{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"Modules": [
		{
			"Name": "ElectraPlayerRuntime",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [ "WinGDK" ]
		},
		{
			"Name": "ElectraPlayerPlugin",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [ "WinGDK" ]
		},
		{
			"Name": "ElectraPlayerFactory",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "WinGDK" ]
		}
	]
}