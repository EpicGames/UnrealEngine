// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraPlayerFactory_WinGDK : ElectraPlayerFactory
	{
		public ElectraPlayerFactory_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
		}

		protected override bool IsSupportedPlatform(ReadOnlyTargetRules Target)
		{
			return true;
		}
	}
}
