// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ElectraPlayerPluginHandler_WinGDK : ElectraPlayerPluginHandler
	{ 
		public ElectraPlayerPluginHandler_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			
		}
		
		protected override bool DoesPlatformSupportElectra(ReadOnlyTargetRules InTarget)
		{
			return false;
		}
	}
}