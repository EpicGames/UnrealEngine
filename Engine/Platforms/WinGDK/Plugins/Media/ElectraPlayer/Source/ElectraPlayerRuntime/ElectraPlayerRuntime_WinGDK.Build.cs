// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ElectraPlayerRuntime_WinGDK : ElectraPlayerRuntime
	{
		public ElectraPlayerRuntime_WinGDK( ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}