{
  "FileVersion": 1,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "Electra Player",
  "Description": "Cross platform media player for local files and internet streaming.\nAlso provides optimized local mp4 file only player (Protron) for desktop machines.",
  "Category": "Media Players",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "DocsURL": "https://docs.unrealengine.com/en-US/Engine/MediaFramework/Overview",
  "MarketplaceURL": "",
  "SupportURL": "",
  "EnabledByDefault": false,
  "CanContainContent": false,
  "IsBetaVersion": false,
  "Installed": false,
  "Modules": [
    {
      "Name": "ElectraPlayerPluginHandler",
      "Type": "RuntimeNoCommandlet",
      "LoadingPhase": "PreLoadingScreen"
    },
    {
      "Name": "ElectraPlayerRuntime",
      "Type": "RuntimeNoCommandlet",
      "LoadingPhase": "PreLoadingScreen",
      "PlatformAllowList": [ "Win64", "Mac", "IOS", "TVOS", "Android", "Linux" ],
      "TargetDenyList": [ "Server" ]
    },
    {
      "Name": "ElectraPlayerPlugin",
      "Type": "RuntimeNoCommandlet",
      "LoadingPhase": "PreLoadingScreen",
      "PlatformAllowList": [ "Win64", "Mac", "IOS", "TVOS", "Android", "Linux" ],
      "TargetDenyList": [ "Server" ]
    },
    {
      "Name": "ElectraPlayerFactory",
      "Type": "Editor",
      "LoadingPhase": "PostEngineInit",
      "PlatformAllowList": [ "Win64", "Mac", "Linux" ],
      "TargetDenyList": [ "Server" ]
    },
    {
      "Name": "ElectraPlayerFactory",
      "Type": "RuntimeNoCommandlet",
      "LoadingPhase": "PostEngineInit",
      "PlatformAllowList": [ "Win64", "Mac", "IOS", "TVOS", "Android", "Linux" ],
      "TargetDenyList": [ "Server" ]
    },

    {
      "Name": "ElectraProtron",
      "Type": "RuntimeNoCommandlet",
      "LoadingPhase": "PreLoadingScreen",
      "PlatformAllowList": [ "Win64", "Mac" ],
      "TargetDenyList": [ "Server" ]
    },
    {
      "Name": "ElectraProtronFactory",
      "Type": "Editor",
      "LoadingPhase": "PostEngineInit",
      "PlatformAllowList": [ "Win64", "Mac" ],
      "TargetDenyList": [ "Server" ]
    },
    {
      "Name": "ElectraProtronFactory",
      "Type": "RuntimeNoCommandlet",
      "LoadingPhase": "PostEngineInit",
      "PlatformAllowList": [ "Win64", "Mac" ],
      "TargetDenyList": [ "Server" ]
    }
  ],

  "Plugins": [
    {
      "Name": "ElectraUtil",
      "Enabled": true
    },
    {
      "Name": "ElectraCodecs",
      "Enabled": true
    },
    {
      "Name": "ElectraCDM",
      "Enabled": true
    },
    {
      "Name": "ElectraSubtitles",
      "Enabled": true
    },
    {
      "Name": "MP4Utilities",
      "Enabled": true
    },
    {
      "Name": "OpusDecoderElectra",
      "Enabled": true
    },
    {
      "Name": "D3D12VideoDecodersElectra",
      "Enabled": true
    }
  ]
}
