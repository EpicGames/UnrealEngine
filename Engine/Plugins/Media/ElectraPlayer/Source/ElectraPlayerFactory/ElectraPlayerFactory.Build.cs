// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ElectraPlayerFactory : ModuleRules
    {
        public ElectraPlayerFactory(ReadOnlyTargetRules Target) : base(Target)
        {
		    bLegalToDistributeObjectCode = true;

            bool bSupportedPlatform = IsSupportedPlatform(Target);

			DynamicallyLoadedModuleNames.AddRange(
                new string[] {
                    "Media",
                });

            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "MediaAssets",
                });

            PrivateIncludePathModuleNames.Add("Media");

            if (bSupportedPlatform)
            {
                PrivateDependencyModuleNames.Add("ElectraBase");
                PrivateIncludePathModuleNames.Add("ElectraPlayerPlugin");
                PublicDefinitions.Add("UE_PLATFORM_ELECTRAPLAYER=1");
            }

            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                });

            if (Target.Type == TargetType.Editor)
            {
                DynamicallyLoadedModuleNames.Add("Settings");
                PrivateIncludePathModuleNames.Add("Settings");
            }

            if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
            {
                PublicDefinitions.Add("_CRT_SECURE_NO_WARNINGS");
            }

            if (bSupportedPlatform)
            {
                DynamicallyLoadedModuleNames.Add("ElectraPlayerPlugin");
            }
        }

		protected virtual bool IsSupportedPlatform(ReadOnlyTargetRules Target)
        {
			return Target.IsInPlatformGroup(UnrealPlatformGroup.Windows)
				|| Target.IsInPlatformGroup(UnrealPlatformGroup.Apple)
				|| Target.IsInPlatformGroup(UnrealPlatformGroup.Android)
				|| Target.IsInPlatformGroup(UnrealPlatformGroup.Unix)
				;
		}
    }
}
