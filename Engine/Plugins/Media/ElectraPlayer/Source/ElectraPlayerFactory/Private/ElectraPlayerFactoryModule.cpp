// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraPlayerFactoryPrivate.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IMediaModule.h"
#include "IMediaOptions.h"
#include "IMediaPlayerFactory.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"

#ifndef UE_PLATFORM_ELECTRAPLAYER
#define UE_PLATFORM_ELECTRAPLAYER 0
#endif
#if UE_PLATFORM_ELECTRAPLAYER
#include "IElectraPlayerPluginModule.h"
#include "Utilities/URLParser.h"
#endif

DEFINE_LOG_CATEGORY(LogElectraPlayerFactory);

#define LOCTEXT_NAMESPACE "ElectraPlayerFactoryModule"

/**
 * Implements the ElectraPlayerFactory module.
 */
class ElectraPlayerFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:
	ElectraPlayerFactoryModule() { }

public:
	// IMediaPlayerFactory interface

	bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
#if UE_PLATFORM_ELECTRAPLAYER
		// Split the URL apart.
		Electra::FURL_RFC3986 UrlParser;
		if (!UrlParser.Parse(Url))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("MalformedURI", "The URI '{0}' could not be parsed"), FText::FromString(Url)));
			}
		}
		// Check scheme
		FString Scheme = UrlParser.GetScheme();
		if (Scheme.IsEmpty())
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}
			return 0;
		}
		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}
			return 0;
		}

		// Check for known extensions
		TArray<FString> PathComponents;
		UrlParser.GetPathComponents(PathComponents);
		FString LowerCaseExtension = PathComponents.Num() ? FPaths::GetExtension(PathComponents.Last().ToLower()) : FString();
		// If the extension is known, we are confident that we can play this.
		// At this point there is no information provided on the codecs used in the media,
		// so we cannot check for this.
		if (SupportedFileExtensions.Contains(LowerCaseExtension))
		{
			return 100;
		}
		// For http URLs, if there is no extension then we can't be sure. Return a lower confidence.
		// If the scheme is file:// then we can actually demand there to be an extension, so if it is missing, too bad.
		else if (LowerCaseExtension.IsEmpty() && (Scheme.Equals(TEXT("https")) || Scheme.Equals(TEXT("http"))))
		{
			return 20;
		}
#endif
		return 0;
	}

	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
#if UE_PLATFORM_ELECTRAPLAYER
		auto PlayerModule = FModuleManager::LoadModulePtr<IElectraPlayerPluginModule>("ElectraPlayerPlugin");
		return (PlayerModule != nullptr) ? PlayerModule->CreatePlayer(EventSink) : nullptr;
#else
		return nullptr;
#endif
	}

	FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Electra Player");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("ElectraPlayer"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0x94ee3f80, 0x8e604292, 0xb4d24dd5, 0xfdade1c2);
		return PlayerPluginGUID;
	}

	const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::AudioSamples) ||
				(Feature == EMediaFeature::AudioTracks) ||
				//(Feature == EMediaFeature::CaptionTracks) ||
				(Feature == EMediaFeature::MetadataTracks) ||
				//(Feature == EMediaFeature::OverlaySamples) ||
				(Feature == EMediaFeature::SubtitleTracks) ||
				(Feature == EMediaFeature::VideoSamples) ||
				(Feature == EMediaFeature::VideoTracks));
	}

public:

	// IModuleInterface interface

	void StartupModule() override
	{
		// supported platforms
#if UE_PLATFORM_ELECTRAPLAYER
		MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		check(MediaModule);

		AddSupportedPlatform(FGuid(0x3619ea87, 0xde704a48, 0xbb155175, 0x4423c26a));
		AddSupportedPlatform(FGuid(0xd1d5f296, 0xff834a87, 0xb20faaa9, 0xd6b8e9a6));
		AddSupportedPlatform(FGuid(0x988eba73, 0xf971495b, 0xafb09639, 0xf8c796bd));
		AddSupportedPlatform(FGuid(0x003be296, 0x17004f0c, 0x8e1f7860, 0x81efbb1f));
		AddSupportedPlatform(FGuid(0xb80decd6, 0x997a4b3f, 0x92063970, 0xe572c0db));
		AddSupportedPlatform(FGuid(0xb67dd9c6, 0x77694fd5, 0xb2b0c8bf, 0xe0c1c673));
		AddSupportedPlatform(FGuid(0x30ebce04, 0x2c8247bd, 0xaf873017, 0x5a27ed45));
		AddSupportedPlatform(FGuid(0x21f5cd78, 0xc2824344, 0xa0f32e55, 0x28059b27));
		AddSupportedPlatform(FGuid(0x941259d5, 0x0a2746aa, 0xadc0ba84, 0x4790ad8a));
		AddSupportedPlatform(FGuid(0xccf05903, 0x822b47e1, 0xb2236a28, 0xdfd78817));
		AddSupportedPlatform(FGuid(0x5636fbc1, 0xd2b54f62, 0xac8e7d4f, 0xb184b45a));
		AddSupportedPlatform(FGuid(0xb596ce6f, 0xd8324a9c, 0x84e9f880, 0x21322535));
		AddSupportedPlatform(FGuid(0x115de4fe, 0x241b465b, 0x970a872f, 0x3167492a));
		AddSupportedPlatform(FGuid(0xc0b45a33, 0x9de340c7, 0xbce24c47, 0x15c3babf));
        AddSupportedPlatform(FGuid(0xa478294f, 0xbd0d4ec0, 0x8830b6d4, 0xd219c1a4));
		AddSupportedPlatform(FGuid(0xae496f22, 0x95534328, 0xbd035b4c, 0x919dc51a));

		// supported schemes
		SupportedUriSchemes.Add(TEXT("http"));
		SupportedUriSchemes.Add(TEXT("https"));
		SupportedUriSchemes.Add(TEXT("file"));

		// supported file extensions
		SupportedFileExtensions.Add(TEXT("mp4"));
		SupportedFileExtensions.Add(TEXT("m4v"));
		SupportedFileExtensions.Add(TEXT("m4a"));
		SupportedFileExtensions.Add(TEXT("mov"));
		SupportedFileExtensions.Add(TEXT("mpd"));
		SupportedFileExtensions.Add(TEXT("m3u8"));
		SupportedFileExtensions.Add(TEXT("mkv"));
		SupportedFileExtensions.Add(TEXT("mka"));
		SupportedFileExtensions.Add(TEXT("webm"));
		SupportedFileExtensions.Add(TEXT("mp3"));
		SupportedFileExtensions.Add(TEXT("mpa"));
		SupportedFileExtensions.Add(TEXT("aac"));
#endif

		// register player factory
		MediaModule->RegisterPlayerFactory(*this);
	}

	void ShutdownModule() override
	{
		// Get media module once more to be sure it is still there
		MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		// unregister player factory
		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}
	}

private:
	void AddSupportedPlatform(const FGuid& PlatformGuid)
	{
		FName PlatformName = MediaModule->GetPlatformName(PlatformGuid);
		if (!PlatformName.IsNone())
		{
			SupportedPlatforms.Add(PlatformName.ToString());
		}
	}

	/** Media module */
	IMediaModule* MediaModule;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(ElectraPlayerFactoryModule, ElectraPlayerFactory);
