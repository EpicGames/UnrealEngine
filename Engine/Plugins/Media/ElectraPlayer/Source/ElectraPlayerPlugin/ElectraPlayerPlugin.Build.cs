// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;
using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class ElectraPlayerPlugin: ModuleRules
	{
		public ElectraPlayerPlugin(ReadOnlyTargetRules Target) : base(Target)
		{
			bLegalToDistributeObjectCode = true;

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"RHI",
					"ElectraPlayerRuntime"
				});

			PrivateIncludePathModuleNames.AddRange(
			    new string[] {
				    "Media",
			});

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.Add("Engine");
			}
		}
	}
}
