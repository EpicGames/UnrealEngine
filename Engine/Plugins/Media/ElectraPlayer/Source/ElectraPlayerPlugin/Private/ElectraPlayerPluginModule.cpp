// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"
#include "RHI.h"


#include "IElectraPlayerRuntimeModule.h"
#include "IElectraPlayerPluginModule.h"			// for loading the Electra player runtime
#include "IElectraPlayerInterface.h"			// for metrics delegate declaration


#define LOCTEXT_NAMESPACE "ElectraPlayerPluginModule"
DECLARE_LOG_CATEGORY_EXTERN(LogElectraPlayerPlugin, Log, All);
DEFINE_LOG_CATEGORY(LogElectraPlayerPlugin);

// -----------------------------------------------------------------------------------------------------------------------------------

class FElectraPlayerPluginModule : public IElectraPlayerPluginModule
{
public:
	bool IsInitialized() const override
	{
		return bInitialized;
	}

	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& InEventSink) override
	{
		if (bInitialized)
		{
			return FElectraPlayerRuntimeFactory::CreatePlayer(InEventSink, SendAnalyticMetricsDelegate, SendAnalyticMetricsPerMinuteDelegate, ReportVideoStreamingErrorDelegate, ReportSubtitlesMetricsDelegate);
		}
		return nullptr;
	}

	void SendAnalyticMetrics(const TSharedPtr<IAnalyticsProviderET>& AnalyticsProvider, const FGuid& PlayerGuid) override
	{
		SendAnalyticMetricsDelegate.Broadcast(AnalyticsProvider, PlayerGuid);
	}

	void SendAnalyticMetricsPerMinute(const TSharedPtr<IAnalyticsProviderET>& AnalyticsProvider) override
	{
		SendAnalyticMetricsPerMinuteDelegate.Broadcast(AnalyticsProvider);
	}

	void ReportVideoStreamingError(const FGuid& PlayerGuid, const FString& LastError) override
	{
		ReportVideoStreamingErrorDelegate.Broadcast(PlayerGuid, LastError);
	}

	void ReportSubtitlesMetrics(const FGuid& PlayerGuid, const FString& URL, double ResponseTime, const FString& LastError) override
	{
		ReportSubtitlesMetricsDelegate.Broadcast(PlayerGuid, URL, ResponseTime, LastError);
	}

	void StartupModule() override
	{
		// Check that we have the player module and that it has initialized successfully.
		if (FModuleManager::Get().GetModule("ElectraPlayerRuntime"))
		{
			IElectraPlayerRuntimeModule* ElectraPlayer = &FModuleManager::Get().GetModuleChecked<IElectraPlayerRuntimeModule>("ElectraPlayerRuntime");
			if (!bInitialized && ElectraPlayer && ElectraPlayer->IsInitialized())
			{
				// Detect cooking and other commandlets that run with NullRHI
				if (GDynamicRHI && RHIGetInterfaceType() != ERHIInterfaceType::Null)
				{
					bInitialized = true;
				}
				else
				{
					UE_LOGF(LogElectraPlayerPlugin, Log, "Dummy Dynamic RHI detected. Electra Player plugin is not initialized.");
				}
			}
		}
	}

	void ShutdownModule() override
	{
		bInitialized = false;
	}

private:
	FElectraPlayerSendAnalyticMetricsDelegate SendAnalyticMetricsDelegate;
	FElectraPlayerSendAnalyticMetricsPerMinuteDelegate SendAnalyticMetricsPerMinuteDelegate;
	FElectraPlayerReportVideoStreamingErrorDelegate ReportVideoStreamingErrorDelegate;
	FElectraPlayerReportSubtitlesMetricsDelegate ReportSubtitlesMetricsDelegate;
	bool bInitialized = false;
};

IMPLEMENT_MODULE(FElectraPlayerPluginModule, ElectraPlayerPlugin);

#undef LOCTEXT_NAMESPACE
