// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"

class IAnalyticsProviderET;
class IMediaEventSink;
class IMediaPlayer;


/**
 * Interface for the ElectraPlayerPlugin module.
 */
class IElectraPlayerPluginModule
	: public IModuleInterface
{
public:
	virtual ~IElectraPlayerPluginModule() = default;

	/**
	 * Is the ElectraPlayerPlugin module initialized?
	 * @return True if the module is initialized.
	 */
	virtual bool IsInitialized() const = 0;

	/**
	 * Creates a media player
	 *
	 * @param EventSink The object that receives media events from the player.
	 * @return A new media player, or nullptr if a player couldn't be created.
	 */
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) = 0;

	virtual void SendAnalyticMetrics(const TSharedPtr<IAnalyticsProviderET>& AnalyticsProvider, const FGuid& PlayerGuid) = 0;
	virtual void SendAnalyticMetricsPerMinute(const TSharedPtr<IAnalyticsProviderET>& AnalyticsProvider) = 0;

	virtual void ReportVideoStreamingError(const FGuid& PlayerGuid, const FString& LastError) = 0;
	virtual void ReportSubtitlesMetrics(const FGuid& PlayerGuid, const FString& URL, double ResponseTime, const FString& LastError) = 0;
};
