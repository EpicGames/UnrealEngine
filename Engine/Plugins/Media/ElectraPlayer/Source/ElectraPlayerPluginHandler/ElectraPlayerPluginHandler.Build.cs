// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ElectraPlayerPluginHandler : ModuleRules
	{ 
		public ElectraPlayerPluginHandler(ReadOnlyTargetRules Target) : base(Target)
		{
			if (DoesPlatformSupportElectra(Target) && Target.Type != TargetType.Server)
			{
				PublicDefinitions.Add("HAVE_ELECTRA=1");
				PublicDependencyModuleNames.Add("ElectraPlayerRuntime");
				PublicDependencyModuleNames.Add("ElectraPlayerPlugin");
			}
			else
			{
				PublicDefinitions.Add("HAVE_ELECTRA=0");
			}

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);
		}
		
		virtual protected bool DoesPlatformSupportElectra(ReadOnlyTargetRules InTarget) 
		{
			return InTarget.Platform == UnrealTargetPlatform.Win64 || 
			       InTarget.Platform == UnrealTargetPlatform.IOS || 
			       InTarget.Platform == UnrealTargetPlatform.Mac ||
			       InTarget.Platform == UnrealTargetPlatform.Linux ||
			       InTarget.Platform == UnrealTargetPlatform.Android;
		}
	}
}
