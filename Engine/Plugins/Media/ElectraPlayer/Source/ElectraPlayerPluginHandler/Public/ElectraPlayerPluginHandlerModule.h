// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#define UE_API ELECTRAPLAYERPLUGINHANDLER_API

class FElectraPlayerPluginHandlerModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
};

#undef UE_API
