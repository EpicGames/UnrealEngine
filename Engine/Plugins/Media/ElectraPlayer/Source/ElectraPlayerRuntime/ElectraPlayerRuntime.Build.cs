// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class ElectraPlayerRuntime: ModuleRules
	{
		public ElectraPlayerRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			//
			// Common setup...
			//
			bLegalToDistributeObjectCode = true;
			IWYUSupport = IWYUSupport.None;

			bRequiresPlatformSDK = true;

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
                    "RHI",
					"RenderCore",
					"Json",
					"MediaUtils",
                    "Media",
					"ElectraBase",
					"ElectraSamples",
					"ElectraCodecFactory",
					"ElectraDecoders",
					"ElectraHTTPStream",
					"ElectraCDM",
					"ElectraSubtitles",
                    "MP4Utilities",
                    "MP4Boxes",
					"XmlParser",
					"SoundTouchZ",
                    "HTTP"
				});
			if (Target.bCompileAgainstEngine)
			{
				// Added to allow debug rendering if used in UE context
				PrivateDependencyModuleNames.Add("Engine");
			}

			PrivateIncludePaths.AddRange(
				new string[] {
					"ElectraPlayerRuntime/Private/Runtime",
				});

			//
			// Common platform setup...
			//
			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDependencyModuleNames.Add("DirectX");

				PrivateDefinitions.Add("_CRT_SECURE_NO_WARNINGS=1");

				PrivateDependencyModuleNames.AddAll("D3D11RHI", "D3D12RHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");

				PrivateDefinitions.Add("ELECTRA_HAVE_DX11");	// video decoding for DX11 enabled (Win8+)

				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
				{
					PublicAdditionalLibraries.AddRange(new string[] {
						Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dxerr.lib"),
					});
				}

				PublicSystemLibraries.AddRange(new string[] {
					"strmiids.lib",
					"legacy_stdio_definitions.lib",
					"Dxva2.lib",
				});

				// Delay-load all MF DLLs to be able to check Windows version for compatibility in `StartupModule` before loading them manually
				PublicSystemLibraries.Add("mfplat.lib");
				PublicDelayLoadDLLs.Add("mfplat.dll");
				PublicSystemLibraries.Add("mfuuid.lib");

				PublicIncludePaths.Add("$(ModuleDir)/Public/Windows");
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
			{
				PrivateDependencyModuleNames.Add("MetalRHI");
				PublicFrameworks.AddRange(
				new string[] {
								"CoreMedia",
								"CoreVideo",
								"AVFoundation",
								"AudioToolbox",
								"VideoToolbox",
								"QuartzCore"
				});

				PublicIncludePaths.Add("$(ModuleDir)/Public/Apple");
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Android) )
			{
				PublicIncludePaths.Add("$(ModuleDir)/Public/Android");
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) )
			{
				PublicIncludePaths.Add("$(ModuleDir)/Public/Linux");

				AddEngineThirdPartyPrivateStaticDependencies(Target, "libav");
			}
		}
	}
}
