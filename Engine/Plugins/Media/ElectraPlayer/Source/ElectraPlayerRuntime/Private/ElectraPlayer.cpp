// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraPlayer.h"
#include "ElectraPlayerPrivate.h"
#include "Misc/Optional.h"
#include "Misc/SecureHash.h"

#include "MediaSamples.h"
#include "MediaPlayerOptions.h"
#include "IMediaPlayerLifecycleManager.h"

#include "PlayerRuntimeGlobal.h"
#include "Player/AdaptiveStreamingPlayer.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/ABRRules/ABROptionKeynames.h"
#include "Utilities/Utilities.h"
#include "Utilities/URLParser.h"
#include "Utilities/StringHelpers.h"
#include "MediaMetaDataDecoderOutput.h"
#include "IElectraMetadataSample.h"
#include "IMediaMetadataItem.h"
#include "MediaStreamMetadata.h"

#include "ElectraDecodersPlatformResources.h"

#include "RHIGlobals.h"

//-----------------------------------------------------------------------------

CSV_DEFINE_CATEGORY_MODULE(ELECTRAPLAYERRUNTIME_API, ElectraPlayer, false);

DECLARE_CYCLE_STAT(TEXT("FElectraPlayer::TickInput"), STAT_ElectraPlayer_ElectraPlayer_TickInput, STATGROUP_ElectraPlayer);
DECLARE_CYCLE_STAT(TEXT("FElectraPlayer::StaticResourceRequest"), STAT_ElectraPlayer_ElectraPlayer_StaticResourceRequest, STATGROUP_ElectraPlayer);
DECLARE_CYCLE_STAT(TEXT("FElectraPlayer::PlayerEvents"), STAT_ElectraPlayer_ElectraPlayer_PlayerEvents, STATGROUP_ElectraPlayer);
DECLARE_CYCLE_STAT(TEXT("FElectraPlayer::QueryOptions"), STAT_ElectraPlayer_ElectraPlayer_QueryOptions, STATGROUP_ElectraPlayer);

//-----------------------------------------------------------------------------

#include "HAL/IConsoleManager.h"
static TAutoConsoleVariable<bool> CVarElectraRetainPools(
	TEXT("Electra.RetainPools"),
	false,
	TEXT("Retains video and audio sample pools in player for reuse.\n")
	TEXT(" false: don't retain, create new pools for each instance; true: retain pools"),
	ECVF_Default);

//-----------------------------------------------------------------------------


// Prefix to use in querying for a custom analytic value through QueryOptions()
#define CUSTOM_ANALYTIC_METRIC_QUERYOPTION_KEY TEXT("ElectraCustomAnalytic")
// Prefix to use in the metric event to set the custom value.
#define CUSTOM_ANALYTIC_METRIC_KEYNAME TEXT("Custom")

namespace ElectraMediaOptions
{
	static const FName GetSafeMediaOptions(TEXT("GetSafeMediaOptions"));
	static const FName ContentID(TEXT("content_id"));
	static const FName ElectraCMCDConfig(TEXT("ElectraCMCDConfig"));
	static const FName ElectraNoPreloading(TEXT("ElectraNoPreloading"));
	static const FName PlaylistProperties(TEXT("playlist_properties"));
	static const FName ElectraInitialBitrate(TEXT("ElectraInitialBitrate"));
	static const FName MaxElectraVerticalResolution(TEXT("MaxElectraVerticalResolution"));
	static const FName MaxElectraVerticalResolutionOf60fpsVideos(TEXT("MaxElectraVerticalResolutionOf60fpsVideos"));
	static const FName ElectraLivePresentationOffset(TEXT("ElectraLivePresentationOffset"));
	static const FName ElectraThrowErrorWhenRebuffering(TEXT("ElectraThrowErrorWhenRebuffering"));
	static const FName ElectraGetDenyStreamCode(TEXT("ElectraGetDenyStreamCode"));
	static const FName MaxResolutionForMediaStreaming(TEXT("MaxResolutionForMediaStreaming"));
	static const FName ElectraMaxStreamingBandwidth(TEXT("ElectraMaxStreamingBandwidth"));
	static const FName Mimetype(TEXT("mimetype"));
	static const FName CodecOptions[] = { TEXT("excluded_codecs_video"), TEXT("excluded_codecs_audio"), TEXT("excluded_codecs_subtitles"), TEXT("preferred_codecs_video"), TEXT("preferred_codecs_audio"),TEXT("preferred_codecs_subtitles") };
	static const FName ElectraGetPlaylistData(TEXT("ElectraGetPlaylistData"));
	static const FName ElectraGetLicenseKeyData(TEXT("ElectraGetLicenseKeyData"));
	static const FName ElectraGetPlaystartPosFromSeekPositions(TEXT("ElectraGetPlaystartPosFromSeekPositions"));



	enum class EOptionType
	{
		MaxVerticalStreamResolution = 0,
		MaxBandwidthForStreaming,
		PlayListData,
		LicenseKeyData,
		CustomAnalyticsMetric,
		PlaystartPosFromSeekPositions
	};
	static Electra::FVariantValue GetOptionValue(TWeakPtr<IElectraSafeMediaOptionInterface, ESPMode::ThreadSafe> InFromOptions, EOptionType InWhich, const Electra::FVariantValue& DefaultValue = Electra::FVariantValue())
	{
		Electra::FVariantValue Value;
		TSharedPtr<IElectraSafeMediaOptionInterface, ESPMode::ThreadSafe> SafeOptions = InFromOptions.Pin();
		if (SafeOptions.IsValid())
		{
			IElectraSafeMediaOptionInterface::FScopedLock SafeLock(SafeOptions);
			IMediaOptions *Options = SafeOptions->GetMediaOptionInterface();
			if (Options)
			{
				switch(InWhich)
				{
					default:
					{
						break;
					}
					case EOptionType::MaxVerticalStreamResolution:
					{
						return FVariantValue((int64)Options->GetMediaOption(MaxResolutionForMediaStreaming, (int64)0));
					}
					case EOptionType::MaxBandwidthForStreaming:
					{
						return FVariantValue((int64)Options->GetMediaOption(ElectraMaxStreamingBandwidth, (int64)0));
					}
					case EOptionType::PlayListData:
					{
						if (Options->HasMediaOption(ElectraGetPlaylistData))
						{
							check(DefaultValue.IsType(FVariantValue::EDataType::TypeFString));
							return FVariantValue(Options->GetMediaOption(ElectraGetPlaylistData, DefaultValue.GetFString()));
						}
						break;
					}
					case EOptionType::LicenseKeyData:
					{
						if (Options->HasMediaOption(ElectraGetLicenseKeyData))
						{
							check(DefaultValue.IsType(FVariantValue::EDataType::TypeFString));
							return FVariantValue(Options->GetMediaOption(ElectraGetLicenseKeyData, DefaultValue.GetFString()));
						}
						break;
					}
					case EOptionType::CustomAnalyticsMetric:
					{
						check(DefaultValue.IsType(FVariantValue::EDataType::TypeFString));
						if (DefaultValue.IsType(FVariantValue::EDataType::TypeFString))
						{
							FName OptionKey(*DefaultValue.GetFString());
							if (Options->HasMediaOption(OptionKey))
							{
								return FVariantValue(Options->GetMediaOption(OptionKey, FString()));
							}
						}
						break;
					}
					case EOptionType::PlaystartPosFromSeekPositions:
					{
						if (Options->HasMediaOption(ElectraGetPlaystartPosFromSeekPositions))
						{
							check(DefaultValue.IsType(FVariantValue::EDataType::TypeSharedPointer));
							TSharedPtr<TArray<FTimespan>, ESPMode::ThreadSafe> PosArray = DefaultValue.GetSharedPointer<TArray<FTimespan>>();
							if (PosArray.IsValid())
							{
								TSharedPtr<FElectraSeekablePositions, ESPMode::ThreadSafe> Res = StaticCastSharedPtr<FElectraSeekablePositions, IMediaOptions::FDataContainer, ESPMode::ThreadSafe>(Options->GetMediaOption(ElectraGetPlaystartPosFromSeekPositions, MakeShared<FElectraSeekablePositions, ESPMode::ThreadSafe>(*PosArray)));
								if (Res.IsValid() && Res->Data.Num())
								{
									return FVariantValue(int64(Res->Data[0].GetTicks())); // return HNS
								}
							}
							return FVariantValue();
						}
						break;
					}
				}
			}
		}
		return Value;
	}

}

//-----------------------------------------------------------------------------

#if UE_BUILD_SHIPPING
#define HIDE_URLS_FROM_LOG	1
#else
#define HIDE_URLS_FROM_LOG	0
#endif

static FString SanitizeMessage(FString InMessage)
{
#if !HIDE_URLS_FROM_LOG
	return MoveTemp(InMessage);
#else
	int32 searchPos = 0;
	while(1)
	{
		static FString SchemeStr(TEXT("://"));
		static FString DotDotDotStr(TEXT("..."));
		static FString TermChars(TEXT("'\",; "));
		int32 schemePos = InMessage.Find(SchemeStr, ESearchCase::IgnoreCase, ESearchDir::FromStart, searchPos);
		if (schemePos != INDEX_NONE)
		{
			schemePos += SchemeStr.Len();
			// There may be a generic user message following a potential URL that we do not want to clobber.
			// We search for any next character that tends to end a URL in a user message, like one of ['",; ]
			int32 EndPos = InMessage.Len();
			int32 Start = schemePos;
			while(Start < EndPos)
			{
				int32 pos;
				if (TermChars.FindChar(InMessage[Start], pos))
				{
					break;
				}
				++Start;
			}
			InMessage.RemoveAt(schemePos, Start-schemePos);
			InMessage.InsertAt(schemePos, DotDotDotStr);
			searchPos = schemePos + SchemeStr.Len();
		}
		else
		{
			break;
		}
	}
	return InMessage;
#endif
}


//-----------------------------------------------------------------------------

class FMetaDataDecoderOutput : public IMetaDataDecoderOutput
{
public:
	virtual ~FMetaDataDecoderOutput() = default;
	const void* GetData() override									{ return Data.GetData(); }
	FTimespan GetDuration() const override							{ return Duration; }
	uint32 GetSize() const override									{ return (uint32) Data.Num(); }
	FDecoderTimeStamp GetTime() const override						{ return PresentationTime; }
	EOrigin GetOrigin() const override								{ return Origin; }
	EDispatchedMode GetDispatchedMode() const override				{ return DispatchedMode; }
	const FString& GetSchemeIdUri() const override					{ return SchemeIdUri; }
	const FString& GetValue() const override						{ return Value; }
	const FString& GetID() const override							{ return ID; }
	TOptional<FDecoderTimeStamp> GetTrackBaseTime() const override	{ return TrackBaseTime; }
	void SetTime(FDecoderTimeStamp& InTime) override				{ PresentationTime = InTime; }

	TArray<uint8> Data;
	FDecoderTimeStamp PresentationTime;
	FTimespan Duration;
	EOrigin Origin;
	EDispatchedMode DispatchedMode;
	FString SchemeIdUri;
	FString Value;
	FString ID;
	TOptional<FDecoderTimeStamp> TrackBaseTime;
};

class FElectraBinarySample : public IElectraBinarySample
{
public:
	~FElectraBinarySample() = default;
	const void* GetData() override							{ return Metadata->GetData(); }
	uint32 GetSize() const override							{ return Metadata->GetSize(); }
	FGuid GetGUID() const override							{ return IElectraBinarySample::GetSampleTypeGUID(); }
	const FString& GetSchemeIdUri() const override			{ return Metadata->GetSchemeIdUri(); }
	const FString& GetValue() const override				{ return Metadata->GetValue(); }
	const FString& GetID() const override					{ return Metadata->GetID(); }
	EDispatchedMode GetDispatchedMode() const override
	{
		switch(Metadata->GetDispatchedMode())
		{
			default:
			case IMetaDataDecoderOutput::EDispatchedMode::OnReceive:
			{
				return FElectraBinarySample::EDispatchedMode::OnReceive;
			}
			case IMetaDataDecoderOutput::EDispatchedMode::OnStart:
			{
				return FElectraBinarySample::EDispatchedMode::OnStart;
			}
		}
	}

	EOrigin GetOrigin() const override
	{
		switch(Metadata->GetOrigin())
		{
			default:
			case IMetaDataDecoderOutput::EOrigin::TimedMetadata:
			{
				return FElectraBinarySample::EOrigin::TimedMetadata;
			}
			case IMetaDataDecoderOutput::EOrigin::EventStream:
			{
				return FElectraBinarySample::EOrigin::EventStream;
			}
			case IMetaDataDecoderOutput::EOrigin::InbandEventStream:
			{
				return FElectraBinarySample::EOrigin::InbandEventStream;
			}
		}
	}

	FMediaTimeStamp GetTime() const override
	{
		FDecoderTimeStamp ts = Metadata->GetTime();
		return FMediaTimeStamp(ts.Time, ts.SequenceIndex);
	}

	FTimespan GetDuration() const override
	{
		FTimespan Duration = Metadata->GetDuration();
		// A zero duration might cause the metadata sample fall through the cracks later
		// so set it to a short 1ms instead.
		if (Duration.IsZero())
		{
			Duration = FTimespan::FromMilliseconds(1);
		}
		return Duration;
	}

	TOptional<FMediaTimeStamp> GetTrackBaseTime() const	override
	{
		TOptional<FMediaTimeStamp> ms;
		TOptional<FDecoderTimeStamp> ts = Metadata->GetTime();
		if (ts.IsSet())
		{
			ms = FMediaTimeStamp(ts.GetValue().Time, ts.GetValue().SequenceIndex);
		}
		return ms;
	}

	IMetaDataDecoderOutputPtr Metadata;
};


//-----------------------------------------------------------------------------

std::atomic<uint32> FElectraPlayer::NextPlayerUniqueID { 0 };

//-----------------------------------------------------------------------------
/**
 *	Construction of new player
 */
FElectraPlayer::FElectraPlayer(IMediaEventSink& InEventSink,
					 FElectraPlayerSendAnalyticMetricsDelegate& InSendAnalyticMetricsDelegate,
					 FElectraPlayerSendAnalyticMetricsPerMinuteDelegate& InSendAnalyticMetricsPerMinuteDelegate,
					 FElectraPlayerReportVideoStreamingErrorDelegate& InReportVideoStreamingErrorDelegate,
					 FElectraPlayerReportSubtitlesMetricsDelegate& InReportSubtitlesFileMetricsDelegate)
	: EventSink(&InEventSink)
	, SendAnalyticMetricsDelegate(InSendAnalyticMetricsDelegate)
	, SendAnalyticMetricsPerMinuteDelegate(InSendAnalyticMetricsPerMinuteDelegate)
	, ReportVideoStreamingErrorDelegate(InReportVideoStreamingErrorDelegate)
	, ReportSubtitlesMetricsDelegate(InReportSubtitlesFileMetricsDelegate)
{
	CSV_EVENT(ElectraPlayer, TEXT("Player Creation"));

	bRetainPoolsForReuse = CVarElectraRetainPools.GetValueOnAnyThread();
	if (bRetainPoolsForReuse)
	{
		OutputAudioSamplePool = MakeShareable(new FElectraAudioSamplePool);
		OutputTexturePool = MakeShareable(new FElectraTextureSamplePool);
	#if PLATFORM_ANDROID
		OutputTexturePoolDecoderID = FElectraDecodersPlatformResources::RegisterOutputTexturePool(OutputTexturePool);
	#endif
	}

	EmptyMediaSamples.Reset(new FMediaSamples);
	CurrentPlaybackRange = TRange<FTimespan>::Empty();

	AppTerminationHandler = MakeSharedTS<Electra::FApplicationTerminationHandler>();
	AppTerminationHandler->Terminate = [this]() { CloseInternal(); };
	Electra::AddTerminationNotificationHandler(AppTerminationHandler);

	SendAnalyticMetricsDelegate.AddRaw(this, &FElectraPlayer::SendAnalyticMetrics);
	SendAnalyticMetricsPerMinuteDelegate.AddRaw(this, &FElectraPlayer::SendAnalyticMetricsPerMinute);
	ReportVideoStreamingErrorDelegate.AddRaw(this, &FElectraPlayer::ReportVideoStreamingError);
	ReportSubtitlesMetricsDelegate.AddRaw(this, &FElectraPlayer::ReportSubtitlesMetrics);

	FString	OSMinor;
	AnalyticsGPUType = GRHIAdapterName.TrimStartAndEnd();
	FPlatformMisc::GetOSVersions(AnalyticsOSVersion, OSMinor);
	AnalyticsOSVersion.TrimStartAndEndInline();
	AnalyticsInstanceEventCount = 0;
	NumQueuedAnalyticEvents = 0;

	ClearToDefaultState();
	bHasPendingError = false;
	bHasClosedDueToError = false;
}

//-----------------------------------------------------------------------------
/**
 *	Cleanup destructor
 */
FElectraPlayer::~FElectraPlayer()
{
	CSV_EVENT(ElectraPlayer, TEXT("Player Destruction"));

	Electra::RemoveTerminationNotificationHandler(AppTerminationHandler);
	AppTerminationHandler.Reset();

	CloseInternal();

	if (bRetainPoolsForReuse)
	{
	#if PLATFORM_ANDROID
		FElectraDecodersPlatformResources::UnregisterOutputTexturePool(OutputTexturePoolDecoderID);
	#endif
	}

	SendAnalyticMetricsDelegate.RemoveAll(this);
	SendAnalyticMetricsPerMinuteDelegate.RemoveAll(this);
	ReportVideoStreamingErrorDelegate.RemoveAll(this);
	ReportSubtitlesMetricsDelegate.RemoveAll(this);
	UE_LOGF(LogElectraPlayer, Verbose, "[%u] ~FElectraPlayer() finished.", PlayerUniqueID);

	if (AsyncResourceReleaseNotification.IsValid())
	{
		AsyncResourceReleaseNotification->Signal(IMediaPlayerLifecycleManagerDelegate::ResourceFlags_OutputBuffers);
	}
}


void FElectraPlayer::ClearToDefaultState()
{
	PlayerState.Reset();
	NumTracksAudio = 0;
	NumTracksVideo = 0;
	NumTracksSubtitle = 0;
	SelectedQuality = 0;
	SelectedVideoTrackIndex = -1;
	SelectedAudioTrackIndex = -1;
	SelectedSubtitleTrackIndex = -1;
	bVideoTrackIndexDirty = true;
	bAudioTrackIndexDirty = true;
	bSubtitleTrackIndexDirty = true;
	bInitialSeekPerformed = false;
	bIsFirstBuffering = true;
	CurrentStreamMetadata.Reset();
	CurrentlyActiveVideoStreamFormat.Reset();
	DeferredPlayerEvents.Empty();
	MediaUrl.Empty();
}


void FElectraPlayer::SendMediaSinkEvent(EMediaEvent InEventToSend)
{
	if (EventSink)
	{
		EventSink->ReceiveMediaEvent(InEventToSend);
	}
}



bool FElectraPlayer::OpenInternal(const FString& InUrl, const IMediaOptions* InOptions, const FMediaPlayerOptions* InPlayerOptions)
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);
	CSV_EVENT(ElectraPlayer, TEXT("Open"));

	CloseInternal();
	ClearToDefaultState();

	FGuid SessionID = FGuid::NewGuid();

	PlayerUniqueID = ++NextPlayerUniqueID;

	// Check that we have a valid option interface.
	if (InOptions == nullptr)
	{
		UE_LOGF(LogElectraPlayer, Error, "[%u] IMediaPlayer::Open: Options == nullptr", PlayerUniqueID);
		SendMediaSinkEvent(EMediaEvent::MediaOpenFailed);
		return false;
	}
	// Get the safe option interface to poll for changes during playback.
	OptionInterface = StaticCastSharedPtr<IElectraSafeMediaOptionInterface>(InOptions->GetMediaOption(ElectraMediaOptions::GetSafeMediaOptions, TSharedPtr<IElectraSafeMediaOptionInterface, ESPMode::ThreadSafe>()));
	UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaPlayer::Open", PlayerUniqueID);

	// Create the static resource provider using the safe media option interface.
	StaticResourceProvider = MakeShared<FAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe>(OptionInterface);

	// Prepare the start options
	PlaystartOptions.Reset();
	FName Environment;
	if (InPlayerOptions)
	{
		if (InPlayerOptions->Loop != EMediaPlayerOptionBooleanOverride::UseMediaPlayerSetting)
		{
			bEnableLooping = InPlayerOptions->Loop == EMediaPlayerOptionBooleanOverride::Enabled;
			UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaPlayer::Open: Setting initial loop state to %ls", PlayerUniqueID, bEnableLooping ? TEXT("enabled") : TEXT("disabled"));
		}
		if (InPlayerOptions->SeekTimeType != EMediaPlayerOptionSeekTimeType::Ignored)
		{
			// If an explicit time is set we only use it if it is not zero. Zero is the default value and most of the time means nothing in particular.
			// Playback will start at the beginning for VoD anyway and if this is an event using zero causes problems
			if (InPlayerOptions->SeekTime > FTimespan::Zero())
			{
				PlaystartOptions.TimeOffset = InPlayerOptions->SeekTime;
				UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaPlayer::Open: Setting initial seek time to %.3f seconds", PlayerUniqueID, InPlayerOptions->SeekTime.GetTotalSeconds());
			}
			else
			{
				UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaPlayer::Open: Setting initial seek time of zero will be ignored as it implies the beginning.", PlayerUniqueID);
			}
		}
		if (InPlayerOptions->TrackSelection == EMediaPlayerOptionTrackSelectMode::UseLanguageCodes)
		{
			if (InPlayerOptions->TracksByLanguage.Video.Len())
			{
				PlaystartOptions.InitialVideoTrackAttributes.Language_RFC4647 = InPlayerOptions->TracksByLanguage.Video;
				UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Asking for initial video language \"%ls\"", PlayerUniqueID, *InPlayerOptions->TracksByLanguage.Video);
			}
			if (InPlayerOptions->TracksByLanguage.Audio.Len())
			{
				PlaystartOptions.InitialAudioTrackAttributes.Language_RFC4647 = InPlayerOptions->TracksByLanguage.Audio;
				UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Asking for initial audio language \"%ls\"", PlayerUniqueID, *InPlayerOptions->TracksByLanguage.Audio);
			}
			if (InPlayerOptions->TracksByLanguage.Subtitle.Len())
			{
				PlaystartOptions.InitialSubtitleTrackAttributes.Language_RFC4647 = InPlayerOptions->TracksByLanguage.Subtitle;
				UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Asking for initial subtitle language \"%ls\"", PlayerUniqueID, *InPlayerOptions->TracksByLanguage.Subtitle);
			}
		}
		else if (InPlayerOptions->TrackSelection == EMediaPlayerOptionTrackSelectMode::UseTrackOptionIndices)
		{
			PlaystartOptions.InitialVideoTrackAttributes.OverrideIndex = InPlayerOptions->Tracks.Video;
			PlaystartOptions.InitialAudioTrackAttributes.OverrideIndex = InPlayerOptions->Tracks.Audio;
			PlaystartOptions.InitialSubtitleTrackAttributes.OverrideIndex = InPlayerOptions->Tracks.Subtitle;
		}
		const FVariant* Env = InPlayerOptions->InternalCustomOptions.Find(MediaPlayerOptionValues::Environment());
		Environment = Env ? Env->GetValue<FName>() : Environment;
	}
	bool bNoPreloading = InOptions->GetMediaOption(ElectraMediaOptions::ElectraNoPreloading, (bool)false);
	if (bNoPreloading)
	{
		PlaystartOptions.bDoNotPreload = true;
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: No preloading after opening media", PlayerUniqueID);
	}

	// Set up options to initialize the internal player with.
	Electra::FParamDict PlayerOptions;

	// Set the session ID to be the same we will use for analytics here (see below)
	PlayerOptions.Set(Electra::OptionKeySessionID, Electra::FVariantValue(SessionID.ToString(EGuidFormats::DigitsWithHyphensLower)));
	// Ask for content ID. If none is provided we construct one from the URL.
	FString ContentID = InOptions->GetMediaOption(ElectraMediaOptions::ContentID, FString());
	if (ContentID.IsEmpty())
	{
		FString UrlToHash(InUrl.TrimStartAndEnd());
		Electra::FURL_RFC3986 up;
		if (up.Parse(UrlToHash))
		{
			UrlToHash = up.GetHost() + up.GetPath(false, false);
		}
		ContentID = FMD5::HashAnsiString(*UrlToHash);
	}
	// Set the content ID.
	PlayerOptions.Set(Electra::OptionKeyContentID, Electra::FVariantValue(ContentID));

	for(auto &CodecOption : ElectraMediaOptions::CodecOptions)
	{
		FString Value = InOptions->GetMediaOption(CodecOption, FString());
		if (Value.Len())
		{
			PlayerOptions.Set(CodecOption, Electra::FVariantValue(Value));
		}
	}
	// Required playlist properties?
	FString PlaylistProperties = InOptions->GetMediaOption(ElectraMediaOptions::PlaylistProperties, FString());
	if (PlaylistProperties.Len())
	{
		PlayerOptions.Set(Electra::OptionKeyPlaylistProperties, Electra::FVariantValue(PlaylistProperties));
	}
	// CMCD configuration
	FString CMCDConfiguration = InOptions->GetMediaOption(ElectraMediaOptions::ElectraCMCDConfig, FString());
	if (CMCDConfiguration.Len())
	{
		PlayerOptions.Set(Electra::OptionKeyCMCDConfiguration, Electra::FVariantValue(CMCDConfiguration));
	}
	// Timecode parsing?
	if (InPlayerOptions && InPlayerOptions->InternalCustomOptions.Find(MediaPlayerOptionValues::ParseTimecodeInfo()))
	{
		PlayerOptions.Set(Electra::OptionKeyParseTimecodeInfo, FVariantValue());
	}

	// Check for one-time initialization options that can't be changed during playback.
	int64 InitialStreamBitrate = InOptions->GetMediaOption(ElectraMediaOptions::ElectraInitialBitrate, (int64)-1);
	if (InitialStreamBitrate > 0)
	{
		PlayerOptions.Set(Electra::OptionKeyInitialBitrate, Electra::FVariantValue(InitialStreamBitrate));
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Using initial bitrate of %d bits/second", PlayerUniqueID, (int32)InitialStreamBitrate);
	}
	FString MediaMimeType = InOptions->GetMediaOption(ElectraMediaOptions::Mimetype, FString());
	if (MediaMimeType.Len())
	{
		PlayerOptions.Set(Electra::OptionKeyMimeType, Electra::FVariantValue(MediaMimeType));
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Setting media mime type to \"%ls\"", PlayerUniqueID, *MediaMimeType);
	}
	int64 MaxVerticalHeight = InOptions->GetMediaOption(ElectraMediaOptions::MaxElectraVerticalResolution, (int64)-1);
	if (MaxVerticalHeight > 0)
	{
		PlayerOptions.Set(Electra::OptionKeyMaxVerticalResolution, Electra::FVariantValue(MaxVerticalHeight));
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Limiting vertical resolution to %d for all streams", PlayerUniqueID, (int32)MaxVerticalHeight);
	}
	int64 MaxVerticalHeightAt60 = InOptions->GetMediaOption(ElectraMediaOptions::MaxElectraVerticalResolutionOf60fpsVideos, (int64)-1);
	if (MaxVerticalHeightAt60 > 0)
	{
		PlayerOptions.Set(Electra::OptionKeyMaxVerticalResolutionAbove30fps, Electra::FVariantValue(MaxVerticalHeightAt60));
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Limiting vertical resolution to %d for streams >30fps", PlayerUniqueID, (int32)MaxVerticalHeightAt60);
	}
	double LiveEdgeDistanceForNormalPresentation = InOptions->GetMediaOption(ElectraMediaOptions::ElectraLivePresentationOffset, (double)-1.0);
	if (LiveEdgeDistanceForNormalPresentation > 0.0)
	{
		PlayerOptions.Set(Electra::OptionKeyLiveSeekableEndOffset, Electra::FVariantValue(Electra::FTimeValue().SetFromSeconds(LiveEdgeDistanceForNormalPresentation)));
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Setting distance to live edge for normal presentations to %.3f seconds", PlayerUniqueID, LiveEdgeDistanceForNormalPresentation);
	}
	bool bThrowErrorWhenRebuffering = InOptions->GetMediaOption(ElectraMediaOptions::ElectraThrowErrorWhenRebuffering, (bool)false);
	if (bThrowErrorWhenRebuffering)
	{
		PlayerOptions.Set(Electra::OptionThrowErrorWhenRebuffering, Electra::FVariantValue(bThrowErrorWhenRebuffering));
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: Throw playback error when rebuffering", PlayerUniqueID);
	}
	FString CDNHTTPStatusDenyStream = InOptions->GetMediaOption(ElectraMediaOptions::ElectraGetDenyStreamCode, FString());
	if (CDNHTTPStatusDenyStream.Len())
	{
		int32 HTTPStatus = -1;
		LexFromString(HTTPStatus, *CDNHTTPStatusDenyStream);
		if (HTTPStatus > 0 && HTTPStatus < 1000)
		{
			PlayerOptions.Set(Electra::ABR::OptionKeyABR_CDNSegmentDenyHTTPStatus, Electra::FVariantValue((int64)HTTPStatus));
			UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaPlayer::Open: CDN HTTP status %d will deny a stream permanently", PlayerUniqueID, HTTPStatus);
		}
	}

	// Check if there is an environment specified in which this player is used.
	// Certain optimization settings apply for dedicated environments.
	if (Environment == MediaPlayerOptionValues::Environment_Preview() || Environment == MediaPlayerOptionValues::Environment_Sequencer())
	{
		PlayerOptions.Set(Electra::OptionKeyWorkerThreads, Electra::FVariantValue(FString(TEXT("worker"))));
	}

	// Check for options that can be changed during playback and apply them at startup already.
	// If a media source supports the MaxResolutionForMediaStreaming option then we can override the max resolution.
	int64 DefaultValue = 0;
	int64 MaxVerticalStreamResolution = InOptions->GetMediaOption(ElectraMediaOptions::MaxResolutionForMediaStreaming, DefaultValue);
	if (MaxVerticalStreamResolution != 0)
	{
		UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaPlayer::Open: Limiting max resolution to %d", PlayerUniqueID, (int32)MaxVerticalStreamResolution);
		PlaystartOptions.MaxVerticalStreamResolution = (int32)MaxVerticalStreamResolution;
	}

	int64 MaxBandwidthForStreaming = InOptions->GetMediaOption(ElectraMediaOptions::ElectraMaxStreamingBandwidth, (int64)0);
	if (MaxBandwidthForStreaming > 0)
	{
		UE_LOGF(LogElectraPlayer, Log, "[%u] Limiting max streaming bandwidth to %d bps", PlayerUniqueID, (int32)MaxBandwidthForStreaming);
		PlaystartOptions.MaxBandwidthForStreaming = (int32)MaxBandwidthForStreaming;
	}

	AnalyticsInstanceEventCount = 0;
	QueuedAnalyticEvents.Empty();
	NumQueuedAnalyticEvents = 0;
	// Create a guid string for the analytics. We do this here and not in the constructor in case the same instance is used over again.
	AnalyticsInstanceGuid = SessionID.ToString(EGuidFormats::Digits);
	UpdateAnalyticsCustomValues();
	// Start statistics with a clean slate.
	Statistics.Reset();

	// Get a writable copy of the URL so we can sanitize it if necessary.
	MediaUrl = InUrl.TrimStartAndEnd();
	bHasPendingError = false;
	bHasClosedDueToError = false;


	// Create a new empty player structure. This contains the actual player instance, its associated renderers and sample queues.
	TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> NewPlayer = MakeShared<FInternalPlayerImpl, ESPMode::ThreadSafe>();

	// Create the internal player and register ourselves as metrics receiver and static resource provider.
	IAdaptiveStreamingPlayer::FCreateParam CreateParams;
	if (!bRetainPoolsForReuse)
	{
		CreateParams.OutputTexturePool = MakeShareable(new FElectraTextureSamplePool);
		CreateParams.OutputAudioSamplePool = MakeShareable(new FElectraAudioSamplePool);
	#if PLATFORM_ANDROID
		OutputTexturePoolDecoderID = FElectraDecodersPlatformResources::RegisterOutputTexturePool(CreateParams.OutputTexturePool);
	#endif
	}
	else
	{
		CreateParams.OutputTexturePool = OutputTexturePool;
		CreateParams.OutputAudioSamplePool = OutputAudioSamplePool;
	}
	PlayerOptions.Set(Electra::OptionKeyOutputTexturePoolID, FVariantValue(OutputTexturePoolDecoderID));
	CreateParams.ExternalPlayerGUID = PlayerGuid;
	FString WorkerThreadOption = PlayerOptions.GetValue(Electra::OptionKeyWorkerThreads).SafeGetFString(TEXT("shared"));
	CreateParams.WorkerThreads = WorkerThreadOption.Equals(TEXT("worker"), ESearchCase::IgnoreCase) ? IAdaptiveStreamingPlayer::FCreateParam::EWorkerThreads::DedicatedWorker :
									WorkerThreadOption.Equals(TEXT("worker_and_events"), ESearchCase::IgnoreCase) ? IAdaptiveStreamingPlayer::FCreateParam::EWorkerThreads::DedicatedWorkerAndEventDispatch :
									IAdaptiveStreamingPlayer::FCreateParam::EWorkerThreads::Shared;
	NewPlayer->AdaptivePlayer = IAdaptiveStreamingPlayer::Create(CreateParams);
	NewPlayer->AdaptivePlayer->AddMetricsReceiver(this);
	NewPlayer->AdaptivePlayer->SetStaticResourceProviderCallback(StaticResourceProvider);

	// Create a new media player event receiver and register it to receive all non player internal events as soon as they are received.
	MediaPlayerEventReceiver = MakeSharedTS<FAEMSEventReceiver>();
	MediaPlayerEventReceiver->GetEventReceivedDelegate().BindRaw(this, &FElectraPlayer::OnMediaPlayerEventReceived);
	NewPlayer->AdaptivePlayer->AddAEMSReceiver(MediaPlayerEventReceiver, TEXT("*"), TEXT(""), IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnReceive);
	NewPlayer->AdaptivePlayer->Initialize(PlayerOptions);

	// Check for options that can be changed during playback and apply them at startup already.
	// If a media source supports the MaxResolutionForMediaStreaming option then we can override the max resolution.
	if (PlaystartOptions.MaxVerticalStreamResolution.IsSet())
	{
		NewPlayer->AdaptivePlayer->SetMaxResolution(0, PlaystartOptions.MaxVerticalStreamResolution.GetValue());
	}
	if (PlaystartOptions.MaxBandwidthForStreaming.IsSet())
	{
		NewPlayer->AdaptivePlayer->SetBitrateCeiling(PlaystartOptions.MaxBandwidthForStreaming.GetValue());
	}

	// Apply options that may have been set prior to calling Open().
	// Set these only if they have defined values as to not override what might have been set in the PlayerOptions.
	if (bFrameAccurateSeeking.IsSet())
	{
		NewPlayer->AdaptivePlayer->EnableFrameAccurateSeeking(bFrameAccurateSeeking.GetValue());
	}

    // Set the player member variable to the new player so we can use our internal configuration methods on the new player.
    CurrentPlayer = MoveTemp(NewPlayer);
	if (bEnableLooping.IsSet())
	{
		SetLooping(bEnableLooping.GetValue());
	}
	if (!CurrentPlaybackRange.IsEmpty())
	{
		SetPlaybackTimeRange(CurrentPlaybackRange);
	}
	// Issue load of the playlist.
	UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaPlayer::Open(%ls)", PlayerUniqueID, *SanitizeMessage(MediaUrl));
	CurrentPlayer->AdaptivePlayer->LoadManifest(MediaUrl);

	return true;
}


//-----------------------------------------------------------------------------
/**
 *	Close / Shutdown player
 */
void FElectraPlayer::CloseInternal()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	OptionInterface.Reset();

	TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return;
	}

	UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaPlayer::Close()", PlayerUniqueID);
	CSV_EVENT(ElectraPlayer, TEXT("Close"));


	// For all intents and purposes the player can be considered closed here now already.
	PlayerState.State = EMediaState::Closed;
	MediaUrl.Empty();
	PlaystartOptions.TimeOffset.Reset();
	PlaystartOptions.InitialAudioTrackAttributes.Reset();
	CurrentPlaybackRange = TRange<FTimespan>::Empty();
	bFrameAccurateSeeking.Reset();
	bEnableLooping.Reset();

	// Detach ourselves from receiving any player events.
	if (Player->AdaptivePlayer.IsValid())
	{
		if (MediaPlayerEventReceiver.IsValid())
		{
			MediaPlayerEventReceiver->GetEventReceivedDelegate().Unbind();
			Player->AdaptivePlayer->RemoveAEMSReceiver(MediaPlayerEventReceiver, TEXT("*"), TEXT(""), IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnReceive);
			MediaPlayerEventReceiver.Reset();
		}
		Player->AdaptivePlayer->SetStaticResourceProviderCallback(nullptr);
		Player->AdaptivePlayer->RemoveMetricsReceiver(this);
	}

	// Clear any pending static resource requests now.
	StaticResourceProvider->ClearPendingRequests();

	HandlePlayerEventPlaybackStopped();
	LogStatistics();
	// Enqueue the last media events. They may get sent in TickInternal() as long as we are still getting ticked.
	DeferredMediaEvents.Enqueue(EMediaEvent::TracksChanged);
	DeferredMediaEvents.Enqueue(EMediaEvent::MediaClosed);

	// Clear out the player instance now.
	CurrentPlayer.Reset();

	// Kick off asynchronous closing now.
	FInternalPlayerImpl::DoCloseAsync(MoveTemp(Player), PlayerUniqueID, AsyncResourceReleaseNotification, bRetainPoolsForReuse==false, OutputTexturePoolDecoderID);
}

void FElectraPlayer::FInternalPlayerImpl::DoCloseAsync(TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe>&& InPlayer, uint32 InPlayerID, TSharedPtr<IAsyncResourceReleaseNotification, ESPMode::ThreadSafe> InAsyncResourceReleaseNotification, bool bReleasePools, int64 TexturePoolDecoderID)
{
	TFunction<void()> CloseTask = [InPlayer, InPlayerID, InAsyncResourceReleaseNotification, bReleasePools, TexturePoolDecoderID]()
	{
		double TimeCloseBegan = FPlatformTime::Seconds();
		InPlayer->AdaptivePlayer->Stop();
		InPlayer->AdaptivePlayer.Reset();

		if (bReleasePools)
		{
		#if PLATFORM_ANDROID
			FElectraDecodersPlatformResources::UnregisterOutputTexturePool(TexturePoolDecoderID);
		#endif
		}

		double TimeCloseEnded = FPlatformTime::Seconds();
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] DoCloseAsync() finished after %.3f msec!", InPlayerID, (TimeCloseEnded - TimeCloseBegan) * 1000.0);
		if (InAsyncResourceReleaseNotification.IsValid())
		{
			InAsyncResourceReleaseNotification->Signal(IMediaPlayerLifecycleManagerDelegate::ResourceFlags_Decoder);
		}
	};
	if (GIsRunning)
	{
		FMediaRunnable::EnqueueAsyncTask(MoveTemp(CloseTask));
	}
	else
	{
		CloseTask();
	}
}





float FElectraPlayer::FPlayerState::GetRate() const
{
	return IntendedPlayRate.IsSet() ? IntendedPlayRate.GetValue() : CurrentPlayRate;
}

EMediaState FElectraPlayer::FPlayerState::GetState() const
{
	if (IntendedPlayRate.IsSet() && (State == EMediaState::Playing || State == EMediaState::Paused || State == EMediaState::Stopped))
	{
		return IntendedPlayRate.GetValue() != 0.0f ? EMediaState::Playing : EMediaState::Paused;
	}
	return State;
}

EMediaStatus FElectraPlayer::FPlayerState::GetStatus() const
{
	return Status;
}

void FElectraPlayer::FPlayerState::SetIntendedPlayRate(float InIntendedRate)
{
	IntendedPlayRate = InIntendedRate;
}

void FElectraPlayer::FPlayerState::SetPlayRateFromPlayer(float InCurrentPlayerPlayRate)
{
	CurrentPlayRate = InCurrentPlayerPlayRate;
	// If reverse playback is selected even though it is not supported, leave it set as such.
	if (IntendedPlayRate.IsSet() && IntendedPlayRate.GetValue() >= 0.0f)
	{
		IntendedPlayRate.Reset();
	}
}





void FElectraPlayer::TickInternal(FTimespan DeltaTime, FTimespan Timecode)
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_ElectraPlayer_TickInput);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, TickInput);
	// Handle the internal player, if we have one.
	TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> Player = CurrentPlayer;
	if (Player.IsValid() && PlayerState.State != EMediaState::Error)
	{
		{
		// Handle static resource fetch requests.
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_ElectraPlayer_StaticResourceRequest);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, StaticResourceRequest);
		StaticResourceProvider->ProcessPendingStaticResourceRequests();
		}

		{
		// Check for option changes
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_ElectraPlayer_QueryOptions);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, QueryOptions);
		FVariantValue VerticalResoLimit = ElectraMediaOptions::GetOptionValue(OptionInterface, ElectraMediaOptions::EOptionType::MaxVerticalStreamResolution, FVariantValue());
		if (VerticalResoLimit.IsValid())
		{
			int64 NewVerticalStreamResolution = VerticalResoLimit.GetInt64();
			if (NewVerticalStreamResolution != PlaystartOptions.MaxVerticalStreamResolution.Get(0))
			{
				PlaystartOptions.MaxVerticalStreamResolution = NewVerticalStreamResolution;
				UE_LOGF(LogElectraPlayer, Log, "[%u] Limiting max vertical resolution to %d", PlayerUniqueID, (int32)NewVerticalStreamResolution);
				Player->AdaptivePlayer->SetMaxResolution(0, (int32)NewVerticalStreamResolution);
			}
		}

		FVariantValue BandwidthLimit = ElectraMediaOptions::GetOptionValue(OptionInterface, ElectraMediaOptions::EOptionType::MaxBandwidthForStreaming, FVariantValue());
		if (BandwidthLimit.IsValid())
		{
			int64 NewBandwidthForStreaming = BandwidthLimit.GetInt64();
			if (NewBandwidthForStreaming != PlaystartOptions.MaxBandwidthForStreaming.Get(0))
			{
				PlaystartOptions.MaxBandwidthForStreaming = NewBandwidthForStreaming;
				UE_LOGF(LogElectraPlayer, Log, "[%u] Limiting max streaming bandwidth to %d bps", PlayerUniqueID, (int32)NewBandwidthForStreaming);
				Player->AdaptivePlayer->SetBitrateCeiling((int32)NewBandwidthForStreaming);
			}
		}
		}

		{
		// Process accumulated player events.
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_ElectraPlayer_PlayerEvents);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, PlayerEvents);
		HandleDeferredPlayerEvents();
		if (bHasPendingError)
		{
			bHasPendingError = false;
			if (PlayerState.State == EMediaState::Preparing)
			{
				DeferredMediaEvents.Enqueue(EMediaEvent::MediaOpenFailed);
			}
			else if (PlayerState.State == EMediaState::Playing)
			{
				DeferredMediaEvents.Enqueue(EMediaEvent::MediaClosed);
			}

			bHasClosedDueToError = true;
			CloseInternal();
			PlayerState.State = EMediaState::Error;
		}
		}
	}
	else
	{
		DeferredPlayerEvents.Empty();
	}

	// Forward enqueued media events. We do this even with no current internal player to ensure all pending events are sent and none are lost.
	EMediaEvent Event;
	while(DeferredMediaEvents.Dequeue(Event))
	{
		SendMediaSinkEvent(Event);
	}
}

bool FElectraPlayer::IsLive()
{
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		Electra::FTimeValue Dur = Player->AdaptivePlayer->GetDuration();
		if (Dur.IsValid())
		{
			return Dur.IsInfinity();
		}
	}
	// Default assumption is Live playback.
	return true;
}

void FElectraPlayer::TriggerFirstSeekIfNecessary()
{
	if (!bInitialSeekPerformed)
	{
		bInitialSeekPerformed = true;

		// Set up the initial playback position
		IAdaptiveStreamingPlayer::FSeekParam playParam;

		// First we look at any potential time offset specified in the playstart options.
		if (PlaystartOptions.TimeOffset.IsSet())
		{
			FTimespan Target;
			CalculateTargetSeekTime(Target, PlaystartOptions.TimeOffset.GetValue());
			playParam.Time.SetFromHNS(Target.GetTicks());
		}
		else
		{
			// Do not set a start time, let the player pick one.
			//playParam.Time.SetToZero();
		}

		// Check with the media options if it wants to start somewhere else.
		TSharedPtr<TArray<FTimespan>, ESPMode::ThreadSafe> SeekablePositions = MakeShared<TArray<FTimespan>, ESPMode::ThreadSafe>();
		FVariantValue PlaystartPos = ElectraMediaOptions::GetOptionValue(OptionInterface, ElectraMediaOptions::EOptionType::PlaystartPosFromSeekPositions, FVariantValue(SeekablePositions));
		if (PlaystartPos.IsValid())
		{
			check(PlaystartPos.IsType(FVariantValue::EDataType::TypeInt64));
			playParam.Time.SetFromHNS(PlaystartPos.GetInt64());
		}

		// Trigger buffering at the intended start time.
		auto Player = CurrentPlayer;
		if (Player.IsValid())
		{
			// If the media is flagged as a replay event then the start time is also implicitly provided and we must not set one.
			IAdaptiveStreamingPlayer::FEventReplayState evtrs;
			Player->AdaptivePlayer->GetReplayEventState(evtrs);
			if (evtrs.bIsReplayEvent)
			{
				playParam.Time.SetToInvalid();
			}
			Player->AdaptivePlayer->SeekTo(playParam);
		}
	}
}

void FElectraPlayer::CalculateTargetSeekTime(FTimespan& OutTargetTime, const FTimespan& InTime)
{
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		Electra::FTimeValue newTime;
		Electra::FTimeRange playRange;
		newTime.SetFromHNS(InTime.GetTicks());
		Player->AdaptivePlayer->GetSeekableRange(playRange);

		// Seek semantics are different for VoD and Live.
		// For VoD we assume the timeline to be from [0 .. duration) and not offset to what may have been an original airdate in UTC, and the seek time
		// needs to fall into that range.
		// For Live the timeline is assumed to be UTC wallclock time in [UTC-DVRwindow .. UTC) and the seek time is an offset BACKWARDS from the UTC Live edge
		// into content already aired.
		if (IsLive())
		{
			// If the target is maximum we treat it as going to the Live edge.
			if (InTime == FTimespan::MaxValue())
			{
				OutTargetTime = InTime;
				return;
			}
			// In case the seek time has been given as a negative number we negate it.
			if (newTime.GetAsHNS() < 0)
			{
				newTime = Electra::FTimeValue::GetZero() - newTime;
			}
			// We want to go that far back from the Live edge.
			newTime = playRange.End - newTime;
			// Need to clamp this to the beginning of the timeline.
			if (newTime < playRange.Start)
			{
				newTime = playRange.Start;
			}
		}
		else
		{
			// For VoD we clamp the time into the timeline only when it would fall off the beginning.
			// We purposely allow to seek outside the duration which will trigger an 'ended' event.
			// This is to make sure that a game event during which a VoD asset is played and synchronized
			// to the beginning of the event itself will not play the last n seconds for people who have
			// joined the event when it is already over.
			if (newTime < playRange.Start)
			{
				newTime = playRange.Start;
			}
			/*
			else if (newTime > playRange.End)
			{
				newTime = playRange.End;
			}
			*/
		}

		OutTargetTime = FTimespan(newTime.GetAsHNS());
	}
}



TSharedPtr<Electra::FTrackMetadata, ESPMode::ThreadSafe> FElectraPlayer::GetTrackStreamMetadata(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return nullptr;
	}
	TArray<Electra::FTrackMetadata> TrackMetaData;
	if (InTrackType == EMediaTrackType::Video)
	{
		Player->AdaptivePlayer->GetTrackMetadata(TrackMetaData, Electra::EStreamType::Video);
	}
	else if (InTrackType == EMediaTrackType::Audio)
	{
		Player->AdaptivePlayer->GetTrackMetadata(TrackMetaData, Electra::EStreamType::Audio);
	}
	else if (InTrackType == EMediaTrackType::Subtitle)
	{
		Player->AdaptivePlayer->GetTrackMetadata(TrackMetaData, Electra::EStreamType::Subtitle);
	}
	if (InTrackIndex >= 0 && InTrackIndex < TrackMetaData.Num())
	{
		return MakeShared<Electra::FTrackMetadata, ESPMode::ThreadSafe>(TrackMetaData[InTrackIndex]);
	}
	return nullptr;
}





void FElectraPlayer::HandleDeferredPlayerEvents()
{
	TSharedPtrTS<FPlayerMetricEventBase> Event;
	while(DeferredPlayerEvents.Dequeue(Event))
	{
		switch(Event->Type)
		{
			case FPlayerMetricEventBase::EType::OpenSource:
			{
				FPlayerMetricEvent_OpenSource* Ev = static_cast<FPlayerMetricEvent_OpenSource*>(Event.Get());
				HandlePlayerEventOpenSource(Ev->URL);
				break;
			}
			case FPlayerMetricEventBase::EType::ReceivedMainPlaylist:
			{
				FPlayerMetricEvent_ReceivedMainPlaylist* Ev = static_cast<FPlayerMetricEvent_ReceivedMainPlaylist*>(Event.Get());
				HandlePlayerEventReceivedMainPlaylist(Ev->EffectiveURL);
				break;
			}
			case FPlayerMetricEventBase::EType::ReceivedPlaylists:
			{
				HandlePlayerEventReceivedPlaylists();
				break;
			}
			case FPlayerMetricEventBase::EType::TracksChanged:
			{
				HandlePlayerEventTracksChanged();
				break;
			}
			case FPlayerMetricEventBase::EType::PlaylistDownload:
			{
				FPlayerMetricEvent_PlaylistDownload* Ev = static_cast<FPlayerMetricEvent_PlaylistDownload*>(Event.Get());
				HandlePlayerEventPlaylistDownload(Ev->PlaylistDownloadStats);
				break;
			}
			case FPlayerMetricEventBase::EType::BufferingStart:
			{
				FPlayerMetricEvent_BufferingStart* Ev = static_cast<FPlayerMetricEvent_BufferingStart*>(Event.Get());
				HandlePlayerEventBufferingStart(Ev->BufferingReason);
				break;
			}
			case FPlayerMetricEventBase::EType::BufferingEnd:
			{
				FPlayerMetricEvent_BufferingEnd* Ev = static_cast<FPlayerMetricEvent_BufferingEnd*>(Event.Get());
				HandlePlayerEventBufferingEnd(Ev->BufferingReason);
				break;
			}
			case FPlayerMetricEventBase::EType::Bandwidth:
			{
				FPlayerMetricEvent_Bandwidth* Ev = static_cast<FPlayerMetricEvent_Bandwidth*>(Event.Get());
				HandlePlayerEventBandwidth(Ev->EffectiveBps, Ev->ThroughputBps, Ev->LatencyInSeconds);
				break;
			}
			case FPlayerMetricEventBase::EType::BufferUtilization:
			{
				FPlayerMetricEvent_BufferUtilization* Ev = static_cast<FPlayerMetricEvent_BufferUtilization*>(Event.Get());
				HandlePlayerEventBufferUtilization(Ev->BufferStats);
				break;
			}
			case FPlayerMetricEventBase::EType::SegmentDownload:
			{
				FPlayerMetricEvent_SegmentDownload* Ev = static_cast<FPlayerMetricEvent_SegmentDownload*>(Event.Get());
				HandlePlayerEventSegmentDownload(Ev->SegmentDownloadStats);
				break;
			}
			case FPlayerMetricEventBase::EType::LicenseKey:
			{
				FPlayerMetricEvent_LicenseKey* Ev = static_cast<FPlayerMetricEvent_LicenseKey*>(Event.Get());
				HandlePlayerEventLicenseKey(Ev->LicenseKeyStats);
				break;
			}
			case FPlayerMetricEventBase::EType::DataAvailabilityChange:
			{
				FPlayerMetricEvent_DataAvailabilityChange* Ev = static_cast<FPlayerMetricEvent_DataAvailabilityChange*>(Event.Get());
				HandlePlayerEventDataAvailabilityChange(Ev->DataAvailability);
				break;
			}
			case FPlayerMetricEventBase::EType::VideoQualityChange:
			{
				FPlayerMetricEvent_VideoQualityChange* Ev = static_cast<FPlayerMetricEvent_VideoQualityChange*>(Event.Get());
				HandlePlayerEventVideoQualityChange(Ev->NewBitrate, Ev->PreviousBitrate, Ev->bIsDrasticDownswitch);
				break;
			}
			case FPlayerMetricEventBase::EType::AudioQualityChange:
			{
				FPlayerMetricEvent_AudioQualityChange* Ev = static_cast<FPlayerMetricEvent_AudioQualityChange*>(Event.Get());
				HandlePlayerEventAudioQualityChange(Ev->NewBitrate, Ev->PreviousBitrate, Ev->bIsDrasticDownswitch);
				break;
			}
			case FPlayerMetricEventBase::EType::CodecFormatChange:
			{
				FPlayerMetricEvent_CodecFormatChange* Ev = static_cast<FPlayerMetricEvent_CodecFormatChange*>(Event.Get());
				HandlePlayerEventCodecFormatChange(Ev->NewDecodingFormat);
				break;
			}
			case FPlayerMetricEventBase::EType::PrerollStart:
			{
				HandlePlayerEventPrerollStart();
				break;
			}
			case FPlayerMetricEventBase::EType::PrerollEnd:
			{
				HandlePlayerEventPrerollEnd();
				break;
			}
			case FPlayerMetricEventBase::EType::PlaybackStart:
			{
				HandlePlayerEventPlaybackStart();
				break;
			}
			case FPlayerMetricEventBase::EType::PlaybackPaused:
			{
				HandlePlayerEventPlaybackPaused();
				break;
			}
			case FPlayerMetricEventBase::EType::PlaybackResumed:
			{
				HandlePlayerEventPlaybackResumed();
				break;
			}
			case FPlayerMetricEventBase::EType::PlaybackEnded:
			{
				HandlePlayerEventPlaybackEnded();
				break;
			}
			case FPlayerMetricEventBase::EType::JumpInPlayPosition:
			{
				FPlayerMetricEvent_JumpInPlayPosition* Ev = static_cast<FPlayerMetricEvent_JumpInPlayPosition*>(Event.Get());
				HandlePlayerEventJumpInPlayPosition(Ev->ToNewTime, Ev->FromTime, Ev->TimejumpReason);
				break;
			}
			case FPlayerMetricEventBase::EType::PlaybackStopped:
			{
				HandlePlayerEventPlaybackStopped();
				break;
			}
			case FPlayerMetricEventBase::EType::SeekCompleted:
			{
				HandlePlayerEventSeekCompleted();
				break;
			}
			case FPlayerMetricEventBase::EType::MediaMetadataChanged:
			{
				FPlayerMetricEvent_MediaMetadataChange* Ev = static_cast<FPlayerMetricEvent_MediaMetadataChange*>(Event.Get());
				HandlePlayerMediaMetadataChanged(Ev->NewMetadata);
				break;
			}
			case FPlayerMetricEventBase::EType::Error:
			{
				FPlayerMetricEvent_Error* Ev = static_cast<FPlayerMetricEvent_Error*>(Event.Get());
				HandlePlayerEventError(Ev->ErrorReason);
				break;
			}
			case FPlayerMetricEventBase::EType::LogMessage:
			{
				FPlayerMetricEvent_LogMessage* Ev = static_cast<FPlayerMetricEvent_LogMessage*>(Event.Get());
				HandlePlayerEventLogMessage(Ev->LogLevel, Ev->LogMessage, Ev->PlayerWallclockMilliseconds);
				break;
			}
			case FPlayerMetricEventBase::EType::DroppedVideoFrame:
			{
				FPlayerMetricEvent_DroppedFrame* Ev = static_cast<FPlayerMetricEvent_DroppedFrame*>(Event.Get());
				HandlePlayerEventDroppedVideoFrame(Ev->FrameStats);
				break;
			}
			case FPlayerMetricEventBase::EType::DroppedAudioFrame:
			{
				FPlayerMetricEvent_DroppedFrame* Ev = static_cast<FPlayerMetricEvent_DroppedFrame*>(Event.Get());
				HandlePlayerEventDroppedAudioFrame(Ev->FrameStats);
				break;
			}
			default:
			{
				break;
			}
		}
	}
}

void FElectraPlayer::HandlePlayerEventOpenSource(const FString& URL)
{
	PlayerState.Status = PlayerState.Status | EMediaStatus::Connecting;

	PlayerState.State = EMediaState::Preparing;
	DeferredMediaEvents.Enqueue(EMediaEvent::MediaConnecting);

	UE_LOGF(LogElectraPlayer, Verbose, "[%u] Opening stream at \"%ls\"", PlayerUniqueID, *SanitizeMessage(URL));

	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	Statistics.AddMessageToHistory(TEXT("Opening stream"));
	Statistics.InitialURL = URL;
	Statistics.TimeAtOpen = FPlatformTime::Seconds();
	Statistics.LastState  = "Opening";

	// Enqueue an "OpenSource" event.
	static const FString kEventNameElectraOpenSource(TEXT("Electra.OpenSource"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraOpenSource))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraOpenSource);
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("URL"), *URL));
		EnqueueAnalyticsEvent(AnalyticEvent);
	}
}

void FElectraPlayer::HandlePlayerEventReceivedMainPlaylist(const FString& EffectiveURL)
{
	UE_LOGF(LogElectraPlayer, Verbose, "[%u] Received main playlist from \"%ls\"", PlayerUniqueID, *SanitizeMessage(EffectiveURL));

	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	Statistics.AddMessageToHistory(TEXT("Got main playlist"));
	// Note the time it took to get the main playlist
	Statistics.TimeToLoadMainPlaylist = FPlatformTime::Seconds() - Statistics.TimeAtOpen;
	Statistics.LastState = "Preparing";

	// Enqueue a "MainPlaylist" event.
	static const FString kEventNameElectraMainPlaylist(TEXT("Electra.MainPlaylist"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraMainPlaylist))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraMainPlaylist);
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("URL"), *EffectiveURL));
		EnqueueAnalyticsEvent(AnalyticEvent);
	}
}

void FElectraPlayer::HandlePlayerEventReceivedPlaylists()
{
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return;
	}

	UE_LOGF(LogElectraPlayer, Verbose, "[%u] Received initial stream playlists", PlayerUniqueID);

	PlayerState.Status = PlayerState.Status & ~EMediaStatus::Connecting;

	MediaStateOnPreparingFinished();

	Electra::FTimeRange MediaTimeline;
	Electra::FTimeValue MediaDuration;
	Player->AdaptivePlayer->GetTimelineRange(MediaTimeline);
	MediaDuration = Player->AdaptivePlayer->GetDuration();

	// Update statistics
	StatisticsLock.Lock();
	Statistics.AddMessageToHistory(TEXT("Got initial playlists"));
	// Note the time it took to get the stream playlist
	Statistics.TimeToLoadStreamPlaylists = FPlatformTime::Seconds() - Statistics.TimeAtOpen;
	Statistics.LastState = "Idle";
	// Establish the timeline and duration.
	Statistics.MediaTimelineAtStart = MediaTimeline;
	Statistics.MediaTimelineAtEnd = MediaTimeline;
	Statistics.MediaDuration = MediaDuration.IsInfinity() ? -1.0 : MediaDuration.GetAsSeconds();
	Statistics.VideoQualityPercentages.Empty();
	Statistics.AudioQualityPercentages.Empty();
	Statistics.VideoSegmentBitratesStreamed.Empty();
	Statistics.AudioSegmentBitratesStreamed.Empty();
	Statistics.NumVideoSegmentsStreamed = 0;
	Statistics.NumAudioSegmentsStreamed = 0;
	StatisticsLock.Unlock();
	// Get the video bitrates and populate our number of segments per bitrate map.
	TArray<Electra::FTrackMetadata> VideoStreamMetaData;
	Player->AdaptivePlayer->GetTrackMetadata(VideoStreamMetaData, Electra::EStreamType::Video);
	NumTracksVideo = VideoStreamMetaData.Num();
	if (NumTracksVideo)
	{
		for(int32 i=0; i<VideoStreamMetaData[0].StreamDetails.Num(); ++i)
		{
			StatisticsLock.Lock();
			Statistics.VideoSegmentBitratesStreamed.Add(VideoStreamMetaData[0].StreamDetails[i].Bandwidth, 0);
			Statistics.VideoQualityPercentages.Add(VideoStreamMetaData[0].StreamDetails[i].Bandwidth, 0);
			StatisticsLock.Unlock();
			UE_LOGF(LogElectraPlayer, Verbose, "[%u] Found %d * %d video stream at bitrate %d", PlayerUniqueID,
												VideoStreamMetaData[0].StreamDetails[i].CodecInformation.GetResolution().Width,
												VideoStreamMetaData[0].StreamDetails[i].CodecInformation.GetResolution().Height,
												VideoStreamMetaData[0].StreamDetails[i].Bandwidth);
		}
	}
	SelectedVideoTrackIndex = NumTracksVideo ? 0 : -1;

	// Get the audio bitrates and populate our number of segments per bitrate map.
	TArray<Electra::FTrackMetadata> AudioStreamMetaData;
	Player->AdaptivePlayer->GetTrackMetadata(AudioStreamMetaData, Electra::EStreamType::Audio);
	NumTracksAudio = AudioStreamMetaData.Num();
	if (NumTracksAudio)
	{
		for(int32 i=0; i<AudioStreamMetaData[0].StreamDetails.Num(); ++i)
		{
			StatisticsLock.Lock();
			Statistics.AudioSegmentBitratesStreamed.Add(AudioStreamMetaData[0].StreamDetails[i].Bandwidth, 0);
			Statistics.AudioQualityPercentages.Add(AudioStreamMetaData[0].StreamDetails[i].Bandwidth, 0);
			StatisticsLock.Unlock();
			UE_LOGF(LogElectraPlayer, Verbose, "[%u] Found audio stream at bitrate %d", PlayerUniqueID,
												AudioStreamMetaData[0].StreamDetails[i].Bandwidth);
		}
	}
	SelectedAudioTrackIndex = NumTracksAudio ? 0 : -1;

	TArray<Electra::FTrackMetadata> SubtitleStreamMetaData;
	Player->AdaptivePlayer->GetTrackMetadata(SubtitleStreamMetaData, Electra::EStreamType::Subtitle);
	NumTracksSubtitle = SubtitleStreamMetaData.Num();

	// Set the initial video track selection attributes.
	Electra::FStreamSelectionAttributes InitialVideoAttributes;
	InitialVideoAttributes.Kind = PlaystartOptions.InitialVideoTrackAttributes.Kind;
	InitialVideoAttributes.Language_RFC4647 = PlaystartOptions.InitialVideoTrackAttributes.Language_RFC4647;
	InitialVideoAttributes.OverrideIndex = PlaystartOptions.InitialVideoTrackAttributes.OverrideIndex;
	Player->AdaptivePlayer->SetInitialStreamAttributes(Electra::EStreamType::Video, InitialVideoAttributes);

	// Set the initial audio track selection attributes.
	Electra::FStreamSelectionAttributes InitialAudioAttributes;
	InitialAudioAttributes.Kind = PlaystartOptions.InitialAudioTrackAttributes.Kind;
	InitialAudioAttributes.Language_RFC4647 = PlaystartOptions.InitialAudioTrackAttributes.Language_RFC4647;
	InitialAudioAttributes.OverrideIndex = PlaystartOptions.InitialAudioTrackAttributes.OverrideIndex;
	Player->AdaptivePlayer->SetInitialStreamAttributes(Electra::EStreamType::Audio, InitialAudioAttributes);

	// Set the initial subtitle track selection attributes.
	Electra::FStreamSelectionAttributes InitialSubtitleAttributes;
	InitialSubtitleAttributes.Kind = PlaystartOptions.InitialSubtitleTrackAttributes.Kind;
	InitialSubtitleAttributes.Language_RFC4647 = PlaystartOptions.InitialSubtitleTrackAttributes.Language_RFC4647;
	InitialSubtitleAttributes.OverrideIndex = PlaystartOptions.InitialSubtitleTrackAttributes.OverrideIndex;
	Player->AdaptivePlayer->SetInitialStreamAttributes(Electra::EStreamType::Subtitle, InitialSubtitleAttributes);

	// Enqueue a "PlaylistsLoaded" event.
	static const FString kEventNameElectraPlaylistLoaded(TEXT("Electra.PlaylistsLoaded"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraPlaylistLoaded))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraPlaylistLoaded);
		EnqueueAnalyticsEvent(AnalyticEvent);
	}

	// Trigger preloading unless forbidden.
	if (PlaystartOptions.bDoNotPreload == false)
	{
		TriggerFirstSeekIfNecessary();
	}
}

void FElectraPlayer::HandlePlayerEventTracksChanged()
{
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		TArray<Electra::FTrackMetadata> VideoStreamMetaData;
		Player->AdaptivePlayer->GetTrackMetadata(VideoStreamMetaData, Electra::EStreamType::Video);
		NumTracksVideo = VideoStreamMetaData.Num();
		if (NumTracksVideo)
		{
			for(int32 i=0; i<VideoStreamMetaData[0].StreamDetails.Num(); ++i)
			{
				UE_LOGF(LogElectraPlayer, Verbose, "[%u] Found %d * %d video stream at bitrate %d", PlayerUniqueID,
													VideoStreamMetaData[0].StreamDetails[i].CodecInformation.GetResolution().Width,
													VideoStreamMetaData[0].StreamDetails[i].CodecInformation.GetResolution().Height,
													VideoStreamMetaData[0].StreamDetails[i].Bandwidth);
			}
		}

		TArray<Electra::FTrackMetadata> AudioStreamMetaData;
		Player->AdaptivePlayer->GetTrackMetadata(AudioStreamMetaData, Electra::EStreamType::Audio);
		NumTracksAudio = AudioStreamMetaData.Num();
		if (NumTracksAudio)
		{
			for(int32 i=0; i<AudioStreamMetaData[0].StreamDetails.Num(); ++i)
			{
				UE_LOGF(LogElectraPlayer, Verbose, "[%u] Found audio stream at bitrate %d", PlayerUniqueID,
													AudioStreamMetaData[0].StreamDetails[i].Bandwidth);
			}
		}

		TArray<Electra::FTrackMetadata> SubtitleStreamMetaData;
		Player->AdaptivePlayer->GetTrackMetadata(SubtitleStreamMetaData, Electra::EStreamType::Subtitle);
		NumTracksSubtitle = SubtitleStreamMetaData.Num();

		bVideoTrackIndexDirty = true;
		bAudioTrackIndexDirty = true;
		bSubtitleTrackIndexDirty = true;

		DeferredMediaEvents.Enqueue(EMediaEvent::TracksChanged);
	}
}

void FElectraPlayer::HandlePlayerEventPlaylistDownload(const Electra::Metrics::FPlaylistDownloadStats& PlaylistDownloadStats)
{
	// To reduce the number of playlist events during a Live presentation we will only report the initial playlist load
	// and later on only failed loads but not successful ones.
	bool bReport = PlaylistDownloadStats.LoadType == Electra::Playlist::ELoadType::Initial || !PlaylistDownloadStats.bWasSuccessful;
	if (bReport)
	{
		static const FString kEventNameElectraPlaylistDownload(TEXT("Electra.PlaylistDownload"));
		if (Electra::IsAnalyticsEventEnabled(kEventNameElectraPlaylistDownload))
		{
			// Enqueue a "PlaylistDownload" event.
			TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraPlaylistDownload);
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("URL"), *PlaylistDownloadStats.Url.URL));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("Failure"), *PlaylistDownloadStats.FailureReason));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("ListType"), Electra::Playlist::GetPlaylistTypeString(PlaylistDownloadStats.ListType)));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("LoadType"), Electra::Playlist::GetPlaylistLoadTypeString(PlaylistDownloadStats.LoadType)));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("HTTPStatus"), PlaylistDownloadStats.HTTPStatusCode));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("Retry"), PlaylistDownloadStats.RetryNumber));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("bSuccess"), PlaylistDownloadStats.bWasSuccessful));
			EnqueueAnalyticsEvent(AnalyticEvent);
		}
	}
	// If unsuccessful keep track of the type of error.
	if (!PlaylistDownloadStats.bWasSuccessful && !PlaylistDownloadStats.bWasAborted)
	{
		FScopeLock Lock(&StatisticsLock);
		if (PlaylistDownloadStats.HTTPStatusCode == 404)
		{
			++Statistics.NumErr404;
		}
		else if (PlaylistDownloadStats.HTTPStatusCode >= 400 && PlaylistDownloadStats.HTTPStatusCode < 500)
		{
			++Statistics.NumErr4xx;
		}
		else if (PlaylistDownloadStats.HTTPStatusCode >= 500 && PlaylistDownloadStats.HTTPStatusCode < 600)
		{
			++Statistics.NumErr5xx;
		}
		else if (PlaylistDownloadStats.bDidTimeout)
		{
			++Statistics.NumErrTimeouts;
		}
		else
		{
			++Statistics.NumErrConnDrops;
		}
	}
}

void FElectraPlayer::HandlePlayerEventLicenseKey(const Electra::Metrics::FLicenseKeyStats& LicenseKeyStats)
{
	// TBD
	if (LicenseKeyStats.bWasSuccessful)
	{
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] License key obtained", PlayerUniqueID);
		FScopeLock Lock(&StatisticsLock);
		Statistics.AddMessageToHistory(TEXT("Obtained license key"));
	}
	else
	{
		UE_LOGF(LogElectraPlayer, Log, "[%u] License key error \"%ls\"", PlayerUniqueID, *LicenseKeyStats.FailureReason);
		FScopeLock Lock(&StatisticsLock);
		Statistics.AddMessageToHistory(TEXT("License key error"));
	}
}

void FElectraPlayer::HandlePlayerEventDataAvailabilityChange(const Electra::Metrics::FDataAvailabilityChange& DataAvailability)
{
	// Pass this event up to the media player facade. We do not act on this here right now.
	if (DataAvailability.StreamType == Electra::EStreamType::Video)
	{
		if (DataAvailability.Availability == Electra::Metrics::FDataAvailabilityChange::EAvailability::DataAvailable)
		{
			DeferredMediaEvents.Enqueue(EMediaEvent::Internal_VideoSamplesAvailable);
		}
		else if (DataAvailability.Availability == Electra::Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable)
		{
			DeferredMediaEvents.Enqueue(EMediaEvent::Internal_VideoSamplesUnavailable);
		}
	}
	else if (DataAvailability.StreamType == Electra::EStreamType::Audio)
	{
		if (DataAvailability.Availability == Electra::Metrics::FDataAvailabilityChange::EAvailability::DataAvailable)
		{
			DeferredMediaEvents.Enqueue(EMediaEvent::Internal_AudioSamplesAvailable);
		}
		else if (DataAvailability.Availability == Electra::Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable)
		{
			DeferredMediaEvents.Enqueue(EMediaEvent::Internal_AudioSamplesUnavailable);
		}
	}
}

void FElectraPlayer::HandlePlayerEventBufferingStart(Electra::Metrics::EBufferingReason BufferingReason)
{
	PlayerState.Status = PlayerState.Status | EMediaStatus::Buffering;

	// In case a seek was performed right away the reason would be `Seeking`, but we want to
	// track it as `Initial` for statistics reasons and to make sure we won't miss sending `TracksChanged`.
	if (bIsFirstBuffering)
	{
		BufferingReason = Electra::Metrics::EBufferingReason::Initial;
	}

	// Send TracksChanged on the initial buffering event. Prior to that we do not know where in the stream
	// playback will begin and what tracks are available there.
	if (BufferingReason == Electra::Metrics::EBufferingReason::Initial)
	{
		// Mark the track indices as dirty in order to get the current active ones again.
		// This is necessary since the player may have made a different selection given the
		// initial track preferences we gave it.
		bVideoTrackIndexDirty = true;
		bAudioTrackIndexDirty = true;
		bSubtitleTrackIndexDirty = true;
		DeferredMediaEvents.Enqueue(EMediaEvent::TracksChanged);
	}

	DeferredMediaEvents.Enqueue(EMediaEvent::MediaBuffering);

	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	Statistics.TimeAtBufferingBegin = FPlatformTime::Seconds();
	switch(BufferingReason)
	{
		case Electra::Metrics::EBufferingReason::Initial:
		{
			Statistics.bIsInitiallyDownloading = true;
			Statistics.LastState = "Buffering";
			break;
		}
		case Electra::Metrics::EBufferingReason::Seeking:
		{
			Statistics.LastState = "Seeking";
			break;
		}
		case Electra::Metrics::EBufferingReason::Rebuffering:
		{
			++Statistics.NumTimesRebuffered;
			Statistics.LastState = "Rebuffering";
			break;
		}
	}
	// Enqueue a "BufferingStart" event.
	static const FString kEventNameElectraBufferingStart(TEXT("Electra.BufferingStart"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraBufferingStart))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraBufferingStart);
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("Type"), Electra::Metrics::GetBufferingReasonString(BufferingReason)));
		EnqueueAnalyticsEvent(AnalyticEvent);
	}

	FString Msg = FString::Printf(TEXT("%s buffering starts"), Electra::Metrics::GetBufferingReasonString(BufferingReason));
	Statistics.AddMessageToHistory(Msg);

	UE_LOGF(LogElectraPlayer, Verbose, "[%u] %ls", PlayerUniqueID, *Msg);
	CSV_EVENT(ElectraPlayer, TEXT("Buffering starts"));
}

void FElectraPlayer::HandlePlayerEventBufferingEnd(Electra::Metrics::EBufferingReason BufferingReason)
{
	// Note: While this event signals the end of buffering the player will now immediately transition into the pre-rolling
	//       state from which a playback start is not quite possible yet and would incur a slight delay until it is.
	//       To avoid this we keep the state as buffering until the pre-rolling phase has also completed.
	//PlayerState.Status = PlayerState.Status & ~EMediaStatus::Buffering;

	// In case a seek was performed right away the reason would be `Seeking`, but we want to track it as `Initial` for statistics.
	if (bIsFirstBuffering)
	{
		BufferingReason = Electra::Metrics::EBufferingReason::Initial;
		bIsFirstBuffering = false;
	}

	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	double BufferingDuration = FPlatformTime::Seconds() - Statistics.TimeAtBufferingBegin;
	switch(BufferingReason)
	{
		case Electra::Metrics::EBufferingReason::Initial:
		{
			Statistics.InitialBufferingDuration = BufferingDuration;
			break;
		}
		case Electra::Metrics::EBufferingReason::Seeking:
		{
			// End of seek buffering is not relevant here.
			break;
		}
		case Electra::Metrics::EBufferingReason::Rebuffering:
		{
			if (BufferingDuration > Statistics.LongestRebufferingDuration)
			{
				Statistics.LongestRebufferingDuration = BufferingDuration;
			}
			Statistics.TotalRebufferingDuration += BufferingDuration;
			break;
		}
	}

	// Enqueue a "BufferingEnd" event.
	static const FString kEventNameElectraBufferingEnd(TEXT("Electra.BufferingEnd"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraBufferingEnd))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraBufferingEnd);
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("Type"), Electra::Metrics::GetBufferingReasonString(BufferingReason)));
		EnqueueAnalyticsEvent(AnalyticEvent);
	}
	UE_LOGF(LogElectraPlayer, Verbose, "[%u] %ls buffering ended after %.3fs", PlayerUniqueID, Electra::Metrics::GetBufferingReasonString(BufferingReason), BufferingDuration);
	Statistics.AddMessageToHistory(TEXT("Buffering ended"));
	Statistics.LastState = "Ready";

	CSV_EVENT(ElectraPlayer, TEXT("Buffering ends"));
}

void FElectraPlayer::HandlePlayerEventBandwidth(int64 EffectiveBps, int64 ThroughputBps, double LatencyInSeconds)
{
//	FScopeLock Lock(&StatisticsLock);
	UE_LOGF(LogElectraPlayer, VeryVerbose, "[%u] Observed bandwidth of %lld Kbps; throughput = %lld Kbps; latency = %.3fs", PlayerUniqueID, EffectiveBps/1000, ThroughputBps/1000, LatencyInSeconds);
}

void FElectraPlayer::HandlePlayerEventBufferUtilization(const Electra::Metrics::FBufferStats& BufferStats)
{
//	FScopeLock Lock(&StatisticsLock);
}

void FElectraPlayer::HandlePlayerEventSegmentDownload(const Electra::Metrics::FSegmentDownloadStats& SegmentDownloadStats)
{
	// Cached responses are not actual network traffic, so we ignore them.
	if (SegmentDownloadStats.bIsCachedResponse)
	{
		return;
	}
	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	if (SegmentDownloadStats.StreamType == Electra::EStreamType::Video)
	{
		Statistics.NumVideoDatabytesStreamed += SegmentDownloadStats.NumBytesDownloaded;
		if (!Statistics.VideoSegmentBitratesStreamed.Contains(SegmentDownloadStats.Bitrate))
		{
			Statistics.VideoSegmentBitratesStreamed.Add(SegmentDownloadStats.Bitrate, 0);
		}
		++Statistics.VideoSegmentBitratesStreamed[SegmentDownloadStats.Bitrate];

		++Statistics.NumVideoSegmentsStreamed;
		if (!Statistics.VideoQualityPercentages.Contains(SegmentDownloadStats.Bitrate))
		{
			Statistics.VideoQualityPercentages.Add(SegmentDownloadStats.Bitrate, 0);
		}
		for(auto& It : Statistics.VideoQualityPercentages)
		{
			const uint32 NumAt = Statistics.VideoSegmentBitratesStreamed[It.Key];
			const int32 AsPercentage = FMath::RoundToInt(100.0 * (double)NumAt / (double)Statistics.NumVideoSegmentsStreamed);
			It.Value = AsPercentage;
		}

		if (Statistics.bIsInitiallyDownloading)
		{
			Statistics.InitialBufferingBandwidth.AddSample(8*SegmentDownloadStats.NumBytesDownloaded / (SegmentDownloadStats.TimeToDownload > 0.0 ? SegmentDownloadStats.TimeToDownload : 1.0), SegmentDownloadStats.TimeToFirstByte);
			if (Statistics.InitialBufferingDuration > 0.0)
			{
				Statistics.bIsInitiallyDownloading = false;
			}
		}
	}
	else if (SegmentDownloadStats.StreamType == Electra::EStreamType::Audio)
	{
		Statistics.NumAudioDatabytesStreamed += SegmentDownloadStats.NumBytesDownloaded;
		if (!Statistics.AudioSegmentBitratesStreamed.Contains(SegmentDownloadStats.Bitrate))
		{
			Statistics.AudioSegmentBitratesStreamed.Add(SegmentDownloadStats.Bitrate, 0);
		}
		++Statistics.AudioSegmentBitratesStreamed[SegmentDownloadStats.Bitrate];

		++Statistics.NumAudioSegmentsStreamed;
		if (!Statistics.AudioQualityPercentages.Contains(SegmentDownloadStats.Bitrate))
		{
			Statistics.AudioQualityPercentages.Add(SegmentDownloadStats.Bitrate, 0);
		}
		for(auto& It : Statistics.AudioQualityPercentages)
		{
			const uint32 NumAt = Statistics.AudioSegmentBitratesStreamed[It.Key];
			const int32 AsPercentage = FMath::RoundToInt(100.0 * (double)NumAt / (double)Statistics.NumAudioSegmentsStreamed);
			It.Value = AsPercentage;
		}

		if (Statistics.bIsInitiallyDownloading && NumTracksVideo == 0)	// Do this just for audio-only presentations.
		{
			Statistics.InitialBufferingBandwidth.AddSample(8*SegmentDownloadStats.NumBytesDownloaded / (SegmentDownloadStats.TimeToDownload > 0.0 ? SegmentDownloadStats.TimeToDownload : 1.0), SegmentDownloadStats.TimeToFirstByte);
			if (Statistics.InitialBufferingDuration > 0.0)
			{
				Statistics.bIsInitiallyDownloading = false;
			}
		}
	}
	if (SegmentDownloadStats.bWasSuccessful)
	{
		UE_LOGF(LogElectraPlayer, VeryVerbose, "[%u] Downloaded %ls segment at bitrate %d: Playback time = %.3fs, duration = %.3fs, download time = %.3fs, URL=%ls \"%ls\"", PlayerUniqueID, Electra::GetStreamTypeName(SegmentDownloadStats.StreamType), SegmentDownloadStats.Bitrate, SegmentDownloadStats.PresentationTime, SegmentDownloadStats.Duration, SegmentDownloadStats.TimeToDownload, *SegmentDownloadStats.Range, *SanitizeMessage(SegmentDownloadStats.URL.URL));
	}
	else if (SegmentDownloadStats.bWasAborted)
	{
		++Statistics.NumSegmentDownloadsAborted;
	}
	if (!SegmentDownloadStats.bWasSuccessful || SegmentDownloadStats.RetryNumber)
	{
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] %ls segment download issue (%ls): retry:%d, success:%d, aborted:%d, filler:%d", PlayerUniqueID, Electra::Metrics::GetSegmentTypeString(SegmentDownloadStats.SegmentType), *SegmentDownloadStats.FailureReason, SegmentDownloadStats.RetryNumber, SegmentDownloadStats.bWasSuccessful, SegmentDownloadStats.bWasAborted, SegmentDownloadStats.bInsertedFillerData);

		if (SegmentDownloadStats.FailureReason.Len())
		{
			FString Msg;
			if (!SegmentDownloadStats.bWasAborted)
			{
				Msg = FString::Printf(TEXT("%s segment download issue on representation %s, bitrate %d, retry %d: %s"), Electra::Metrics::GetSegmentTypeString(SegmentDownloadStats.SegmentType),
					*SegmentDownloadStats.RepresentationID, SegmentDownloadStats.Bitrate, SegmentDownloadStats.RetryNumber, *SegmentDownloadStats.FailureReason);
			}
			else
			{
				Msg = FString::Printf(TEXT("%s segment download issue on representation %s, bitrate %d, aborted: %s"), Electra::Metrics::GetSegmentTypeString(SegmentDownloadStats.SegmentType),
					*SegmentDownloadStats.RepresentationID, SegmentDownloadStats.Bitrate, *SegmentDownloadStats.FailureReason);
			}
			Statistics.AddMessageToHistory(Msg);
		}

		static const FString kEventNameElectraSegmentIssue(TEXT("Electra.SegmentIssue"));
		if (Electra::IsAnalyticsEventEnabled(kEventNameElectraSegmentIssue))
		{
			// Enqueue a "SegmentIssue" event.
			TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraSegmentIssue);
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("URL"), *SegmentDownloadStats.URL.URL));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("Failure"), *SegmentDownloadStats.FailureReason));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("SegmentType"), Electra::Metrics::GetSegmentTypeString(SegmentDownloadStats.SegmentType)));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("HTTPStatus"), SegmentDownloadStats.HTTPStatusCode));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("Retry"), SegmentDownloadStats.RetryNumber));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("bSuccess"), SegmentDownloadStats.bWasSuccessful));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("TimeToFirstByte"), SegmentDownloadStats.TimeToFirstByte));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("ByteSize"), SegmentDownloadStats.ByteSize));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumBytesDownloaded"), SegmentDownloadStats.NumBytesDownloaded));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("bWasAborted"), SegmentDownloadStats.bWasAborted));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("bDidTimeout"), SegmentDownloadStats.bDidTimeout));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("bParseFailure"), SegmentDownloadStats.bParseFailure));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("bInsertedFillerData"), SegmentDownloadStats.bInsertedFillerData));
			EnqueueAnalyticsEvent(AnalyticEvent);
		}
	}
	if (!SegmentDownloadStats.bWasSuccessful && !SegmentDownloadStats.bWasAborted)
	{
		if (SegmentDownloadStats.HTTPStatusCode == 404)
		{
			++Statistics.NumErr404;
		}
		else if (SegmentDownloadStats.HTTPStatusCode >= 400 && SegmentDownloadStats.HTTPStatusCode < 500)
		{
			++Statistics.NumErr4xx;
		}
		else if (SegmentDownloadStats.HTTPStatusCode >= 500 && SegmentDownloadStats.HTTPStatusCode < 600)
		{
			++Statistics.NumErr5xx;
		}
		else if (SegmentDownloadStats.bDidTimeout)
		{
			++Statistics.NumErrTimeouts;
		}
		else if (SegmentDownloadStats.bParseFailure)
		{
			++Statistics.NumErrOther;
		}
		else
		{
			++Statistics.NumErrConnDrops;
		}
	}
}

void FElectraPlayer::HandlePlayerEventVideoQualityChange(int32 NewBitrate, int32 PreviousBitrate, bool bIsDrasticDownswitch)
{
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return;
	}
	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	if (PreviousBitrate == 0)
	{
		Statistics.InitialVideoStreamBitrate = NewBitrate;
	}
	else
	{
		if (bIsDrasticDownswitch)
		{
			++Statistics.NumVideoQualityDrasticDownswitches;
		}
		if (NewBitrate > PreviousBitrate)
		{
			++Statistics.NumVideoQualityUpswitches;
		}
		else
		{
			++Statistics.NumVideoQualityDownswitches;
		}
	}
	if (bIsDrasticDownswitch)
	{
		UE_LOGF(LogElectraPlayer, Log, "[%u] Player switched video quality drastically down to %d bps from %d bps. %d upswitches, %d downswitches (%d drastic ones)", PlayerUniqueID, NewBitrate, PreviousBitrate, Statistics.NumVideoQualityUpswitches, Statistics.NumVideoQualityDownswitches, Statistics.NumVideoQualityDrasticDownswitches);
	}
	else
	{
		UE_LOGF(LogElectraPlayer, Log, "[%u] Player switched video quality to %d bps from %d bps. %d upswitches, %d downswitches (%d drastic ones)", PlayerUniqueID, NewBitrate, PreviousBitrate, Statistics.NumVideoQualityUpswitches, Statistics.NumVideoQualityDownswitches, Statistics.NumVideoQualityDrasticDownswitches);
	}

	int32 prvWidth  = Statistics.CurrentlyActiveResolutionWidth;
	int32 prvHeight = Statistics.CurrentlyActiveResolutionHeight;
	// Get the current playlist URL
	TArray<Electra::FTrackMetadata> VideoStreamMetaData;
	Player->AdaptivePlayer->GetTrackMetadata(VideoStreamMetaData, Electra::EStreamType::Video);
	if (VideoStreamMetaData.Num())
	{
		for(int32 i=0; i<VideoStreamMetaData[0].StreamDetails.Num(); ++i)
		{
			if (VideoStreamMetaData[0].StreamDetails[i].Bandwidth == NewBitrate)
			{
				SelectedQuality = i;
				Statistics.CurrentlyActivePlaylistURL = VideoStreamMetaData[0].ID;
				Statistics.CurrentlyActiveResolutionWidth = VideoStreamMetaData[0].StreamDetails[i].CodecInformation.GetResolution().Width;
				Statistics.CurrentlyActiveResolutionHeight = VideoStreamMetaData[0].StreamDetails[i].CodecInformation.GetResolution().Height;
				break;
			}
		}
	}

	// Enqueue a "VideoQualityChange" event.
	static const FString kEventNameElectraVideoQualityChange(TEXT("Electra.VideoQualityChange"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraVideoQualityChange))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraVideoQualityChange);
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("OldBitrate"), PreviousBitrate));
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("NewBitrate"), NewBitrate));
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("bIsDrasticDownswitch"), bIsDrasticDownswitch));
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("OldResolution"), *FString::Printf(TEXT("%d*%d"), prvWidth, prvHeight)));
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("NewResolution"), *FString::Printf(TEXT("%d*%d"), Statistics.CurrentlyActiveResolutionWidth, Statistics.CurrentlyActiveResolutionHeight)));
		EnqueueAnalyticsEvent(AnalyticEvent);
	}

	Statistics.AddMessageToHistory(FString::Printf(TEXT("Video bitrate change from %d to %d"), PreviousBitrate, NewBitrate));

	CSV_EVENT(ElectraPlayer, TEXT("VideoQualityChange %d -> %d"), PreviousBitrate, NewBitrate);
}

void FElectraPlayer::HandlePlayerEventAudioQualityChange(int32 NewBitrate, int32 PreviousBitrate, bool bIsDrasticDownswitch)
{
	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	if (PreviousBitrate == 0)
	{
		Statistics.InitialAudioStreamBitrate = NewBitrate;
	}
	else
	{
		if (bIsDrasticDownswitch)
		{
			++Statistics.NumAudioQualityDrasticDownswitches;
		}
		if (NewBitrate > PreviousBitrate)
		{
			++Statistics.NumAudioQualityUpswitches;
		}
		else
		{
			++Statistics.NumAudioQualityDownswitches;
		}
	}
	if (bIsDrasticDownswitch)
	{
		UE_LOGF(LogElectraPlayer, Log, "[%u] Player switched audio quality drastically down to %d bps from %d bps. %d upswitches, %d downswitches (%d drastic ones)", PlayerUniqueID, NewBitrate, PreviousBitrate, Statistics.NumAudioQualityUpswitches, Statistics.NumAudioQualityDownswitches, Statistics.NumAudioQualityDrasticDownswitches);
	}
	else
	{
		UE_LOGF(LogElectraPlayer, Log, "[%u] Player switched audio quality to %d bps from %d bps. %d upswitches, %d downswitches (%d drastic ones)", PlayerUniqueID, NewBitrate, PreviousBitrate, Statistics.NumAudioQualityUpswitches, Statistics.NumAudioQualityDownswitches, Statistics.NumAudioQualityDrasticDownswitches);
	}

	// Enqueue a "AudioQualityChange" event.
	static const FString kEventNameElectraAudioQualityChange(TEXT("Electra.AudioQualityChange"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraAudioQualityChange))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraAudioQualityChange);
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("OldBitrate"), PreviousBitrate));
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("NewBitrate"), NewBitrate));
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("bIsDrasticDownswitch"), bIsDrasticDownswitch));
		EnqueueAnalyticsEvent(AnalyticEvent);
	}

	Statistics.AddMessageToHistory(FString::Printf(TEXT("Audio bitrate change from %d to %d"), PreviousBitrate, NewBitrate));

	CSV_EVENT(ElectraPlayer, TEXT("AudioQualityChange %d -> %d"), PreviousBitrate, NewBitrate);
}

void FElectraPlayer::HandlePlayerEventCodecFormatChange(const Electra::FStreamCodecInformation& NewDecodingFormat)
{
	if (NewDecodingFormat.IsVideoCodec())
	{
		FScopeLock lock(&VideoFormatLock);
		CurrentlyActiveVideoStreamFormat.Bitrate = NewDecodingFormat.GetBitrate();
		CurrentlyActiveVideoStreamFormat.Resolution.X = NewDecodingFormat.GetResolution().Width;
		CurrentlyActiveVideoStreamFormat.Resolution.Y = NewDecodingFormat.GetResolution().Height;
		CurrentlyActiveVideoStreamFormat.FrameRate = NewDecodingFormat.GetFrameRate().IsValid() ? NewDecodingFormat.GetFrameRate().GetAsDouble() : 0.0;
	}
}

void FElectraPlayer::HandlePlayerEventPrerollStart()
{
	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	Statistics.TimeAtPrerollBegin = FPlatformTime::Seconds();
	UE_LOGF(LogElectraPlayer, Verbose, "[%u] Player starts prerolling to warm decoders and renderers", PlayerUniqueID);

	// Enqueue a "PrerollStart" event.
	static const FString kEventNameElectraPrerollStart(TEXT("Electra.PrerollStart"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraPrerollStart))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraPrerollStart);
		EnqueueAnalyticsEvent(AnalyticEvent);
	}
}

void FElectraPlayer::HandlePlayerEventPrerollEnd()
{
	// Note: See comments in ReportBufferingEnd()
	//       Preroll follows at the end of buffering and we keep the buffering state until preroll has finished as well.
	PlayerState.Status = PlayerState.Status & ~EMediaStatus::Buffering;

	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	if (Statistics.TimeForInitialPreroll < 0.0)
	{
		Statistics.TimeForInitialPreroll = FPlatformTime::Seconds() - Statistics.TimeAtPrerollBegin;
	}
	UE_LOGF(LogElectraPlayer, Verbose, "[%u] Player prerolling complete", PlayerUniqueID);
	Statistics.LastState = "Ready";

	DeferredMediaEvents.Enqueue(EMediaEvent::MediaBufferingComplete);

	// Enqueue a "PrerollEnd" event.
	static const FString kEventNameElectraPrerollEnd(TEXT("Electra.PrerollEnd"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraPrerollEnd))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraPrerollEnd);
		EnqueueAnalyticsEvent(AnalyticEvent);
	}
}

void FElectraPlayer::HandlePlayerEventPlaybackStart()
{
	PlayerState.Status = PlayerState.Status & ~EMediaStatus::Buffering;
	MediaStateOnPlay();
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		// Update statistics
		FScopeLock Lock(&StatisticsLock);
		double PlayPos = Player->AdaptivePlayer->GetPlayPosition().GetAsSeconds();
		if (Statistics.PlayPosAtStart < 0.0)
		{
			Statistics.PlayPosAtStart = PlayPos;
		}
		Statistics.LastState = "Playing";
		UE_LOGF(LogElectraPlayer, Log, "[%u] Playback started at play position %.3f", PlayerUniqueID, PlayPos);
		Statistics.AddMessageToHistory(TEXT("Playback started"));

		// Enqueue a "Start" event.
		static const FString kEventNameElectraStart(TEXT("Electra.Start"));
		if (Electra::IsAnalyticsEventEnabled(kEventNameElectraStart))
		{
			TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraStart);
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("PlayPos"), PlayPos));
			EnqueueAnalyticsEvent(AnalyticEvent);
		}
	}
}

void FElectraPlayer::HandlePlayerEventPlaybackPaused()
{
	MediaStateOnPause();
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		FScopeLock Lock(&StatisticsLock);
		Statistics.LastState = "Paused";
		double PlayPos = Player->AdaptivePlayer->GetPlayPosition().GetAsSeconds();
		UE_LOGF(LogElectraPlayer, Log, "[%u] Playback paused at play position %.3f", PlayerUniqueID, PlayPos);
		Statistics.AddMessageToHistory(TEXT("Playback paused"));

		// Enqueue a "Pause" event.
		static const FString kEventNameElectraPause(TEXT("Electra.Pause"));
		if (Electra::IsAnalyticsEventEnabled(kEventNameElectraPause))
		{
			TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraPause);
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("PlayPos"), PlayPos));
			EnqueueAnalyticsEvent(AnalyticEvent);
		}
	}
}

void FElectraPlayer::HandlePlayerEventPlaybackResumed()
{
	MediaStateOnPlay();
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		FScopeLock Lock(&StatisticsLock);
		Statistics.LastState = "Playing";
		double PlayPos = Player->AdaptivePlayer->GetPlayPosition().GetAsSeconds();
		UE_LOGF(LogElectraPlayer, Log, "[%u] Playback resumed at play position %.3f", PlayerUniqueID, PlayPos);
		Statistics.AddMessageToHistory(TEXT("Playback resumed"));

		// Enqueue a "Resume" event.
		static const FString kEventNameElectraResume(TEXT("Electra.Resume"));
		if (Electra::IsAnalyticsEventEnabled(kEventNameElectraResume))
		{
			TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraResume);
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("PlayPos"), PlayPos));
			EnqueueAnalyticsEvent(AnalyticEvent);
		}
	}
}

void FElectraPlayer::HandlePlayerEventPlaybackEnded()
{
	UpdatePlayEndStatistics();
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		double PlayPos = Player->AdaptivePlayer->GetPlayPosition().GetAsSeconds();

		// Update statistics
		FScopeLock Lock(&StatisticsLock);
		Statistics.LastState = "Ended";
		Statistics.bDidPlaybackEnd = true;
		UE_LOGF(LogElectraPlayer, Log, "[%u] Playback reached end at play position %.3f", PlayerUniqueID, PlayPos);
		Statistics.AddMessageToHistory(TEXT("Playback ended"));

		// Enqueue an "End" event.
		static const FString kEventNameElectraEnd(TEXT("Electra.End"));
		if (Electra::IsAnalyticsEventEnabled(kEventNameElectraEnd))
		{
			TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraEnd);
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("PlayPos"), PlayPos));
			EnqueueAnalyticsEvent(AnalyticEvent);
		}

		MediaStateOnEndReached();
	}
}

void FElectraPlayer::HandlePlayerEventJumpInPlayPosition(const Electra::FTimeValue& ToNewTime, const Electra::FTimeValue& FromTime, Electra::Metrics::ETimeJumpReason TimejumpReason)
{
	Electra::FTimeRange MediaTimeline;
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		Player->AdaptivePlayer->GetTimelineRange(MediaTimeline);

		// Update statistics
		FScopeLock Lock(&StatisticsLock);
		if (TimejumpReason == Electra::Metrics::ETimeJumpReason::UserSeek)
		{
			if (ToNewTime > FromTime)
			{
				++Statistics.NumTimesForwarded;
			}
			else if (ToNewTime < FromTime)
			{
				++Statistics.NumTimesRewound;
			}
			UE_LOGF(LogElectraPlayer, Verbose, "[%u] Jump in play position from %.3f to %.3f", PlayerUniqueID, FromTime.GetAsSeconds(), ToNewTime.GetAsSeconds());
		}
		else if (TimejumpReason == Electra::Metrics::ETimeJumpReason::Looping)
		{
			++Statistics.NumTimesLooped;
			Electra::IAdaptiveStreamingPlayer::FLoopState loopState;
			Player->AdaptivePlayer->GetLoopState(loopState);
			UE_LOGF(LogElectraPlayer, Log, "[%u] Looping (%lld) from %.3f to %.3f", PlayerUniqueID, loopState.Count, FromTime.GetAsSeconds(), ToNewTime.GetAsSeconds());
			Statistics.AddMessageToHistory(TEXT("Looped"));
		}

		// Enqueue a "PositionJump" event.
		static const FString kEventNameElectraPositionJump(TEXT("Electra.PositionJump"));
		if (Electra::IsAnalyticsEventEnabled(kEventNameElectraPositionJump))
		{
			TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraPositionJump);
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("From"), FromTime.GetAsSeconds()));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("To"), ToNewTime.GetAsSeconds()));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("Cause"), Electra::Metrics::GetTimejumpReasonString(TimejumpReason)));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("MediaTimeline.Start"), MediaTimeline.Start.GetAsSeconds(-1.0)));
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("MediaTimeline.End"), MediaTimeline.End.GetAsSeconds(-1.0)));
			EnqueueAnalyticsEvent(AnalyticEvent);
		}
	}
}

void FElectraPlayer::HandlePlayerEventPlaybackStopped()
{
	UpdatePlayEndStatistics();
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		double PlayPos = Player->AdaptivePlayer->GetPlayPosition().GetAsSeconds();

		// Update statistics
		FScopeLock Lock(&StatisticsLock);
		Statistics.bDidPlaybackEnd = true;
		// Note: we do not change Statistics.LastState since we want to keep the state the player was in when it got closed.
		UE_LOGF(LogElectraPlayer, Log, "[%u] Playback stopped. Last play position %.3f", PlayerUniqueID, PlayPos);
		Statistics.AddMessageToHistory(TEXT("Stopped"));

		// Enqueue a "Stop" event.
		static const FString kEventNameElectraStop(TEXT("Electra.Stop"));
		if (Electra::IsAnalyticsEventEnabled(kEventNameElectraStop))
		{
			TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraStop);
			AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("PlayPos"), PlayPos));
			EnqueueAnalyticsEvent(AnalyticEvent);
		}
	}
}

void FElectraPlayer::HandlePlayerEventSeekCompleted()
{
	UE_LOGF(LogElectraPlayer, Verbose, "[%u] Seek completed", PlayerUniqueID);
	MediaStateOnSeekFinished();
}

void FElectraPlayer::HandlePlayerMediaMetadataChanged(const TSharedPtrTS<Electra::FMetadataParser>& InMetadata)
{
	if (InMetadata.IsValid())
	{
		TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> PlayerMeta = InMetadata->GetMediaStreamMetadata();
		if (PlayerMeta.IsValid())
		{
			CurrentStreamMetadata = MoveTemp(PlayerMeta);
			DeferredMediaEvents.Enqueue(EMediaEvent::MetadataChanged);
		}
	}
}

void FElectraPlayer::HandlePlayerEventError(const FString& ErrorReason)
{
	bHasPendingError = true;

	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	// If there is already an error do not overwrite it. First come, first serve!
	if (Statistics.LastError.Len() == 0)
	{
		Statistics.LastError = ErrorReason;
	}
	// Note: we do not change Statistics.LastState to something like 'error' because we want to know the state the player was in when it errored.
	UE_LOGF(LogElectraPlayer, Error, "[%u] ReportError: \"%ls\"", PlayerUniqueID, *SanitizeMessage(ErrorReason));
	Statistics.AddMessageToHistory(FString::Printf(TEXT("Error: %s"), *SanitizeMessage(ErrorReason)));
	FString MessageHistory;
	for(auto &msg : Statistics.MessageHistoryBuffer)
	{
		MessageHistory.Append(FString::Printf(TEXT("%8.3f: %s"), msg.TimeSinceStart, *msg.Message));
		MessageHistory.Append(TEXT("<br>"));
	}

	// Enqueue an "Error" event.
	static const FString kEventNameElectraError(TEXT("Electra.Error"));
	if (Electra::IsAnalyticsEventEnabled(kEventNameElectraError))
	{
		TSharedPtr<FAnalyticsEvent> AnalyticEvent = CreateAnalyticsEvent(kEventNameElectraError);
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("Reason"), *ErrorReason));
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("LastState"), *Statistics.LastState));
		AnalyticEvent->ParamArray.Add(FAnalyticsEventAttribute(TEXT("MessageHistory"), MessageHistory));
		EnqueueAnalyticsEvent(AnalyticEvent);
	}
}

void FElectraPlayer::HandlePlayerEventLogMessage(Electra::IInfoLog::ELevel InLogLevel, const FString& InLogMessage, int64 InPlayerWallclockMilliseconds)
{
	FString m(SanitizeMessage(InLogMessage));
	switch(InLogLevel)
	{
		case Electra::IInfoLog::ELevel::Error:
		{
			UE_LOGF(LogElectraPlayer, Error, "[%u] %ls", PlayerUniqueID, *m);
			FScopeLock Lock(&StatisticsLock);
			Statistics.AddMessageToHistory(m);
			break;
		}
		case Electra::IInfoLog::ELevel::Warning:
		{
			UE_LOGF(LogElectraPlayer, Warning, "[%u] %ls", PlayerUniqueID, *m);
			FScopeLock Lock(&StatisticsLock);
			Statistics.AddMessageToHistory(m);
			break;
		}
		case Electra::IInfoLog::ELevel::Info:
		{
			UE_LOGF(LogElectraPlayer, Log, "[%u] %ls", PlayerUniqueID, *m);
			break;
		}
		case Electra::IInfoLog::ELevel::Verbose:
		{
			UE_LOGF(LogElectraPlayer, Verbose, "[%u] %ls", PlayerUniqueID, *m);
			break;
		}
	}
}

void FElectraPlayer::HandlePlayerEventDroppedVideoFrame(const Metrics::FFrameStats& InFrameStats)
{
	Statistics.NumTotalAddedVideoFrames = InFrameStats.NumTotal;
	Statistics.NumTotalDroppedVideoFrames = InFrameStats.NumDropped;
	Statistics.AvgVideoDropsPerInterval = InFrameStats.AvgDropsPerInterval;
	Statistics.MinVideoDropsPerInterval = InFrameStats.MinDropsPerInterval;
	Statistics.MaxVideoDropsPerInterval = InFrameStats.MaxDropsPerInterval;
	Statistics.VideoDropIntervalMs = InFrameStats.DropIntervalMs;
	Statistics.VideoNumIntervals = InFrameStats.NumIntervals;
	Statistics.VideoNumIntervalsWithDrops = InFrameStats.NumIntervalsWithDrops;
}

void FElectraPlayer::HandlePlayerEventDroppedAudioFrame(const Metrics::FFrameStats& InFrameStats)
{
}

void FElectraPlayer::FStatistics::AddMessageToHistory(FString InMessage)
{
	if (MessageHistoryBuffer.Num() >= 20)
	{
		MessageHistoryBuffer.RemoveAt(0);
	}
	double Now = FPlatformTime::Seconds();
	FStatistics::FHistoryEntry he;
	he.Message = MoveTemp(InMessage);
	he.TimeSinceStart = TimeAtOpen < 0.0 ? 0.0 : Now - TimeAtOpen;
	MessageHistoryBuffer.Emplace(MoveTemp(he));
}

void FElectraPlayer::UpdatePlayEndStatistics()
{
	TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> Player = CurrentPlayer;
	if (!Player.IsValid() || !Player->AdaptivePlayer.IsValid())
	{
		return;
	}

	double PlayPos = Player->AdaptivePlayer->GetPlayPosition().GetAsSeconds();
	Electra::FTimeRange MediaTimeline;
	Electra::FTimeValue MediaDuration;
	Player->AdaptivePlayer->GetTimelineRange(MediaTimeline);
	MediaDuration = Player->AdaptivePlayer->GetDuration();
	// Update statistics
	FScopeLock Lock(&StatisticsLock);
	if (Statistics.PlayPosAtStart >= 0.0 && Statistics.PlayPosAtEnd < 0.0)
	{
		Statistics.PlayPosAtEnd = PlayPos;
	}
	// Update the media timeline end.
	Statistics.MediaTimelineAtEnd = MediaTimeline;
	// Also re-set the duration in case it changed dynamically.
	Statistics.MediaDuration = MediaDuration.IsInfinity() ? -1.0 : MediaDuration.GetAsSeconds();
}

void FElectraPlayer::LogStatistics()
{
	FString VideoSegsPercentage;
	FString AudioSegsPercentage;

	FScopeLock Lock(&StatisticsLock);

	int32 Idx=0;
	for(auto& It : Statistics.VideoQualityPercentages)
	{
		VideoSegsPercentage += FString::Printf(TEXT("%d/%d: %d%%\n"), Idx++, It.Key, It.Value);
	}
	Idx=0;
	for(auto& It : Statistics.AudioQualityPercentages)
	{
		AudioSegsPercentage += FString::Printf(TEXT("%d/%d: %d%%\n"), Idx++, It.Key, It.Value);
	}

	UE_LOG(LogElectraPlayer, Verbose, TEXT(
		"[%u] Electra player statistics:\n"\
		"OS: %s\n"\
		"GPU Adapter: %s\n"
		"URL: %s\n"\
		"Time after main playlist loaded: %.3fs\n"\
		"Time after stream playlists loaded: %.3fs\n"\
		"Time for initial buffering: %.3fs\n"\
		"Initial video stream bitrate: %d bps\n"\
		"Initial audio stream bitrate: %d bps\n"\
		"Initial buffering bandwidth bps: %.3f\n"\
		"Initial buffering latency: %.3fs\n"\
		"Time for initial preroll: %.3fs\n"\
		"Number of times moved forward: %d\n"\
		"Number of times moved backward: %d\n"\
		"Number of times looped: %d\n"\
		"Number of times rebuffered: %d\n"\
		"Total time spent rebuffering: %.3fs\n"\
		"Longest rebuffering time: %.3fs\n"\
		"First media timeline start: %.3fs\n"\
		"First media timeline end: %.3fs\n"\
		"Last media timeline start: %.3fs\n"\
		"Last media timeline end: %.3fs\n"\
		"Media duration: %.3fs\n"\
		"Play position at start: %.3fs\n"\
		"Play position at end: %.3fs\n"\
		"Number of video quality upswitches: %d\n"\
		"Number of video quality downswitches: %d\n"\
		"Number of video drastic downswitches: %d\n"\
		"Number of audio quality upswitches: %d\n"\
		"Number of audio quality downswitches: %d\n"\
		"Number of audio drastic downswitches: %d\n"\
		"Bytes of video data streamed: %lld from %lld segments\n"\
		"Bytes of audio data streamed: %lld from %lld segments\n"\
		"Video quality percentage:\n%s"\
		"Audio quality percentage:\n%s"\
		"Video frames dropped: %lld of %lld total, %u avg (%u max) per %u ms interval (drops in %.1f%% of all intervals)\n"\
		"Currently active playlist URL: %s\n"\
		"Currently active resolution: %d * %d\n" \
		"Current state: %s\n" \
		"404 errors: %u\n" \
		"4xx errors: %u\n" \
		"5xx errors: %u\n" \
		"Timeouts: %u\n" \
		"Connection failures: %u\n" \
		"Other failures: %u\n" \
		"Last issue: %s\n"
	),
		PlayerUniqueID,
		*FString::Printf(TEXT("%s"), *AnalyticsOSVersion),
		*AnalyticsGPUType,
		*SanitizeMessage(Statistics.InitialURL),
		Statistics.TimeToLoadMainPlaylist,
		Statistics.TimeToLoadStreamPlaylists,
		Statistics.InitialBufferingDuration,
		Statistics.InitialVideoStreamBitrate,
		Statistics.InitialAudioStreamBitrate,
		Statistics.InitialBufferingBandwidth.GetAverageBandwidth(),
		Statistics.InitialBufferingBandwidth.GetAverageLatency(),
		Statistics.TimeForInitialPreroll,
		Statistics.NumTimesForwarded,
		Statistics.NumTimesRewound,
		Statistics.NumTimesLooped,
		Statistics.NumTimesRebuffered,
		Statistics.TotalRebufferingDuration,
		Statistics.LongestRebufferingDuration,
		Statistics.MediaTimelineAtStart.Start.GetAsSeconds(-1.0),
		Statistics.MediaTimelineAtStart.End.GetAsSeconds(-1.0),
		Statistics.MediaTimelineAtEnd.Start.GetAsSeconds(-1.0),
		Statistics.MediaTimelineAtEnd.End.GetAsSeconds(-1.0),
		Statistics.MediaDuration,
		Statistics.PlayPosAtStart,
		Statistics.PlayPosAtEnd,
		Statistics.NumVideoQualityUpswitches,
		Statistics.NumVideoQualityDownswitches,
		Statistics.NumVideoQualityDrasticDownswitches,
		Statistics.NumAudioQualityUpswitches,
		Statistics.NumAudioQualityDownswitches,
		Statistics.NumAudioQualityDrasticDownswitches,
		(long long int)Statistics.NumVideoDatabytesStreamed,
		(long long int)Statistics.NumVideoSegmentsStreamed,
		(long long int)Statistics.NumAudioDatabytesStreamed,
		(long long int)Statistics.NumAudioSegmentsStreamed,
		*VideoSegsPercentage,
		*AudioSegsPercentage,
		(long long int)Statistics.NumTotalDroppedVideoFrames,
		(long long int)Statistics.NumTotalAddedVideoFrames,
		Statistics.AvgVideoDropsPerInterval,
		Statistics.MaxVideoDropsPerInterval,
		Statistics.VideoDropIntervalMs,
		Statistics.VideoNumIntervals ? ((int64)Statistics.VideoNumIntervalsWithDrops * 10000 / Statistics.VideoNumIntervals) / 100.0 : 0.0,
		*SanitizeMessage(Statistics.CurrentlyActivePlaylistURL),
		Statistics.CurrentlyActiveResolutionWidth,
		Statistics.CurrentlyActiveResolutionHeight,
		*Statistics.LastState,
		Statistics.NumErr404,
		Statistics.NumErr4xx,
		Statistics.NumErr5xx,
		Statistics.NumErrTimeouts,
		Statistics.NumErrConnDrops,
		Statistics.NumErrOther,
		*SanitizeMessage(Statistics.LastError)
	);

	if (Statistics.LastError.Len())
	{
		FString MessageHistory;
		for(auto &msg : Statistics.MessageHistoryBuffer)
		{
			MessageHistory.Append(FString::Printf(TEXT("%8.3f: %s"), msg.TimeSinceStart, *msg.Message));
			MessageHistory.Append(TEXT("\n"));
		}
		UE_LOGF(LogElectraPlayer, Verbose, "Most recent log messages:\n%ls", *MessageHistory);
	}
}


void FElectraPlayer::SendAnalyticMetrics(const TSharedPtr<IAnalyticsProviderET>& AnalyticsProvider, const FGuid& InPlayerGuid)
{
	if (PlayerGuid != InPlayerGuid)
	{
		return;
	}
	if (!AnalyticsProvider.IsValid())
	{
		return;
	}

	if (!Statistics.bDidPlaybackEnd)
	{
		UE_LOGF(LogElectraPlayer, Verbose, "[%u] Submitting analytics during playback, some data may be incomplete", PlayerUniqueID);
		// Try to fill in some of the blanks.
		UpdatePlayEndStatistics();
	}

	UE_LOGF(LogElectraPlayer, Verbose, "[%u] Submitting analytics", PlayerUniqueID);


	// First emit all enqueued events before sending the final one.
	SendPendingAnalyticMetrics(AnalyticsProvider);


	TArray<FAnalyticsEventAttribute> ParamArray;
	UpdateAnalyticsCustomValues();
	AddCommonAnalyticsAttributes(ParamArray);
	StatisticsLock.Lock();
	FString MessageHistory;
	for(auto &msg : Statistics.MessageHistoryBuffer)
	{
		MessageHistory.Append(FString::Printf(TEXT("%8.3f: %s"), msg.TimeSinceStart, *msg.Message));
		MessageHistory.Append(TEXT("<br>"));
	}
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("URL"), Statistics.InitialURL));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("LastState"), Statistics.LastState));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("MessageHistory"), MessageHistory));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("LastError"), Statistics.LastError));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("FinalVideoResolution"), FString::Printf(TEXT("%d*%d"), Statistics.CurrentlyActiveResolutionWidth, Statistics.CurrentlyActiveResolutionHeight)));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("TimeElapsedToMainPlaylist"), Statistics.TimeToLoadMainPlaylist));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("TimeElapsedToPlaylists"), Statistics.TimeToLoadStreamPlaylists));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("InitialAvgBufferingBandwidth"), Statistics.InitialBufferingBandwidth.GetAverageBandwidth()));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("InitialAvgBufferingLatency"), Statistics.InitialBufferingBandwidth.GetAverageLatency()));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("InitialVideoBitrate"), Statistics.InitialVideoStreamBitrate));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("InitialAudioBitrate"), Statistics.InitialAudioStreamBitrate));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("InitialBufferingDuration"), Statistics.InitialBufferingDuration));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("InitialPrerollDuration"), Statistics.TimeForInitialPreroll));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("TimeElapsedUntilReady"), Statistics.TimeForInitialPreroll + Statistics.TimeAtPrerollBegin - Statistics.TimeAtOpen));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("MediaTimeline.First.Start"), Statistics.MediaTimelineAtStart.Start.GetAsSeconds(-1.0)));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("MediaTimeline.First.End"), Statistics.MediaTimelineAtStart.End.GetAsSeconds(-1.0)));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("MediaTimeline.Last.Start"), Statistics.MediaTimelineAtEnd.Start.GetAsSeconds(-1.0)));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("MediaTimeline.Last.End"), Statistics.MediaTimelineAtEnd.End.GetAsSeconds(-1.0)));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("MediaDuration"), Statistics.MediaDuration));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("PlayPosAtStart"), Statistics.PlayPosAtStart));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("PlayPosAtEnd"), Statistics.PlayPosAtEnd));
// FIXME: the difference is pointless as it does not tell how long playback was really performed for unless we are tracking an uninterrupted playback of a Live session.
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("PlaybackDuration"), Statistics.PlayPosAtEnd >= 0.0 ? Statistics.PlayPosAtEnd - Statistics.PlayPosAtStart : 0.0));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumTimesMovedForward"), (uint32) Statistics.NumTimesForwarded));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumTimesMovedBackward"), (uint32) Statistics.NumTimesRewound));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumTimesLooped"), (uint32) Statistics.NumTimesLooped));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("AbortedSegmentDownloads"), (uint32) Statistics.NumSegmentDownloadsAborted));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumQualityUpswitches"), (uint32) Statistics.NumVideoQualityUpswitches));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumQualityDownswitches"), (uint32) Statistics.NumVideoQualityDownswitches));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumQualityDrasticDownswitches"), (uint32) Statistics.NumVideoQualityDrasticDownswitches));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("AudioQualityUpswitches"), (uint32) Statistics.NumAudioQualityUpswitches));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("AudioQualityDownswitches"), (uint32) Statistics.NumAudioQualityDownswitches));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("AudioQualityDrasticDownswitches"), (uint32) Statistics.NumAudioQualityDrasticDownswitches));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Rebuffering.Num"), (uint32)Statistics.NumTimesRebuffered));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Rebuffering.AvgDuration"), Statistics.NumTimesRebuffered > 0 ? Statistics.TotalRebufferingDuration / Statistics.NumTimesRebuffered : 0.0));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Rebuffering.MaxDuration"), Statistics.LongestRebufferingDuration));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumBytesStreamedAudio"), (double) Statistics.NumAudioDatabytesStreamed));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumBytesStreamedVideo"), (double) Statistics.NumVideoDatabytesStreamed));
	FString SegsPerStream;
	for(const TPair<int32, uint32>& pair : Statistics.VideoSegmentBitratesStreamed)
	{
		SegsPerStream += FString::Printf(TEXT("%d:%u;"), pair.Key, pair.Value);
	}
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("VideoSegmentFetchStats"), *SegsPerStream));
	SegsPerStream.Empty();
	for(const TPair<int32, uint32>& pair : Statistics.AudioSegmentBitratesStreamed)
	{
		SegsPerStream += FString::Printf(TEXT("%d:%u;"), pair.Key, pair.Value);
	}
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("AudioSegmentFetchStats"), *SegsPerStream));

	// Quality buckets by percentage
	int32 qbIdx = 0;
	for(auto &qbIt : (Statistics.NumVideoSegmentsStreamed ? Statistics.VideoQualityPercentages : Statistics.AudioQualityPercentages))
	{
		ParamArray.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("qp%d"), qbIdx++), (int32) qbIt.Value));
	}
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumVideoSegmentsFetched"), (uint32) Statistics.NumVideoSegmentsStreamed));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumAudioSegmentsFetched"), (uint32) Statistics.NumAudioSegmentsStreamed));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumTotalSegmentsFetched"), (uint32) Statistics.NumVideoSegmentsStreamed + Statistics.NumAudioSegmentsStreamed));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumVideoFramesAddedTotal"), (uint32) Statistics.NumTotalAddedVideoFrames));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumVideoFramesDroppedTotal"), (uint32) Statistics.NumTotalDroppedVideoFrames));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("VideoDropIntervalMs"), (uint32) Statistics.VideoDropIntervalMs));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("AvgVideoDropsPerInterval"), (uint32) Statistics.AvgVideoDropsPerInterval));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("MinVideoDropsPerInterval"), (uint32) Statistics.MinVideoDropsPerInterval));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("MaxVideoDropsPerInterval"), (uint32) Statistics.MaxVideoDropsPerInterval));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("VideoNumIntervals"), (uint32) Statistics.VideoNumIntervals));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("VideoNumIntervalsWithDrops"), (uint32) Statistics.VideoNumIntervalsWithDrops));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("VideoIntervalsWithDropsPercentage"), (double)(Statistics.VideoNumIntervals ? ((int64)Statistics.VideoNumIntervalsWithDrops * 10000 / Statistics.VideoNumIntervals) / 100.0 : 0.0)));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Num404"), (uint32) Statistics.NumErr404));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Num4xx"), (uint32) Statistics.NumErr4xx));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Num5xx"), (uint32) Statistics.NumErr5xx));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumTimeouts"), (uint32) Statistics.NumErrTimeouts));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumConnDrops"), (uint32) Statistics.NumErrConnDrops));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("NumErrOther"), (uint32) Statistics.NumErrOther));

	StatisticsLock.Unlock();

	AnalyticsProvider->RecordEvent(TEXT("Electra.FinalMetrics"), MoveTemp(ParamArray));
}

void FElectraPlayer::SendAnalyticMetricsPerMinute(const TSharedPtr<IAnalyticsProviderET>& AnalyticsProvider)
{
	SendPendingAnalyticMetrics(AnalyticsProvider);

	TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> Player = CurrentPlayer;
	if (Player.Get() && Player->AdaptivePlayer->IsPlaying())
	{
		TArray<FAnalyticsEventAttribute> ParamArray;
		UpdateAnalyticsCustomValues();
		AddCommonAnalyticsAttributes(ParamArray);
		StatisticsLock.Lock();
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("URL"), Statistics.CurrentlyActivePlaylistURL));
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("VideoResolution"), FString::Printf(TEXT("%d*%d"), Statistics.CurrentlyActiveResolutionWidth, Statistics.CurrentlyActiveResolutionHeight)));
		StatisticsLock.Unlock();
		AnalyticsProvider->RecordEvent(TEXT("Electra.PerMinuteMetrics"), MoveTemp(ParamArray));
	}
}

void FElectraPlayer::SendPendingAnalyticMetrics(const TSharedPtr<IAnalyticsProviderET>& AnalyticsProvider)
{
	FScopeLock Lock(&StatisticsLock);
	TSharedPtr<FAnalyticsEvent> AnalyticEvent;
	while(QueuedAnalyticEvents.Dequeue(AnalyticEvent))
	{
		AnalyticsProvider->RecordEvent(*AnalyticEvent->EventName, MoveTemp(AnalyticEvent->ParamArray));
	}
	NumQueuedAnalyticEvents = 0;
}


void FElectraPlayer::ReportVideoStreamingError(const FGuid& InPlayerGuid, const FString& LastError)
{
	if (PlayerGuid != InPlayerGuid)
	{
		return;
	}

	FScopeLock Lock(&StatisticsLock);
	// Only replace a blank string with a non-blank string. We want to preserve
	// existing last error messages, as they will be the root of the problem.
	if (LastError.Len() > 0 && Statistics.LastError.Len() == 0)
	{
		Statistics.LastError = LastError;
	}
}

void FElectraPlayer::ReportSubtitlesMetrics(const FGuid& InPlayerGuid, const FString& URL, double ResponseTime, const FString& LastError)
{
}

void FElectraPlayer::MediaStateOnPreparingFinished()
{
	if (!ensure(PlayerState.State == EMediaState::Preparing))
	{
		return;
	}

	CSV_EVENT(ElectraPlayer, TEXT("MediaStateOnPreparingFinished"));

	PlayerState.State = EMediaState::Stopped;
	// Only report MediaOpened here and *not* TracksChanged as well.
	// We do not know where playback will start at and what tracks are available at that point.
	DeferredMediaEvents.Enqueue(EMediaEvent::MediaOpened);
}

bool FElectraPlayer::MediaStateOnPlay()
{
	if (PlayerState.State != EMediaState::Stopped && PlayerState.State != EMediaState::Paused)
	{
		return false;
	}

	CSV_EVENT(ElectraPlayer, TEXT("MediaStateOnPlay"));

	double CurrentRate = 1.0;
	TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> Player = CurrentPlayer;
	if (Player.IsValid() && Player->AdaptivePlayer.IsValid())
	{
		CurrentRate = Player->AdaptivePlayer->GetPlayRate();
	}

	PlayerState.State = EMediaState::Playing;
	PlayerState.SetPlayRateFromPlayer(CurrentRate);

	DeferredMediaEvents.Enqueue(EMediaEvent::PlaybackResumed);
	return true;
}

bool FElectraPlayer::MediaStateOnPause()
{
	if (PlayerState.State != EMediaState::Playing)
	{
		return false;
	}

	CSV_EVENT(ElectraPlayer, TEXT("MediaStateOnPause"));

	PlayerState.State = EMediaState::Paused;
	PlayerState.SetPlayRateFromPlayer(0.0f);

	DeferredMediaEvents.Enqueue(EMediaEvent::PlaybackSuspended);
	return true;
}

void FElectraPlayer::MediaStateOnEndReached()
{
	CSV_EVENT(ElectraPlayer, TEXT("MediaStateOnEndReached"));

	switch(PlayerState.State)
	{
		case EMediaState::Preparing:
		case EMediaState::Playing:
		case EMediaState::Paused:
		case EMediaState::Stopped:
		{
			DeferredMediaEvents.Enqueue(EMediaEvent::PlaybackEndReached);
			break;
		}
		default:
		{
			break;
		}
	}
	PlayerState.State = EMediaState::Stopped;
}

void FElectraPlayer::MediaStateOnSeekFinished()
{
	CSV_EVENT(ElectraPlayer, TEXT("MediaStateOnSeekFinished"));
	DeferredMediaEvents.Enqueue(EMediaEvent::SeekCompleted);
}


// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------

TSharedPtr<FElectraPlayer::FAnalyticsEvent> FElectraPlayer::CreateAnalyticsEvent(FString InEventName)
{
	TSharedPtr<FAnalyticsEvent> Ev = MakeShared<FAnalyticsEvent>();
	Ev->EventName = MoveTemp(InEventName);
	AddCommonAnalyticsAttributes(Ev->ParamArray);
	return Ev;
}

void FElectraPlayer::AddCommonAnalyticsAttributes(TArray<FAnalyticsEventAttribute>& InOutParamArray)
{
	InOutParamArray.Add(FAnalyticsEventAttribute(TEXT("SessionId"), AnalyticsInstanceGuid));
	InOutParamArray.Add(FAnalyticsEventAttribute(TEXT("EventNum"), AnalyticsInstanceEventCount));
	InOutParamArray.Add(FAnalyticsEventAttribute(TEXT("Utc"), static_cast<double>(FDateTime::UtcNow().ToUnixTimestamp())));
	InOutParamArray.Add(FAnalyticsEventAttribute(TEXT("OS"), FString::Printf(TEXT("%s"), *AnalyticsOSVersion)));
	InOutParamArray.Add(FAnalyticsEventAttribute(TEXT("GPUAdapter"), AnalyticsGPUType));
	++AnalyticsInstanceEventCount;
	StatisticsLock.Lock();
	for(int32 nI=0, nIMax=UE_ARRAY_COUNT(AnalyticsCustomValues); nI<nIMax; ++nI)
	{
		if (AnalyticsCustomValues[nI].Len())
		{
			InOutParamArray.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("%s%d"), CUSTOM_ANALYTIC_METRIC_KEYNAME, nI), AnalyticsCustomValues[nI]));
		}
	}
	StatisticsLock.Unlock();
}

void FElectraPlayer::UpdateAnalyticsCustomValues()
{
	StatisticsLock.Lock();
	for(int32 nI=0, nIMax=UE_ARRAY_COUNT(AnalyticsCustomValues); nI<nIMax; ++nI)
	{
		FVariantValue Value = ElectraMediaOptions::GetOptionValue(OptionInterface, ElectraMediaOptions::EOptionType::CustomAnalyticsMetric, FVariantValue(FString::Printf(TEXT("%s%d"), CUSTOM_ANALYTIC_METRIC_QUERYOPTION_KEY, nI)));
		if (Value.IsValid() && Value.GetDataType() == FVariantValue::EDataType::TypeFString)
		{
			AnalyticsCustomValues[nI] = Value.GetFString();
		}
	}
	StatisticsLock.Unlock();
}


void FElectraPlayer::EnqueueAnalyticsEvent(TSharedPtr<FAnalyticsEvent> InAnalyticEvent)
{
	FScopeLock Lock(&StatisticsLock);
	// Since analytics are popped from the outside only we check if we have accumulated a lot without them having been retrieved.
	// To prevent those from growing beyond leap and bounds we limit ourselves to 100.
	while(NumQueuedAnalyticEvents > 100)
	{
		QueuedAnalyticEvents.Pop();
		--NumQueuedAnalyticEvents;
	}
	QueuedAnalyticEvents.Enqueue(InAnalyticEvent);
	++NumQueuedAnalyticEvents;
}

// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------


void FElectraPlayer::FAdaptiveStreamingPlayerResourceProvider::ProvideStaticPlaybackDataForURL(TSharedPtr<Electra::IAdaptiveStreamingPlayerResourceRequest, ESPMode::ThreadSafe> InOutRequest)
{
	check(InOutRequest.IsValid());
	PendingStaticResourceRequests.Enqueue(InOutRequest);
}

void FElectraPlayer::FAdaptiveStreamingPlayerResourceProvider::ProcessPendingStaticResourceRequests()
{
	TSharedPtr<Electra::IAdaptiveStreamingPlayerResourceRequest, ESPMode::ThreadSafe> InOutRequest;
	while(PendingStaticResourceRequests.Dequeue(InOutRequest))
	{
		check(InOutRequest->GetResourceType() == Electra::IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Empty ||
			  InOutRequest->GetResourceType() == Electra::IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Playlist ||
			  InOutRequest->GetResourceType() == Electra::IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::LicenseKey);
		if (InOutRequest->GetResourceType() == Electra::IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Playlist)
		{
			FVariantValue Value = ElectraMediaOptions::GetOptionValue(OptionInterface, ElectraMediaOptions::EOptionType::PlayListData, FVariantValue(InOutRequest->GetResourceURL()));
			if (Value.IsValid())
			{
				FString PlaylistData = Value.GetFString();
				if (!PlaylistData.IsEmpty() && PlaylistData != InOutRequest->GetResourceURL())
				{
					TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> ResponseDataPtr = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();
					Electra::StringHelpers::StringToArray(*ResponseDataPtr, PlaylistData);
					InOutRequest->SetPlaybackData(ResponseDataPtr, 0);
				}
			}
		}
		else if (InOutRequest->GetResourceType() == Electra::IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::LicenseKey)
		{
			FVariantValue Value = ElectraMediaOptions::GetOptionValue(OptionInterface, ElectraMediaOptions::EOptionType::LicenseKeyData, FVariantValue(InOutRequest->GetResourceURL()));
			if (Value.IsValid())
			{
				FString LicenseKeyData = Value.GetFString();
				if (!LicenseKeyData.IsEmpty() && LicenseKeyData != InOutRequest->GetResourceURL())
				{
					TArray<uint8> BinKey;
					BinKey.AddUninitialized(LicenseKeyData.Len());
					BinKey.SetNum(HexToBytes(LicenseKeyData, BinKey.GetData()));
					TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> ResponseDataPtr = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>(BinKey);
					InOutRequest->SetPlaybackData(ResponseDataPtr, 0);
				}
			}
		}
		InOutRequest->SignalDataReady();
	}
}

void FElectraPlayer::FAdaptiveStreamingPlayerResourceProvider::ClearPendingRequests()
{
	PendingStaticResourceRequests.Empty();
}



// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------

void FElectraPlayer::OnMediaPlayerEventReceived(TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> InEvent, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode)
{
#if !UE_BUILD_SHIPPING
	const TCHAR* const Origins[] = { TEXT("Playlist"), TEXT("Inband"), TEXT("TimedMetadata"), TEXT("n/a") };
	UE_LOGF(LogElectraPlayer, Verbose, "[%u] %ls event %ls with \"%ls\", \"%ls\", \"%ls\" PTS @ %.3f for %.3fs", PlayerUniqueID,
		Origins[Electra::Utils::Min((int32)InEvent->GetOrigin(), (int32)UE_ARRAY_COUNT(Origins)-1)],
		InDispatchMode==IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnReceive?TEXT("received"):TEXT("started"),
		*InEvent->GetSchemeIdUri(), *InEvent->GetValue(), *InEvent->GetID(),
		InEvent->GetPresentationTime().GetAsSeconds(), InEvent->GetDuration().GetAsSeconds());
#endif

	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		Electra::FTimeRange MediaTimeline;
		Player->AdaptivePlayer->GetTimelineRange(MediaTimeline);

		// Create a binary media sample of our extended format and pass it up.
		TSharedPtr<FMetaDataDecoderOutput, ESPMode::ThreadSafe> Meta = MakeShared<FMetaDataDecoderOutput, ESPMode::ThreadSafe>();
		switch(InDispatchMode)
		{
			default:
			case IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnReceive:
			{
				Meta->DispatchedMode = FMetaDataDecoderOutput::EDispatchedMode::OnReceive;
				break;
			}
			case IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnStart:
			{
				Meta->DispatchedMode = FMetaDataDecoderOutput::EDispatchedMode::OnStart;
				break;
			}
		}
		switch(InEvent->GetOrigin())
		{
			default:
			case IAdaptiveStreamingPlayerAEMSEvent::EOrigin::TimedMetadata:
			{
				Meta->Origin = FMetaDataDecoderOutput::EOrigin::TimedMetadata;
				break;
			}
			case IAdaptiveStreamingPlayerAEMSEvent::EOrigin::EventStream:
			{
				Meta->Origin = FMetaDataDecoderOutput::EOrigin::EventStream;
				break;
			}
			case IAdaptiveStreamingPlayerAEMSEvent::EOrigin::InbandEventStream:
			{
				Meta->Origin = FMetaDataDecoderOutput::EOrigin::InbandEventStream;
				break;
			}
		}
		Meta->Data = InEvent->GetMessageData();
		Meta->SchemeIdUri = InEvent->GetSchemeIdUri();
		Meta->Value = InEvent->GetValue();
		Meta->ID = InEvent->GetID(),
		Meta->Duration = InEvent->GetDuration().GetAsTimespan();
		Meta->PresentationTime = FDecoderTimeStamp(InEvent->GetPresentationTime().GetAsTimespan(), 0);
		// Set the current timeline start as the metadata track's zero point. This is only useful if the timeline does not
		// actually change over time. The use of the base time is therefore tied to knowledge by the using code that the
		// timeline will be fixed.
		Meta->TrackBaseTime = FDecoderTimeStamp(MediaTimeline.Start.GetAsTimespan(), MediaTimeline.Start.GetSequenceIndex());

		TSharedRef<FElectraBinarySample, ESPMode::ThreadSafe> MetaDataSample = MakeShared<FElectraBinarySample, ESPMode::ThreadSafe>();
		MetaDataSample->Metadata = Meta;
		Player->AdaptivePlayer->GetMediaSamples()->AddMetadata(MetaDataSample);
	}
}

// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------


void FElectraPlayer::Close()
{
	CloseInternal();
}

FString FElectraPlayer::GetInfo() const
{
	return FString();
}

FGuid FElectraPlayer::GetPlayerPluginGUID() const
{
	// Same GUID as used in the player factory.
	static FGuid PlayerPluginGUID(0x94ee3f80, 0x8e604292, 0xb4d24dd5, 0xfdade1c2);
	return PlayerPluginGUID;
}

IMediaSamples& FElectraPlayer::GetSamples()
{
	auto Player = CurrentPlayer;
	return Player.IsValid() ? *Player->AdaptivePlayer->GetMediaSamples() : *EmptyMediaSamples;
}

FString FElectraPlayer::GetStats() const
{
	return FString();
}

FString FElectraPlayer::GetUrl() const
{
	return MediaUrl;
}

bool FElectraPlayer::Open(const FString& InUrl, const IMediaOptions* InOptions)
{
	return Open(InUrl, InOptions, nullptr);
}

bool FElectraPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive, const FString& InOriginalUrl, const IMediaOptions* InOptions)
{
	// Opening from an archive is not supported.
	return false;
}

bool FElectraPlayer::Open(const FString& InUrl, const IMediaOptions* InOptions, const FMediaPlayerOptions* InPlayerOptions)
{
	return OpenInternal(InUrl, InOptions, InPlayerOptions);
}

FVariant FElectraPlayer::GetMediaInfo(FName InInfoName) const
{
	const TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> Player = CurrentPlayer;
	return Player.IsValid() ? Player->AdaptivePlayer->GetMediaInfo(InInfoName).ToFVariant() : FVariant();
}

TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> FElectraPlayer::GetMediaMetadata() const
{
	return CurrentStreamMetadata;
}

void FElectraPlayer::SetGuid(const FGuid& InGuid)
{
	PlayerGuid = InGuid;
}

void FElectraPlayer::TickInput(FTimespan InDeltaTime, FTimespan InTimecode)
{
	TickInternal(InDeltaTime, InTimecode);
}

bool FElectraPlayer::FlushOnSeekStarted() const
{
	return false;
}

bool FElectraPlayer::FlushOnSeekCompleted() const
{
	return false;
}

bool FElectraPlayer::GetPlayerFeatureFlag(EFeatureFlag InFlag) const
{
	switch(InFlag)
	{
		case EFeatureFlag::AllowShutdownOnClose:
		{
			return bHasClosedDueToError;
		}
		case EFeatureFlag::UsePlaybackTimingV2:
		{
			return true;
		}
		case EFeatureFlag::PlayerUsesInternalFlushOnSeek:
		{
			return true;
		}
		case EFeatureFlag::IsTrackSwitchSeamless:
		{
			return true;
		}
		case EFeatureFlag::PlayerSelectsDefaultTracks:
		{
			return true;
		}
		default:
		{
			return IMediaPlayer::GetPlayerFeatureFlag(InFlag);
		}
	}
}

bool FElectraPlayer::SetAsyncResourceReleaseNotification(IAsyncResourceReleaseNotificationRef InAsyncDestructNotification)
{
	AsyncResourceReleaseNotification = InAsyncDestructNotification;
	return true;
}

uint32 FElectraPlayer::GetNewResourcesOnOpen() const
{
	return IMediaPlayerLifecycleManagerDelegate::ResourceFlags_Decoder;
}

bool FElectraPlayer::QueryCacheState(EMediaCacheState InState, TRangeSet<FTimespan>& OutTimeRanges) const
{
	// Note: The data of time ranges returned here will not actually get "cached" as
	//       it is always only transient. We thus report the ranges only for `Loaded` and `Loading`,
	//       but never for `Cached`!
	switch(InState)
	{
		case EMediaCacheState::Loaded:
		case EMediaCacheState::Loading:
		case EMediaCacheState::Pending:
		{
			auto Player = CurrentPlayer;
			if (Player.IsValid())
			{
				// When asked to provide what's already loaded we look at what we have in the sample queue
				// and add that to the result. These samples have already left the player but are ready
				// for use.
				if (InState == EMediaCacheState::Loaded)
				{
					TArray<TRange<FMediaTimeStamp>> QueuedRange;
					if (Player->AdaptivePlayer->GetMediaSamples()->PeekVideoSampleTimeRanges(QueuedRange) && QueuedRange.Num())
					{
						OutTimeRanges.Add(TRange<FTimespan>(QueuedRange[0].GetLowerBoundValue().Time, QueuedRange.Last().GetUpperBoundValue().Time));
					}
				}

				IAdaptiveStreamingPlayer::FStreamBufferInfo bi;
				// Query video first.
				Player->AdaptivePlayer->QueryStreamBufferInfo(bi, Electra::EStreamType::Video);
				// If that is not active query audio.
				if (!bi.bIsBufferActive)
				{
					Player->AdaptivePlayer->QueryStreamBufferInfo(bi, Electra::EStreamType::Audio);
				}
				if (bi.bIsBufferActive)
				{
					auto AddRanges = [](TRangeSet<FTimespan>& OutRanges, const TArray<Electra::FTimeRange>& InRanges) -> void
					{
						for(int32 i=0; i<InRanges.Num(); ++i)
						{
							OutRanges.Add(TRange<FTimespan>(InRanges[i].Start.GetAsTimespan(), InRanges[i].End.GetAsTimespan()));
						}
					};

					switch(InState)
					{
						case EMediaCacheState::Loaded:
						{
							AddRanges(OutTimeRanges, bi.TimeEnqueued);
							break;
						}
						case EMediaCacheState::Loading:
						{
							AddRanges(OutTimeRanges, bi.TimeAvailable);
							break;
						}
						case EMediaCacheState::Pending:
						{
							AddRanges(OutTimeRanges, bi.TimeRequested);
							break;
						}
					}
				}
			}
			return true;
		}
	}
	return false;
}

bool FElectraPlayer::CanControl(EMediaControl InControl) const
{
	const EMediaState CurrentState = GetState();
	if (InControl == EMediaControl::BlockOnFetch)
	{
		return CurrentState == EMediaState::Playing || CurrentState == EMediaState::Paused;
	}
	else if (InControl == EMediaControl::Pause)
	{
		return CurrentState == EMediaState::Playing;
	}
	else if (InControl == EMediaControl::Resume)
	{
		return CurrentState == EMediaState::Paused || CurrentState == EMediaState::Stopped;
	}
	else if (InControl == EMediaControl::Seek || InControl == EMediaControl::Scrub)
	{
		return CurrentState == EMediaState::Playing || CurrentState == EMediaState::Paused || CurrentState == EMediaState::Stopped;
	}
	else if (InControl == EMediaControl::PlaybackRange)
	{
		return true;
	}
	return false;
}

FTimespan FElectraPlayer::GetDuration() const
{
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return FTimespan::Zero();
	}
	Electra::FTimeValue Dur = Player->AdaptivePlayer->GetDuration();
	return Dur.IsValid() ? (Dur.IsInfinity() ? FTimespan::MaxValue() : Dur.GetAsTimespan()) : FTimespan();
}

float FElectraPlayer::GetRate() const
{
	return PlayerState.GetRate();
}

EMediaState FElectraPlayer::GetState() const
{
	return PlayerState.GetState();
}

EMediaStatus FElectraPlayer::GetStatus() const
{
	return PlayerState.GetStatus();
}

TRangeSet<float> FElectraPlayer::GetSupportedRates(EMediaRateThinning InThinning) const
{
	TRangeSet<float> Res;
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return Res;
	}
	TArray<TRange<double>> SupportedRanges;
	Player->AdaptivePlayer->GetSupportedRates(InThinning == EMediaRateThinning::Unthinned ? IAdaptiveStreamingPlayer::EPlaybackRateType::Unthinned : IAdaptiveStreamingPlayer::EPlaybackRateType::Thinned).GetRanges(SupportedRanges);
	for(auto &Rate : SupportedRanges)
	{
		TRange<float> r;
		if (Rate.HasLowerBound())
		{
			r.SetLowerBound(TRange<float>::BoundsType::Inclusive((float) Rate.GetLowerBoundValue()));
		}
		if (Rate.HasUpperBound())
		{
			r.SetUpperBound(TRange<float>::BoundsType::Inclusive((float) Rate.GetUpperBoundValue()));
		}
		Res.Add(r);
	}
	return Res;
}

FTimespan FElectraPlayer::GetTime() const
{
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return FTimespan::Zero();
	}
	Electra::FTimeValue playerTime = Player->AdaptivePlayer->GetPlayPosition();
	return playerTime.GetAsTimespan();
}

bool FElectraPlayer::IsLooping() const
{
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		IAdaptiveStreamingPlayer::FLoopState loopState;
		Player->AdaptivePlayer->GetLoopState(loopState);
		return loopState.bIsEnabled;
	}
	return bEnableLooping.Get(false);
}

bool FElectraPlayer::SetLooping(bool bInLooping)
{
	bEnableLooping = bInLooping;
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return false;
	}
	IAdaptiveStreamingPlayer::FLoopParam loop;
	loop.bEnableLooping = bEnableLooping.GetValue();
	Player->AdaptivePlayer->SetLooping(loop);
	UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaPlayer::SetLooping(%ls)", PlayerUniqueID, bEnableLooping.GetValue()?TEXT("true"):TEXT("false"));
	return true;
}

bool FElectraPlayer::SetRate(float InRate)
{
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return false;
	}
	// Set the intended rate, which *may* be set negative. This is not supported and we put the adaptive player into pause
	// if this happens, but we keep the intended rate set nevertheless.
	PlayerState.SetIntendedPlayRate(InRate);
	if (InRate <= 0.0f)
	{
		Player->AdaptivePlayer->Pause();
	}
	else
	{
		if (Player->AdaptivePlayer->IsPaused() || !Player->AdaptivePlayer->IsPlaying())
		{
			TriggerFirstSeekIfNecessary();
			Player->AdaptivePlayer->Resume();
		}
	}
	UE_LOGF(LogElectraPlayer, Log, "[%u] IMediaControls::SetRate(%.3f)", PlayerUniqueID, InRate);
	CSV_EVENT(ElectraPlayer, TEXT("Setting Rate"));
	IAdaptiveStreamingPlayer::FTrickplayParams Params;
	Player->AdaptivePlayer->SetPlayRate((double) InRate, Params);
	return true;
}

bool FElectraPlayer::Seek(const FTimespan& InNewTime, const FMediaSeekParams& InAdditionalParams)
{
	auto Player = CurrentPlayer;
	if (!Player.IsValid())
	{
		return false;
	}
	UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaControls::Seek() to %#.4f (%ls)", PlayerUniqueID, InNewTime.GetTotalSeconds(), *InNewTime.ToString(TEXT("%h:%m:%s.%f")));
	CSV_EVENT(ElectraPlayer, TEXT("Seeking"));
	FTimespan Target;
	CalculateTargetSeekTime(Target, InNewTime);
	Electra::IAdaptiveStreamingPlayer::FSeekParam seek;
	if (Target != FTimespan::MaxValue())
	{
		seek.Time.SetFromTimespan(Target);
	}
	check(InAdditionalParams.NewSequenceIndex.IsSet());
	seek.NewSequenceIndex = InAdditionalParams.NewSequenceIndex;
	bInitialSeekPerformed = true;
	Player->AdaptivePlayer->SeekTo(seek);
	return true;
}

TRange<FTimespan> FElectraPlayer::GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet) const
{
	TRange<FTimespan> Range(CurrentPlaybackRange);
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		switch(InRangeToGet)
		{
			case EMediaTimeRangeType::Absolute:
			{
				Electra::FTimeRange Timeline;
				Player->AdaptivePlayer->GetTimelineRange(Timeline);
				if (Timeline.IsValid())
				{
					Range.SetLowerBound(Timeline.Start.GetAsTimespan());
					Range.SetUpperBound(Timeline.End.GetAsTimespan());
				}
				else
				{
					Electra::FTimeValue Dur = Player->AdaptivePlayer->GetDuration();
					if (Dur.IsValid())
					{
						Range.SetLowerBound(FTimespan(0));
						Range.SetUpperBound(Dur.IsInfinity() ? FTimespan::MaxValue() : Dur.GetAsTimespan());
					}
				}
				break;
			}
			case EMediaTimeRangeType::Current:
			{
				Electra::IAdaptiveStreamingPlayer::FPlaybackRange Current;
				Player->AdaptivePlayer->GetPlaybackRange(Current);
				if (Current.Start.IsSet() && Current.End.IsSet())
				{
					Range.SetLowerBound(Current.Start.GetValue().GetAsTimespan());
					Range.SetUpperBound(Current.End.GetValue().GetAsTimespan());
				}
				else
				{
					return GetPlaybackTimeRange(EMediaTimeRangeType::Absolute);
				}
				break;
			}
		}
	}
	return Range;
}

bool FElectraPlayer::SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange)
{
	CurrentPlaybackRange = InTimeRange;
	auto Player = CurrentPlayer;
	if (Player.IsValid())
	{
		// Ranges cannot be set on Live streams.
		Electra::FTimeValue Dur = Player->AdaptivePlayer->GetDuration();
		if (Dur.IsValid() && Dur.IsInfinity())
		{
			return false;
		}
		if (!CurrentPlaybackRange.IsEmpty())
		{
			Electra::IAdaptiveStreamingPlayer::FPlaybackRange Range;
			if (CurrentPlaybackRange.HasLowerBound())
			{
				Range.Start = Electra::FTimeValue().SetFromTimespan(CurrentPlaybackRange.GetLowerBoundValue());
			}
			if (CurrentPlaybackRange.HasUpperBound())
			{
				Range.End = Electra::FTimeValue().SetFromTimespan(CurrentPlaybackRange.GetUpperBoundValue());
			}
			Player->AdaptivePlayer->SetPlaybackRange(Range);
			if (Range.Start.IsSet() && Range.End.IsSet())
			{
				UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaControls::SetPlaybackTimeRange(%#.4f - %#.4f)", PlayerUniqueID, Range.Start.GetValue().GetAsSeconds(), Range.End.GetValue().GetAsSeconds());
			}
			else if (Range.Start.IsSet())
			{
				UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaControls::SetPlaybackTimeRange(%#.4f - n/a)", PlayerUniqueID, Range.Start.GetValue().GetAsSeconds());
			}
			else if (Range.End.IsSet())
			{
				UE_LOGF(LogElectraPlayer, Verbose, "[%u] IMediaControls::SetPlaybackTimeRange(n/a - %#.4f)", PlayerUniqueID, Range.End.GetValue().GetAsSeconds());
			}
		}
	}
	return true;
}

bool FElectraPlayer::GetAudioTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	if (InTrackIndex >= 0 && InTrackIndex < NumTracksAudio && InFormatIndex == 0)
	{
		TSharedPtr<Electra::FTrackMetadata, ESPMode::ThreadSafe> Meta = GetTrackStreamMetadata(EMediaTrackType::Audio, InTrackIndex);
		if (Meta.IsValid())
		{
			const Electra::FStreamCodecInformation& ci = Meta->HighestBandwidthCodec;
			OutFormat.BitsPerSample = 16;
			OutFormat.NumChannels = (uint32)ci.GetNumberOfChannels();
			OutFormat.SampleRate = (uint32)ci.GetSamplingRate();
			OutFormat.TypeName = ci.GetHumanReadableCodecName();
			return true;
		}
	}
	return false;
}

int32 FElectraPlayer::GetNumTracks(EMediaTrackType InTrackType) const
{
	if (InTrackType == EMediaTrackType::Audio)
	{
		return NumTracksAudio;
	}
	else if (InTrackType == EMediaTrackType::Video)
	{
		return NumTracksVideo;
	}
	else if (InTrackType == EMediaTrackType::Subtitle)
	{
		return NumTracksSubtitle;
	}
	return 0;
}

int32 FElectraPlayer::GetNumTrackFormats(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	// Right now we only have a single format per track
	if ((InTrackType == EMediaTrackType::Video && NumTracksVideo != 0) ||
		(InTrackType == EMediaTrackType::Audio && NumTracksAudio != 0) ||
		(InTrackType == EMediaTrackType::Subtitle && NumTracksSubtitle != 0))
	{
		return 1;
	}
	return 0;
}

int32 FElectraPlayer::GetSelectedTrack(EMediaTrackType InTrackType) const
{
	/*
		To reduce the overhead of this function we check for the track the underlying player has
		actually selected only when we were told the tracks changed.

		It is possible that the underlying player changes the track automatically as playback progresses.
		For instance, when playing a DASH stream consisting of several periods the player needs to re-select
		the audio stream when transitioning from one period into the next, which may change the index of
		the selected track.
	*/

	auto CheckAndReselectTrack = [this](Electra::EStreamType InStreamType, bool& InOutDirtyFlag, int32& InOutSelectedIndex, int32 InNumTracks) -> int32
	{
		if (InOutDirtyFlag)
		{
			if (InNumTracks == 0)
			{
				InOutSelectedIndex = -1;
			}
			else
			{
				auto Player = CurrentPlayer;
				if (Player.IsValid())
				{
					if (Player->AdaptivePlayer->IsTrackDeselected(InStreamType))
					{
						InOutSelectedIndex = -1;
						InOutDirtyFlag = false;
					}
					else
					{
						Electra::FStreamSelectionAttributes Attributes;
						Player->AdaptivePlayer->GetSelectedTrackAttributes(Attributes, InStreamType);
						if (Attributes.OverrideIndex.IsSet())
						{
							InOutSelectedIndex = Attributes.OverrideIndex.GetValue();
							InOutDirtyFlag = false;
						}
					}
				}
			}
		}
		return InOutSelectedIndex;
	};

	// Electra does not have caption or metadata tracks, handle only video, audio and subtitles.
	if (InTrackType == EMediaTrackType::Video)
	{
		return CheckAndReselectTrack(Electra::EStreamType::Video, bVideoTrackIndexDirty, SelectedVideoTrackIndex, NumTracksVideo);
	}
	else if (InTrackType == EMediaTrackType::Audio)
	{
		return CheckAndReselectTrack(Electra::EStreamType::Audio, bAudioTrackIndexDirty, SelectedAudioTrackIndex, NumTracksAudio);
	}
	else if (InTrackType == EMediaTrackType::Subtitle)
	{
		return CheckAndReselectTrack(Electra::EStreamType::Subtitle, bSubtitleTrackIndexDirty, SelectedSubtitleTrackIndex, NumTracksSubtitle);
	}
	return -1;
}

FText FElectraPlayer::GetTrackDisplayName(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<Electra::FTrackMetadata, ESPMode::ThreadSafe> Meta = GetTrackStreamMetadata(InTrackType, InTrackIndex);
	if (Meta.IsValid())
	{
		if (InTrackType == EMediaTrackType::Video)
		{
			if (!Meta->Label.IsEmpty())
			{
				return FText::FromString(Meta->Label);
			}
			return FText::FromString(FString::Printf(TEXT("Video Track ID %s"), *Meta->ID));
		}
		else if (InTrackType == EMediaTrackType::Audio)
		{
			if (!Meta->Label.IsEmpty())
			{
				return FText::FromString(Meta->Label);
			}
			return FText::FromString(FString::Printf(TEXT("Audio Track ID %s"), *Meta->ID));
		}
		else if (InTrackType == EMediaTrackType::Subtitle)
		{
			FString Name;
			if (!Meta->Label.IsEmpty())
			{
				Name = FString::Printf(TEXT("%s (%s)"), *Meta->Label, *Meta->HighestBandwidthCodec.GetCodecSpecifierRFC6381());
			}
			else
			{
				Name = FString::Printf(TEXT("Subtitle Track ID %s (%s)"), *Meta->ID, *Meta->HighestBandwidthCodec.GetCodecSpecifierRFC6381());
			}
			return FText::FromString(Name);
		}
	}
	return FText();
}

int32 FElectraPlayer::GetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	if ((InTrackType == EMediaTrackType::Audio && NumTracksAudio > 0) ||
		(InTrackType == EMediaTrackType::Video && NumTracksVideo > 0) ||
		(InTrackType == EMediaTrackType::Subtitle && NumTracksSubtitle > 0))
	{
		// Right now we only have a single format per track so we return format index 0 at all times.
		return 0;
	}
	return -1;
}

FString FElectraPlayer::GetTrackLanguage(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<Electra::FTrackMetadata, ESPMode::ThreadSafe> Meta = GetTrackStreamMetadata(InTrackType, InTrackIndex);
	if (Meta.IsValid())
	{
		if (InTrackType == EMediaTrackType::Audio)
		{
			// Audio does not need to include the script tag (but video does as it could include burned in subtitles)
			return Meta->LanguageTagRFC5646.Get(true, false, true, false, false, false);
		}
		else
		{
			return Meta->LanguageTagRFC5646.Get(true, true, true, false, false, false);
		}
	}
	return FString();
}

FString FElectraPlayer::GetTrackName(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	return FString();
}

bool FElectraPlayer::GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if (InTrackIndex >= 0 && InTrackIndex < NumTracksVideo && InFormatIndex == 0)
	{
		TSharedPtr<Electra::FTrackMetadata, ESPMode::ThreadSafe> Meta = GetTrackStreamMetadata(EMediaTrackType::Video, InTrackIndex);
		if (Meta.IsValid())
		{
			const Electra::FStreamCodecInformation& ci = Meta->HighestBandwidthCodec;
			OutFormat.Dim.X = ci.GetResolution().Width;
			OutFormat.Dim.Y = ci.GetResolution().Height;

			OutFormat.FrameRate = 0.0f;
			if (ci.GetFrameRate().IsValid())
			{
				OutFormat.FrameRate = (float)ci.GetFrameRate().GetAsDouble();
			}
			else
			{
				FScopeLock lock(&VideoFormatLock);
				OutFormat.FrameRate = CurrentlyActiveVideoStreamFormat.FrameRate;
			}
			OutFormat.FrameRates = TRange<float>{ OutFormat.FrameRate };
			OutFormat.TypeName = ci.GetHumanReadableCodecName();
			return true;
		}
	}
	return false;
}

TOptional<FTimespan> FElectraPlayer::GetTrackDuration(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<Electra::FTrackMetadata, ESPMode::ThreadSafe> Meta = GetTrackStreamMetadata(InTrackType, InTrackIndex);
	return Meta.IsValid() ? Meta->Duration : TOptional<FTimespan>();
}


/**
 * Selects a specified track for playback.
 *
 * Note:
 *   There is currently no concept of selecting a track based on metadata, only by index.
 *   The idea being that before selecting a track by index the application needs to check
 *   the metadata beforehand (eg. call GetTrackLanguage()) to figure out the index of the
 *   track it wants to play.
 *
 *   The underlying player however needs to select tracks based on metadata alone instead
 *   of an index in case the track layout changes dynamically during playback.
 *   For example, a part of the presentation could have both English and French audio,
 *   followed by a part (say, an advertisement) that only has English audio, followed
 *   by the continued regular part that has both. Without any user intervention the
 *   player needs to automatically switch from French to English and back to French, or
 *   index 1 -> 0 -> 1 (assuming French was the starting language of choice).
 *   Indices are therefore meaningless to the underlying player.
 *
 *   SelectTrack() is currently called implicitly by FMediaPlayerFacade::SelectDefaultTracks()
 *   when EMediaEvent::TracksChanged is received. This is why this event is NOT sent out
 *   in HandlePlayerEventTracksChanged() when the underlying player notifies us about a
 *   change in track layout.
 *   Other than the very first track selection made by the facade this method should only
 *   be called from a direct user interaction.
 */
bool FElectraPlayer::SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	auto PerformSelection = [this, InTrackType, InTrackIndex](int32& OutSelectedTrackIndex, Electra::FStreamSelectionAttributes& OutSelectionAttributes) -> bool
	{
		Electra::EStreamType StreamType = InTrackType == EMediaTrackType::Video ? Electra::EStreamType::Video :
										  InTrackType == EMediaTrackType::Audio ? Electra::EStreamType::Audio :
										  InTrackType == EMediaTrackType::Subtitle ? Electra::EStreamType::Subtitle :
										  Electra::EStreamType::Unsupported;
		// Select a track or deselect?
		if (InTrackIndex >= 0)
		{
			// Check if the track index exists by checking the presence of the track metadata.
			// If for some reason the index is not valid the selection will not be changed.
			TSharedPtr<Electra::FTrackMetadata, ESPMode::ThreadSafe> Meta = GetTrackStreamMetadata(InTrackType, InTrackIndex);
			if (Meta.IsValid())
			{
				// Switch only when the track index has changed.
				if (GetSelectedTrack(InTrackType) != InTrackIndex)
				{
					Electra::FStreamSelectionAttributes TrackAttributes;
					TrackAttributes.OverrideIndex = InTrackIndex;

					OutSelectionAttributes.OverrideIndex = InTrackIndex;
					if (!Meta->Kind.IsEmpty())
					{
						TrackAttributes.Kind = Meta->Kind;
						OutSelectionAttributes.Kind = Meta->Kind;
					}
					TrackAttributes.Language_RFC4647 = Meta->LanguageTagRFC5646.Get(true, true, true, false, false, false);
					OutSelectionAttributes.Language_RFC4647 = TrackAttributes.Language_RFC4647;
					TrackAttributes.Codec = Meta->HighestBandwidthCodec.GetCodecName();
					OutSelectionAttributes.Codec = Meta->HighestBandwidthCodec.GetCodecName();

					TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> Player = CurrentPlayer;
					if (Player.IsValid() && Player->AdaptivePlayer.IsValid())
					{
						Player->AdaptivePlayer->SelectTrackByAttributes(StreamType, TrackAttributes);
					}

					OutSelectedTrackIndex = InTrackIndex;
				}
				return true;
			}
		}
		else
		{
			// Deselect track.
			OutSelectionAttributes.OverrideIndex = -1;
			OutSelectedTrackIndex = -1;
			TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> Player = CurrentPlayer;
			if (Player.IsValid() && Player->AdaptivePlayer.IsValid())
			{
				Player->AdaptivePlayer->DeselectTrack(StreamType);
			}
			return true;
		}
		return false;
	};

	if (InTrackType == EMediaTrackType::Video)
	{
		return PerformSelection(SelectedVideoTrackIndex, PlaystartOptions.InitialVideoTrackAttributes);
	}
	else if (InTrackType == EMediaTrackType::Audio)
	{
		return PerformSelection(SelectedAudioTrackIndex, PlaystartOptions.InitialAudioTrackAttributes);
	}
	else if (InTrackType == EMediaTrackType::Subtitle)
	{
		return PerformSelection(SelectedSubtitleTrackIndex, PlaystartOptions.InitialSubtitleTrackAttributes);
	}
	return false;
}

bool FElectraPlayer::SetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex, int32 InFormatIndex)
{
	return false;
}

bool FElectraPlayer::SetVideoTrackFrameRate(int32 InTrackIndex, int32 InFormatIndex, float InFrameRate)
{
	return false;
}

void FElectraPlayer::ReportOpenSource(const FString& InURL)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_OpenSource>(InURL));
}

void FElectraPlayer::ReportReceivedMainPlaylist(const FString& InEffectiveURL)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_ReceivedMainPlaylist>(InEffectiveURL));
}

void FElectraPlayer::ReportReceivedPlaylists()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::ReceivedPlaylists));
}

void FElectraPlayer::ReportTracksChanged()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::TracksChanged));
}

void FElectraPlayer::ReportPlaylistDownload(const Metrics::FPlaylistDownloadStats& InPlaylistDownloadStats)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_PlaylistDownload>(InPlaylistDownloadStats));
}

void FElectraPlayer::ReportBufferingStart(Metrics::EBufferingReason InBufferingReason)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_BufferingStart>(InBufferingReason));
}

void FElectraPlayer::ReportBufferingEnd(Metrics::EBufferingReason InBufferingReason)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_BufferingEnd>(InBufferingReason));
}

void FElectraPlayer::ReportBandwidth(int64 InEffectiveBps, int64 InThroughputBps, double InLatencyInSeconds)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_Bandwidth>(InEffectiveBps, InThroughputBps, InLatencyInSeconds));
}

void FElectraPlayer::ReportBufferUtilization(const Metrics::FBufferStats& InBufferStats)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_BufferUtilization>(InBufferStats));
}

void FElectraPlayer::ReportSegmentDownload(const Metrics::FSegmentDownloadStats& InSegmentDownloadStats)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_SegmentDownload>(InSegmentDownloadStats));
}

void FElectraPlayer::ReportLicenseKey(const Metrics::FLicenseKeyStats& InLicenseKeyStats)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_LicenseKey>(InLicenseKeyStats));
}

void FElectraPlayer::ReportDataAvailabilityChange(const Metrics::FDataAvailabilityChange& InDataAvailability)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_DataAvailabilityChange>(InDataAvailability));
}

void FElectraPlayer::ReportVideoQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_VideoQualityChange>(InNewBitrate, InPreviousBitrate, bInIsDrasticDownswitch));
}

void FElectraPlayer::ReportAudioQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_AudioQualityChange>(InNewBitrate, InPreviousBitrate, bInIsDrasticDownswitch));
}

void FElectraPlayer::ReportDecodingFormatChange(const FStreamCodecInformation& InNewDecodingFormat)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_CodecFormatChange>(InNewDecodingFormat));
}

void FElectraPlayer::ReportPrerollStart()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::PrerollStart));
}

void FElectraPlayer::ReportPrerollEnd()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::PrerollEnd));
}

void FElectraPlayer::ReportPlaybackStart()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::PlaybackStart));
}

void FElectraPlayer::ReportPlaybackPaused()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::PlaybackPaused));
}

void FElectraPlayer::ReportPlaybackResumed()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::PlaybackResumed));
}

void FElectraPlayer::ReportPlaybackEnded()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::PlaybackEnded));
}

void FElectraPlayer::ReportJumpInPlayPosition(const FTimeValue& InToNewTime, const FTimeValue& InFromTime, Metrics::ETimeJumpReason InTimejumpReason)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_JumpInPlayPosition>(InToNewTime, InFromTime, InTimejumpReason));
}

void FElectraPlayer::ReportPlaybackStopped()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::PlaybackStopped));
}

void FElectraPlayer::ReportSeekCompleted()
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEventBase>(FPlayerMetricEventBase::EType::SeekCompleted));
}

void FElectraPlayer::ReportMediaMetadataChanged(TSharedPtrTS<Electra::FMetadataParser> InMetadata)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_MediaMetadataChange>(InMetadata));
}

void FElectraPlayer::ReportError(const FString& InErrorReason)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_Error>(InErrorReason));
}

void FElectraPlayer::ReportLogMessage(IInfoLog::ELevel InLogLevel, const FString& InLogMessage, int64 InPlayerWallclockMilliseconds)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_LogMessage>(InLogLevel, InLogMessage, InPlayerWallclockMilliseconds));
}

void FElectraPlayer::ReportVideoFrameStats(const Metrics::FFrameStats& InFrameStats)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_DroppedFrame>(FPlayerMetricEventBase::EType::DroppedVideoFrame, InFrameStats));
}

void FElectraPlayer::ReportAudioFrameStats(const Metrics::FFrameStats& InFrameStats)
{
	DeferredPlayerEvents.Enqueue(MakeSharedTS<FPlayerMetricEvent_DroppedFrame>(FPlayerMetricEventBase::EType::DroppedAudioFrame, InFrameStats));
}

// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------------------

TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> FElectraPlayerRuntimeFactory::CreatePlayer(IMediaEventSink& InEventSink,
	FElectraPlayerSendAnalyticMetricsDelegate& InSendAnalyticMetricsDelegate,
	FElectraPlayerSendAnalyticMetricsPerMinuteDelegate& InSendAnalyticMetricsPerMinuteDelegate,
	FElectraPlayerReportVideoStreamingErrorDelegate& InReportVideoStreamingErrorDelegate,
	FElectraPlayerReportSubtitlesMetricsDelegate& InReportSubtitlesFileMetricsDelegate)
{
	TSharedPtr<FElectraPlayer, ESPMode::ThreadSafe> NewPlayer = MakeShared<FElectraPlayer, ESPMode::ThreadSafe>(InEventSink, InSendAnalyticMetricsDelegate, InSendAnalyticMetricsPerMinuteDelegate, InReportVideoStreamingErrorDelegate, InReportSubtitlesFileMetricsDelegate);
	return NewPlayer;
}
