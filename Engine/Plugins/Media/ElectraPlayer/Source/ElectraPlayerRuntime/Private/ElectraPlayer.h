// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraPlayerInterface.h"

#include "Containers/UnrealString.h"
#include "Containers/Queue.h"
#include "Containers/SortedMap.h"
#include "Templates/SharedPointer.h"
#include "Templates/Greater.h"
#include "Logging/LogMacros.h"
#include "Misc/Optional.h"
#include "Misc/Guid.h"
#include "Math/Range.h"

#include "IMediaPlayer.h"
#include "IMediaCache.h"
#include "IMediaView.h"
#include "IMediaControls.h"
#include "IMediaTracks.h"
#include "IMediaEventSink.h"
#include "IMediaPlayerLifecycleManager.h"
#include "MediaSamples.h"
#include "MediaSampleQueue.h"

#include "IAnalyticsProviderET.h"

#include "OutputHandler.h"
#include "ElectraTextureSample.h"
#include "IElectraAudioSample.h"

#include "Player/AdaptiveStreamingPlayer.h"
#include "MediaStreamMetadata.h"
#include "PlayerRuntimeGlobal.h"
#include "StreamTypes.h"

#include <atomic>



class IMetaDataDecoderOutput;
using IMetaDataDecoderOutputPtr = TSharedPtr<IMetaDataDecoderOutput, ESPMode::ThreadSafe>;

using namespace Electra;


class FElectraPlayer
	: public IMediaPlayer
	, public IMediaCache
	, public IMediaControls
	, public IMediaTracks
	, public IMediaView
	, public IAdaptiveStreamingPlayerMetrics
	, public TSharedFromThis<FElectraPlayer, ESPMode::ThreadSafe>
{
public:
	FElectraPlayer(IMediaEventSink& InEventSink,
			  FElectraPlayerSendAnalyticMetricsDelegate& InSendAnalyticMetricsDelegate,
			  FElectraPlayerSendAnalyticMetricsPerMinuteDelegate& InSendAnalyticMetricsPerMinuteDelegate,
			  FElectraPlayerReportVideoStreamingErrorDelegate& InReportVideoStreamingErrorDelegate,
			  FElectraPlayerReportSubtitlesMetricsDelegate& InReportSubtitlesFileMetricsDelegate);
	virtual ~FElectraPlayer();

	//-------------------------------------------------------------------------
	// Methods from IMediaPlayer
	//
	void Close() override;
	IMediaCache& GetCache() override
	{ return *this; }
	IMediaControls& GetControls() override
	{ return *this; }
	FString GetInfo() const override;
	FGuid GetPlayerPluginGUID() const override;
	IMediaSamples& GetSamples() override;
	FString GetStats() const override;
	IMediaTracks& GetTracks() override
	{ return *this; }
	FString GetUrl() const override;
	IMediaView& GetView() override
	{ return *this; }
	bool Open(const FString& InUrl, const IMediaOptions* InOptions) override;
	bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive, const FString& InOriginalUrl, const IMediaOptions* InOptions) override;
	bool Open(const FString& InUrl, const IMediaOptions* InOptions, const FMediaPlayerOptions* InPlayerOptions) override;
	FVariant GetMediaInfo(FName InInfoName) const override;
	TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> GetMediaMetadata() const override;
	void SetGuid(const FGuid& InGuid) override;
	void TickInput(FTimespan InDeltaTime, FTimespan InTimecode) override;
	bool FlushOnSeekStarted() const override;
	bool FlushOnSeekCompleted() const override;
	bool GetPlayerFeatureFlag(EFeatureFlag InFlag) const override;
	bool SetAsyncResourceReleaseNotification(IAsyncResourceReleaseNotificationRef InAsyncDestructNotification) override;
	uint32 GetNewResourcesOnOpen() const override;

	//-------------------------------------------------------------------------
	// Methods from IMediaCache
	//
	bool QueryCacheState(EMediaCacheState InState, TRangeSet<FTimespan>& OutTimeRanges) const override;

	//-------------------------------------------------------------------------
	// Methods from IMediaControls
	//
	bool CanControl(EMediaControl InControl) const override;
	FTimespan GetDuration() const override;
	float GetRate() const override;
	EMediaState GetState() const override;
	EMediaStatus GetStatus() const override;
	TRangeSet<float> GetSupportedRates(EMediaRateThinning InThinning) const override;
	FTimespan GetTime() const override;
	bool IsLooping() const override;
	bool Seek(const FTimespan& InTime) override
	{ check(!"You have to call the override with additional options!"); return false; }
	bool SetLooping(bool bInLooping) override;
	bool SetRate(float InRate) override;
	bool Seek(const FTimespan& InNewTime, const FMediaSeekParams& InAdditionalParams) override;
	TRange<FTimespan> GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet) const override;
	bool SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange) override;

	//-------------------------------------------------------------------------
	// Methods from IMediaTracks
	//
	bool GetAudioTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	int32 GetNumTracks(EMediaTrackType InTrackType) const override;
	int32 GetNumTrackFormats(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	int32 GetSelectedTrack(EMediaTrackType InTrackType) const override;
	FText GetTrackDisplayName(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	int32 GetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	FString GetTrackLanguage(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	FString GetTrackName(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	bool GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	bool SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex) override;
	bool SetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex, int32 InFormatIndex) override;
	bool SetVideoTrackFrameRate(int32 InTrackIndex, int32 InFormatIndex, float InFrameRate) override;
	TOptional<FTimespan> GetTrackDuration(EMediaTrackType InTrackType, int32 InTrackIndex) const override;

	//-------------------------------------------------------------------------
	// Methods from IAdaptiveStreamingPlayerMetrics
	//
	void ReportOpenSource(const FString& InURL) override;
	void ReportReceivedMainPlaylist(const FString& InEffectiveURL) override;
	void ReportReceivedPlaylists() override;
	void ReportTracksChanged() override;
	void ReportPlaylistDownload(const Metrics::FPlaylistDownloadStats& InPlaylistDownloadStats) override;
	void ReportBufferingStart(Metrics::EBufferingReason InBufferingReason) override;
	void ReportBufferingEnd(Metrics::EBufferingReason InBufferingReason) override;
	void ReportBandwidth(int64 InEffectiveBps, int64 InThroughputBps, double InLatencyInSeconds) override;
	void ReportBufferUtilization(const Metrics::FBufferStats& InBufferStats) override;
	void ReportSegmentDownload(const Metrics::FSegmentDownloadStats& InSegmentDownloadStats) override;
	void ReportLicenseKey(const Metrics::FLicenseKeyStats& InLicenseKeyStats) override;
	void ReportDataAvailabilityChange(const Metrics::FDataAvailabilityChange& InDataAvailability) override;
	void ReportVideoQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch) override;
	void ReportAudioQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch) override;
	void ReportDecodingFormatChange(const FStreamCodecInformation& InNewDecodingFormat) override;
	void ReportPrerollStart() override;
	void ReportPrerollEnd() override;
	void ReportPlaybackStart() override;
	void ReportPlaybackPaused() override;
	void ReportPlaybackResumed() override;
	void ReportPlaybackEnded() override;
	void ReportJumpInPlayPosition(const FTimeValue& InToNewTime, const FTimeValue& InFromTime, Metrics::ETimeJumpReason InTimejumpReason) override;
	void ReportPlaybackStopped() override;
	void ReportSeekCompleted() override;
	void ReportMediaMetadataChanged(TSharedPtrTS<Electra::FMetadataParser> InMetadata) override;
	void ReportError(const FString& InErrorReason) override;
	void ReportLogMessage(IInfoLog::ELevel InLogLevel, const FString& InLogMessage, int64 InPlayerWallclockMilliseconds) override;
	void ReportVideoFrameStats(const Metrics::FFrameStats& InFrameStats) override;
	void ReportAudioFrameStats(const Metrics::FFrameStats& InFrameStats) override;


	//-------------------------------------------------------------------------
	// Implementation methods
	//
	void PlatformSuspendOrResumeDecoders(bool bInSuspend, const Electra::FParamDict& InOptions);

	void SendAnalyticMetrics(const TSharedPtr<IAnalyticsProviderET>& InAnalyticsProvider, const FGuid& InPlayerGuid);
	void SendAnalyticMetricsPerMinute(const TSharedPtr<IAnalyticsProviderET>& InAnalyticsProvider);
	void SendPendingAnalyticMetrics(const TSharedPtr<IAnalyticsProviderET>& InAnalyticsProvider);
	void ReportVideoStreamingError(const FGuid& InPlayerGuid, const FString& InLastError);
	void ReportSubtitlesMetrics(const FGuid& InPlayerGuid, const FString& InURL, double InResponseTime, const FString& InLastError);

private:
	struct FAnalyticsEvent;
	void ClearToDefaultState();
	void SendMediaSinkEvent(EMediaEvent InEventToSend);
	bool OpenInternal(const FString& InUrl, const IMediaOptions* InOptions, const FMediaPlayerOptions* InPlayerOptions);
	void CloseInternal();
	void TickInternal(FTimespan InDeltaTime, FTimespan InTimecode);
	void TriggerFirstSeekIfNecessary();
	void CalculateTargetSeekTime(FTimespan& OutTargetTime, const FTimespan& InTime);
	bool IsLive();
	TSharedPtr<FTrackMetadata, ESPMode::ThreadSafe> GetTrackStreamMetadata(EMediaTrackType InTrackType, int32 InTrackIndex) const;
	void AddCommonAnalyticsAttributes(TArray<FAnalyticsEventAttribute>& InOutParamArray);
	TSharedPtr<FAnalyticsEvent> CreateAnalyticsEvent(FString InEventName);
	void EnqueueAnalyticsEvent(TSharedPtr<FAnalyticsEvent> InAnalyticEvent);
	void UpdateAnalyticsCustomValues();
	void UpdatePlayEndStatistics();
	void LogStatistics();
	void OnMediaPlayerEventReceived(TSharedPtr<IAdaptiveStreamingPlayerAEMSEvent, ESPMode::ThreadSafe> InEvent, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode);
	void HandleDeferredPlayerEvents();
	void HandlePlayerEventOpenSource(const FString& InURL);
	void HandlePlayerEventReceivedMainPlaylist(const FString& InEffectiveURL);
	void HandlePlayerEventReceivedPlaylists();
	void HandlePlayerEventTracksChanged();
	void HandlePlayerEventPlaylistDownload(const Metrics::FPlaylistDownloadStats& InPlaylistDownloadStats);
	void HandlePlayerEventBufferingStart(Metrics::EBufferingReason InBufferingReason);
	void HandlePlayerEventBufferingEnd(Metrics::EBufferingReason InBufferingReason);
	void HandlePlayerEventBandwidth(int64 InEffectiveBps, int64 InThroughputBps, double InLatencyInSeconds);
	void HandlePlayerEventBufferUtilization(const Metrics::FBufferStats& InBufferStats);
	void HandlePlayerEventSegmentDownload(const Metrics::FSegmentDownloadStats& InSegmentDownloadStats);
	void HandlePlayerEventLicenseKey(const Metrics::FLicenseKeyStats& InLicenseKeyStats);
	void HandlePlayerEventDataAvailabilityChange(const Metrics::FDataAvailabilityChange& InDataAvailability);
	void HandlePlayerEventVideoQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch);
	void HandlePlayerEventAudioQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch);
	void HandlePlayerEventCodecFormatChange(const Electra::FStreamCodecInformation& InNewDecodingFormat);
	void HandlePlayerEventPrerollStart();
	void HandlePlayerEventPrerollEnd();
	void HandlePlayerEventPlaybackStart();
	void HandlePlayerEventPlaybackPaused();
	void HandlePlayerEventPlaybackResumed();
	void HandlePlayerEventPlaybackEnded();
	void HandlePlayerEventJumpInPlayPosition(const FTimeValue& InToNewTime, const FTimeValue& InFromTime, Metrics::ETimeJumpReason InTimejumpReason);
	void HandlePlayerEventPlaybackStopped();
	void HandlePlayerEventSeekCompleted();
	void HandlePlayerMediaMetadataChanged(const TSharedPtrTS<Electra::FMetadataParser>& InMetadata);
	void HandlePlayerEventError(const FString& InErrorReason);
	void HandlePlayerEventLogMessage(IInfoLog::ELevel InLogLevel, const FString& InLogMessage, int64 InPlayerWallclockMilliseconds);
	void HandlePlayerEventDroppedVideoFrame(const Metrics::FFrameStats& InFrameStats);
	void HandlePlayerEventDroppedAudioFrame(const Metrics::FFrameStats& InFrameStats);
	void MediaStateOnPreparingFinished();
	bool MediaStateOnPlay();
	bool MediaStateOnPause();
	void MediaStateOnEndReached();
	void MediaStateOnSeekFinished();

	//********************************************************************************************************************
private:
	DECLARE_DELEGATE_TwoParams(FOnMediaPlayerEventReceivedDelegate, TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> /*InEvent*/, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode /*InDispatchMode*/);
	class FAEMSEventReceiver : public IAdaptiveStreamingPlayerAEMSReceiver
	{
	public:
		virtual ~FAEMSEventReceiver() = default;
		FOnMediaPlayerEventReceivedDelegate& GetEventReceivedDelegate()
		{ return EventReceivedDelegate; }
	private:
		virtual void OnMediaPlayerEventReceived(TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> InEvent, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode) override
		{ EventReceivedDelegate.ExecuteIfBound(InEvent, InDispatchMode); }
		FOnMediaPlayerEventReceivedDelegate EventReceivedDelegate;
	};

	class FAdaptiveStreamingPlayerResourceProvider : public IAdaptiveStreamingPlayerResourceProvider
	{
	public:
		FAdaptiveStreamingPlayerResourceProvider(TWeakPtr<IElectraSafeMediaOptionInterface, ESPMode::ThreadSafe> InOptionInterface)
		{ OptionInterface = MoveTemp(InOptionInterface); }
		virtual ~FAdaptiveStreamingPlayerResourceProvider() = default;

		virtual void ProvideStaticPlaybackDataForURL(TSharedPtr<IAdaptiveStreamingPlayerResourceRequest, ESPMode::ThreadSafe> InOutRequest) override;

		void ProcessPendingStaticResourceRequests();
		void ClearPendingRequests();
	private:
		TWeakPtr<IElectraSafeMediaOptionInterface, ESPMode::ThreadSafe> OptionInterface;
		/** Requests for static resource fetches we want to perform on the main thread **/
		TQueue<TSharedPtr<IAdaptiveStreamingPlayerResourceRequest, ESPMode::ThreadSafe>, EQueueMode::Mpsc> PendingStaticResourceRequests;
	};

	class FInternalPlayerImpl
	{
	public:
		/** The media player itself **/
		TSharedPtr<IAdaptiveStreamingPlayer, ESPMode::ThreadSafe> AdaptivePlayer;

		static void DoCloseAsync(TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe>&& InPlayer, uint32 InPlayerID, TSharedPtr<IAsyncResourceReleaseNotification, ESPMode::ThreadSafe> InAsyncResourceReleaseNotification, bool bReleasePools, int64 TexturePoolDecoderID);
	};

	struct FPlaystartOptions
	{
		void Reset()
		{
			TimeOffset.Reset();
			InitialVideoTrackAttributes.Reset();
			InitialAudioTrackAttributes.Reset();
			InitialSubtitleTrackAttributes.Reset();
			MaxVerticalStreamResolution.Reset();
			MaxBandwidthForStreaming.Reset();
			bDoNotPreload = false;
		}
		TOptional<FTimespan>		TimeOffset;
		FStreamSelectionAttributes	InitialVideoTrackAttributes;
		FStreamSelectionAttributes	InitialAudioTrackAttributes;
		FStreamSelectionAttributes	InitialSubtitleTrackAttributes;
		TOptional<int32>			MaxVerticalStreamResolution;
		TOptional<int32>			MaxBandwidthForStreaming;
		bool						bDoNotPreload = false;
	};

	struct FVideoStreamFormat
	{
		void Reset()
		{
			Resolution = FIntPoint::ZeroValue;
			FrameRate = 0.0;
			Bitrate = 0;
		}
		FIntPoint Resolution {};
		double FrameRate = 0.0;
		int32 Bitrate = 0;
	};

	struct FPlayerState
	{
		FPlayerState()
		{ Reset(); }
		TOptional<float> IntendedPlayRate;
		float CurrentPlayRate;
		EMediaState State;
		EMediaStatus Status;

		void Reset()
		{
			IntendedPlayRate.Reset();
			CurrentPlayRate = 0.0f;
			State = EMediaState::Closed;
			Status = EMediaStatus::None;
		}

		float GetRate() const;
		EMediaState GetState() const;
		EMediaStatus GetStatus() const;
		void SetIntendedPlayRate(float InIntendedRate);
		void SetPlayRateFromPlayer(float InCurrentPlayerPlayRate);
	};


	struct FPlayerMetricEventBase
	{
		enum class EType
		{
			OpenSource,
			ReceivedMainPlaylist,
			ReceivedPlaylists,
			TracksChanged,
			PlaylistDownload,
			BufferingStart,
			BufferingEnd,
			Bandwidth,
			BufferUtilization,
			SegmentDownload,
			LicenseKey,
			DataAvailabilityChange,
			VideoQualityChange,
			AudioQualityChange,
			CodecFormatChange,
			PrerollStart,
			PrerollEnd,
			PlaybackStart,
			PlaybackPaused,
			PlaybackResumed,
			PlaybackEnded,
			JumpInPlayPosition,
			PlaybackStopped,
			SeekCompleted,
			MediaMetadataChanged,
			Error,
			LogMessage,
			DroppedVideoFrame,
			DroppedAudioFrame
		};
		FPlayerMetricEventBase(EType InType) : Type(InType) {}
		virtual ~FPlayerMetricEventBase() = default;
		EType Type;
	};
	struct FPlayerMetricEvent_OpenSource : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_OpenSource(const FString& InURL) : FPlayerMetricEventBase(EType::OpenSource), URL(InURL) {}
		FString URL;
	};
	struct FPlayerMetricEvent_ReceivedMainPlaylist : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_ReceivedMainPlaylist(const FString& InEffectiveURL) : FPlayerMetricEventBase(EType::ReceivedMainPlaylist), EffectiveURL(InEffectiveURL) {}
		FString EffectiveURL;
	};
	struct FPlayerMetricEvent_PlaylistDownload : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_PlaylistDownload(const Metrics::FPlaylistDownloadStats& InPlaylistDownloadStats) : FPlayerMetricEventBase(EType::PlaylistDownload), PlaylistDownloadStats(InPlaylistDownloadStats) {}
		Metrics::FPlaylistDownloadStats PlaylistDownloadStats;
	};
	struct FPlayerMetricEvent_BufferingStart : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_BufferingStart(Metrics::EBufferingReason InBufferingReason) : FPlayerMetricEventBase(EType::BufferingStart), BufferingReason(InBufferingReason) {}
		Metrics::EBufferingReason BufferingReason;
	};
	struct FPlayerMetricEvent_BufferingEnd : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_BufferingEnd(Metrics::EBufferingReason InBufferingReason) : FPlayerMetricEventBase(EType::BufferingEnd), BufferingReason(InBufferingReason) {}
		Metrics::EBufferingReason BufferingReason;
	};
	struct FPlayerMetricEvent_Bandwidth : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_Bandwidth(int64 InEffectiveBps, int64 InThroughputBps, double InLatencyInSeconds) : FPlayerMetricEventBase(EType::Bandwidth), EffectiveBps(InEffectiveBps), ThroughputBps(InThroughputBps), LatencyInSeconds(InLatencyInSeconds) {}
		int64 EffectiveBps;
		int64 ThroughputBps;
		double LatencyInSeconds;
	};
	struct FPlayerMetricEvent_BufferUtilization : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_BufferUtilization(const Metrics::FBufferStats& InBufferStats) : FPlayerMetricEventBase(EType::BufferUtilization), BufferStats(InBufferStats) {}
		Metrics::FBufferStats BufferStats;
	};
	struct FPlayerMetricEvent_SegmentDownload : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_SegmentDownload(const Metrics::FSegmentDownloadStats& InSegmentDownloadStats) : FPlayerMetricEventBase(EType::SegmentDownload), SegmentDownloadStats(InSegmentDownloadStats) {}
		Metrics::FSegmentDownloadStats SegmentDownloadStats;
	};
	struct FPlayerMetricEvent_LicenseKey : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_LicenseKey(const Metrics::FLicenseKeyStats& InLicenseKeyStats) : FPlayerMetricEventBase(EType::LicenseKey), LicenseKeyStats(InLicenseKeyStats) {}
		Metrics::FLicenseKeyStats LicenseKeyStats;
	};
	struct FPlayerMetricEvent_DataAvailabilityChange : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_DataAvailabilityChange(const Metrics::FDataAvailabilityChange& InDataAvailability) : FPlayerMetricEventBase(EType::DataAvailabilityChange), DataAvailability(InDataAvailability) {}
		Metrics::FDataAvailabilityChange DataAvailability;
	};
	struct FPlayerMetricEvent_VideoQualityChange : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_VideoQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch) : FPlayerMetricEventBase(EType::VideoQualityChange), NewBitrate(InNewBitrate), PreviousBitrate(InPreviousBitrate), bIsDrasticDownswitch(bInIsDrasticDownswitch) {}
		int32 NewBitrate;
		int32 PreviousBitrate;
		bool bIsDrasticDownswitch;
	};
	struct FPlayerMetricEvent_AudioQualityChange : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_AudioQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch) : FPlayerMetricEventBase(EType::AudioQualityChange), NewBitrate(InNewBitrate), PreviousBitrate(InPreviousBitrate), bIsDrasticDownswitch(bInIsDrasticDownswitch) {}
		int32 NewBitrate;
		int32 PreviousBitrate;
		bool bIsDrasticDownswitch;
	};
	struct FPlayerMetricEvent_CodecFormatChange : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_CodecFormatChange(const FStreamCodecInformation& InNewDecodingFormat) : FPlayerMetricEventBase(EType::CodecFormatChange), NewDecodingFormat(InNewDecodingFormat) {}
		FStreamCodecInformation NewDecodingFormat;
	};
	struct FPlayerMetricEvent_JumpInPlayPosition : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_JumpInPlayPosition(const FTimeValue& InToNewTime, const FTimeValue& InFromTime, Metrics::ETimeJumpReason InTimejumpReason) : FPlayerMetricEventBase(EType::JumpInPlayPosition), ToNewTime(InToNewTime), FromTime(InFromTime), TimejumpReason(InTimejumpReason) {}
		FTimeValue ToNewTime;
		FTimeValue FromTime;
		Metrics::ETimeJumpReason TimejumpReason;
	};
	struct FPlayerMetricEvent_MediaMetadataChange : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_MediaMetadataChange(const TSharedPtrTS<Electra::FMetadataParser>& InMetadata) : FPlayerMetricEventBase(EType::MediaMetadataChanged), NewMetadata(InMetadata) {}
		TSharedPtrTS<Electra::FMetadataParser> NewMetadata;
	};
	struct FPlayerMetricEvent_Error : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_Error(const FString& InErrorReason) : FPlayerMetricEventBase(EType::Error), ErrorReason(InErrorReason) {}
		FString ErrorReason;
	};
	struct FPlayerMetricEvent_LogMessage : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_LogMessage(IInfoLog::ELevel InLogLevel, const FString& InLogMessage, int64 InPlayerWallclockMilliseconds) : FPlayerMetricEventBase(EType::LogMessage), LogLevel(InLogLevel), LogMessage(InLogMessage), PlayerWallclockMilliseconds(InPlayerWallclockMilliseconds) {}
		IInfoLog::ELevel LogLevel;
		FString LogMessage;
		int64 PlayerWallclockMilliseconds;
	};
	struct FPlayerMetricEvent_DroppedFrame : public FPlayerMetricEventBase
	{
		FPlayerMetricEvent_DroppedFrame(EType InType, const Metrics::FFrameStats& InFrameStats)	: FPlayerMetricEventBase(InType), FrameStats(InFrameStats) {}
		Metrics::FFrameStats FrameStats;
	};


	/** Analytics event */
	struct FAnalyticsEvent
	{
		FString EventName;
		TArray<FAnalyticsEventAttribute> ParamArray;
	};


	class FAverageValue
	{
	public:
		FAverageValue()
		: Samples(nullptr)
		, NumSamples(0)
		, MaxSamples(0)
		{
		}
		~FAverageValue()
		{
			delete[] Samples;
		}
		void SetNumSamples(int32 InMaxSamples)
		{
			check(InMaxSamples > 0);
			delete[] Samples;
			NumSamples = 0;
			MaxSamples = InMaxSamples;
			Samples = new double[MaxSamples];
		}
		void AddValue(double Value)
		{
			Samples[NumSamples % MaxSamples] = Value;
			++NumSamples;
		}
		void Reset()
		{
			NumSamples = 0;
		}
		double GetAverage() const
		{
			double Avg = 0.0;
			if (NumSamples > 0)
			{
				double Sum = 0.0;
				int32 Last = NumSamples <= MaxSamples ? NumSamples : MaxSamples;
				for (int32 i = 0; i < Last; ++i)
				{
					Sum += Samples[i];
				}
				Avg = Sum / Last;
			}
			return Avg;
		}
	private:
		double*	Samples;
		int32	NumSamples;
		int32	MaxSamples;
	};

	struct FStatistics
	{
		struct FBandwidth
		{
			FBandwidth()
			{
				Bandwidth.SetNumSamples(3);
				Latency.SetNumSamples(3);
				Reset();
			}
			void Reset()
			{
				Bandwidth.Reset();
				Latency.Reset();
			}
			void AddSample(double InBytesPerSecond, double InLatency)
			{
				Bandwidth.AddValue(InBytesPerSecond);
				Latency.AddValue(InLatency);
			}
			double GetAverageBandwidth() const
			{
				return Bandwidth.GetAverage();
			}
			double GetAverageLatency() const
			{
				return Latency.GetAverage();
			}
			FAverageValue	Bandwidth;
			FAverageValue	Latency;
		};
		struct FHistoryEntry
		{
			double TimeSinceStart = 0.0;
			FString Message;
		};
		FStatistics()
		{
			Reset();
		}
		void Reset()
		{
			InitialURL.Empty();
			CurrentlyActivePlaylistURL.Empty();
			LastError.Empty();
			LastState = "Empty";
			TimeAtOpen = -1.0;
			TimeToLoadMainPlaylist = -1.0;
			TimeToLoadStreamPlaylists = -1.0;
			InitialBufferingDuration = -1.0;
			InitialVideoStreamBitrate = 0;
			InitialAudioStreamBitrate = 0;
			TimeAtPrerollBegin = -1.0;
			TimeForInitialPreroll = -1.0;
			NumTimesRebuffered = 0;
			NumTimesForwarded = 0;
			NumTimesRewound = 0;
			NumTimesLooped = 0;
			TimeAtBufferingBegin = 0.0;
			TotalRebufferingDuration = 0.0;
			LongestRebufferingDuration = 0.0;
			PlayPosAtStart = -1.0;
			PlayPosAtEnd = -1.0;
			NumVideoQualityUpswitches = 0;
			NumVideoQualityDownswitches = 0;
			NumVideoQualityDrasticDownswitches = 0;
			NumAudioQualityUpswitches = 0;
			NumAudioQualityDownswitches = 0;
			NumAudioQualityDrasticDownswitches = 0;
			NumVideoDatabytesStreamed = 0;
			NumAudioDatabytesStreamed = 0;
			NumSegmentDownloadsAborted = 0;
			CurrentlyActiveResolutionWidth = 0;
			CurrentlyActiveResolutionHeight = 0;
			VideoSegmentBitratesStreamed.Empty();
			AudioSegmentBitratesStreamed.Empty();
			VideoQualityPercentages.Empty();
			AudioQualityPercentages.Empty();
			NumVideoSegmentsStreamed = 0;
			NumAudioSegmentsStreamed = 0;
			InitialBufferingBandwidth.Reset();
			bIsInitiallyDownloading = false;
			bDidPlaybackEnd = false;
			MediaTimelineAtStart.Reset();
			MediaTimelineAtEnd.Reset();
			MediaDuration = 0.0;
			MessageHistoryBuffer.Empty();
			NumErr404 = 0;
			NumErr4xx = 0;
			NumErr5xx = 0;
			NumErrTimeouts = 0;
			NumErrConnDrops = 0;
			NumErrOther = 0;
			NumTotalAddedVideoFrames = 0;
			NumTotalDroppedVideoFrames = 0;
			AvgVideoDropsPerInterval = 0;
			MinVideoDropsPerInterval = 0;
			MaxVideoDropsPerInterval = 0;
			VideoDropIntervalMs = 0;
			VideoNumIntervals = 0;
			VideoNumIntervalsWithDrops = 0;
		}
		void AddMessageToHistory(FString InMessage);

		// n elements by descending bitrate holding a percentage of this bitrate being used
		using FQualityPercentages = TSortedMap<int32, int32, FDefaultAllocator, TGreater<int32>>;

		FString					InitialURL;
		FString					CurrentlyActivePlaylistURL;
		FString					LastError;
		FString					LastState;		// "Empty", "Opening", "Preparing", "Buffering", "Idle", "Ready", "Playing", "Paused", "Seeking", "Rebuffering", "Ended"
		double					TimeAtOpen;
		double					TimeToLoadMainPlaylist;
		double					TimeToLoadStreamPlaylists;
		double					InitialBufferingDuration;
		int32					InitialVideoStreamBitrate;
		int32					InitialAudioStreamBitrate;
		double					TimeAtPrerollBegin;
		double					TimeForInitialPreroll;
		int32					NumTimesRebuffered;
		int32					NumTimesForwarded;
		int32					NumTimesRewound;
		int32					NumTimesLooped;
		double					TimeAtBufferingBegin;
		double					TotalRebufferingDuration;
		double					LongestRebufferingDuration;
		double					PlayPosAtStart;
		double					PlayPosAtEnd;
		int32					NumVideoQualityUpswitches;
		int32					NumVideoQualityDownswitches;
		int32					NumVideoQualityDrasticDownswitches;
		int32					NumAudioQualityUpswitches;
		int32					NumAudioQualityDownswitches;
		int32					NumAudioQualityDrasticDownswitches;
		int64					NumVideoDatabytesStreamed;
		int64					NumAudioDatabytesStreamed;
		int32					NumSegmentDownloadsAborted;
		int32					CurrentlyActiveResolutionWidth;
		int32					CurrentlyActiveResolutionHeight;
		TMap<int32, uint32>		VideoSegmentBitratesStreamed;		// key=video stream bitrate, value=number of segments loaded at this rate
		TMap<int32, uint32>		AudioSegmentBitratesStreamed;		// key=audio stream bitrate, value=number of segments loaded at this rate
		FQualityPercentages		VideoQualityPercentages;
		FQualityPercentages		AudioQualityPercentages;
		uint32					NumVideoSegmentsStreamed;
		uint32					NumAudioSegmentsStreamed;
		FBandwidth				InitialBufferingBandwidth;
		bool					bIsInitiallyDownloading;
		bool					bDidPlaybackEnd;
		FTimeRange				MediaTimelineAtStart;
		FTimeRange				MediaTimelineAtEnd;
		double					MediaDuration;
		TArray<FHistoryEntry>	MessageHistoryBuffer;
		uint32					NumErr404;
		uint32					NumErr4xx;
		uint32					NumErr5xx;
		uint32					NumErrTimeouts;
		uint32					NumErrConnDrops;
		uint32					NumErrOther;
		uint64					NumTotalAddedVideoFrames;
		uint64					NumTotalDroppedVideoFrames;
		uint32					AvgVideoDropsPerInterval;
		uint32					MinVideoDropsPerInterval;
		uint32					MaxVideoDropsPerInterval;
		uint32					VideoDropIntervalMs;
		uint32					VideoNumIntervals;
		uint32					VideoNumIntervalsWithDrops;
	};


	/** Media event sink */
	IMediaEventSink* EventSink = nullptr;

	/** Metric delegates */
	FElectraPlayerSendAnalyticMetricsDelegate& SendAnalyticMetricsDelegate;
	FElectraPlayerSendAnalyticMetricsPerMinuteDelegate&	SendAnalyticMetricsPerMinuteDelegate;
	FElectraPlayerReportVideoStreamingErrorDelegate& ReportVideoStreamingErrorDelegate;
	FElectraPlayerReportSubtitlesMetricsDelegate& ReportSubtitlesMetricsDelegate;

	/** Empty sample queue */
	TUniquePtr<FMediaSamples> EmptyMediaSamples;

	/** Option interface */
	TWeakPtr<IElectraSafeMediaOptionInterface, ESPMode::ThreadSafe> OptionInterface;

	/** Current player stream metadata */
	mutable TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> CurrentMetadata;
	mutable bool bMetadataChanged = false;

	/** Output sample pools */
	TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> OutputTexturePool;
	int64 OutputTexturePoolDecoderID = 0;
	TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> OutputAudioSamplePool;
	bool bRetainPoolsForReuse = false;

	/** Application system event handler */
	TSharedPtr<Electra::FApplicationTerminationHandler, ESPMode::ThreadSafe> AppTerminationHandler;

	/** Media player Guid */
	FGuid PlayerGuid;
	uint32 PlayerUniqueID = 0;
	static std::atomic<uint32> NextPlayerUniqueID;

	/** Analytics */
	TQueue<TSharedPtr<FAnalyticsEvent>> QueuedAnalyticEvents;
	FString AnalyticsCustomValues[8];
	FString AnalyticsOSVersion;
	FString AnalyticsGPUType;
	FString AnalyticsInstanceGuid;
	uint32 AnalyticsInstanceEventCount;
	std::atomic<int32> NumQueuedAnalyticEvents { 0 };

	/** Statistics */
	FCriticalSection StatisticsLock;
	FStatistics Statistics;

	/** External resource provider. */
	TSharedPtr<FAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe> StaticResourceProvider;

	/** Internal states. */
	FPlaystartOptions PlaystartOptions;
	FString MediaUrl;
	TOptional<bool> bFrameAccurateSeeking;
	TOptional<bool> bEnableLooping;

	TQueue<TSharedPtrTS<FPlayerMetricEventBase>> DeferredPlayerEvents;
	TQueue<EMediaEvent> DeferredMediaEvents;
	mutable FCriticalSection VideoFormatLock;
	FVideoStreamFormat CurrentlyActiveVideoStreamFormat;
	int32 NumTracksAudio = 0;
	int32 NumTracksVideo = 0;
	int32 NumTracksSubtitle = 0;
	int32 SelectedQuality = 0;
	mutable int32 SelectedVideoTrackIndex = -1;
	mutable int32 SelectedAudioTrackIndex = -1;
	mutable int32 SelectedSubtitleTrackIndex = -1;
	mutable bool bVideoTrackIndexDirty = true;
	mutable bool bAudioTrackIndexDirty = true;
	mutable bool bSubtitleTrackIndexDirty = true;
	bool bInitialSeekPerformed = false;
	bool bIsFirstBuffering = true;
	TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> CurrentStreamMetadata;

	TSharedPtr<FInternalPlayerImpl, ESPMode::ThreadSafe> CurrentPlayer;
	TSharedPtr<FAEMSEventReceiver, ESPMode::ThreadSafe> MediaPlayerEventReceiver;
	TSharedPtr<IAsyncResourceReleaseNotification, ESPMode::ThreadSafe> AsyncResourceReleaseNotification;

	TRange<FTimespan> CurrentPlaybackRange;
	FPlayerState PlayerState;
	bool bHasPendingError = false;
	bool bHasClosedDueToError = false;
};
