// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"

#include "PlayerRuntimeGlobal.h"
#include "Core/MediaInterlocked.h"

#include "IElectraPlayerRuntimeModule.h"
#include "ElectraPlayerPrivate.h"
#include "ElectraPlayer.h"

DEFINE_LOG_CATEGORY(LogElectraPlayer);
DEFINE_LOG_CATEGORY(LogElectraPlayerStats);

#define LOCTEXT_NAMESPACE "ElectraPlayerRuntimeModule"

// -----------------------------------------------------------------------------------------------------------------------------------

// Implements the ElectraPlayer Runtime Module module.
class FElectraPlayerRuntimeModule : public IElectraPlayerRuntimeModule
{
public:
	bool IsInitialized() const override
	{
		return bInitialized;
	}

	void StartupModule() override
	{
		LLM_SCOPE_BYNAME(TEXT("StartupModule"));
		check(!bInitialized);
	
		// Read Analytics Events from ini configuring the events sent during playback
		Electra::Configuration ElectraConfig;
		FString AnalyticsEventsFromIni;
		if (GConfig->GetString(TEXT("ElectraPlayer"), TEXT("AnalyticsEvents"), AnalyticsEventsFromIni, GEngineIni))
		{
			// Parse comma delimited strings into arrays
			TArray<FString> EnabledAnalyticsEvents;
			AnalyticsEventsFromIni.ParseIntoArray(EnabledAnalyticsEvents, TEXT(","), /*bCullEmpty=*/true);
			for (auto EnabledEvent : EnabledAnalyticsEvents)
			{
				EnabledEvent.TrimStartAndEndInline();
				ElectraConfig.EnabledAnalyticsEvents.Add(EnabledEvent, true);
			}
		}
		UE_LOGF(LogElectraPlayer, Verbose, "Found %d enabled ElectraPlayer analytic events.", ElectraConfig.EnabledAnalyticsEvents.Num());

		// Core
		Electra::Startup(ElectraConfig);

		bInitialized = true;
	}

	void ShutdownModule() override
	{
		if (bInitialized)
		{
			// Wait for players to have terminated. If this fails then do not shutdown the sub components
			// to avoid potential hangs in there.
			if (Electra::WaitForAllPlayersToHaveTerminated())
			{
				Electra::Shutdown();
			}
			else
			{
				UE_LOGF(LogElectraPlayer, Warning, "Shutting down with active player instances. This could lead to problems.");

				// At least unbind all the application notification handlers.
				Electra::Shutdown();
			}
			bInitialized = false;
		}
	}

private:
	bool bInitialized = false;
};

IMPLEMENT_MODULE(FElectraPlayerRuntimeModule, ElectraPlayerRuntime);

#undef LOCTEXT_NAMESPACE
