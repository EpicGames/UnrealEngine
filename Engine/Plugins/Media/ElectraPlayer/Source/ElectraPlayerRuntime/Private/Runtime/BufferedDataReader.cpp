// Copyright Epic Games, Inc. All Rights Reserved.

#include "BufferedDataReader.h"
#include "InfoLog.h"

DECLARE_LOG_CATEGORY_EXTERN(LogElectraBufferReader, Log, All);
DEFINE_LOG_CATEGORY(LogElectraBufferReader);

namespace Electra
{

	enum EBufferedDataReaderError
	{
		FileTooShort = 1,
		ReadAssetDataFailed = 2,
	};

	void FBufferedDataReader::CreateNewArea(int64 NumBytes, int64 FromOffset)
	{
		if (TotalDataSize >= 0 && FromOffset + NumBytes > TotalDataSize)
		{
			UE_LOGF(LogElectraBufferReader, VeryVerbose, "CreateNewArea(): clamp request to end of file size");
			NumBytes = TotalDataSize - FromOffset;
		}

		TUniquePtr<FArea> NewArea = MakeUnique<FArea>();
		NewArea->Data = (const uint8*) FMemory::Malloc(NumBytes);
		if (NewArea->Data)
		{
			NewArea->Size = NumBytes;
			NewArea->StartOffset = FromOffset;
			int64 TotalSize = -1;
			int64 NumRead = DataProvider->OnReadAssetData((void*) NewArea->Data, NumBytes, FromOffset, &TotalSize);
			UE_LOGF(LogElectraBufferReader, VeryVerbose, "CreateNewArea(): read %lld bytes from offset %lld; total file size = %lld, received %lld bytes", (long long int)NumBytes, (long long int)FromOffset, (long long int)TotalSize, (long long int)NumRead);
			if (TotalSize >= 0)
			{
				TotalDataSize = TotalSize;
			}
			if (NumRead >= 0)
			{
				if (NumRead < NumBytes || (TotalSize >= 0 && FromOffset + NumRead >= TotalSize))
				{
					NewArea->bEOS = true;
				}
				// Any existing area before this one can't be at EOS any more.
				for(auto &a : Areas)
				{
					if (a->StartOffset < FromOffset)
					{
						a->bEOS = false;
					}
					else
					{
						break;
					}
				}
				NewArea->Size = NumRead;
				BytesRemainingInArea = NewArea->Size;
				CurrentArea = NewArea.Get();
				Areas.Emplace(MoveTemp(NewArea));
				Areas.Sort([](const TUniquePtr<FArea>& a, const TUniquePtr<FArea>& b)
				{
					return a->StartOffset < b->StartOffset;
				});
			}
			else
			{
				IDataProvider::EError Error = static_cast<IDataProvider::EError>(NumRead);
				if (Error == IDataProvider::EError::Failed)
				{
					LastError.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::BufferedDataReader).SetCode(EBufferedDataReaderError::ReadAssetDataFailed).SetMessage(TEXT("OnReadAssetData() failed"));
				}
			}
		}
	}


	bool FBufferedDataReader::EnlargeCurrentAreaBy(int64 NumBytesToAdd)
	{
		// Reading past the known end of the file?
		if (TotalDataSize >= 0 && CurrentArea->StartOffset + CurrentArea->Size + NumBytesToAdd > TotalDataSize)
		{
			UE_LOGF(LogElectraBufferReader, VeryVerbose, "EnlargeCurrentAreaBy(): clamp request of %lld to %lld at end of file", (long long int)NumBytesToAdd, (long long int)(TotalDataSize - (CurrentArea->StartOffset + CurrentArea->Size)));
			NumBytesToAdd = TotalDataSize - (CurrentArea->StartOffset + CurrentArea->Size);
		}

		// Check if there is an area following the current one and whether we would reach into it.
		int32 CurrentIndex = Areas.IndexOfByPredicate([&](const TUniquePtr<FArea>& e) { return e.Get() == CurrentArea; });
		check(CurrentIndex != INDEX_NONE);
		if (CurrentIndex == INDEX_NONE)
		{
			return false;
		}
		FArea* Next = Areas.Num() > CurrentIndex+1 ? Areas[CurrentIndex+1].Get() : nullptr;
		int64 ReallocSize = CurrentArea->Size + NumBytesToAdd;
		// Will we reach into the next area if we enlarge this one by the given amount?
		if (Next && CurrentArea->StartOffset + CurrentArea->Size + NumBytesToAdd >= Next->StartOffset)
		{
			UE_LOGF(LogElectraBufferReader, VeryVerbose, "EnlargeCurrentAreaBy(): reaching into area of offset %lld, clamping", (long long int)Next->StartOffset);

			// Yes. Trim the amount to read to avoid overlap.
			NumBytesToAdd = Next->StartOffset - (CurrentArea->StartOffset + CurrentArea->Size);
			// We need one large block for the current block plus the amount to read plus the next block.
			ReallocSize = CurrentArea->Size + NumBytesToAdd + Next->Size;
		}
		else
		{
			// No, the areas will remain separate. No need to consolidate after reading.
			Next = nullptr;
		}

		CurrentArea->Data = (const uint8*) FMemory::Realloc((void*)CurrentArea->Data, ReallocSize);
		if (CurrentArea->Data)
		{
			int64 TotalSize = -1;
			int64 NumRead = DataProvider->OnReadAssetData((void*)(CurrentArea->Data + CurrentArea->Size), NumBytesToAdd, CurrentArea->StartOffset + CurrentArea->Size, &TotalSize);
			UE_LOGF(LogElectraBufferReader, VeryVerbose, "CreateNewArea(): read %lld bytes from offset %lld; total file size = %lld, received %lld bytes", (long long int)NumBytesToAdd, (long long int)(CurrentArea->StartOffset + CurrentArea->Size), (long long int)TotalSize, (long long int)NumRead);
			if (TotalSize >= 0)
			{
				TotalDataSize = TotalSize;
			}
			if (NumRead >= 0)
			{
				if (NumRead < NumBytesToAdd || (TotalSize >= 0 && CurrentArea->StartOffset + CurrentArea->Size + NumRead >= TotalSize))
				{
					UE_LOGF(LogElectraBufferReader, VeryVerbose, "EnlargeCurrentAreaBy(): reached the end of the file");
					CurrentArea->bEOS = true;
				}
				CurrentArea->Size += NumRead;
				BytesRemainingInArea += NumRead;
				// Consolidate forward?
				if (Next)
				{
					UE_LOGF(LogElectraBufferReader, VeryVerbose, "EnlargeCurrentAreaBy(): merging blocks from offsets %lld and %lld into one", (long long int)CurrentArea->StartOffset, (long long int)Next->StartOffset);
					FMemory::Memcpy((void*)(CurrentArea->Data + CurrentArea->Size), Next->Data, Next->Size);
					CurrentArea->bEOS = Next->bEOS;
					CurrentArea->Size += Next->Size;
					BytesRemainingInArea += Next->Size;
					Areas.RemoveAt(CurrentIndex + 1);
				}
				return true;
			}
			else
			{
				IDataProvider::EError Error = static_cast<IDataProvider::EError>(NumRead);
				if (Error == IDataProvider::EError::Failed)
				{
					LastError.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::BufferedDataReader).SetCode(EBufferedDataReaderError::ReadAssetDataFailed).SetMessage(TEXT("OnReadAssetData() failed"));
				}
			}
		}
		return false;
	}

	bool FBufferedDataReader::PrepareToRead(int64 NumBytes)
	{
		// No area?
		if (!CurrentArea)
		{
			// Create a new area and fill it with data.
			CreateNewArea(NumBytes > kDefaultReadSize ? NumBytes: kDefaultReadSize, CurrentOffset);
			UpdateReadDataPointer();
			return CurrentArea != nullptr;
		}
		// Not enough data in the area?
		while(BytesRemainingInArea < NumBytes && !CurrentArea->bEOS)
		{
			// Enlarge the area by at least a default size to avoid repeated resizes.
			if (!EnlargeCurrentAreaBy((NumBytes > kDefaultReadSize ? NumBytes: kDefaultReadSize) - BytesRemainingInArea))
			{
				return false;
			}
		}
		if (BytesRemainingInArea < NumBytes)
		{
			return false;
		}
		UpdateReadDataPointer();
		return true;
	}

	bool FBufferedDataReader::IsAtEOS()
	{
		if (CurrentArea)
		{
			return BytesRemainingInArea == 0 && CurrentArea->bEOS;
		}
		return false;
	}


	bool FBufferedDataReader::SkipOver(int64 NumBytes)
	{
#if 1
		return SeekTo(CurrentOffset + NumBytes);
#else
		if (!PrepareToRead(NumBytes))
		{
			return false;
		}
		if (BytesRemainingInArea < NumBytes)
		{
			LastError.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::BufferedDataReader).SetCode(EBufferedDataReaderError::FileTooShort).SetMessage(TEXT("File too short to skip requested amount of bytes"));
			return false;
		}
		BytesRemainingInArea -= NumBytes;
		CurrentOffset += NumBytes;
		return true;
#endif
	}


	bool FBufferedDataReader::ReadByteArray(TArray<uint8>& OutValue, int64 NumBytes)
	{
		if (!PrepareToRead(NumBytes))
		{
			return false;
		}
		NumBytes = NumBytes <= BytesRemainingInArea ? NumBytes : BytesRemainingInArea;
		OutValue.AddUninitialized(NumBytes);
		FMemory::Memcpy(OutValue.GetData(), ReadDataPointer, NumBytes);
		CurrentOffset += NumBytes;
		BytesRemainingInArea -= NumBytes;
		return true;
	}

	bool FBufferedDataReader::PeekU8(uint8& OutValue)
	{
		if (!PrepareToRead(sizeof(uint8)))
		{
			return false;
		}
		OutValue = *ReadDataPointer;
		return true;
	}
	bool FBufferedDataReader::PeekU16BE(uint16& OutValue)
	{
		if (!PrepareToRead(sizeof(uint16)))
		{
			return false;
		}
		const uint16* rp = reinterpret_cast<const uint16*>(ReadDataPointer);
		#if PLATFORM_LITTLE_ENDIAN
			OutValue = Utils::EndianSwap(*rp);
		#else
			OutValue = *rp;
		#endif
		return true;
	}
	bool FBufferedDataReader::PeekU32BE(uint32& OutValue)
	{
		if (!PrepareToRead(sizeof(uint32)))
		{
			return false;
		}
		const uint32* rp = reinterpret_cast<const uint32*>(ReadDataPointer);
		#if PLATFORM_LITTLE_ENDIAN
			OutValue = Utils::EndianSwap(*rp);
		#else
			OutValue = *rp;
		#endif
		return true;
	}
	bool FBufferedDataReader::PeekU64BE(uint64& OutValue)
	{
		if (!PrepareToRead(sizeof(uint64)))
		{
			return false;
		}
		const uint64* rp = reinterpret_cast<const uint64*>(ReadDataPointer);
		#if PLATFORM_LITTLE_ENDIAN
			OutValue = Utils::EndianSwap(*rp);
		#else
			OutValue = *rp;
		#endif
		return true;
	}
	bool FBufferedDataReader::PeekU16LE(uint16& OutValue)
	{
		if (!PrepareToRead(sizeof(uint16)))
		{
			return false;
		}
		const uint16* rp = reinterpret_cast<const uint16*>(ReadDataPointer);
		#if PLATFORM_LITTLE_ENDIAN
			OutValue = *rp;
		#else
			OutValue = Utils::EndianSwap(*rp);
		#endif
		return true;
	}
	bool FBufferedDataReader::PeekU32LE(uint32& OutValue)
	{
		if (!PrepareToRead(sizeof(uint32)))
		{
			return false;
		}
		const uint32* rp = reinterpret_cast<const uint32*>(ReadDataPointer);
		#if PLATFORM_LITTLE_ENDIAN
			OutValue = *rp;
		#else
			OutValue = Utils::EndianSwap(*rp);
		#endif
		return true;
	}
	bool FBufferedDataReader::PeekU64LE(uint64& OutValue)
	{
		if (!PrepareToRead(sizeof(uint64)))
		{
			return false;
		}
		const uint64* rp = reinterpret_cast<const uint64*>(ReadDataPointer);
		#if PLATFORM_LITTLE_ENDIAN
			OutValue = *rp;
		#else
			OutValue = Utils::EndianSwap(*rp);
		#endif
		return true;
	}

	bool FBufferedDataReader::ReadU8(uint8& OutValue)
	{
		if (PeekU8(OutValue))
		{
			CurrentOffset += sizeof(uint8);
			BytesRemainingInArea -= sizeof(uint8);
			return true;
		}
		return false;
	}
	bool FBufferedDataReader::ReadU16LE(uint16& OutValue)
	{
		if (PeekU16LE(OutValue))
		{
			CurrentOffset += sizeof(uint16);
			BytesRemainingInArea -= sizeof(uint16);
			return true;
		}
		return false;
	}
	bool FBufferedDataReader::ReadU32LE(uint32& OutValue)
	{
		if (PeekU32LE(OutValue))
		{
			CurrentOffset += sizeof(uint32);
			BytesRemainingInArea -= sizeof(uint32);
			return true;
		}
		return false;
	}
	bool FBufferedDataReader::ReadU64LE(uint64& OutValue)
	{
		if (PeekU64LE(OutValue))
		{
			CurrentOffset += sizeof(uint64);
			BytesRemainingInArea -= sizeof(uint64);
			return true;
		}
		return false;
	}
	bool FBufferedDataReader::ReadU16BE(uint16& OutValue)
	{
		if (PeekU16BE(OutValue))
		{
			CurrentOffset += sizeof(uint16);
			BytesRemainingInArea -= sizeof(uint16);
			return true;
		}
		return false;
	}
	bool FBufferedDataReader::ReadU32BE(uint32& OutValue)
	{
		if (PeekU32BE(OutValue))
		{
			CurrentOffset += sizeof(uint32);
			BytesRemainingInArea -= sizeof(uint32);
			return true;
		}
		return false;
	}
	bool FBufferedDataReader::ReadU64BE(uint64& OutValue)
	{
		if (PeekU64BE(OutValue))
		{
			CurrentOffset += sizeof(uint64);
			BytesRemainingInArea -= sizeof(uint64);
			return true;
		}
		return false;
	}

	bool FBufferedDataReader::SeekTo(int64 AbsolutePosition)
	{
		if (TotalDataSize >= 0 && AbsolutePosition > TotalDataSize)
		{
			UE_LOGF(LogElectraBufferReader, VeryVerbose, "SeekTo(): Seeking beyond the end of the file");
			return false;
		}

		// Check if we have the area at the target position.
		FArea* DestinationArea = nullptr;
		for(int32 i=0,iMax=Areas.Num(); i<iMax; ++i)
		{
			// We allow to set the position on the end of an area (<=) which will automatically resize the area
			// on the next read. That way we do not need to consolidate blocks backwards.
			if (AbsolutePosition >= Areas[i]->StartOffset && AbsolutePosition <= Areas[i]->StartOffset+Areas[i]->Size)
			{
				DestinationArea = Areas[i].Get();
				break;
			}
		}
		if (DestinationArea)
		{
			CurrentArea = DestinationArea;
			CurrentOffset = AbsolutePosition;
			BytesRemainingInArea = DestinationArea->StartOffset + DestinationArea->Size - AbsolutePosition;
			UpdateReadDataPointer();
		}
		else
		{
			// Area not loaded yet. Set data pointers to zero so the next read will
			// load in the area at that location.
			CurrentArea = nullptr;
			CurrentOffset = AbsolutePosition;
			BytesRemainingInArea = 0;
			ReadDataPointer = nullptr;
		}
		return true;
	}



} // namespace Electra
