// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Build.h"
#include "Utilities/Utilities.h"
#include "ErrorDetail.h"

namespace Electra
{
	class IGenericDataReader
	{
	public:
		virtual ~IGenericDataReader() = default;
		/**
		 * Read n bytes of data starting at offset o into the provided buffer.
		 *
		 * Reading must return the number of bytes asked to get, if necessary by blocking.
		 * If a read error prevents reading the number of bytes -1 must be returned.
		 *
		 * @param IntoBuffer Buffer into which to store the data bytes. If nullptr is passed the data must be skipped over.
		 * @param NumBytesToRead The number of bytes to read. Must not read more bytes and should be no less than requested.
		 * @return The number of bytes read or -1 on a read error. If the read would go beyond the size of the file then
		 *         returning fewer bytes than requested is permitted in this case ONLY.
		 */
		virtual int64 ReadData(void* InDestinationBuffer, int64 InNumBytesToRead, int64 InFromOffset) = 0;

		/**
		 * Returns the current internal read offset.
		 */
		virtual int64 GetCurrentOffset() const = 0;

		/**
		 * Returns the total size of the file.
		 * The size should be available at least after the first call to ReadData().
		 * If the total length is not known, return -1.
		 */
		virtual int64 GetTotalSize() const = 0;

		/**
		 * Checks if reading of the file and therefore parsing has been aborted.
		 *
		 * @return true if reading/parsing has been aborted, false otherwise.
		 */
		virtual bool HasReadBeenAborted() const = 0;

		/**
		 * Checks if the data source has reached the End Of File (EOF) and cannot provide any additional data.
		 *
		 * @return If EOF has been reached returns true, otherwise false.
		 */
		virtual bool HasReachedEOF() const = 0;
	};



	class FBufferedDataReader
	{
	public:
		class IDataProvider
		{
		public:
			enum class EError
			{
				Failed = -1,
				EOS = -2,
				Aborted = -3
			};
			virtual int64 OnReadAssetData(void* Destination, int64 NumBytes, int64 FromOffset, int64* OutTotalSize) = 0;
		};

		FBufferedDataReader(IDataProvider* InDataProvider)
			: DataProvider(InDataProvider)
		{ }

		bool Failed() const
		{
			return LastError.IsSet();
		}
		FErrorDetail GetLastError() const
		{
			return LastError;
		}

		int64 GetCurrentOffset() const
		{
			return CurrentOffset;
		}

		int64 GetTotalDataSize() const
		{
			return TotalDataSize;
		}

		bool PrepareToRead(int64 NumBytes);
		bool ReadU8(uint8& OutValue);
		bool ReadU16LE(uint16& OutValue);
		bool ReadU32LE(uint32& OutValue);
		bool ReadU64LE(uint64& OutValue);
		bool ReadU16BE(uint16& OutValue);
		bool ReadU32BE(uint32& OutValue);
		bool ReadU64BE(uint64& OutValue);
		bool PeekU8(uint8& OutValue);
		bool PeekU16LE(uint16& OutValue);
		bool PeekU32LE(uint32& OutValue);
		bool PeekU64LE(uint64& OutValue);
		bool PeekU16BE(uint16& OutValue);
		bool PeekU32BE(uint32& OutValue);
		bool PeekU64BE(uint64& OutValue);
		bool SkipOver(int64 NumBytes);
		bool ReadByteArray(TArray<uint8>& OutValue, int64 NumBytes);
		bool SeekTo(int64 AbsolutePosition);
		bool IsAtEOS();
	protected:
		enum
		{
			kDefaultReadSize = 65536
		};

		struct FArea
		{
			~FArea()
			{
				FMemory::Free((void*) Data);
			}
			const uint8* Data = nullptr;
			int64 Size = 0;
			int64 StartOffset = 0;
			bool bEOS = false;
		};

		FArea* FindAreaForOffset(int64 Offset);
		void CreateNewArea(int64 NumBytes, int64 FromOffset);
		bool EnlargeCurrentAreaBy(int64 NumBytesToAdd);

		void UpdateReadDataPointer()
		{
			ReadDataPointer = !CurrentArea ? nullptr : CurrentArea->Data + CurrentOffset - CurrentArea->StartOffset;
		}

		IDataProvider* DataProvider = nullptr;
		FErrorDetail LastError;

		TArray<TUniquePtr<FArea>> Areas;
		int64 TotalDataSize = -1;

		FArea* CurrentArea = nullptr;
		int64 BytesRemainingInArea = 0;

		int64 CurrentOffset = 0;
		const uint8* ReadDataPointer = nullptr;
	};

} // namespace Electra
