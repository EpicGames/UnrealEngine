// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/LowLevelMemTracker.h"
#include "Stats/Stats.h"

#include "PlayerCore.h"
#include "PlayerRuntimeGlobal.h"
#include "ElectraPlayerPrivate.h"

#include "StreamAccessUnitBuffer.h"
#include "Decoder/AudioDecoder.h"
#include "OutputHandler.h"
#include "Player/PlayerSessionServices.h"
#include "Player/AdaptivePlayerMetadataKeynames.h"
#include "Utilities/Utilities.h"
#include "Utilities/StringHelpers.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/AudioChannelMapper.h"
#include "MP4Utilities.h"

// Error codes must be in the 1000-1999 range. 1-999 is reserved for the decoder implementation.
#define ERRCODE_AUDIO_INTERNAL_COULD_NOT_CREATE_DECODER			1001
#define ERRCODE_AUDIO_INTERNAL_COULD_NOT_CREATE_SAMPLE_POOL		1002
#define ERRCODE_AUDIO_INTERNAL_COULD_NOT_GET_SAMPLE_BUFFER		1003
#define ERRCODE_AUDIO_INTERNAL_UNSUPPORTED_NUMBER_OF_CHANNELS	1004
#define ERRCODE_AUDIO_INTERNAL_UNSUPPORTED_OUTPUT_FORMAT		1005
#define ERRCODE_AUDIO_INTERNAL_FAILED_TO_CONVERT_OUTPUT			1006


#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputAudio.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"

/***************************************************************************************************************************************************/

DECLARE_CYCLE_STAT(TEXT("FAudioDecoderImpl::Decode()"), STAT_ElectraPlayer_AudioDecode, STATGROUP_ElectraPlayer);
DECLARE_CYCLE_STAT(TEXT("FAudioDecoderImpl::ConvertOutput()"), STAT_ElectraPlayer_AudioConvertOutput, STATGROUP_ElectraPlayer);


namespace Electra
{

class FAudioDecoderImpl : public IAudioDecoder, public FMediaThread
{
public:
	FAudioDecoderImpl();
	virtual ~FAudioDecoderImpl();

	static TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> GetDecoderFactory(TMap<FString, FVariant>& OutAddtlCfg, const FCodecTypeFormat& InCodecTypeFormat);

	void SetPlayerSessionServices(IPlayerSessionServices* SessionServices) override;
	void Open(TSharedPtrTS<FAccessUnit::CodecData> InCodecData) override;
	void Close() override;
	void DrainForCodecChange() override;
	void SetOutputHandler(TSharedPtr<FOutputHandlerAudio, ESPMode::ThreadSafe> InOutputHandler) override;
	void SuspendOrResumeDecoder(bool bSuspend, const FParamDict& InOptions) override;
	void AUdataPushAU(FAccessUnit* AccessUnit) override;
	void AUdataPushEOD() override;
	void AUdataClearEOD() override;
	void AUdataFlushEverything() override;
	void SetAUInputBufferListener(IAccessUnitBufferListener* Listener) override;
	void SetReadyBufferListener(IDecoderOutputBufferListener* Listener) override;

private:
	struct FDecoderInput
	{
		~FDecoderInput()
		{
			FAccessUnit::Release(AccessUnit);
		}

		FAccessUnit*	AccessUnit = nullptr;
		bool			bHasBeenPrepared = false;
		int64			PTS = 0;
		int64			EndPTS = 0;
		FTimeValue		AdjustedPTS;
		FTimeValue		AdjustedDuration;
		FTimeValue		StartOverlapDuration;
		FTimeValue		EndOverlapDuration;
	};

	struct FCurrentOutputFormat
	{
		void Reset()
		{
			SampleRate = 0;
			NumChannels = 0;
		}
		int32 SampleRate = 0;
		int32 NumChannels = 0;
	};

	enum class EDecodingState
	{
		NormalDecoding,
		Draining,
		NeedsReset,
		CodecChange
	};

	enum class EAUChangeFlags
	{
		None = 0x00,
		CSDChanged = 0x01,
		Discontinuity = 0x02,
		CodecChange = 0x04
	};
	FRIEND_ENUM_CLASS_FLAGS(EAUChangeFlags);

	enum class ESendMode
	{
		SendSilence,
		SendEOS
	};

	void StartThread();
	void StopThread();
	void WorkerThread();

	void HandleApplicationHasEnteredForeground();
	void HandleApplicationWillEnterBackground();

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();

	void CreateDecoderOutputPool();
	void DestroyDecoderOutputPool();

	void ReturnUnusedOutputBuffer();

	void NotifyReadyBufferListener(bool bHaveOutput);

	bool CanReceiveOutputFrame()
	{
		bool bCanRcv = true;
		if (OutputHandler)
		{
			bCanRcv = OutputHandler->CanReceiveOutputSample();
		}
		return bCanRcv;
	}

	EAUChangeFlags GetAndPrepareInputAU();
	void PrepareAU(TSharedPtrTS<FDecoderInput> AU);
	IElectraDecoder::ECSDCompatibility IsCompatibleWith();

	bool PostError(int32 ApiReturnValue, const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);
	bool PostError(const IElectraDecoder::FError& InDecoderError);
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

	IElectraDecoder::EOutputStatus HandleOutput();
	bool HandleDecoding();
	bool SendSilenceOrEOS(ESendMode InMode);
	void StartDraining(EDecodingState InNextStateAfterDraining);
	bool CheckForFlush();
	bool CheckBackgrounding();

private:
	TSharedPtrTS<FAccessUnit::CodecData>									InitialCodecSpecificData;

	TAccessUnitQueue<TSharedPtrTS<FDecoderInput>>							NextAccessUnits;
	TArray<TSharedPtrTS<FDecoderInput>>										InDecoderInput;
	TSharedPtrTS<FDecoderInput>												CurrentAccessUnit;
	TOptional<int64>														CurrentSequenceIndex;
	TOptional<int64>														NextExpectedDTSHNS;
	EDecodingState															CurrentDecodingState = EDecodingState::NormalDecoding;
	EDecodingState															NextDecodingStateAfterDrain = EDecodingState::NormalDecoding;
	bool																	bIsDecoderClean = true;
	bool																	bDrainAfterDecode = false;

	TSharedPtrTS<FAccessUnit::CodecData>									CurrentCodecData;
	FCurrentOutputFormat													CurrentOutputFormat;
	TMap<FString, FVariant>													CSDOptions;
	bool																	bIsFirstAccessUnit = true;
	bool																	bInDummyDecodeMode = false;
	bool																	bDrainForCodecChange = false;

	bool																	bError = false;

	FMediaEvent																TerminateThreadSignal;
	FMediaEvent																FlushDecoderSignal;
	FMediaEvent																DecoderFlushedSignal;
	bool																	bThreadStarted = false;

	FMediaEvent																ApplicationRunningSignal;
	FMediaEvent																ApplicationSuspendConfirmedSignal;
	TSharedPtrTS<FFGBGNotificationHandlers>									FGBGHandlers;
	int32																	ApplicationSuspendCount = 0;

	TSharedPtr<FOutputHandlerAudio, ESPMode::ThreadSafe>					OutputHandler;
	int32																	MaxOutputBuffers = 0;

	FCriticalSection														ListenerMutex;
	IAccessUnitBufferListener*												InputBufferListener = nullptr;
	IDecoderOutputBufferListener*											ReadyBufferListener = nullptr;

	IPlayerSessionServices* 												SessionServices = nullptr;


	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>						DecoderInstance;
	bool																	bIsAdaptiveDecoder = false;
	bool																	bMustBeSuspendedInBackground = false;

	FElectraAudioSamplePtr													CurrentOutputBuffer;
	FAudioChannelMapper														ChannelMapper;
};
ENUM_CLASS_FLAGS(FAudioDecoderImpl::EAUChangeFlags);


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

bool IAudioDecoder::CanDecodeStream(const FStreamCodecInformation& InCodecInfo)
{
	Electra::FCodecTypeFormat ctf = InCodecInfo.ToCodecTypeFormat();
	TMap<FString, FVariant> AddtlCfg;
	return FAudioDecoderImpl::GetDecoderFactory(AddtlCfg, ctf).IsValid();
}

IAudioDecoder* IAudioDecoder::Create()
{
	return new FAudioDecoderImpl;
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FAudioDecoderImpl::GetDecoderFactory(TMap<FString, FVariant>& OutAddtlCfg, const FCodecTypeFormat& InCodecTypeFormat)
{
	IElectraCodecFactoryModule* FactoryModule = static_cast<IElectraCodecFactoryModule*>(FModuleManager::Get().GetModule(TEXT("ElectraCodecFactory")));
	if (!FactoryModule || !InCodecTypeFormat.Properties.IsType<FCodecTypeFormat::FAudio>())
	{
		return nullptr;
	}
	TMap<FString, FVariant> NoAdditionalOptions;
	return FactoryModule->GetBestDecoderFactoryForFormat(OutAddtlCfg, InCodecTypeFormat, NoAdditionalOptions);
}


FAudioDecoderImpl::FAudioDecoderImpl()
	: FMediaThread("ElectraPlayer::Audio decoder")
{
}

FAudioDecoderImpl::~FAudioDecoderImpl()
{
	Close();
}

void FAudioDecoderImpl::SetPlayerSessionServices(IPlayerSessionServices* InSessionServices)
{
	SessionServices = InSessionServices;
}

void FAudioDecoderImpl::Open(TSharedPtrTS<FAccessUnit::CodecData> InCodecData)
{
	InitialCodecSpecificData = InCodecData;
	StartThread();
}

void FAudioDecoderImpl::Close()
{
	StopThread();
}

void FAudioDecoderImpl::DrainForCodecChange()
{
	bDrainForCodecChange = true;
}

void FAudioDecoderImpl::SetOutputHandler(TSharedPtr<FOutputHandlerAudio, ESPMode::ThreadSafe> InOutputHandler)
{
	OutputHandler = InOutputHandler;
}

void FAudioDecoderImpl::SuspendOrResumeDecoder(bool bSuspend, const FParamDict& InOptions)
{
	check(!"This has not yet been implemented. Time to do so now.");
}

void FAudioDecoderImpl::AUdataPushAU(FAccessUnit* InAccessUnit)
{
	InAccessUnit->AddRef();

	TSharedPtrTS<FDecoderInput> NextAU = MakeSharedTS<FDecoderInput>();
	NextAU->AccessUnit = InAccessUnit;
	NextAccessUnits.Enqueue(MoveTemp(NextAU));
}

void FAudioDecoderImpl::AUdataPushEOD()
{
	NextAccessUnits.SetEOD();
}

void FAudioDecoderImpl::AUdataClearEOD()
{
	NextAccessUnits.ClearEOD();
}

void FAudioDecoderImpl::AUdataFlushEverything()
{
	FlushDecoderSignal.Signal();
	DecoderFlushedSignal.WaitAndReset();
}

void FAudioDecoderImpl::SetAUInputBufferListener(IAccessUnitBufferListener* InListener)
{
	FScopeLock lock(&ListenerMutex);
	InputBufferListener = InListener;
}

void FAudioDecoderImpl::SetReadyBufferListener(IDecoderOutputBufferListener* InListener)
{
	FScopeLock lock(&ListenerMutex);
	ReadyBufferListener = InListener;
}

void FAudioDecoderImpl::StartThread()
{
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FAudioDecoderImpl::WorkerThread));
	bThreadStarted = true;
}

void FAudioDecoderImpl::StopThread()
{
	if (bThreadStarted)
	{
		TerminateThreadSignal.Signal();
		ThreadWaitDone();
		bThreadStarted = false;
	}
}

void FAudioDecoderImpl::CreateDecoderOutputPool()
{
	if (OutputHandler)
	{
		FParamDict poolOpts;
		// Note: Get the default value of 8 from some config option?
		int64 NumOutputFrames = 8;
		poolOpts.Set(OutputHandlerOptionKeys::NumBuffers, FVariantValue(NumOutputFrames));
		if (OutputHandler->PreparePool(poolOpts))
		{
			MaxOutputBuffers = (int32) OutputHandler->GetPoolProperties().GetValue(OutputHandlerOptionKeys::NumBuffers).GetInt64();
			return;
		}
	}
	PostError(0, TEXT("Failed to create sample pool"), ERRCODE_AUDIO_INTERNAL_COULD_NOT_CREATE_SAMPLE_POOL);
}

void FAudioDecoderImpl::DestroyDecoderOutputPool()
{
	if (OutputHandler)
	{
		OutputHandler->ClosePool();
	}
}

void FAudioDecoderImpl::NotifyReadyBufferListener(bool bHaveOutput)
{
	if (ReadyBufferListener)
	{
		IDecoderOutputBufferListener::FDecodeReadyStats stats;
		stats.OutputBufferPoolSize = MaxOutputBuffers;
		if ((stats.NumElementsInDecoder = InDecoderInput.Num()) != 0)
		{
			stats.InDecoderTimeRangePTS.Start = InDecoderInput[0]->AccessUnit->PTS;
			stats.InDecoderTimeRangePTS.End = InDecoderInput.Last()->AccessUnit->PTS + InDecoderInput.Last()->AccessUnit->Duration;
		}
		if (CurrentAccessUnit)
		{
			stats.InDecoderTimeRangePTS.Start = Utils::Min(CurrentAccessUnit->AccessUnit->PTS, stats.InDecoderTimeRangePTS.Start.IsValid() ? stats.InDecoderTimeRangePTS.Start : FTimeValue::GetPositiveInfinity());
			stats.InDecoderTimeRangePTS.End = Utils::Max(CurrentAccessUnit->AccessUnit->PTS + CurrentAccessUnit->AccessUnit->Duration, stats.InDecoderTimeRangePTS.End.IsValid() ? stats.InDecoderTimeRangePTS.End : FTimeValue::GetNegativeInfinity());
		}
		stats.InDecoderTimeRangePTS.End.SetSequenceIndex(stats.InDecoderTimeRangePTS.Start.GetSequenceIndex());
		stats.bOutputStalled = !bHaveOutput;
		stats.bEODreached = NextAccessUnits.ReachedEOD() && !CurrentOutputBuffer;
		ListenerMutex.Lock();
		if (ReadyBufferListener)
		{
			ReadyBufferListener->DecoderOutputReady(stats);
		}
		ListenerMutex.Unlock();
	}
}

bool FAudioDecoderImpl::PostError(int32 ApiReturnValue, const FString& Message, uint16 Code, UEMediaError Error)
{
	bError = true;
	if (SessionServices)
	{
		FErrorDetail err;
		err.SetError(Error != UEMEDIA_ERROR_OK ? Error : UEMEDIA_ERROR_DETAIL);
		err.SetFacility(Facility::EFacility::AudioDecoder);
		err.SetCode(Code);
		err.SetMessage(Message);
		err.SetPlatformMessage(FString::Printf(TEXT("%d (0x%08x)"), (int32) ApiReturnValue, (int32) ApiReturnValue));
		SessionServices->PostError(err);
	}
	return false;
}

bool FAudioDecoderImpl::PostError(const IElectraDecoder::FError& InDecoderError)
{
	bError = true;
	if (SessionServices)
	{
		FErrorDetail err;
		err.SetError(UEMEDIA_ERROR_DETAIL);
		err.SetFacility(Facility::EFacility::AudioDecoder);
		err.SetCode(InDecoderError.GetCode());
		err.SetMessage(InDecoderError.GetMessage());
		err.SetPlatformMessage(FString::Printf(TEXT("%d (0x%08x)"), (int32) InDecoderError.GetSdkCode(), InDecoderError.GetSdkCode()));
		SessionServices->PostError(err);
	}
	return false;
}

void FAudioDecoderImpl::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (SessionServices)
	{
		SessionServices->PostLog(Facility::EFacility::AudioDecoder, Level, Message);
	}
}

void FAudioDecoderImpl::HandleApplicationHasEnteredForeground()
{
	int32 Count = FPlatformAtomics::InterlockedDecrement(&ApplicationSuspendCount);
	if (Count == 0)
	{
		ApplicationRunningSignal.Signal();
	}
}

void FAudioDecoderImpl::HandleApplicationWillEnterBackground()
{
	int32 Count = FPlatformAtomics::InterlockedIncrement(&ApplicationSuspendCount);
	if (Count == 1)
	{
		ApplicationRunningSignal.Reset();
	}
}

bool FAudioDecoderImpl::InternalDecoderCreate()
{
	InternalDecoderDestroy();

	if (!CurrentCodecData.IsValid())
	{
		return PostError(0, TEXT("No CSD to create audio decoder with"), ERRCODE_AUDIO_INTERNAL_COULD_NOT_CREATE_DECODER);
	}

	FCodecTypeFormat CodecTypeFormat = CurrentCodecData->ParsedInfo.ToCodecTypeFormat();
	if (CodecTypeFormat.DCR.IsEmpty() && CurrentCodecData->RawCSD.Num())
	{
		CodecTypeFormat.DCR = CurrentCodecData->RawCSD;
	}
	if (CodecTypeFormat.CSD.IsEmpty() && CurrentCodecData->CodecSpecificData.Num())
	{
		CodecTypeFormat.CSD = CurrentCodecData->CodecSpecificData;
	}

	TMap<FString, FVariant> AddtlCfg;
	TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> Factory = GetDecoderFactory(AddtlCfg, CodecTypeFormat);
	if (!Factory.IsValid())
	{
		return PostError(-2, TEXT("No decoder factory found to create an audio decoder"), ERRCODE_AUDIO_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
	if (SessionServices)
	{
		SessionServices->GetMediaInfoDictionary().Set(MediaInfoAudioDecoderProvider, FVariantValue(Factory->GetProviderInformation().GetName()));
	}

	DecoderInstance = Factory->CreateDecoder(CodecTypeFormat, AddtlCfg);
	if (!DecoderInstance.IsValid() || DecoderInstance->GetError().IsSet())
	{
		InternalDecoderDestroy();
		return PostError(-2, TEXT("Failed to create audio decoder"), ERRCODE_AUDIO_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
	if (DecoderInstance->GetType() != IElectraDecoder::EType::Audio)
	{
		InternalDecoderDestroy();
		return PostError(-2, TEXT("Created decoder is not an audio decoder!"), ERRCODE_AUDIO_INTERNAL_COULD_NOT_CREATE_DECODER);
	}

	TMap<FString, FVariant> Features;
	DecoderInstance->GetFeatures(Features);
	bIsAdaptiveDecoder = ElectraDecodersUtil::GetVariantValueSafeI64(Features, IElectraDecoderFeature::IsAdaptive, 0) != 0;
	bMustBeSuspendedInBackground = MustSuspendDecodersInBackground(ElectraDecodersUtil::GetVariantValueSafeI64(Features, IElectraDecoderFeature::MustBeSuspendedInBackground, 0) != 0);
	if (bMustBeSuspendedInBackground)
	{
		FGBGHandlers = MakeSharedTS<FFGBGNotificationHandlers>();
		FGBGHandlers->WillEnterBackground = [this]() { HandleApplicationWillEnterBackground(); };
		FGBGHandlers->HasEnteredForeground = [this]() { HandleApplicationHasEnteredForeground(); };
		if (AddBGFGNotificationHandler(FGBGHandlers))
		{
			HandleApplicationWillEnterBackground();
		}
	}


	return true;
}

void FAudioDecoderImpl::InternalDecoderDestroy()
{
	if (FGBGHandlers.IsValid())
	{
		RemoveBGFGNotificationHandler(FGBGHandlers);
		FGBGHandlers.Reset();
	}
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Close();
		DecoderInstance.Reset();
	}
	bIsAdaptiveDecoder = false;
}

void FAudioDecoderImpl::ReturnUnusedOutputBuffer()
{
	if (CurrentOutputBuffer && OutputHandler)
	{
		OutputHandler->ReturnOutputSample(CurrentOutputBuffer, FOutputHandlerAudio::EReturnSampleType::DontSendToQueue);
		CurrentOutputBuffer.Reset();
	}
}

void FAudioDecoderImpl::PrepareAU(TSharedPtrTS<FDecoderInput> AU)
{
	if (!AU->bHasBeenPrepared)
	{
		AU->bHasBeenPrepared = true;

		// Does this AU fall (partially) outside the range for rendering?
		FTimeValue StartTime = AU->AccessUnit->PTS;
		FTimeValue EndTime = AU->AccessUnit->PTS + AU->AccessUnit->Duration;
		AU->PTS = StartTime.GetAsHNS();		// The PTS we give the decoder no matter any adjustment.
		AU->EndPTS = EndTime.GetAsHNS();
		AU->StartOverlapDuration.SetToZero();
		AU->EndOverlapDuration.SetToZero();
		if (AU->AccessUnit->EarliestPTS.IsValid())
		{
			// If the end time of the AU is before the earliest render PTS we do not need to decode it.
			if (EndTime <= AU->AccessUnit->EarliestPTS)
			{
				StartTime.SetToInvalid();
			}
			else if (StartTime < AU->AccessUnit->EarliestPTS)
			{
				AU->StartOverlapDuration = AU->AccessUnit->EarliestPTS - StartTime;
				StartTime = AU->AccessUnit->EarliestPTS;
			}
		}
		if (StartTime.IsValid() && AU->AccessUnit->LatestPTS.IsValid())
		{
			// If the start time is behind the latest render PTS we donot need to decode it.
			if (StartTime >= AU->AccessUnit->LatestPTS)
			{
				StartTime.SetToInvalid();
			}
			else if (EndTime >= AU->AccessUnit->LatestPTS)
			{
				AU->EndOverlapDuration = EndTime - AU->AccessUnit->LatestPTS;
				EndTime = AU->AccessUnit->LatestPTS;
			}
		}
		AU->AdjustedPTS = StartTime;
		AU->AdjustedDuration = EndTime - StartTime;
		if (AU->AdjustedDuration <= FTimeValue::GetZero())
		{
			AU->AdjustedPTS.SetToInvalid();
		}
	}
}

FAudioDecoderImpl::EAUChangeFlags FAudioDecoderImpl::GetAndPrepareInputAU()
{
	EAUChangeFlags NewAUFlags = EAUChangeFlags::None;

	// Upcoming codec change?
	if (bDrainForCodecChange)
	{
		return EAUChangeFlags::CodecChange;
	}

	// When draining we do not ask for any new input.
	if (CurrentDecodingState == EDecodingState::Draining)
	{
		return NewAUFlags;
	}

	// Need a new access unit?
	if (!CurrentAccessUnit.IsValid())
	{
		// Notify the buffer listener that we will now be needing an AU for our input buffer.
		if (InputBufferListener && NextAccessUnits.IsEmpty())
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_AudioDecode);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, AudioDecode);
			IAccessUnitBufferListener::FBufferStats	stats;
			stats.bEODSignaled = NextAccessUnits.GetEOD();
			stats.bEODReached = NextAccessUnits.ReachedEOD();
			ListenerMutex.Lock();
			if (InputBufferListener)
			{
				InputBufferListener->DecoderInputNeeded(stats);
			}
			ListenerMutex.Unlock();
		}

		// Get the AU to be decoded if one is there.
		if (NextAccessUnits.Wait(1000 * 10))
		{
			NextAccessUnits.Dequeue(CurrentAccessUnit);
			if (CurrentAccessUnit.IsValid())
			{
				PrepareAU(CurrentAccessUnit);
				// Is there a discontinuity/break in sequence of sorts?
				if (CurrentAccessUnit->AccessUnit->bTrackChangeDiscontinuity ||
					(!bInDummyDecodeMode && CurrentAccessUnit->AccessUnit->bIsDummyData) ||
					(CurrentSequenceIndex.IsSet() && CurrentSequenceIndex.GetValue() != CurrentAccessUnit->AccessUnit->PTS.GetSequenceIndex()))
				{
					NewAUFlags |= EAUChangeFlags::Discontinuity;
				}
				else if (DecoderConfig::bCheckForDTSTimejump && NextExpectedDTSHNS.IsSet() && NextExpectedDTSHNS.GetValue() > CurrentAccessUnit->AccessUnit->DTS.GetAsHNS() + DecoderConfig::BackwardsTimejumpThresholdHNS)
				{
					if (DecoderConfig::bDrainDecoderOnDetectedBackwardsTimejump)
					{
						LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Audio timestamp jumped back unexpectedly by %.4fs. Draining the decoder before continuing."), (NextExpectedDTSHNS.GetValue() - CurrentAccessUnit->AccessUnit->DTS.GetAsHNS()) / 10000000.0));
						NewAUFlags |= EAUChangeFlags::Discontinuity;
					}
					else
					{
						LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Audio timestamp jumped back unexpectedly by %.4fs."), (NextExpectedDTSHNS.GetValue() - CurrentAccessUnit->AccessUnit->DTS.GetAsHNS()) / 10000000.0));
					}
				}
				else if (DecoderConfig::bCheckForDTSTimejump && NextExpectedDTSHNS.IsSet() && NextExpectedDTSHNS.GetValue() < CurrentAccessUnit->AccessUnit->DTS.GetAsHNS() - DecoderConfig::ForwardTimejumpThresholdHNS)
				{
					LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Audio timestamp jumped forward unexpectedly by %.4fs."), (CurrentAccessUnit->AccessUnit->DTS.GetAsHNS() - NextExpectedDTSHNS.GetValue()) / 10000000.0));
				}
				CurrentSequenceIndex = CurrentAccessUnit->AccessUnit->PTS.GetSequenceIndex();
				NextExpectedDTSHNS = (CurrentAccessUnit->AccessUnit->DTS + CurrentAccessUnit->AccessUnit->Duration).GetAsHNS();

				// Did the codec specific data change?
				if (CurrentCodecData.IsValid() && CurrentAccessUnit->AccessUnit->AUCodecData.IsValid() && CurrentCodecData->CodecSpecificData != CurrentAccessUnit->AccessUnit->AUCodecData->CodecSpecificData)
				{
					if (!bIsAdaptiveDecoder)
					{
						NewAUFlags |= EAUChangeFlags::CSDChanged;
					}
				}
				if (CurrentCodecData != CurrentAccessUnit->AccessUnit->AUCodecData && CurrentAccessUnit->AccessUnit->AUCodecData.IsValid())
				{
					CurrentCodecData = CurrentAccessUnit->AccessUnit->AUCodecData;
					CSDOptions.Empty();
					// Prepare the codec type format structure.
					FCodecTypeFormat ctf = CurrentCodecData->ParsedInfo.ToCodecTypeFormat();
					if (CurrentCodecData->RawCSD.Num())
					{
						ctf.DCR = CurrentCodecData->RawCSD;
					}
					if (CurrentCodecData->CodecSpecificData.Num())
					{
						ctf.CSD = CurrentCodecData->CodecSpecificData;
					}
					// Get CSD options from codec type format
					if (ctf.DCR.Num())
					{
						CSDOptions.Emplace(TEXT("dcr"), FVariant(ctf.DCR));
					}
					if (ctf.CSD.Num())
					{
						CSDOptions.Emplace(TEXT("csd"), FVariant(ctf.CSD));
					}
					for(auto &It : ctf.ExtraBoxes)
					{
						FString BoxName = MP4Utilities::Printable4CC(It.Key);
						CSDOptions.Emplace(BoxName, It.Value);
					}
				}

				// The very first access unit can't have differences to the one before so we clear the flags.
				if (bIsFirstAccessUnit)
				{
					bIsFirstAccessUnit = false;
					NewAUFlags = EAUChangeFlags::None;
				}
			}
		}
	}
	return NewAUFlags;
}

IElectraDecoder::ECSDCompatibility FAudioDecoderImpl::IsCompatibleWith()
{
	IElectraDecoder::ECSDCompatibility Compatibility = IElectraDecoder::ECSDCompatibility::Compatible;
	if (DecoderInstance.IsValid() && CurrentAccessUnit.IsValid())
	{
		TMap<FString, FVariant> TestCSDOptions;
		if (CurrentAccessUnit->AccessUnit->AUCodecData.IsValid())
		{
			// Prepare the codec type format structure.
			FCodecTypeFormat ctf = CurrentAccessUnit->AccessUnit->AUCodecData->ParsedInfo.ToCodecTypeFormat();
			if (CurrentAccessUnit->AccessUnit->AUCodecData->RawCSD.Num())
			{
				ctf.DCR = CurrentAccessUnit->AccessUnit->AUCodecData->RawCSD;
			}
			if (CurrentAccessUnit->AccessUnit->AUCodecData->CodecSpecificData.Num())
			{
				ctf.CSD = CurrentAccessUnit->AccessUnit->AUCodecData->CodecSpecificData;
			}
			// Get CSD options from codec type format
			if (ctf.DCR.Num())
			{
				TestCSDOptions.Emplace(TEXT("dcr"), FVariant(ctf.DCR));
			}
			if (ctf.CSD.Num())
			{
				TestCSDOptions.Emplace(TEXT("csd"), FVariant(ctf.CSD));
			}
			for(auto &It : ctf.ExtraBoxes)
			{
				FString BoxName = MP4Utilities::Printable4CC(It.Key);
				TestCSDOptions.Emplace(BoxName, It.Value);
			}
			Compatibility = DecoderInstance->IsCompatibleWith(TestCSDOptions);
		}
	}
	return Compatibility;
}

IElectraDecoder::EOutputStatus FAudioDecoderImpl::HandleOutput()
{
	IElectraDecoder::EOutputStatus OutputStatus = IElectraDecoder::EOutputStatus::Available;
	if (DecoderInstance.IsValid())
	{
		// Get output unless flushing or terminating
		while(!TerminateThreadSignal.IsSignaled() && !FlushDecoderSignal.IsSignaled() && (OutputStatus = DecoderInstance->HaveOutput()) == IElectraDecoder::EOutputStatus::Available)
		{
			if (CheckBackgrounding())
			{
				continue;
			}
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_AudioConvertOutput);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, AudioConvertOutput);

			// Check if the output pipeline can accept the output we want to send to it.
			if (!CanReceiveOutputFrame())
			{
				NotifyReadyBufferListener(false);
				return IElectraDecoder::EOutputStatus::TryAgainLater;
			}

			// Need a new output buffer?
			if (!CurrentOutputBuffer && OutputHandler)
			{
				IOutputHandlerBase::EBufferResult bufResult = OutputHandler->ObtainOutputSample(CurrentOutputBuffer);
				if (bufResult != IOutputHandlerBase::EBufferResult::Ok && bufResult != IOutputHandlerBase::EBufferResult::NoBuffer)
				{
					PostError(0, TEXT("Failed to acquire sample buffer"), ERRCODE_AUDIO_INTERNAL_COULD_NOT_GET_SAMPLE_BUFFER);
					return IElectraDecoder::EOutputStatus::Error;
				}
			}
			// Didn't get a buffer? This should not really happen since the output pipeline said it could accept a frame.
			if (!CurrentOutputBuffer)
			{
				NotifyReadyBufferListener(false);
				return IElectraDecoder::EOutputStatus::TryAgainLater;
			}

			NotifyReadyBufferListener(true);

			TSharedPtr<IElectraDecoderAudioOutput, ESPMode::ThreadSafe> Output = StaticCastSharedPtr<IElectraDecoderAudioOutput>(DecoderInstance->GetOutput());
			// No output even though it was advertised? This may happen when a decoder has an internal state change.
			if (!Output.IsValid())
			{
				break;
			}
			if (Output->GetType() != IElectraDecoderOutput::EType::Audio || !Output->IsInterleaved() || Output->GetSampleFormat() != IElectraDecoderAudioOutput::ESampleFormat::Float)
			{
				PostError(0, TEXT("Could not get decoded output due to decoded format being unsupported"), ERRCODE_AUDIO_INTERNAL_UNSUPPORTED_OUTPUT_FORMAT);
				return IElectraDecoder::EOutputStatus::Error;
			}

			// Locate the input AU info that should correspond to this output.
			TSharedPtrTS<FDecoderInput> MatchingInput;
			if (InDecoderInput.Num())
			{
				// Try the frontmost entry. It tends to be that one.
				if (InDecoderInput[0]->PTS == Output->GetUserValue())
				{
					MatchingInput = InDecoderInput[0];
					InDecoderInput.RemoveAt(0);
				}
				else
				{
					// Not the first element. This can happen if the audio decoder aggregates output from multiple inputs.
					for(int32 i=0; i<InDecoderInput.Num(); ++i)
					{
						if (InDecoderInput[i]->PTS == Output->GetUserValue())
						{
							MatchingInput = InDecoderInput[i];
							InDecoderInput.RemoveAt(i);
							break;
						}
						else if (InDecoderInput[i]->PTS + DecoderConfig::RemovalOfOldDecoderInputThresholdHNS < (int64)Output->GetUserValue())
						{
							InDecoderInput.RemoveAt(i);
							--i;
						}
					}
				}
			}
			if (!MatchingInput.IsValid())
			{
				PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_AUDIO_INTERNAL_FAILED_TO_CONVERT_OUTPUT);
				return IElectraDecoder::EOutputStatus::Error;
			}

			const float* PCMBuffer = reinterpret_cast<const float*>(Output->GetData(0));
			const int32 SamplingRate = Output->GetSampleRate();
			const int32 NumberOfChannels = Output->GetNumChannels();
			const int32 NumBytesPerFrame = Output->GetBytesPerFrame();
			const int32 NumBytesPerSample = Output->GetBytesPerSample();
			int32 NumSamplesProduced = Output->GetNumFrames();
			int32 NumDecodedBytes = NumSamplesProduced * NumBytesPerFrame;

			CurrentOutputFormat.SampleRate = SamplingRate;
			CurrentOutputFormat.NumChannels = NumberOfChannels;
			if (!ChannelMapper.IsInitialized())
			{
				if (NumberOfChannels)
				{
					if (ChannelMapper.Initialize(Output))
					{
						// Ok.
					}
					else
					{
						PostError(0, TEXT("Failed to initialize audio channel mapper"), ERRCODE_AUDIO_INTERNAL_UNSUPPORTED_NUMBER_OF_CHANNELS);
						return IElectraDecoder::EOutputStatus::Error;
					}
				}
				else
				{
					PostError(0, TEXT("Bad number (0) of decoded audio channels"), ERRCODE_AUDIO_INTERNAL_UNSUPPORTED_NUMBER_OF_CHANNELS);
					return IElectraDecoder::EOutputStatus::Error;
				}
			}

			int32 ByteOffsetToFirstSample = 0;
			if (MatchingInput->StartOverlapDuration.GetAsHNS() || MatchingInput->EndOverlapDuration.GetAsHNS())
			{
				int32 SkipStartSampleNum = (int32) (MatchingInput->StartOverlapDuration.GetAsHNS() * SamplingRate / 10000000);
				int32 SkipEndSampleNum = (int32) (MatchingInput->EndOverlapDuration.GetAsHNS() * SamplingRate / 10000000);

				if (SkipStartSampleNum + SkipEndSampleNum < NumSamplesProduced)
				{
					ByteOffsetToFirstSample = SkipStartSampleNum * NumBytesPerFrame;
					NumSamplesProduced -= SkipStartSampleNum;
					NumSamplesProduced -= SkipEndSampleNum;
					NumDecodedBytes -= (SkipStartSampleNum + SkipEndSampleNum) * NumBytesPerFrame;
				}
				else
				{
					NumSamplesProduced = 0;
					NumDecodedBytes = 0;
				}
			}

			int32 NumDecodedSamplesPerChannel = NumDecodedBytes / NumBytesPerFrame;
			if (NumDecodedSamplesPerChannel)
			{
				if (CurrentOutputBuffer->AllocateFor(EMediaAudioSampleFormat::Float, ChannelMapper.GetNumTargetChannels(), NumDecodedSamplesPerChannel))
				{
					FMediaTimeStamp PTS(MatchingInput->AdjustedPTS.GetAsTimespan(), MatchingInput->AdjustedPTS.GetSequenceIndex());
					FTimeValue Duration;
					Duration.SetFromND(NumSamplesProduced, SamplingRate);
					CurrentOutputBuffer->SetParameters(SamplingRate, PTS, Duration.GetAsTimespan());
					ChannelMapper.MapChannels(CurrentOutputBuffer->GetWritableBuffer(), CurrentOutputBuffer->GetAllocatedSize(), AdvancePointer(PCMBuffer, ByteOffsetToFirstSample), NumDecodedBytes, NumDecodedSamplesPerChannel);
				}

				OutputHandler->ReturnOutputSample(CurrentOutputBuffer, FOutputHandlerAudio::EReturnSampleType::SendToQueue);
				CurrentOutputBuffer.Reset();
			}
			else
			{
				ReturnUnusedOutputBuffer();
			}
		}
	}
	else if (CurrentDecodingState == EDecodingState::Draining)
	{
		OutputStatus = IElectraDecoder::EOutputStatus::EndOfData;
	}
	return OutputStatus;
}

bool FAudioDecoderImpl::HandleDecoding()
{
	bDrainAfterDecode = false;
	if (CurrentAccessUnit.IsValid())
	{
		// Is this audio packet to be dropped?
		if (!CurrentAccessUnit->AdjustedPTS.IsValid())
		{
			// Even if this access unit won't be decoded, if it is the last in the period and we are
			// not decoding dummy data the decoder must be drained to get the last decoded data out.
			bDrainAfterDecode = CurrentAccessUnit->AccessUnit->bIsLastInPeriod && !bInDummyDecodeMode;
			CurrentAccessUnit.Reset();
			return true;
		}

		if ((bInDummyDecodeMode = CurrentAccessUnit->AccessUnit->bIsDummyData) == true)
		{
			bool bOk = SendSilenceOrEOS(ESendMode::SendSilence);
			CurrentAccessUnit.Reset();
			return bOk;
		}

		if (!DecoderInstance.IsValid())
		{
			if (!InternalDecoderCreate())
			{
				return false;
			}
		}

		if (DecoderInstance.IsValid())
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_AudioDecode);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, AudioDecode);

			IElectraDecoder::FInputAccessUnit DecAU;
			DecAU.Data = CurrentAccessUnit->AccessUnit->AUData;
			DecAU.DataSize = (int32) CurrentAccessUnit->AccessUnit->AUSize;
			DecAU.DTS = CurrentAccessUnit->AccessUnit->DTS.GetAsTimespan();
			DecAU.PTS = CurrentAccessUnit->AccessUnit->PTS.GetAsTimespan();
			DecAU.Duration = CurrentAccessUnit->AccessUnit->Duration.GetAsTimespan();
			DecAU.UserValue = CurrentAccessUnit->PTS;
			DecAU.Flags = CurrentAccessUnit->AccessUnit->bIsSyncSample ? EElectraDecoderFlags::IsSyncSample : EElectraDecoderFlags::None;

			IElectraDecoder::EDecoderError DecErr = DecoderInstance->DecodeAccessUnit(DecAU, CSDOptions);
			if (DecErr == IElectraDecoder::EDecoderError::None)
			{
				InDecoderInput.Emplace(CurrentAccessUnit);
				InDecoderInput.StableSort([](const TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>& a, const TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>& b)
				{
					return a->PTS < b->PTS;
				});

				// If this was the last access unit in a period we need to drain the decoder _after_ having sent it
				// for decoding. We need to get its decoded output.
				bDrainAfterDecode = CurrentAccessUnit->AccessUnit->bIsLastInPeriod;
				CurrentAccessUnit.Reset();
				// Since we decoded something the decoder is no longer clean.
				bIsDecoderClean = false;
			}
			else if (DecErr == IElectraDecoder::EDecoderError::NoBuffer)
			{
				// Try again later...
				return true;
			}
			else
			{
				return PostError(DecoderInstance->GetError());
			}
		}
	}
	return true;
}


bool FAudioDecoderImpl::SendSilenceOrEOS(ESendMode InSendMode)
{
	// We do not currently have a use case for sending an EOS upstream.
	if (InSendMode == ESendMode::SendEOS)
	{
		return true;
	}
	check(CurrentAccessUnit.IsValid());
	check(bIsDecoderClean);

	// Get output unless flushing or terminating
	while(!TerminateThreadSignal.IsSignaled() && !FlushDecoderSignal.IsSignaled())
	{
		// Check if the output pipeline can accept the output we want to send to it.
		bool bCanRcv = CanReceiveOutputFrame();
		if (bCanRcv)
		{
			// Need a new output buffer?
			if (!CurrentOutputBuffer && OutputHandler)
			{
				IOutputHandlerBase::EBufferResult bufResult = OutputHandler->ObtainOutputSample(CurrentOutputBuffer);
				if (bufResult != IOutputHandlerBase::EBufferResult::Ok && bufResult != IOutputHandlerBase::EBufferResult::NoBuffer)
				{
					return PostError(0, TEXT("Failed to acquire sample buffer"), ERRCODE_AUDIO_INTERNAL_COULD_NOT_GET_SAMPLE_BUFFER);
				}
			}
			// Didn't get a buffer? This should not really happen since the output pipeline said it could accept a frame.
			if (CurrentOutputBuffer)
			{
				NotifyReadyBufferListener(true);

				SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_AudioConvertOutput);
				CSV_SCOPED_TIMING_STAT(ElectraPlayer, AudioConvertOutput);

				uint32 NumChannels = CurrentOutputFormat.NumChannels ? CurrentOutputFormat.NumChannels : 2;
				uint32 SampleRate = CurrentOutputFormat.SampleRate ? CurrentOutputFormat.SampleRate : 48000;
				uint32 SamplesPerBlock = CurrentAccessUnit->AdjustedDuration.IsValid() ? CurrentAccessUnit->AdjustedDuration.GetAsHNS() * SampleRate / 10000000 : 0;
				uint32 EmptySize = NumChannels * sizeof(float) * SamplesPerBlock;
				if (EmptySize)
				{
					if (CurrentOutputBuffer->AllocateFor(EMediaAudioSampleFormat::Float, NumChannels, SamplesPerBlock))
					{
						FMediaTimeStamp PTS(CurrentAccessUnit->AdjustedPTS.GetAsTimespan(), CurrentAccessUnit->AdjustedPTS.GetSequenceIndex());
						FTimeValue Duration;
						Duration.SetFromND(SamplesPerBlock, SampleRate);
						CurrentOutputBuffer->SetParameters(SampleRate, PTS, Duration.GetAsTimespan());
						FMemory::Memzero(CurrentOutputBuffer->GetWritableBuffer(), CurrentOutputBuffer->GetAllocatedSize());
					}
					OutputHandler->ReturnOutputSample(CurrentOutputBuffer, FOutputHandlerAudio::EReturnSampleType::SendToQueue);
					CurrentOutputBuffer.Reset();
				}
				else
				{
					ReturnUnusedOutputBuffer();
				}
				return true;
			}
			else
			{
				NotifyReadyBufferListener(false);
				FMediaRunnable::SleepMilliseconds(10);
			}
		}
		else
		{
			NotifyReadyBufferListener(false);
			FMediaRunnable::SleepMilliseconds(10);
		}
	}
	return true;
}


void FAudioDecoderImpl::StartDraining(EDecodingState InNextStateAfterDraining)
{
	if (CurrentDecodingState == EDecodingState::NormalDecoding)
	{
	    // Drain the decoder only when we sent it something to work on.
	    // If it already clean there is no point in doing that.
	    if (!bIsDecoderClean && DecoderInstance.IsValid())
	    {
		    IElectraDecoder::EDecoderError DecErr = DecoderInstance->SendEndOfData();
		    if (DecErr != IElectraDecoder::EDecoderError::None)
		    {
			    PostError(DecoderInstance->GetError());
		    }
	    }
	    // We do however set our internal state to draining in order to pick up any
	    // potentially pending output, reset the channel mapper and clear out pending input.
	    CurrentDecodingState = EDecodingState::Draining;
	    NextDecodingStateAfterDrain = InNextStateAfterDraining;
	    bIsDecoderClean = true;
	}
}


bool FAudioDecoderImpl::CheckForFlush()
{
	// Flush?
	if (FlushDecoderSignal.IsSignaled())
	{
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_AudioDecode);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, AudioDecode);
		if (DecoderInstance.IsValid())
		{
			DecoderInstance->Flush();
		}
		ReturnUnusedOutputBuffer();
		NextAccessUnits.Empty();
		InDecoderInput.Empty();
		CurrentSequenceIndex.Reset();
		NextExpectedDTSHNS.Reset();
		CurrentAccessUnit.Reset();
		CurrentOutputFormat.Reset();
		bIsDecoderClean = true;
		bInDummyDecodeMode = false;
		CurrentDecodingState = EDecodingState::NormalDecoding;

		FlushDecoderSignal.Reset();
		DecoderFlushedSignal.Signal();
		return true;
	}
	return false;
}

bool FAudioDecoderImpl::CheckBackgrounding()
{
	// If in background, wait until we get activated again.
	if (!ApplicationRunningSignal.IsSignaled())
	{
		UE_LOGF(LogElectraPlayer, Log, "FAudioDecoderImpl(%p): OnSuspending", this);
		if (DecoderInstance.IsValid())
		{
			DecoderInstance->Suspend();
		}
		ApplicationSuspendConfirmedSignal.Signal();
		while(!ApplicationRunningSignal.WaitTimeout(100 * 1000) && !TerminateThreadSignal.IsSignaled())
		{
		}
		UE_LOGF(LogElectraPlayer, Log, "FAudioDecoderImpl(%p): OnResuming", this);
		if (DecoderInstance.IsValid())
		{
			DecoderInstance->Resume();
		}
		return true;
	}
	return false;
}


void FAudioDecoderImpl::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	ApplicationRunningSignal.Signal();
	ApplicationSuspendConfirmedSignal.Reset();

	bError = false;
	CurrentOutputBuffer.Reset();
	bIsFirstAccessUnit = true;
	bInDummyDecodeMode = false;
	bIsAdaptiveDecoder = false;
	bDrainAfterDecode = false;
	bIsDecoderClean = true;
	CurrentDecodingState = EDecodingState::NormalDecoding;

	CreateDecoderOutputPool();

	while(!TerminateThreadSignal.IsSignaled())
	{
		if (CheckBackgrounding())
		{
			continue;
		}

		// Is there a pending flush? If so, execute the flush and go back to the top to check if we must terminate now.
		if (CheckForFlush())
		{
			continue;
		}

		// Get the next access unit to decode.
		EAUChangeFlags NewAUFlags = GetAndPrepareInputAU();

		if (!bError)
		{
			// Did the codec specific data change?
			if ((NewAUFlags & EAUChangeFlags::CSDChanged) != EAUChangeFlags::None)
			{
				// If the decoder is not adaptive, ask it how we have to handle the change.
				if (!bIsAdaptiveDecoder)
				{
					IElectraDecoder::ECSDCompatibility Compatibility = IsCompatibleWith();
					if (Compatibility == IElectraDecoder::ECSDCompatibility::Drain || Compatibility == IElectraDecoder::ECSDCompatibility::DrainAndReset)
					{
						StartDraining(Compatibility == IElectraDecoder::ECSDCompatibility::Drain ? EDecodingState::NormalDecoding : EDecodingState::NeedsReset);
					}
				}
			}
			// Is there a discontinuity that requires us to drain the decoder, including a switch to dummy-decoding?
			else if ((NewAUFlags & EAUChangeFlags::Discontinuity) != EAUChangeFlags::None)
			{
				StartDraining(EDecodingState::NormalDecoding);
			}
			// Upcoming codec change?
			else if ((NewAUFlags & EAUChangeFlags::CodecChange) != EAUChangeFlags::None)
			{
				StartDraining(EDecodingState::CodecChange);
			}

			// When draining the decoder we get all the output that we can.
			if (CurrentDecodingState == EDecodingState::Draining)
			{
				IElectraDecoder::EOutputStatus OS = HandleOutput();
				if (OS == IElectraDecoder::EOutputStatus::Error)
				{
					bError = true;
				}
				else if (OS == IElectraDecoder::EOutputStatus::TryAgainLater)
				{
					FMediaRunnable::SleepMilliseconds(10);
				}
				else if (OS == IElectraDecoder::EOutputStatus::EndOfData ||
						 OS == IElectraDecoder::EOutputStatus::NeedInput)
				{
					if (OS == IElectraDecoder::EOutputStatus::EndOfData)
					{
						SendSilenceOrEOS(ESendMode::SendEOS);
					}

					// All output has been retrieved
					InDecoderInput.Empty();
					ChannelMapper.Reset();
					// Continue with next state.
					CurrentDecodingState = NextDecodingStateAfterDrain;
				}
			}

			// Codec change?
			if (CurrentDecodingState == EDecodingState::CodecChange)
			{
				// We are done. Leave the decode loop.
				break;
			}

			// Does the decoder need to be reset?
			if (CurrentDecodingState == EDecodingState::NeedsReset)
			{
				if (DecoderInstance.IsValid())
				{
					if (!DecoderInstance->ResetToCleanStart())
					{
						InternalDecoderDestroy();
					}
				}
				CurrentDecodingState = EDecodingState::NormalDecoding;
			}

			// Handle decoding of either regular or dummy data.
			if (CurrentDecodingState == EDecodingState::NormalDecoding)
			{
				if (HandleOutput() == IElectraDecoder::EOutputStatus::TryAgainLater)
				{
					FMediaRunnable::SleepMilliseconds(10);
				}
				HandleDecoding();
				// If this access unit requires us to drain the decoder we do it now.
				if (bDrainAfterDecode)
				{
					StartDraining(EDecodingState::NormalDecoding);
				}

				// Is the buffer at EOD?
				if (NextAccessUnits.ReachedEOD())
				{
					if (bInDummyDecodeMode)
					{
						SendSilenceOrEOS(ESendMode::SendEOS);
						CurrentDecodingState = EDecodingState::NormalDecoding;
						bInDummyDecodeMode = false;
					}
					else if (!bIsDecoderClean)
					{
						StartDraining(EDecodingState::NormalDecoding);
					}
					else
					{
						NotifyReadyBufferListener(true);
						FMediaRunnable::SleepMilliseconds(10);
					}
				}
			}
		}
		else
		{
			if (CurrentAccessUnit.IsValid() && CurrentAccessUnit->AccessUnit->Duration.IsValid())
			{
				FMediaRunnable::SleepMicroseconds(CurrentAccessUnit->AccessUnit->Duration.GetAsMicroseconds());
			}
			else
			{
				FMediaRunnable::SleepMilliseconds(10);
			}
			CurrentAccessUnit.Reset();
		}
	}

	ReturnUnusedOutputBuffer();
	// Close the decoder.
	InternalDecoderDestroy();
	DestroyDecoderOutputPool();

	// Flush any remaining input data.
	NextAccessUnits.Empty();
	InDecoderInput.Empty();
	CurrentSequenceIndex.Reset();
	NextExpectedDTSHNS.Reset();
	CurrentOutputFormat.Reset();
	CurrentCodecData.Reset();
	ChannelMapper.Reset();

	// On a pending codec change notify the player that we are done.
	if (bDrainForCodecChange)
	{
		// Notify the player that we have finished draining.
		SessionServices->SendMessageToPlayer(FDecoderMessage::Create(FDecoderMessage::EReason::DrainingFinished, this, EStreamType::Audio));
		// We need to wait to get terminated. Also check if flushing is requested and acknowledge if it is.
		while(!TerminateThreadSignal.IsSignaled())
		{
			if (FlushDecoderSignal.WaitTimeoutAndReset(1000 * 10))
			{
				DecoderFlushedSignal.Signal();
			}
		}
	}
}

} // namespace Electra
