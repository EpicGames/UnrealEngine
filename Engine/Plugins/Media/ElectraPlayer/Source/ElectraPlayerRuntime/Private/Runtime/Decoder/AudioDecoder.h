// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

#include "StreamAccessUnitBuffer.h"
#include "Decoder/DecoderBase.h"


namespace Electra
{
	class IPlayerSessionServices;
	class FOutputHandlerAudio;


	class IAudioDecoder : public IDecoderBase, public IAccessUnitBufferInterface, public IDecoderAUBufferDiags, public IDecoderReadyBufferDiags
	{
	public:
		static bool CanDecodeStream(const FStreamCodecInformation& InCodecInfo);

		static IAudioDecoder* Create();

		virtual ~IAudioDecoder() = default;

		virtual void SetOutputHandler(TSharedPtr<FOutputHandlerAudio, ESPMode::ThreadSafe> InOutputHandler) = 0;

		virtual void SetPlayerSessionServices(IPlayerSessionServices* SessionServices) = 0;

		virtual void Open(TSharedPtrTS<FAccessUnit::CodecData> InCodecData) = 0;
		virtual void Close() = 0;
		virtual void DrainForCodecChange() = 0;


		//-------------------------------------------------------------------------
		// Methods from IDecoderBase
		//
		virtual void SuspendOrResumeDecoder(bool bSuspend, const FParamDict& InOptions) = 0;

		//-------------------------------------------------------------------------
		// Methods from IAccessUnitBufferInterface
		//
		//! Pushes an access unit to the decoder. Ownership of the access unit is transferred to the decoder.
		virtual void AUdataPushAU(FAccessUnit* AccessUnit) = 0;
		//! Notifies the decoder that there will be no further access units.
		virtual void AUdataPushEOD() = 0;
		//! Notifies the decoder that there may be further access units.
		virtual void AUdataClearEOD() = 0;
		//! Instructs the decoder to flush all pending input and all already decoded output.
		virtual void AUdataFlushEverything() = 0;

		//-------------------------------------------------------------------------
		// Methods from IDecoderAUBufferDiags
		//
		//! Registers an AU input buffer listener.
		virtual void SetAUInputBufferListener(IAccessUnitBufferListener* Listener) = 0;

		//-------------------------------------------------------------------------
		// Methods from IDecoderReadyBufferDiags
		//
		virtual void SetReadyBufferListener(IDecoderOutputBufferListener* Listener) = 0;
	};


} // namespace Electra

