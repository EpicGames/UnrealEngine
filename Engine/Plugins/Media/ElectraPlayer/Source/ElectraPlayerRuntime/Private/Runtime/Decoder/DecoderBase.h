// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlayerSessionServices.h"
#include "ParameterDictionary.h"

namespace Electra
{
	class IAccessUnitBufferListener;
	class IDecoderOutputBufferListener;

	namespace DecoderConfig
	{
		// Should we check if the DTS of the input AU jumps?
		static const bool bCheckForDTSTimejump = true;
		// When checking the AU DTS, how much does it need to jump back before triggering a decoder drain?
		static const int64 BackwardsTimejumpThresholdHNS = 10 * 1000 * 500;						// 500ms
		// Drain the decoder when a timejump is being detected?
		static const bool bDrainDecoderOnDetectedBackwardsTimejump = false;
		// When checking the AU DTS, how much does it need to jump forward before logging a warning?
		static const int64 ForwardTimejumpThresholdHNS = 10 * 1000 * 500;						// 500ms

		// After what time decoder inputs are to be flushed in case some decoder implementation drops
		// output which we would otherwise keep the source AU around for.
		static const int64 RemovalOfOldDecoderInputThresholdHNS = 10 * 1000 * 1000 * 10;		// 10 seconds
	};


	/**
	 *
	**/
	class IDecoderBase
	{
	public:
		virtual ~IDecoderBase() = default;
		//virtual void SuspendOrResumeDecoder(bool bSuspend, const FParamDict& InOptions) = 0;
	};


	/**
	 *
	**/
	class IDecoderAUBufferDiags
	{
	public:
		virtual ~IDecoderAUBufferDiags() = default;
		virtual void SetAUInputBufferListener(IAccessUnitBufferListener* Listener) = 0;
	};


	/**
	 *
	**/
	class IDecoderReadyBufferDiags
	{
	public:
		virtual ~IDecoderReadyBufferDiags() = default;
		virtual void SetReadyBufferListener(IDecoderOutputBufferListener* Listener) = 0;
	};




	class FDecoderMessage : public IPlayerMessage
	{
	public:
		enum class EReason
		{
			DrainingFinished
		};

		static TSharedPtrTS<IPlayerMessage> Create(EReason InReason, IDecoderBase* InDecoderInstance, EStreamType InStreamType, FStreamCodecInformation::ECodec InCodec=FStreamCodecInformation::ECodec::Unknown)
		{
			TSharedPtrTS<FDecoderMessage> p(new FDecoderMessage(InReason, InDecoderInstance, InStreamType, InCodec));
			return p;
		}

		static const FString& Type()
		{
			static FString TypeName("Decoder");
			return TypeName;
		}

		virtual const FString& GetType() const
		{
			return Type();
		}

		EReason GetReason() const
		{
			return Reason;
		}

		IDecoderBase* GetDecoderInstance() const
		{
			return DecoderInstance;
		}

		EStreamType GetStreamType() const
		{
			return StreamType;
		}

		FStreamCodecInformation::ECodec GetCodec() const
		{
			return Codec;
		}

	private:
		FDecoderMessage(EReason InReason, IDecoderBase* InDecoderInstance, EStreamType InStreamType, FStreamCodecInformation::ECodec InCodec)
			: Codec(InCodec)
			, DecoderInstance(InDecoderInstance)
			, StreamType(InStreamType)
			, Reason(InReason)
		{
		}
		FStreamCodecInformation::ECodec Codec;
		IDecoderBase* DecoderInstance;
		EStreamType StreamType;
		EReason Reason;
	};


} // namespace Electra
