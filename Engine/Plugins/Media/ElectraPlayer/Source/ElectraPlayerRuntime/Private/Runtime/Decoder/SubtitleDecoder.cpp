// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitleDecoder.h"
#include "Features/IModularFeatures.h"
#include "IElectraSubtitleDecoder.h"
#include "IElectraSubtitleModule.h"
#include "IElectraSubtitleSample.h"


namespace Electra
{

namespace SubtitleOptionKeys
{
	static const FName PresentationTimeOffset(TEXT("PresentationTimeOffset"));
	static const FName SendEmptySubtitleDuringGaps(TEXT("sendEmptySubtitleDuringGaps"));
	static const FName SourceID(TEXT("source_id"));
	static const FName Width(TEXT("width"));
	static const FName Height(TEXT("height"));
	static const FName OffsetX(TEXT("offset_x"));
	static const FName OffsetY(TEXT("offset_y"));
	static const FName Timescale(TEXT("timescale"));
}


class FSubtitleDecoderPlugins
{
public:
	static FSubtitleDecoderPlugins& Get()
	{
		static FSubtitleDecoderPlugins This;
		This.CollectSupportedDecoderPlugins();
		return This;
	}

	bool IsSupported(const FString& CodecName);
	TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> CreateDecoder(const FString& CodecName);

private:
	struct FCodecPlugin
	{
		IElectraSubtitleModularFeature* Plugin = nullptr;
		int32 Priority = -1;
	};

	void CollectSupportedDecoderPlugins();

	FCriticalSection Lock;
	TMap<FString, FCodecPlugin> CodecPlugins;
	bool bIsValidList = false;
};


void FSubtitleDecoderPlugins::CollectSupportedDecoderPlugins()
{
	FScopeLock lock(&Lock);
	if (!bIsValidList)
	{
		// Get the list of all the registered modular features implementing a subtitle decoder.
		IModularFeatures::Get().LockModularFeatureList();
		TArray<IElectraSubtitleModularFeature*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IElectraSubtitleModularFeature>(IElectraSubtitlesModule::GetModularFeatureName());
		IModularFeatures::Get().UnlockModularFeatureList();
		for(auto &Plugin : PluginImplementations)
		{
			if (!Plugin)
			{
				continue;
			}
			// Get the names of supported codecs and add them to our list, replacing those of lower priority.
			TArray<FString> CodecNames;
			Plugin->GetSupportedFormats(CodecNames);
			for(auto &CodecName : CodecNames)
			{
				int32 Prio = Plugin->GetPriorityForFormat(CodecName);
				bool bAdd = true;
				for(auto &cp : CodecPlugins)
				{
					if (cp.Key.Equals(CodecName) && Prio < cp.Value.Priority)
					{
						bAdd = false;
						break;
					}
				}
				if (bAdd)
				{
					CodecPlugins.Emplace(CodecName, FCodecPlugin( {Plugin, Prio} ));
				}
			}
		}
		bIsValidList = true;
	}
}

bool FSubtitleDecoderPlugins::IsSupported(const FString& CodecName)
{
	CollectSupportedDecoderPlugins();
	FScopeLock lock(&Lock);
	return CodecPlugins.Contains(CodecName);
}

TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> FSubtitleDecoderPlugins::CreateDecoder(const FString& CodecName)
{
	CollectSupportedDecoderPlugins();
	FScopeLock lock(&Lock);
	FCodecPlugin* Plugin = CodecPlugins.Find(CodecName);
	if (Plugin && Plugin->Plugin)
	{
		return Plugin->Plugin->CreateDecoderForFormat(CodecName);
	}
	return nullptr;
}


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/




/**
 * Generic subtitle decoder
**/
class FSubtitleDecoder : public ISubtitleDecoder
{
public:
	static bool IsSupported(const FString& MimeType, const FString& Codec);

	FSubtitleDecoder(TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> InPluginDecoder);
	virtual ~FSubtitleDecoder();

	void SetPlayerSessionServices(IPlayerSessionServices* SessionServices) override;
	void SetMediaSamples(TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> InMediaSamples) override;

	void Open(const FParamDict& Options) override;
	void Close() override;
	void Start() override;
	void Stop() override;
	void UpdatePlaybackPosition(FTimeValue InAbsolutePosition, FTimeValue InLocalPosition) override;

	FTimeValue GetStreamedDeliveryTimeOffset() override;

	void AUdataPushAU(FAccessUnit* AccessUnit) override;
	void AUdataPushEOD() override;
	void AUdataClearEOD() override;
	void AUdataFlushEverything() override;

	void SetIsSideloaded(bool bInIsSideloaded)
	{
		CurrentConfig.bIsSideloaded = bInIsSideloaded;
	}
	void SetSourceID(const FString& InID)
	{
		CurrentConfig.SourceID = InID;
	}
	void SetCSD(const TArray<uint8> InRawCSD)
	{
		CurrentConfig.RawCSD = InRawCSD;
	}
	void SetCodecInfo(const FStreamCodecInformation& InCodecInfo)
	{
		CurrentConfig.CodecInfo = InCodecInfo;
	}

	bool ResetForTrackChange(FAccessUnit* AccessUnit);


private:
	void OnDecodedSubtitleReceived(ISubtitleDecoderOutputPtr DecodedSubtitle);
	void FlushSamples();

	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> CurrentMediaSamples;

	TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> PluginDecoder;
	FParamDict PluginDecoderOptions;
	IPlayerSessionServices* PlayerSessionServices = nullptr;
	FTimeValue DeliveryOffset;
	FDelegateHandle ReceiveDelegateHandle;
	bool bIsStarted = false;

	struct FCurrentConfig
	{
		FStreamCodecInformation CodecInfo;
		TArray<uint8> RawCSD;
		FString SourceID;
		bool bIsSideloaded = false;
		void Reset()
		{
			CodecInfo.Clear();
			RawCSD.Empty();
			SourceID.Empty();
			bIsSideloaded = false;
		}
	};
	FCurrentConfig CurrentConfig;
};

bool ISubtitleDecoder::IsSupported(const FString& MimeType, const FString& Codec)
{
	return FSubtitleDecoder::IsSupported(MimeType, Codec);
}

ISubtitleDecoder* ISubtitleDecoder::Create(FAccessUnit* AccessUnit)
{
	check(AccessUnit);
	if (AccessUnit && AccessUnit->AUCodecData.IsValid())
	{
		TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> PluginDecoder;
		// For sideloaded data prefer to create the decoder via the Mime type. This allows the decoder to
		// know that the data is there all at once and it needs to handle it in a non-streaming fashion.
		if (AccessUnit->bIsSideloaded)
		{
			PluginDecoder = FSubtitleDecoderPlugins::Get().CreateDecoder(AccessUnit->AUCodecData->ParsedInfo.GetMimeType());
		}
		if (!PluginDecoder.IsValid())
		{
			PluginDecoder = FSubtitleDecoderPlugins::Get().CreateDecoder(AccessUnit->AUCodecData->ParsedInfo.GetCodecSpecifierRFC6381());
		}
		if (PluginDecoder.IsValid())
		{
			FSubtitleDecoder* Decoder = new FSubtitleDecoder(MoveTemp(PluginDecoder));
			Decoder->SetIsSideloaded(AccessUnit->bIsSideloaded);
			Decoder->SetSourceID(AccessUnit->BufferSourceInfo.IsValid() ? AccessUnit->BufferSourceInfo->PeriodAdaptationSetID : FString());
			Decoder->SetCSD(AccessUnit->AUCodecData->RawCSD);
			Decoder->SetCodecInfo(AccessUnit->AUCodecData->ParsedInfo);
			return Decoder;
		}
	}
	return nullptr;
}


bool FSubtitleDecoder::IsSupported(const FString& MimeType, const FString& Codec)
{
	if (MimeType.IsEmpty() && Codec.IsEmpty())
	{
		return false;
	}
	bool bMime = MimeType.Len() ? FSubtitleDecoderPlugins::Get().IsSupported(MimeType) : false;
	bool bCodec = Codec.Len() ? FSubtitleDecoderPlugins::Get().IsSupported(Codec) : false;
	return bMime || bCodec;
}


FSubtitleDecoder::FSubtitleDecoder(TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> InPluginDecoder)
	: PluginDecoder(InPluginDecoder)
{
	DeliveryOffset.SetToZero();
}

FSubtitleDecoder::~FSubtitleDecoder()
{
}

void FSubtitleDecoder::SetPlayerSessionServices(IPlayerSessionServices* InSessionServices)
{
	PlayerSessionServices = InSessionServices;
}

void FSubtitleDecoder::SetMediaSamples(TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> InMediaSamples)
{
	CurrentMediaSamples = MoveTemp(InMediaSamples);
}

void FSubtitleDecoder::Open(const FParamDict& Options)
{
	check(PlayerSessionServices);
	check(PluginDecoder.IsValid());
	if (PluginDecoder.IsValid())
	{
		PluginDecoderOptions = Options;
		FParamDict Addtl(Options);
		Addtl.Set(SubtitleOptionKeys::Width, FVariantValue((int64) CurrentConfig.CodecInfo.GetResolution().Width));
		Addtl.Set(SubtitleOptionKeys::Height, FVariantValue((int64) CurrentConfig.CodecInfo.GetResolution().Height));
		Addtl.Set(SubtitleOptionKeys::OffsetX, FVariantValue((int64) CurrentConfig.CodecInfo.GetTranslation().GetX()));
		Addtl.Set(SubtitleOptionKeys::OffsetY, FVariantValue((int64) CurrentConfig.CodecInfo.GetTranslation().GetY()));
		Addtl.Set(SubtitleOptionKeys::Timescale, FVariantValue((int64) CurrentConfig.CodecInfo.GetFrameRate().GetDenominator()));
		if (PluginDecoder->InitializeStreamWithCSD(CurrentConfig.RawCSD, Addtl))
		{
			DeliveryOffset = PluginDecoder->GetStreamedDeliveryTimeOffset();
			ReceiveDelegateHandle = PluginDecoder->GetParsedSubtitleReceiveDelegate().AddRaw(this, &FSubtitleDecoder::OnDecodedSubtitleReceived);
		}
		else
		{
			PlayerSessionServices->PostError(FErrorDetail().SetFacility(Facility::EFacility::SubtitleDecoder).SetCode(1).SetError(UEMEDIA_ERROR_INTERNAL).SetMessage(FString::Printf(TEXT("Subtitle decoder plugin failed to initialize with CSD"))));
		}
	}
	else
	{
		PlayerSessionServices->PostError(FErrorDetail().SetFacility(Facility::EFacility::SubtitleDecoder).SetCode(1).SetError(UEMEDIA_ERROR_INTERNAL).SetMessage(FString::Printf(TEXT("No suitable subtitle decoder plugin found"))));
	}
}

void FSubtitleDecoder::Close()
{
	if (ReceiveDelegateHandle.IsValid())
	{
		check(PluginDecoder.IsValid());
		PluginDecoder->GetParsedSubtitleReceiveDelegate().Remove(ReceiveDelegateHandle);
		ReceiveDelegateHandle.Reset();
	}
	CurrentMediaSamples.Reset();
	PluginDecoder.Reset();
	CurrentConfig.Reset();
}

void FSubtitleDecoder::Start()
{
	if (PluginDecoder.IsValid())
	{
		PluginDecoder->Start();
	}
	bIsStarted = true;
}

void FSubtitleDecoder::Stop()
{
	if (PluginDecoder.IsValid())
	{
		PluginDecoder->Stop();
	}
	bIsStarted = false;
}

void FSubtitleDecoder::UpdatePlaybackPosition(FTimeValue InAbsolutePosition, FTimeValue InLocalPosition)
{
	if (PluginDecoder.IsValid())
	{
		PluginDecoder->UpdatePlaybackPosition(InAbsolutePosition, InLocalPosition);
	}
}


FTimeValue FSubtitleDecoder::GetStreamedDeliveryTimeOffset()
{
	return DeliveryOffset;
}

void FSubtitleDecoder::OnDecodedSubtitleReceived(ISubtitleDecoderOutputPtr DecodedSubtitle)
{
	if (DecodedSubtitle.IsValid())
	{
		auto MediaSamples = CurrentMediaSamples;
		if (MediaSamples.IsValid())
		{
			FDecoderTimeStamp ts = DecodedSubtitle->GetTime();
			FUTF8ToTCHAR cnv((const ANSICHAR*)DecodedSubtitle->GetData().GetData(), DecodedSubtitle->GetData().Num());

			TSharedRef<FElectraSubtitleSample, ESPMode::ThreadSafe> SubtitleSample = MakeShared<FElectraSubtitleSample, ESPMode::ThreadSafe>();
			SubtitleSample->Timestamp.SetTimeAndIndexValue(ts.Time, ts.SequenceIndex); 
			SubtitleSample->Duration = DecodedSubtitle->GetDuration();
			SubtitleSample->Text = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
			MediaSamples->AddSubtitle(SubtitleSample);
		}
	}
}


bool FSubtitleDecoder::ResetForTrackChange(FAccessUnit* AccessUnit)
{
	bool bWasStarted = bIsStarted;
	Stop();
	if (PluginDecoder.IsValid())
	{
		PluginDecoder->Flush();
	}
	FlushSamples();

	// Do we need to close and create a new decoder?
	if (AccessUnit && AccessUnit->AUCodecData.IsValid())
	{
		FString NewSourceID(AccessUnit->BufferSourceInfo.IsValid() ? AccessUnit->BufferSourceInfo->PeriodAdaptationSetID : FString());
		if (CurrentConfig.SourceID != NewSourceID ||
			CurrentConfig.bIsSideloaded != AccessUnit->bIsSideloaded ||
			CurrentConfig.RawCSD != AccessUnit->AUCodecData->RawCSD ||
			!CurrentConfig.CodecInfo.Equals(AccessUnit->AUCodecData->ParsedInfo))
		{
			auto MediaSamples = CurrentMediaSamples;
			Close();
			if (AccessUnit && AccessUnit->AUCodecData.IsValid())
			{
				// For sideloaded data prefer to create the decoder via the Mime type. This allows the decoder to
				// know that the data is there all at once and it needs to handle it in a non-streaming fashion.
				if (AccessUnit->bIsSideloaded)
				{
					PluginDecoder = FSubtitleDecoderPlugins::Get().CreateDecoder(AccessUnit->AUCodecData->ParsedInfo.GetMimeType());
				}
				if (!PluginDecoder.IsValid())
				{
					PluginDecoder = FSubtitleDecoderPlugins::Get().CreateDecoder(AccessUnit->AUCodecData->ParsedInfo.GetCodecSpecifierRFC6381());
				}
				if (PluginDecoder.IsValid())
				{
					SetIsSideloaded(AccessUnit->bIsSideloaded);
					SetSourceID(NewSourceID);
					SetCSD(AccessUnit->AUCodecData->RawCSD);
					SetCodecInfo(AccessUnit->AUCodecData->ParsedInfo);
					CurrentMediaSamples = MoveTemp(MediaSamples);
					Open(PluginDecoderOptions);
					if (bWasStarted)
					{
						Start();
					}
					return true;
				}
			}
		}
		else
		{
			// No change needed.
			if (bWasStarted)
			{
				Start();
			}
			return true;
		}
	}
	return false;
}


void FSubtitleDecoder::AUdataPushAU(FAccessUnit* AccessUnit)
{
	if (PluginDecoder.IsValid())
	{
		if (!AccessUnit->bIsDummyData)
		{
			if (AccessUnit->bTrackChangeDiscontinuity)
			{
				if (!ResetForTrackChange(AccessUnit))
				{
					return;
				}
			}
			// Set the period and adaptation set ID in the additional options. This allows the plugin to identify
			// whether or not it has already parsed this data before (when seeking for instance).
			FParamDict Addtl;
			if (AccessUnit->BufferSourceInfo.IsValid())
			{
				Addtl.Set(SubtitleOptionKeys::SourceID, FVariantValue(AccessUnit->BufferSourceInfo->PeriodAdaptationSetID));
			}
			if (AccessUnit->AUCodecData.IsValid() && AccessUnit->AUCodecData->ParsedInfo.GetExtras().HaveKey(StreamCodecInformationOptions::PresentationTimeOffset))
			{
				Addtl.Set(SubtitleOptionKeys::PresentationTimeOffset, AccessUnit->AUCodecData->ParsedInfo.GetExtras().GetValue(StreamCodecInformationOptions::PresentationTimeOffset));
			}

			FTimeValue PTS(AccessUnit->PTS);
			FTimeValue Duration(AccessUnit->Duration);
			// Check if the duration is zero. This might happen with the last sample that is usually used to send
			// blank text to remove the subtitle from the screen. We cannot have zero duration for media samples
			// as this leaves their time interval empty, causing issues with timestamp overlap checks.
			if (Duration.IsZero())
			{
				// Set a short duration for overlap tests to work. Adjust the PTS down by that amount as well.
				Duration.SetFromMicroseconds(10);
				PTS -= Duration;
			}
			PluginDecoder->AddStreamedSubtitleData(TArray<uint8>((const uint8*)AccessUnit->AUData, (int32)AccessUnit->AUSize), PTS, Duration, Addtl);
		}
	}
}

void FSubtitleDecoder::AUdataPushEOD()
{
	if (PluginDecoder.IsValid())
	{
		PluginDecoder->SignalStreamedSubtitleEOD();
	}
}

void FSubtitleDecoder::AUdataClearEOD()
{
	if (PluginDecoder.IsValid())
	{
// FIXME: Add this method if required. Right now it is not.
//		PluginDecoder->ClearStreamedSubtitleEOD();
	}
}

void FSubtitleDecoder::AUdataFlushEverything()
{
	if (PluginDecoder.IsValid())
	{
		PluginDecoder->Flush();
	}
	FlushSamples();
}

void FSubtitleDecoder::FlushSamples()
{
	auto MediaSamples = CurrentMediaSamples;
	if (MediaSamples.IsValid())
	{
		TRange<FTimespan> AllTime(FTimespan::MinValue(), FTimespan::MaxValue());
		TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe> FlushSample;
		while(MediaSamples->FetchSubtitle(AllTime, FlushSample))
		{ }
	}
}


} // namespace Electra
