// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "MediaSamples.h"

#include "StreamAccessUnitBuffer.h"
#include "Decoder/DecoderBase.h"


namespace Electra
{
	class IPlayerSessionServices;


	/**
	 * Generic subtitle decoder
	**/
	class ISubtitleDecoder : public IAccessUnitBufferInterface
	{
	public:
		static bool IsSupported(const FString& MimeType, const FString& Codec);

		static ISubtitleDecoder* Create(FAccessUnit* AccessUnit);

		virtual ~ISubtitleDecoder() = default;

		//-------------------------------------------------------------------------
		// State methods
		//
		virtual void SetPlayerSessionServices(IPlayerSessionServices* SessionServices) = 0;
		virtual void SetMediaSamples(TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> InMediaSamples) = 0;
		virtual void Open(const FParamDict& Options) = 0;
		virtual void Close() = 0;
		virtual void Start() = 0;
		virtual void Stop() = 0;
		virtual void UpdatePlaybackPosition(FTimeValue InAbsolutePosition, FTimeValue InLocalPosition) = 0;


		//! Returns a time offset at which streamed subtitles should be delivered ahead of their actual timestamp (if they are available earlier)
		virtual FTimeValue GetStreamedDeliveryTimeOffset() = 0;


		//-------------------------------------------------------------------------
		// Methods from IAccessUnitBufferInterface
		//
		//! Pushes an access unit to the decoder. Ownership of the access unit is transferred to the decoder.
		virtual void AUdataPushAU(FAccessUnit* AccessUnit) = 0;
		//! Notifies the decoder that there will be no further access units.
		virtual void AUdataPushEOD() = 0;
		//! Notifies the decoder that there may be further access units.
		virtual void AUdataClearEOD() = 0;
		//! Instructs the decoder to flush all pending input and all already decoded output.
		virtual void AUdataFlushEverything() = 0;
	};


} // namespace Electra

