// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/LowLevelMemTracker.h"
#include "Stats/Stats.h"

#include "PlayerCore.h"
#include "PlayerRuntimeGlobal.h"
#include "ElectraPlayerPrivate.h"

#include "StreamAccessUnitBuffer.h"
#include "Decoder/VideoDecoder.h"
#include "OutputHandler.h"
#include "Player/PlayerSessionServices.h"
#include "Player/AdaptivePlayerMetadataKeynames.h"
#include "Utilities/Utilities.h"
#include "Utilities/StringHelpers.h"
#include "SynchronizedClock.h"

#include "MediaDecoderOutput.h"
#include "Utils/VideoDecoderHelpers.h"

#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"
#include "ElectraDecodersPlatformResources.h"

// Error codes must be in the 1000-1999 range. 1-999 is reserved for the decoder implementation.
#define ERRCODE_VIDEO_INTERNAL_COULD_NOT_CREATE_DECODER			1001
#define ERRCODE_VIDEO_INTERNAL_COULD_NOT_CREATE_SAMPLE_POOL		1002
#define ERRCODE_VIDEO_INTERNAL_COULD_NOT_GET_SAMPLE_BUFFER		1003
#define ERRCODE_VIDEO_INTERNAL_UNSUPPORTED_OUTPUT_FORMAT		1004
#define ERRCODE_VIDEO_INTERNAL_FAILED_TO_CONVERT_OUTPUT			1005
#define ERRCODE_VIDEO_INTERNAL_FAILED_TO_PARSE_BITSTREAM		1006


/***************************************************************************************************************************************************/

DECLARE_CYCLE_STAT(TEXT("FVideoDecoderImpl::Decode()"), STAT_ElectraPlayer_VideoDecode, STATGROUP_ElectraPlayer);
DECLARE_CYCLE_STAT(TEXT("FVideoDecoderImpl::ConvertOutput()"), STAT_ElectraPlayer_VideoConvertOutput, STATGROUP_ElectraPlayer);


namespace Electra
{

class FVideoDecoderImpl : public IVideoDecoder, public FMediaThread
{
public:
	FVideoDecoderImpl();
	virtual ~FVideoDecoderImpl();

	static TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> GetDecoderFactory(TMap<FString, FVariant>& OutAddtlCfg, const FCodecTypeFormat& InCodecTypeFormat);

	void SetPlayerSessionServices(IPlayerSessionServices* SessionServices) override;
	void Open(TSharedPtrTS<FAccessUnit::CodecData> InCodecData, FParamDict&& InAdditionalOptions, const FStreamCodecInformation* InMaxStreamConfiguration) override;
	bool Reopen(TSharedPtrTS<FAccessUnit::CodecData> InCodecData, const FParamDict& InAdditionalOptions, const FStreamCodecInformation* InMaxStreamConfiguration) override;
	void Close() override;
	void DrainForCodecChange() override;
	void SetOutputHandler(TSharedPtr<FOutputHandlerVideo, ESPMode::ThreadSafe> InOutputHandler) override;
	void SuspendOrResumeDecoder(bool bSuspend, const FParamDict& InOptions) override;
	void AUdataPushAU(FAccessUnit* AccessUnit) override;
	void AUdataPushEOD() override;
	void AUdataClearEOD() override;
	void AUdataFlushEverything() override;
	void SetAUInputBufferListener(IAccessUnitBufferListener* Listener) override;
	void SetReadyBufferListener(IDecoderOutputBufferListener* Listener) override;

private:
	struct FDecoderInput
	{
		~FDecoderInput()
		{
			FAccessUnit::Release(AccessUnit);
		}

		IElectraDecoder::FInputAccessUnit DecAU;
		TMap<FString, FVariant> CSDOptions;
		TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> BSI;
		FTimeValue AdjustedPTS;
		FTimeValue AdjustedDuration;
		FAccessUnit* AccessUnit = nullptr;
		int64 PTS = 0;
		bool bHasBeenPrepared = false;
		bool bMaySkipDecoding = false;
	};

	enum class EDecodingState
	{
		NormalDecoding,
		Draining,
		NeedsReset,
		CodecChange,
		ReplayDecoding
	};

	enum class ENextDecodingState
	{
		NormalDecoding,
		ReplayDecoding,
		Error
	};

	enum class EAUChangeFlags
	{
		None = 0x00,
		CSDChanged = 0x01,
		Discontinuity = 0x02,
		CodecChange = 0x04
	};
	FRIEND_ENUM_CLASS_FLAGS(EAUChangeFlags);

	void StartThread();
	void StopThread();
	void WorkerThread();

	void HandleApplicationHasEnteredForeground();
	void HandleApplicationWillEnterBackground();

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();

	void CreateDecoderOutputPool();
	void DestroyDecoderOutputPool();

	void ReturnUnusedOutputBuffer();

	void NotifyReadyBufferListener(bool bHaveOutput);

	bool CanReceiveOutputFrame()
	{
		bool bCanRcv = true;
		if (OutputHandler)
		{
			bCanRcv = OutputHandler->CanReceiveOutputSample();
		}
		return bCanRcv;
	}

	EAUChangeFlags GetAndPrepareInputAU();
	EAUChangeFlags PrepareAU(TSharedPtrTS<FDecoderInput> AU);
	IElectraDecoder::ECSDCompatibility IsCompatibleWith();

	bool PostError(int32 ApiReturnValue, const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);
	bool PostError(const IElectraDecoder::FError& InDecoderError);
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

	IElectraDecoder::EOutputStatus HandleOutput();
	ENextDecodingState HandleDecoding();
	ENextDecodingState HandleReplaying();
	bool HandleDummyDecoding();
	void StartDraining(EDecodingState InNextStateAfterDraining);
	bool CheckForFlush();
	bool CheckBackgrounding();

private:
	FCodecTypeFormat														InitialCodecTypeFormat;
	TSharedPtrTS<FAccessUnit::CodecData>									InitialCodecSpecificData;
	FParamDict																InitialAdditionalOptions;
	TOptional<FStreamCodecInformation>										InitialMaxStreamProperties;

	TAccessUnitQueue<TSharedPtrTS<FDecoderInput>>							NextAccessUnits;

	TAccessUnitQueue<TSharedPtrTS<FDecoderInput>>							ReplayAccessUnits;
	TAccessUnitQueue<TSharedPtrTS<FDecoderInput>>							ReplayingAccessUnits;
	TSharedPtrTS<FDecoderInput>												ReplayAccessUnit;

	TArray<TSharedPtrTS<FDecoderInput>>										InDecoderInput;
	TMap<FString, FVariant>													CurrentActiveCSD;
	TSharedPtrTS<FDecoderInput>												CurrentAccessUnit;
	TOptional<int64>														CurrentSequenceIndex;
	TOptional<int64>														NextExpectedDTSHNS;
	EDecodingState															CurrentDecodingState = EDecodingState::NormalDecoding;
	EDecodingState															NextDecodingStateAfterDrain = EDecodingState::NormalDecoding;
	bool																	bIsDecoderClean = true;
	bool																	bDrainAfterDecode = false;
	int32																	MinLoopSleepTimeMsec = 0;

	bool																	bIsFirstAccessUnit = true;
	bool																	bInDummyDecodeMode = false;
	bool																	bDrainForCodecChange = false;
	bool																	bWaitForSyncSample = true;
	bool																	bWarnedMissingSyncSample = false;

	int32																	NumInitialSkippedFrames = 0;
	int32																	NumInitialSkippedDecodingFrames = 0;
	bool																	bIsStartOfSequence = true;

	bool																	bError = false;

	FMediaEvent																TerminateThreadSignal;
	FMediaEvent																FlushDecoderSignal;
	FMediaEvent																DecoderFlushedSignal;
	bool																	bThreadStarted = false;

	FMediaEvent																ApplicationRunningSignal;
	FMediaEvent																ApplicationSuspendConfirmedSignal;
	TSharedPtrTS<FFGBGNotificationHandlers>									FGBGHandlers;
	int32																	ApplicationSuspendCount = 0;

	TSharedPtr<FOutputHandlerVideo, ESPMode::ThreadSafe>					OutputHandler;
	int32																	MaxOutputBuffers = 0;

	FCriticalSection														ListenerMutex;
	IAccessUnitBufferListener*												InputBufferListener = nullptr;
	IDecoderOutputBufferListener*											ReadyBufferListener = nullptr;

	IPlayerSessionServices* 												SessionServices = nullptr;

	TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe>					DecoderFactory;
	TMap<FString, FVariant>													DecoderFactoryAddtlCfg;

	FElectraDecodersPlatformResources::IDecoderPlatformResource*			DecoderPlatformResource = nullptr;

	TMap<FString, FVariant>													DecoderConfigOptions;
	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>						DecoderInstance;
	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe>		DecoderBitstreamProcessor;

	bool																	bIsAdaptiveDecoder = false;
	bool																	bSupportsDroppingOutput = false;
	bool																	bNeedsReplayData = true;
	bool																	bMustBeSuspendedInBackground = false;

	TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe>				CurrentDecoderOutput;
	TOptional<MPEG::FColorimetryHelper>										CurrentColorimetry;
	TOptional<MPEG::FHDRHelper>												CurrentHDR;

	FElectraTextureSamplePtr												CurrentOutputBuffer;
	FParamDict																DummyBufferSampleProperties;
};
ENUM_CLASS_FLAGS(FVideoDecoderImpl::EAUChangeFlags);


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

bool IVideoDecoder::CanDecodeStream(const FStreamCodecInformation& InCodecInfo)
{
	Electra::FCodecTypeFormat ctf = InCodecInfo.ToCodecTypeFormat();
	TMap<FString, FVariant> AddtlCfg;
	return FVideoDecoderImpl::GetDecoderFactory(AddtlCfg, ctf).IsValid();
}

IVideoDecoder* IVideoDecoder::Create()
{
	return new FVideoDecoderImpl;
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FVideoDecoderImpl::GetDecoderFactory(TMap<FString, FVariant>& OutAddtlCfg, const FCodecTypeFormat& InCodecTypeFormat)
{
	IElectraCodecFactoryModule* FactoryModule = static_cast<IElectraCodecFactoryModule*>(FModuleManager::Get().GetModule(TEXT("ElectraCodecFactory")));
	if (!FactoryModule || !InCodecTypeFormat.Properties.IsType<FCodecTypeFormat::FVideo>())
	{
		return nullptr;
	}
	TMap<FString, FVariant> NoAdditionalOptions;
	return FactoryModule->GetBestDecoderFactoryForFormat(OutAddtlCfg, InCodecTypeFormat, NoAdditionalOptions);
}


FVideoDecoderImpl::FVideoDecoderImpl()
	: FMediaThread("ElectraPlayer::Video decoder")
{
}

FVideoDecoderImpl::~FVideoDecoderImpl()
{
	Close();
}

void FVideoDecoderImpl::SetPlayerSessionServices(IPlayerSessionServices* InSessionServices)
{
	SessionServices = InSessionServices;
}

void FVideoDecoderImpl::Open(TSharedPtrTS<FAccessUnit::CodecData> InCodecData, FParamDict&& InAdditionalOptions, const FStreamCodecInformation* InMaxStreamConfiguration)
{
	InitialCodecSpecificData = InCodecData;
	InitialAdditionalOptions = MoveTemp(InAdditionalOptions);
	if (InMaxStreamConfiguration)
	{
		InitialMaxStreamProperties = *InMaxStreamConfiguration;
	}
	StartThread();
}

bool FVideoDecoderImpl::Reopen(TSharedPtrTS<FAccessUnit::CodecData> InCodecData, const FParamDict& InAdditionalOptions, const FStreamCodecInformation* InMaxStreamConfiguration)
{
	// Check if we can be used to decode the next set of streams.
	// If no new information is provided, err on the safe side and say we can't be used for this.
	if (!InCodecData.IsValid() || !InMaxStreamConfiguration)
	{
		return false;
	}
	// Check new against old limits.
	if (InitialMaxStreamProperties.IsSet() && InMaxStreamConfiguration)
	{
		// If the codec has suddenly changed, we cannot be used.
		if (InitialMaxStreamProperties.GetValue().GetCodec() != InMaxStreamConfiguration->GetCodec())
		{
			return false;
		}
		// If this is a H.265 stream of different profile (Main vs. Main10) we cannot be used.
		if (InMaxStreamConfiguration->GetCodec() == FStreamCodecInformation::ECodec::H265 &&
			InMaxStreamConfiguration->GetProfile() != InitialMaxStreamProperties.GetValue().GetProfile())
		{
			return false;
		}
		// If the current maximum resolution is less than what is required now, we cannot be used.
		if (InitialMaxStreamProperties.GetValue().GetResolution().Width  < InMaxStreamConfiguration->GetResolution().Width ||
			InitialMaxStreamProperties.GetValue().GetResolution().Height < InMaxStreamConfiguration->GetResolution().Height)
		{
			return false;
		}
		// Assume at this point that we can be used.
		return true;
	}

	return false;
}

void FVideoDecoderImpl::Close()
{
	StopThread();
}

void FVideoDecoderImpl::DrainForCodecChange()
{
	bDrainForCodecChange = true;
}

void FVideoDecoderImpl::SetOutputHandler(TSharedPtr<FOutputHandlerVideo, ESPMode::ThreadSafe> InOutputHandler)
{
	OutputHandler = MoveTemp(InOutputHandler);
}

void FVideoDecoderImpl::SuspendOrResumeDecoder(bool bSuspend, const FParamDict& InOptions)
{
	check(!"This has not yet been implemented. Time to do so now.");
}

void FVideoDecoderImpl::AUdataPushAU(FAccessUnit* InAccessUnit)
{
	InAccessUnit->AddRef();

	TSharedPtrTS<FDecoderInput> NextAU = MakeSharedTS<FDecoderInput>();
	NextAU->AccessUnit = InAccessUnit;
	NextAccessUnits.Enqueue(MoveTemp(NextAU));
}

void FVideoDecoderImpl::AUdataPushEOD()
{
	NextAccessUnits.SetEOD();
}

void FVideoDecoderImpl::AUdataClearEOD()
{
	NextAccessUnits.ClearEOD();
}

void FVideoDecoderImpl::AUdataFlushEverything()
{
	FlushDecoderSignal.Signal();
	DecoderFlushedSignal.WaitAndReset();
}

void FVideoDecoderImpl::SetAUInputBufferListener(IAccessUnitBufferListener* InListener)
{
	FScopeLock lock(&ListenerMutex);
	InputBufferListener = InListener;
}

void FVideoDecoderImpl::SetReadyBufferListener(IDecoderOutputBufferListener* InListener)
{
	FScopeLock lock(&ListenerMutex);
	ReadyBufferListener = InListener;
}

void FVideoDecoderImpl::StartThread()
{
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FVideoDecoderImpl::WorkerThread));
	bThreadStarted = true;
}

void FVideoDecoderImpl::StopThread()
{
	if (bThreadStarted)
	{
		TerminateThreadSignal.Signal();
		ThreadWaitDone();
		bThreadStarted = false;
	}
}

void FVideoDecoderImpl::CreateDecoderOutputPool()
{
	if (OutputHandler)
	{
		FParamDict poolOpts;
		// Note: Get the default value of 8 from some config option?
		int64 NumOutputFrames = ElectraDecodersUtil::GetVariantValueSafeI64(DecoderConfigOptions, IElectraDecoderFeature::MinimumNumberOfOutputFrames, 8);
		poolOpts.Set(OutputHandlerOptionKeys::NumBuffers, FVariantValue(NumOutputFrames));
		if (OutputHandler->PreparePool(poolOpts))
		{
			MaxOutputBuffers = (int32) OutputHandler->GetPoolProperties().GetValue(OutputHandlerOptionKeys::NumBuffers).GetInt64();
			DecoderFactoryAddtlCfg.Add(TEXT("max_output_buffers"), FVariant((uint32)MaxOutputBuffers));
			return;
		}
	}
	PostError(0, TEXT("Failed to create sample pool"), ERRCODE_VIDEO_INTERNAL_COULD_NOT_CREATE_SAMPLE_POOL);
}

void FVideoDecoderImpl::DestroyDecoderOutputPool()
{
	if (OutputHandler)
	{
		OutputHandler->ClosePool();
	}
}

void FVideoDecoderImpl::NotifyReadyBufferListener(bool bHaveOutput)
{
	if (ReadyBufferListener)
	{
		IDecoderOutputBufferListener::FDecodeReadyStats stats;
		stats.OutputBufferPoolSize = MaxOutputBuffers;
		if ((stats.NumElementsInDecoder = InDecoderInput.Num()) != 0)
		{
			stats.InDecoderTimeRangePTS.Start = InDecoderInput[0]->AccessUnit->PTS;
			stats.InDecoderTimeRangePTS.End = InDecoderInput.Last()->AccessUnit->PTS + InDecoderInput.Last()->AccessUnit->Duration;
		}
		if (CurrentAccessUnit)
		{
			stats.InDecoderTimeRangePTS.Start = Utils::Min(CurrentAccessUnit->AccessUnit->PTS, stats.InDecoderTimeRangePTS.Start.IsValid() ? stats.InDecoderTimeRangePTS.Start : FTimeValue::GetPositiveInfinity());
			stats.InDecoderTimeRangePTS.End = Utils::Max(CurrentAccessUnit->AccessUnit->PTS + CurrentAccessUnit->AccessUnit->Duration, stats.InDecoderTimeRangePTS.End.IsValid() ? stats.InDecoderTimeRangePTS.End : FTimeValue::GetNegativeInfinity());
		}
		stats.InDecoderTimeRangePTS.End.SetSequenceIndex(stats.InDecoderTimeRangePTS.Start.GetSequenceIndex());
		stats.bOutputStalled = !bHaveOutput;
		stats.bEODreached = NextAccessUnits.ReachedEOD() && !CurrentOutputBuffer;
		ListenerMutex.Lock();
		if (ReadyBufferListener)
		{
			ReadyBufferListener->DecoderOutputReady(stats);
		}
		ListenerMutex.Unlock();
	}
}

bool FVideoDecoderImpl::PostError(int32 ApiReturnValue, const FString& Message, uint16 Code, UEMediaError Error)
{
	bError = true;
	if (SessionServices)
	{
		FErrorDetail err;
		err.SetError(Error != UEMEDIA_ERROR_OK ? Error : UEMEDIA_ERROR_DETAIL);
		err.SetFacility(Facility::EFacility::VideoDecoder);
		err.SetCode(Code);
		err.SetMessage(Message);
		err.SetPlatformMessage(FString::Printf(TEXT("%d (0x%08x)"), (int32) ApiReturnValue, (int32) ApiReturnValue));
		SessionServices->PostError(err);
	}
	return false;
}

bool FVideoDecoderImpl::PostError(const IElectraDecoder::FError& InDecoderError)
{
	bError = true;
	if (SessionServices)
	{
		FErrorDetail err;
		err.SetError(UEMEDIA_ERROR_DETAIL);
		err.SetFacility(Facility::EFacility::VideoDecoder);
		err.SetCode(InDecoderError.GetCode());
		err.SetMessage(InDecoderError.GetMessage());
		err.SetPlatformMessage(FString::Printf(TEXT("%d (0x%08x)"), (int32) InDecoderError.GetSdkCode(), InDecoderError.GetSdkCode()));
		SessionServices->PostError(err);
	}
	return false;
}

void FVideoDecoderImpl::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (SessionServices)
	{
		SessionServices->PostLog(Facility::EFacility::VideoDecoder, Level, Message);
	}
}

void FVideoDecoderImpl::HandleApplicationHasEnteredForeground()
{
	int32 Count = FPlatformAtomics::InterlockedDecrement(&ApplicationSuspendCount);
	if (Count == 0)
	{
		ApplicationRunningSignal.Signal();
	}
}

void FVideoDecoderImpl::HandleApplicationWillEnterBackground()
{
	int32 Count = FPlatformAtomics::InterlockedIncrement(&ApplicationSuspendCount);
	if (Count == 1)
	{
		ApplicationRunningSignal.Reset();
	}
}

bool FVideoDecoderImpl::InternalDecoderCreate()
{
	InternalDecoderDestroy();

	if (!DecoderFactory.IsValid())
	{
		return PostError(-2, TEXT("No decoder factory found to create an video decoder"), ERRCODE_VIDEO_INTERNAL_COULD_NOT_CREATE_DECODER);
	}

	// Create platform specifics.
	TMap<FString, FVariant> PlatformSpecificCfg(DecoderFactoryAddtlCfg);
	DecoderPlatformResource = FElectraDecodersPlatformResources::CreatePlatformVideoResource(this, PlatformSpecificCfg);

	// Add in video decoder special options passed from the application.
	TMap<FString, FVariant> DecoderCreateCfg(DecoderFactoryAddtlCfg);
	InitialAdditionalOptions.ConvertKeysStartingWithTo(DecoderCreateCfg, TEXT("output_texturepool_id"), FString());

	DecoderInstance = DecoderFactory->CreateDecoder(InitialCodecTypeFormat, DecoderCreateCfg);
	if (!DecoderInstance.IsValid() || DecoderInstance->GetError().IsSet())
	{
		InternalDecoderDestroy();
		return PostError(-2, TEXT("Failed to create video decoder"), ERRCODE_VIDEO_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
	if (DecoderInstance->GetType() != IElectraDecoder::EType::Video)
	{
		InternalDecoderDestroy();
		return PostError(-2, TEXT("Created decoder is not a video decoder!"), ERRCODE_VIDEO_INTERNAL_COULD_NOT_CREATE_DECODER);
	}

	TMap<FString, FVariant> Features;
	DecoderInstance->GetFeatures(Features);
	bIsAdaptiveDecoder = ElectraDecodersUtil::GetVariantValueSafeI64(Features, IElectraDecoderFeature::IsAdaptive, 0) != 0;
	bSupportsDroppingOutput = ElectraDecodersUtil::GetVariantValueSafeI64(Features, IElectraDecoderFeature::SupportsDroppingOutput, 0) != 0;
	bNeedsReplayData = ElectraDecodersUtil::GetVariantValueSafeI64(Features, IElectraDecoderFeature::NeedReplayDataOnDecoderLoss, 0) != 0;
	// If replay data is not needed we can let go of anything we may have collected (which should be only the first access unit).
	if (!bNeedsReplayData)
	{
		ReplayAccessUnits.Empty();
		ReplayingAccessUnits.Empty();
	}
	bMustBeSuspendedInBackground = MustSuspendDecodersInBackground(ElectraDecodersUtil::GetVariantValueSafeI64(Features, IElectraDecoderFeature::MustBeSuspendedInBackground, 0) != 0);
	if (bMustBeSuspendedInBackground)
	{
		FGBGHandlers = MakeSharedTS<FFGBGNotificationHandlers>();
		FGBGHandlers->WillEnterBackground = [this]() { HandleApplicationWillEnterBackground(); };
		FGBGHandlers->HasEnteredForeground = [this]() { HandleApplicationHasEnteredForeground(); };
		if (AddBGFGNotificationHandler(FGBGHandlers))
		{
			HandleApplicationWillEnterBackground();
		}
	}

	// Get the bitstream processor for this decoder, if it requires one.
	DecoderBitstreamProcessor = DecoderInstance->CreateBitstreamProcessor();
	return true;
}

void FVideoDecoderImpl::InternalDecoderDestroy()
{
	if (FGBGHandlers.IsValid())
	{
		RemoveBGFGNotificationHandler(FGBGHandlers);
		FGBGHandlers.Reset();
	}
	if (DecoderBitstreamProcessor.IsValid())
	{
		DecoderBitstreamProcessor->Clear();
		DecoderBitstreamProcessor.Reset();
	}
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Close();
		DecoderInstance.Reset();
	}
	if (DecoderPlatformResource)
	{
		FElectraDecodersPlatformResources::ReleasePlatformVideoResource(this, DecoderPlatformResource);
		DecoderPlatformResource = nullptr;
	}
	bIsAdaptiveDecoder = false;
	bSupportsDroppingOutput = false;
	bNeedsReplayData = true;
	CurrentActiveCSD.Empty();
	CurrentColorimetry.Reset();
	CurrentHDR.Reset();
}

void FVideoDecoderImpl::ReturnUnusedOutputBuffer()
{
	if (CurrentOutputBuffer && OutputHandler)
	{
		OutputHandler->ReturnOutputSample(CurrentOutputBuffer, FOutputHandlerVideo::EReturnSampleType::DontSendToQueue);
		CurrentOutputBuffer.Reset();
	}
}

FVideoDecoderImpl::EAUChangeFlags FVideoDecoderImpl::PrepareAU(TSharedPtrTS<FDecoderInput> AU)
{
	EAUChangeFlags NewAUFlags = EAUChangeFlags::None;
	if (!AU->bHasBeenPrepared)
	{
		AU->bHasBeenPrepared = true;

		// Does this AU fall (partially) outside the range for rendering?
		FTimeValue StartTime = AU->AccessUnit->PTS;
		FTimeValue EndTime = AU->AccessUnit->PTS + AU->AccessUnit->Duration;
		AU->PTS = StartTime.GetAsHNS();		// The PTS we give the decoder no matter any adjustment.
		if (AU->AccessUnit->EarliestPTS.IsValid())
		{
			// If the end time of the AU is before the earliest render PTS we do not need to decode it.
			if (EndTime <= AU->AccessUnit->EarliestPTS)
			{
				StartTime.SetToInvalid();
				AU->bMaySkipDecoding = true;
			}
			else if (StartTime < AU->AccessUnit->EarliestPTS)
			{
				StartTime = AU->AccessUnit->EarliestPTS;
			}
		}
		if (StartTime.IsValid() && AU->AccessUnit->LatestPTS.IsValid())
		{
			// If the start time is behind the latest render PTS we may have to decode, but not need render.
			if (StartTime >= AU->AccessUnit->LatestPTS)
			{
				StartTime.SetToInvalid();
				// If the decode time is behind the latest render PTS we do not need to decode.
				if (AU->AccessUnit->DTS.IsValid() && AU->AccessUnit->DTS >= AU->AccessUnit->LatestPTS)
				{
					AU->bMaySkipDecoding = true;
				}
			}
			else if (EndTime >= AU->AccessUnit->LatestPTS)
			{
				EndTime = AU->AccessUnit->LatestPTS;
			}
		}
		AU->AdjustedPTS = StartTime;
		AU->AdjustedDuration = EndTime - StartTime;
		if (AU->AdjustedDuration <= FTimeValue::GetZero())
		{
			AU->AdjustedPTS.SetToInvalid();
		}

		// Get the codec specific data
		if (AU->AccessUnit->AUCodecData.IsValid())
		{
			AU->CSDOptions.Emplace(TEXT("csd"), FVariant(CurrentAccessUnit->AccessUnit->AUCodecData->CodecSpecificData));
			AU->CSDOptions.Emplace(TEXT("dcr"), FVariant(CurrentAccessUnit->AccessUnit->AUCodecData->RawCSD));
		}
		else
		{
			AU->CSDOptions = CurrentActiveCSD;
		}
		// Pass dynamic sideband data
		if (AU->AccessUnit->DynamicSidebandData.IsValid())
		{
			for(auto& dynsbIt : *AU->AccessUnit->DynamicSidebandData)
			{
				AU->CSDOptions.Emplace(dynsbIt.Key.ToString(), FVariant(dynsbIt.Value));
			}
		}

		// Set the timestamps in the decoder input description.
		AU->DecAU.DTS = AU->AccessUnit->DTS.GetAsTimespan();
		AU->DecAU.PTS = AU->AccessUnit->PTS.GetAsTimespan();
		AU->DecAU.Duration = AU->AccessUnit->Duration.GetAsTimespan();
		AU->DecAU.UserValue = AU->PTS;
		AU->DecAU.Flags |= AU->AccessUnit->bIsSyncSample ? EElectraDecoderFlags::IsSyncSample : EElectraDecoderFlags::None;

		// Set the bitstream data and size in the decoder input description.
		AU->DecAU.Data = AU->AccessUnit->AUData;
		AU->DecAU.DataSize = AU->AccessUnit->AUSize;
		// We do not need to set up more than the bitstream data and the codec specific info to call the bitstream processor.
		IElectraDecoderBitstreamProcessor::EProcessResult BSResult = DecoderBitstreamProcessor->ProcessInputForDecoding(AU->BSI, AU->DecAU, AU->CSDOptions);
		if (BSResult == IElectraDecoderBitstreamProcessor::EProcessResult::Error)
		{
			PostError(-2, DecoderBitstreamProcessor->GetLastError(), ERRCODE_VIDEO_INTERNAL_FAILED_TO_PARSE_BITSTREAM);
			return NewAUFlags;
		}
		else if (BSResult == IElectraDecoderBitstreamProcessor::EProcessResult::CSDChanged)
		{
			CurrentActiveCSD = AU->CSDOptions;
			NewAUFlags |= EAUChangeFlags::CSDChanged;
		}

		if (bSupportsDroppingOutput && !AU->AdjustedPTS.IsValid())
		{
			NumInitialSkippedFrames += bIsStartOfSequence ? 1 : 0;
			NumInitialSkippedDecodingFrames += (AU->DecAU.Flags & EElectraDecoderFlags::IsDiscardable) == EElectraDecoderFlags::IsDiscardable ? 1 : 0;
			AU->DecAU.Flags |= EElectraDecoderFlags::DoNotOutput;
		}
	}
	return NewAUFlags;
}

FVideoDecoderImpl::EAUChangeFlags FVideoDecoderImpl::GetAndPrepareInputAU()
{
	EAUChangeFlags NewAUFlags = EAUChangeFlags::None;

	// Upcoming codec change?
	if (bDrainForCodecChange)
	{
		return EAUChangeFlags::CodecChange;
	}

	// When draining we do not ask for any new input.
	if (CurrentDecodingState == EDecodingState::Draining)
	{
		return NewAUFlags;
	}

	// Need a new access unit?
	if (!CurrentAccessUnit.IsValid())
	{
		// Notify the buffer listener that we will now be needing an AU for our input buffer.
		if (InputBufferListener && NextAccessUnits.IsEmpty())
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_VideoDecode);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, VideoDecode);
			IAccessUnitBufferListener::FBufferStats	stats;
			stats.bEODSignaled = NextAccessUnits.GetEOD();
			stats.bEODReached = NextAccessUnits.ReachedEOD();
			ListenerMutex.Lock();
			if (InputBufferListener)
			{
				InputBufferListener->DecoderInputNeeded(stats);
			}
			ListenerMutex.Unlock();
		}

		// Get the AU to be decoded if one is there.
		if (NextAccessUnits.Wait(500))
		{
			NextAccessUnits.Dequeue(CurrentAccessUnit);
			if (CurrentAccessUnit.IsValid())
			{
				NewAUFlags = PrepareAU(CurrentAccessUnit);
				// Is there a discontinuity/break in sequence of sorts?
				if (CurrentAccessUnit->AccessUnit->bTrackChangeDiscontinuity ||
					(!bInDummyDecodeMode && CurrentAccessUnit->AccessUnit->bIsDummyData) ||
					(CurrentSequenceIndex.IsSet() && CurrentSequenceIndex.GetValue() != CurrentAccessUnit->AccessUnit->PTS.GetSequenceIndex()))
				{
					NewAUFlags |= EAUChangeFlags::Discontinuity;
				}
				else if (DecoderConfig::bCheckForDTSTimejump && NextExpectedDTSHNS.IsSet() && NextExpectedDTSHNS.GetValue() > CurrentAccessUnit->AccessUnit->DTS.GetAsHNS() + DecoderConfig::BackwardsTimejumpThresholdHNS)
				{
					if (DecoderConfig::bDrainDecoderOnDetectedBackwardsTimejump)
					{
						LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Video timestamp jumped back unexpectedly by %.4fs. Draining the decoder before continuing."), (NextExpectedDTSHNS.GetValue() - CurrentAccessUnit->AccessUnit->DTS.GetAsHNS()) / 10000000.0));
						NewAUFlags |= EAUChangeFlags::Discontinuity;
					}
					else
					{
						LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Video timestamp jumped back unexpectedly by %.4fs."), (NextExpectedDTSHNS.GetValue() - CurrentAccessUnit->AccessUnit->DTS.GetAsHNS()) / 10000000.0));
					}
				}
				else if (DecoderConfig::bCheckForDTSTimejump && NextExpectedDTSHNS.IsSet() && NextExpectedDTSHNS.GetValue() < CurrentAccessUnit->AccessUnit->DTS.GetAsHNS() - DecoderConfig::ForwardTimejumpThresholdHNS)
				{
					LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Video timestamp jumped forward unexpectedly by %.4fs."), (CurrentAccessUnit->AccessUnit->DTS.GetAsHNS() - NextExpectedDTSHNS.GetValue()) / 10000000.0));
				}
				CurrentSequenceIndex = CurrentAccessUnit->AccessUnit->PTS.GetSequenceIndex();
				NextExpectedDTSHNS = (CurrentAccessUnit->AccessUnit->DTS + CurrentAccessUnit->AccessUnit->Duration).GetAsHNS();

				// The very first access unit can't have differences to the one before so we clear the flags.
				if (bIsFirstAccessUnit)
				{
					bIsFirstAccessUnit = false;
					NewAUFlags = EAUChangeFlags::None;
				}

				// If this is a sync frame then we can dump all replay data we have and start from here.
				if ((CurrentAccessUnit->DecAU.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None)
				{
					ReplayAccessUnits.Empty();
				}
				// If the decoder needs to be replayed when lost we need to hold on to the data.
				if (bNeedsReplayData && !CurrentAccessUnit->AccessUnit->bIsDummyData && (CurrentAccessUnit->DecAU.Flags & EElectraDecoderFlags::IsDiscardable) == EElectraDecoderFlags::None)
				{
					ReplayAccessUnits.Enqueue(CurrentAccessUnit);
				}
			}
		}
	}
	return NewAUFlags;
}

IElectraDecoder::ECSDCompatibility FVideoDecoderImpl::IsCompatibleWith()
{
	IElectraDecoder::ECSDCompatibility Compatibility = IElectraDecoder::ECSDCompatibility::Compatible;
	if (DecoderInstance.IsValid() && CurrentAccessUnit.IsValid())
	{
		TMap<FString, FVariant> CSDOptions;
		if (CurrentAccessUnit->AccessUnit->AUCodecData.IsValid())
		{
			CSDOptions.Emplace(TEXT("csd"), FVariant(CurrentAccessUnit->AccessUnit->AUCodecData->CodecSpecificData));
			CSDOptions.Emplace(TEXT("dcr"), FVariant(CurrentAccessUnit->AccessUnit->AUCodecData->RawCSD));
			Compatibility = DecoderInstance->IsCompatibleWith(CSDOptions);
		}
	}
	return Compatibility;
}

IElectraDecoder::EOutputStatus FVideoDecoderImpl::HandleOutput()
{
	IElectraDecoder::EOutputStatus OutputStatus = IElectraDecoder::EOutputStatus::Available;
	if (DecoderInstance.IsValid())
	{
		// Get output unless flushing or terminating
		while(!TerminateThreadSignal.IsSignaled() && !FlushDecoderSignal.IsSignaled() &&
			  (CurrentDecoderOutput.IsValid() || ((OutputStatus = DecoderInstance->HaveOutput()) == IElectraDecoder::EOutputStatus::Available)))
		{
			if (CheckBackgrounding())
			{
				continue;
			}

			// Check if the output pipeline can accept the output we want to send to it.
			if (!CanReceiveOutputFrame())
			{
				NotifyReadyBufferListener(false);
				return IElectraDecoder::EOutputStatus::TryAgainLater;
			}

			// Get the next output from the decoder.
			if (!CurrentDecoderOutput.IsValid())
			{
				CurrentDecoderOutput = StaticCastSharedPtr<IElectraDecoderVideoOutput>(DecoderInstance->GetOutput());
			}
			// No available output although advertised?
			if (!CurrentDecoderOutput.IsValid())
			{
				break;
			}
			// Sanity check.
			if (CurrentDecoderOutput->GetType() != IElectraDecoderOutput::EType::Video)
			{
				PostError(0, TEXT("Could not get decoded output due to decoded format being unsupported"), ERRCODE_VIDEO_INTERNAL_UNSUPPORTED_OUTPUT_FORMAT);
				return IElectraDecoder::EOutputStatus::Error;
			}

			// Need a new output buffer?
			if (!CurrentOutputBuffer && OutputHandler)
			{
				SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_VideoConvertOutput);
				CSV_SCOPED_TIMING_STAT(ElectraPlayer, VideoConvertOutput);
				IOutputHandlerBase::EBufferResult bufResult = OutputHandler->ObtainOutputSample(CurrentOutputBuffer);
				if (bufResult != IOutputHandlerBase::EBufferResult::Ok && bufResult != IOutputHandlerBase::EBufferResult::NoBuffer)
				{
					PostError(0, TEXT("Failed to acquire sample buffer"), ERRCODE_VIDEO_INTERNAL_COULD_NOT_GET_SAMPLE_BUFFER);
					return IElectraDecoder::EOutputStatus::Error;
				}
			}
			// Didn't get a buffer? This should not really happen since the output pipeline said it could accept a frame.
			if (!CurrentOutputBuffer)
			{
				NotifyReadyBufferListener(false);
				return IElectraDecoder::EOutputStatus::TryAgainLater;
			}

			// Check if the output can actually be output or if the decoder says this is not to be output (incorrectly decoded)
			bool bUseOutput = CurrentDecoderOutput->GetOutputType() == IElectraDecoderVideoOutput::EOutputType::Output;
			if (bUseOutput)
			{
				NotifyReadyBufferListener(true);
			}
			if (1)
			{
				SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_VideoConvertOutput);
				CSV_SCOPED_TIMING_STAT(ElectraPlayer, VideoConvertOutput);

				// Locate the input AU info that should correspond to this output.
				TSharedPtrTS<FDecoderInput> MatchingInput;
				if (InDecoderInput.Num())
				{
					// Try the frontmost entry. It should be that one.
					if (InDecoderInput[0]->PTS == CurrentDecoderOutput->GetUserValue())
					{
						MatchingInput = InDecoderInput[0];
						InDecoderInput.RemoveAt(0);
					}
					else
					{
						/*
							Not the first element. This is not expected, but possible if decoding did not start on a SAP type 1
							with PTS's increasing from there. On an open GOP or SAP type 2 or worse there may be frames with
							PTS's earlier than the starting frame.

							It may also be that the decoder could not produce valid output for some of the earlier input because
							of a broken frame or a frame that needed nonexisting frames as references.

							We check if there is a precise match somewhere in our list and use it.
							Any elements in the list that are far too old we remove since it is not likely for the decoder to
							emit those frames at all and we don't want our list to grow too long.
						*/
						for(int32 i=0; i<InDecoderInput.Num(); ++i)
						{
							if (InDecoderInput[i]->PTS == CurrentDecoderOutput->GetUserValue())
							{
								MatchingInput = InDecoderInput[i];
								InDecoderInput.RemoveAt(i);
								break;
							}
							else if (InDecoderInput[i]->PTS + DecoderConfig::RemovalOfOldDecoderInputThresholdHNS < (int64)CurrentDecoderOutput->GetUserValue())
							{
								InDecoderInput.RemoveAt(i);
								--i;
							}
						}
					}
				}
				if (!MatchingInput.IsValid())
				{
					PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_VIDEO_INTERNAL_FAILED_TO_CONVERT_OUTPUT);
					return IElectraDecoder::EOutputStatus::Error;
				}

				bUseOutput = bUseOutput ? MatchingInput->AdjustedPTS.IsValid() : false;
				if (bUseOutput)
				{
					// Create the platform specific decoder output.
					TSharedPtr<FParamDict, ESPMode::ThreadSafe> BufferProperties(new FParamDict);
					BufferProperties->Set(OutputHandlerOptionKeys::PTS, FVariantValue(MatchingInput->AdjustedPTS));
					BufferProperties->Set(OutputHandlerOptionKeys::Duration, FVariantValue(MatchingInput->AdjustedDuration));

					// Set properties from the bitstream messages.
					if (DecoderBitstreamProcessor.IsValid())
					{
						TMap<FString, FVariant> BSIProperties;
						DecoderBitstreamProcessor->SetPropertiesOnOutput(BSIProperties, MatchingInput->BSI);
						if (BSIProperties.Num())
						{
							// Colorimetry?
							TArray<uint8> CommonColorimetry(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::CommonColorimetry));
							if (CommonColorimetry.Num() == sizeof(ElectraDecodersUtil::MPEG::FCommonColorimetry))
							{
								const ElectraDecodersUtil::MPEG::FCommonColorimetry& Colorimetry(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FCommonColorimetry*>(CommonColorimetry.GetData()));
								if (!CurrentColorimetry.IsSet())
								{
									CurrentColorimetry = Electra::MPEG::FColorimetryHelper();
								}
								CurrentColorimetry.GetValue().Update(Colorimetry.colour_primaries, Colorimetry.transfer_characteristics, Colorimetry.matrix_coeffs, Colorimetry.video_full_range_flag, Colorimetry.video_format);
							}

							// HDR parameters?
							TArray<uint8> Mdcv(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::SeiMasteringDisplayColorVolume));
							if (Mdcv.Num() == sizeof(ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume))
							{
								if (!CurrentHDR.IsSet())
								{
									CurrentHDR = Electra::MPEG::FHDRHelper();
								}
								CurrentHDR.GetValue().UpdateWith(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume*>(Mdcv.GetData()));
							}
							TArray<uint8> Clli(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::SeiContentLightLeveInfo));
							if (Clli.Num() == sizeof(ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info))
							{
								if (!CurrentHDR.IsSet())
								{
									CurrentHDR = Electra::MPEG::FHDRHelper();
								}
								CurrentHDR.GetValue().UpdateWith(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info*>(Clli.GetData()));
							}
							TArray<uint8> Altc(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::SeiAlternateTransferCharacteristics));
							if (Altc.Num() == sizeof(ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics))
							{
								if (!CurrentHDR.IsSet())
								{
									CurrentHDR = Electra::MPEG::FHDRHelper();
								}
								CurrentHDR.GetValue().UpdateWith(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics*>(Altc.GetData()));
							}

							// Timecode?
							TArray<uint8> PicTiming(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::CommonPictureTiming));
							if (PicTiming.Num() == sizeof(ElectraDecodersUtil::MPEG::FCommonPictureTiming))
							{
								TSharedPtr<MPEG::FVideoDecoderTimecode, ESPMode::ThreadSafe> NewTimecode = MakeShared<MPEG::FVideoDecoderTimecode, ESPMode::ThreadSafe>();
								NewTimecode->UpdateWith(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FCommonPictureTiming*>(PicTiming.GetData()));
								BufferProperties->Set(IDecoderOutputOptionNames::Timecode, FVariantValue(NewTimecode));
							}
						}

						// Set the colorimetry, if available, on the output properties.
						if (CurrentColorimetry.IsSet())
						{
							CurrentColorimetry.GetValue().UpdateParamDict(*BufferProperties);
							// Also HDR information (which requires colorimetry!) if available.
							if (CurrentHDR.IsSet())
							{
								CurrentHDR.GetValue().SetHDRType(CurrentDecoderOutput->GetNumberOfBits(), CurrentColorimetry.GetValue());
								CurrentHDR.GetValue().UpdateParamDict(*BufferProperties);
							}
						}
					}

					FString DecoderOutputErrorMsg;
					if (!FElectraDecodersPlatformResources::SetupOutputTextureSample(DecoderOutputErrorMsg, CurrentOutputBuffer, CurrentDecoderOutput, BufferProperties, DecoderPlatformResource))
					{
						if (DecoderOutputErrorMsg.IsEmpty())
						{
							PostError(0, TEXT("Failed to set up the decoder output!"), ERRCODE_VIDEO_INTERNAL_FAILED_TO_CONVERT_OUTPUT);
						}
						else
						{
							PostError(0, FString::Printf(TEXT("Failed to set up the decoder output (%s)!"), *DecoderOutputErrorMsg), ERRCODE_VIDEO_INTERNAL_FAILED_TO_CONVERT_OUTPUT);
						}
						return IElectraDecoder::EOutputStatus::Error;
					}

					OutputHandler->ReturnOutputSample(CurrentOutputBuffer, bUseOutput ? FOutputHandlerVideo::EReturnSampleType::SendToQueue : FOutputHandlerVideo::EReturnSampleType::DontSendToQueue);
					CurrentOutputBuffer.Reset();

					if (bIsStartOfSequence && NumInitialSkippedFrames)
					{
						UE_LOGF(LogElectraPlayer, Verbose, "Frame accurate seek skipped %d leading frames of which %d had to be decoded", NumInitialSkippedFrames, NumInitialSkippedFrames-NumInitialSkippedDecodingFrames);
						bIsStartOfSequence = false;
					}
				}
				CurrentDecoderOutput.Reset();
			}
		}
	}
	else if (CurrentDecodingState == EDecodingState::Draining)
	{
		OutputStatus = IElectraDecoder::EOutputStatus::EndOfData;
	}
	// Did the decoder return an error?
	if (OutputStatus == IElectraDecoder::EOutputStatus::Error)
	{
		PostError(DecoderInstance->GetError());
	}
	return OutputStatus;
}

FVideoDecoderImpl::ENextDecodingState FVideoDecoderImpl::HandleDecoding()
{
	bDrainAfterDecode = false;
	if (CurrentAccessUnit.IsValid())
	{
		// If this AU falls outside the range where it is to be rendered and it is also discardable
		// we do not need to process it.
		if ((CurrentAccessUnit->DecAU.Flags & EElectraDecoderFlags::IsDiscardable) != EElectraDecoderFlags::None && CurrentAccessUnit->bMaySkipDecoding)
		{
			// Even if this access unit won't be decoded, if it is the last in the period and we are
			// not decoding dummy data the decoder must be drained to get the last decoded data out.
			bDrainAfterDecode = CurrentAccessUnit->AccessUnit->bIsLastInPeriod && !bInDummyDecodeMode;
			CurrentAccessUnit.Reset();
			NumInitialSkippedFrames += bIsStartOfSequence ? 1 : 0;
			NumInitialSkippedDecodingFrames += bIsStartOfSequence ? 1 : 0;
			return ENextDecodingState::NormalDecoding;
		}

		if ((bInDummyDecodeMode = CurrentAccessUnit->AccessUnit->bIsDummyData) == true)
		{
			ReplayAccessUnits.Empty();
			ReplayingAccessUnits.Empty();
			ReplayAccessUnit.Reset();
			bool bOk = HandleDummyDecoding();
			CurrentAccessUnit.Reset();
			return bOk ? ENextDecodingState::NormalDecoding : ENextDecodingState::Error;
		}

		if (DecoderInstance.IsValid())
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_VideoDecode);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, VideoDecode);

			// Need to wait for a sync sample?
			if (bWaitForSyncSample && (CurrentAccessUnit->DecAU.Flags & EElectraDecoderFlags::IsSyncSample) == EElectraDecoderFlags::None)
			{
				if (!bWarnedMissingSyncSample)
				{
					bWarnedMissingSyncSample = true;
					UE_LOGF(LogElectraPlayer, Warning, "Expected a video sync sample at PTS %lld, but did not get one. The stream may be packaged incorrectly. Dropping frames until one arrives, which may take a while. Please wait!", (long long int)CurrentAccessUnit->DecAU.PTS.GetTicks());
				}
				bDrainAfterDecode = CurrentAccessUnit->AccessUnit->bIsLastInPeriod;
				CurrentAccessUnit.Reset();
				// Report this up as "stalled" so that we get out of prerolling.
				// This case here happens when seeking due to bad sync frame information in the container format
				// and the next sync frame may be too far away to satisfy the prerolling finished condition.
				NotifyReadyBufferListener(false);
				return ENextDecodingState::NormalDecoding;
			}

			IElectraDecoder::EDecoderError DecErr = DecoderInstance->DecodeAccessUnit(CurrentAccessUnit->DecAU, CurrentAccessUnit->CSDOptions);
			if (DecErr == IElectraDecoder::EDecoderError::None)
			{
				if ((CurrentAccessUnit->DecAU.Flags & EElectraDecoderFlags::DoNotOutput) == EElectraDecoderFlags::None)
				{
					InDecoderInput.Emplace(CurrentAccessUnit);
					InDecoderInput.StableSort([](const TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>& a, const TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>& b)
					{
						return a->PTS < b->PTS;
					});
				}
				else
				{
					MinLoopSleepTimeMsec = 0;
				}

				// If this was the last access unit in a period we need to drain the decoder _after_ having sent it
				// for decoding. We need to get its decoded output.
				bDrainAfterDecode = CurrentAccessUnit->AccessUnit->bIsLastInPeriod;
				CurrentAccessUnit.Reset();
				// Since we decoded something the decoder is no longer clean.
				bIsDecoderClean = false;
				// Likewise we are no longer waiting for a sync sample.
				bWaitForSyncSample = false;
			}
			else if (DecErr == IElectraDecoder::EDecoderError::NoBuffer || DecErr == IElectraDecoder::EDecoderError::EndOfData)
			{
				// Try again later...
				return ENextDecodingState::NormalDecoding;
			}
			else if (DecErr == IElectraDecoder::EDecoderError::LostDecoder)
			{
				/*
					Note: We leave the InDecoderInput intact on purpose. Even though we expect the decoder to not return output for
					replay data, we don't really enforce this. So if it does provide output there'd be matching input at least.
					Stale input will be removed with ongoing new output so this is not too big of a deal.
				*/

				// First release all access units we may already be replaying.
				ReplayingAccessUnits.Empty();
				// Then put all replay units into the queue for replaying.
				int32 NumReplayAUs = ReplayAccessUnits.Num();
				for(int32 i=0; i<NumReplayAUs; ++i)
				{
					// Get the frontmost AU from the replay queue
					TSharedPtrTS<FDecoderInput> AU;
					ReplayAccessUnits.Dequeue(AU);
					// And add it back to the end so that the queue will be just as it was when we're done.
					ReplayAccessUnits.Enqueue(AU);
					// Add it to the replaying queue, which is where we need them for replaying.
					if (AU != CurrentAccessUnit)
					{
						ReplayingAccessUnits.Enqueue(AU);
					}
				}
				return ReplayingAccessUnits.Num() ? ENextDecodingState::ReplayDecoding : ENextDecodingState::NormalDecoding;
			}
			else
			{
				PostError(DecoderInstance->GetError());
				return ENextDecodingState::Error;
			}
		}
	}
	return ENextDecodingState::NormalDecoding;
}


FVideoDecoderImpl::ENextDecodingState FVideoDecoderImpl::HandleReplaying()
{
	ENextDecodingState NextState = ENextDecodingState::ReplayDecoding;

	if (!ReplayAccessUnit.IsValid())
	{
		if (!ReplayingAccessUnits.Dequeue(ReplayAccessUnit))
		{
			return ENextDecodingState::NormalDecoding;
		}
	}
	bool bIsLastReplayAU = ReplayingAccessUnits.IsEmpty();

	if (DecoderInstance.IsValid())
	{
		// Set replay flags for this decode call
		ReplayAccessUnit->DecAU.Flags |= EElectraDecoderFlags::IsReplaySample | (bIsLastReplayAU ? EElectraDecoderFlags::IsLastReplaySample : EElectraDecoderFlags::None);
		IElectraDecoder::EDecoderError DecErr = DecoderInstance->DecodeAccessUnit(ReplayAccessUnit->DecAU, ReplayAccessUnit->CSDOptions);
		// Clear them again.
		ReplayAccessUnit->DecAU.Flags &= ~(EElectraDecoderFlags::IsReplaySample | EElectraDecoderFlags::IsLastReplaySample);

		if (DecErr == IElectraDecoder::EDecoderError::None)
		{
			// The decoder must not deliver output from replays, so we must not keep track of the input.
			ReplayAccessUnit.Reset();
			if (bIsLastReplayAU)
			{
				NextState = ENextDecodingState::NormalDecoding;
			}
			// Since we decoded something the decoder is no longer clean.
			bIsDecoderClean = false;
		}
		else if (DecErr == IElectraDecoder::EDecoderError::NoBuffer)
		{
			// Try again later...
			return NextState;
		}
		else if (DecErr == IElectraDecoder::EDecoderError::LostDecoder)
		{
			// First release all access units we may already be replaying.
			ReplayAccessUnit.Reset();
			ReplayingAccessUnits.Empty();
			// Then put all replay units into the queue for replaying.
			int32 NumReplayAUs = ReplayAccessUnits.Num();
			for(int32 i=0; i<NumReplayAUs; ++i)
			{
				// Get the frontmost AU from the replay queue
				TSharedPtrTS<FDecoderInput> AU;
				ReplayAccessUnits.Dequeue(AU);
				// And add it back to the end so that the queue will be just as it was when we're done.
				ReplayAccessUnits.Enqueue(AU);
				// Add it to the replaying queue, which is where we need them for replaying.
				if (AU != CurrentAccessUnit)
				{
					ReplayingAccessUnits.Enqueue(AU);
				}
			}
			return NextState;
		}
		else
		{
			PostError(DecoderInstance->GetError());
			return ENextDecodingState::Error;
		}
	}
	return NextState;
}


bool FVideoDecoderImpl::HandleDummyDecoding()
{
	check(CurrentAccessUnit.IsValid());

	// Get output unless flushing or terminating
	while(!TerminateThreadSignal.IsSignaled() && !FlushDecoderSignal.IsSignaled())
	{
		// Check if the output pipeline can accept the output we want to send to it.
		if (!CanReceiveOutputFrame())
		{
			NotifyReadyBufferListener(false);
			FMediaRunnable::SleepMilliseconds(5);
			continue;
		}

		// Need a new output buffer?
		if (!CurrentOutputBuffer && OutputHandler)
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_VideoConvertOutput);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, VideoConvertOutput);
			IOutputHandlerBase::EBufferResult bufResult = OutputHandler->ObtainOutputSample(CurrentOutputBuffer);
			if (bufResult != IOutputHandlerBase::EBufferResult::Ok && bufResult != IOutputHandlerBase::EBufferResult::NoBuffer)
			{
				return PostError(0, TEXT("Failed to acquire sample buffer"), ERRCODE_VIDEO_INTERNAL_COULD_NOT_GET_SAMPLE_BUFFER);
			}
		}
		// Didn't get a buffer?
		if (!CurrentOutputBuffer)
		{
			NotifyReadyBufferListener(false);
			FMediaRunnable::SleepMilliseconds(5);
			continue;
		}

		NotifyReadyBufferListener(true);

		CurrentOutputBuffer->SetTime(FMediaTimeStamp(CurrentAccessUnit->AdjustedPTS.GetAsTimespan(), CurrentAccessUnit->AdjustedPTS.GetSequenceIndex()));
		CurrentOutputBuffer->SetDuration(CurrentAccessUnit->AdjustedDuration.GetAsTimespan());
		OutputHandler->ReturnOutputSample(CurrentOutputBuffer, FOutputHandlerVideo::EReturnSampleType::DummySample);
		CurrentOutputBuffer.Reset();
		// We must not drain the source buffer too quickly. While our counterpart code in the audio decoder actually
		// produces a usable sample containing silence, we cannot create a usable dummy frame because we have to
		// keep the last good frame on screen. Our sample we have just returned will not actually be sent into the
		// media sample queue and thus any next call to `CanReceiveOutputFrame()` above will always
		// return `true` because the sample queue will not be full, and as a result we race and take new source
		// samples from the buffer so quickly, that the buffer will underrun.
		// To prevent this we put ourselves to sleep for a while. Not the entire sample duration though, but for
		// enough time to hopefully not cause an underrun.
		// NOTE: Technically speaking this is not a good solution because we should not really sleep here as
		//       that is only acceptable at 1x play rate. If playing faster we would need to sleep for a shorter
		//       duration here or not at all. Filler data on missing media segments should not really happen
		//       though, so I'm hoping we're getting by.
		MinLoopSleepTimeMsec = CurrentAccessUnit->AdjustedDuration.GetAsMilliseconds() - 1;
		return true;
	}
	return true;
}


void FVideoDecoderImpl::StartDraining(EDecodingState InNextStateAfterDraining)
{
	if (CurrentDecodingState == EDecodingState::NormalDecoding)
	{
		// Drain the decoder only when we sent it something to work on.
		// If it already clean there is no point in doing that.
		if (!bIsDecoderClean && DecoderInstance.IsValid())
		{
			IElectraDecoder::EDecoderError DecErr = DecoderInstance->SendEndOfData();
			if (DecErr != IElectraDecoder::EDecoderError::None)
			{
				PostError(DecoderInstance->GetError());
			}
		}
		// We do however set our internal state to draining in order to pick up any
		// potentially pending output and clear out pending input.
		CurrentDecodingState = EDecodingState::Draining;
		NextDecodingStateAfterDrain = InNextStateAfterDraining;
		bIsDecoderClean = true;
	}
}


bool FVideoDecoderImpl::CheckForFlush()
{
	// Flush?
	if (FlushDecoderSignal.IsSignaled())
	{
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_VideoDecode);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, VideoDecode);
		if (DecoderInstance.IsValid())
		{
			DecoderInstance->Flush();
		}
		ReturnUnusedOutputBuffer();
		CurrentDecoderOutput.Reset();
		NextAccessUnits.Empty();
		ReplayAccessUnits.Empty();
		ReplayingAccessUnits.Empty();
		ReplayAccessUnit.Reset();
		InDecoderInput.Empty();
		CurrentSequenceIndex.Reset();
		NextExpectedDTSHNS.Reset();
		CurrentAccessUnit.Reset();
		CurrentActiveCSD.Empty();
		CurrentColorimetry.Reset();
		CurrentHDR.Reset();
		bIsDecoderClean = true;
		bInDummyDecodeMode = false;
		bWaitForSyncSample = true;
		bWarnedMissingSyncSample = false;
		CurrentDecodingState = EDecodingState::NormalDecoding;
		NumInitialSkippedFrames = 0;
		NumInitialSkippedDecodingFrames = 0;
		bIsStartOfSequence = true;
		if (DecoderBitstreamProcessor.IsValid())
		{
			DecoderBitstreamProcessor->Clear();
		}
		FlushDecoderSignal.Reset();
		DecoderFlushedSignal.Signal();
		return true;
	}
	return false;
}

bool FVideoDecoderImpl::CheckBackgrounding()
{
	// If in background, wait until we get activated again.
	if (!ApplicationRunningSignal.IsSignaled())
	{
		UE_LOGF(LogElectraPlayer, Log, "FVideoDecoderImpl(%p): OnSuspending", this);
		if (DecoderInstance.IsValid())
		{
			DecoderInstance->Suspend();
		}
		ApplicationSuspendConfirmedSignal.Signal();
		while(!ApplicationRunningSignal.WaitTimeout(100 * 1000) && !TerminateThreadSignal.IsSignaled())
		{
		}
		UE_LOGF(LogElectraPlayer, Log, "FVideoDecoderImpl(%p): OnResuming", this);
		if (DecoderInstance.IsValid())
		{
			DecoderInstance->Resume();
		}
		return true;
	}
	return false;
}


void FVideoDecoderImpl::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	ApplicationRunningSignal.Signal();
	ApplicationSuspendConfirmedSignal.Reset();

	bError = false;
	bIsFirstAccessUnit = true;
	bInDummyDecodeMode = false;
	bIsAdaptiveDecoder = false;
	bSupportsDroppingOutput = false;
	// Start out assuming replay data will be needed. We only know this for sure once we have created a decoder instance.
	bNeedsReplayData = true;
	bDrainAfterDecode = false;
	bIsDecoderClean = true;
	bWaitForSyncSample = true;
	bWarnedMissingSyncSample = false;
	CurrentDecodingState = EDecodingState::NormalDecoding;

	// Clear initial skip frame stats
	NumInitialSkippedFrames = 0;
	NumInitialSkippedDecodingFrames = 0;
	bIsStartOfSequence = true;

	check(InitialCodecSpecificData.IsValid());
	if (InitialCodecSpecificData.IsValid())
	{
		InitialCodecTypeFormat = InitialCodecSpecificData->ParsedInfo.ToCodecTypeFormat();
		if (InitialCodecTypeFormat.DCR.IsEmpty() && InitialCodecSpecificData->RawCSD.Num())
		{
			InitialCodecTypeFormat.DCR = InitialCodecSpecificData->RawCSD;
		}
		if (InitialCodecTypeFormat.CSD.IsEmpty() && InitialCodecSpecificData->CodecSpecificData.Num())
		{
			InitialCodecTypeFormat.CSD = InitialCodecSpecificData->CodecSpecificData;
		}
		DecoderFactory = GetDecoderFactory(DecoderFactoryAddtlCfg, InitialCodecTypeFormat);
		if (InitialMaxStreamProperties.IsSet())
		{
			DecoderFactoryAddtlCfg.Add(TEXT("max_width"), FVariant((uint32)InitialMaxStreamProperties.GetValue().GetResolution().Width));
			DecoderFactoryAddtlCfg.Add(TEXT("max_height"), FVariant((uint32)InitialMaxStreamProperties.GetValue().GetResolution().Height));
			DecoderFactoryAddtlCfg.Add(TEXT("max_bitrate"), FVariant((int64)InitialMaxStreamProperties.GetValue().GetBitrate()));
			if (InitialMaxStreamProperties.GetValue().GetFrameRate().IsValid())
			{
				DecoderFactoryAddtlCfg.Add(TEXT("max_fps"), FVariant((double)InitialMaxStreamProperties.GetValue().GetFrameRate().GetAsDouble()));
				DecoderFactoryAddtlCfg.Add(TEXT("max_fps_n"), FVariant((int64)InitialMaxStreamProperties.GetValue().GetFrameRate().GetNumerator()));
				DecoderFactoryAddtlCfg.Add(TEXT("max_fps_d"), FVariant((uint32)InitialMaxStreamProperties.GetValue().GetFrameRate().GetDenominator()));
			}
			else
			{
				DecoderFactoryAddtlCfg.Add(TEXT("max_fps"), FVariant((double)0.0));
				DecoderFactoryAddtlCfg.Add(TEXT("max_fps_n"), FVariant((int64)0));
				DecoderFactoryAddtlCfg.Add(TEXT("max_fps_d"), FVariant((uint32)0));
			}
			DecoderFactoryAddtlCfg.Add(TEXT("max_codecprofile"), FVariant(InitialMaxStreamProperties.GetValue().GetCodecSpecifierRFC6381()));
		}
		if (DecoderFactory.IsValid())
		{
			DecoderFactory->GetConfigurationOptions(DecoderConfigOptions);
			if (SessionServices)
			{
				SessionServices->GetMediaInfoDictionary().Set(MediaInfoVideoDecoderProvider, FVariantValue(DecoderFactory->GetProviderInformation().GetName()));
			}
		}
	}

	CreateDecoderOutputPool();

	int64 TimeLast = MEDIAutcTime::CurrentMSec();
	const int32 kDefaultMinLoopSleepTimeMS = 5;
	while(!TerminateThreadSignal.IsSignaled())
	{
		if (CheckBackgrounding())
		{
			continue;
		}

		// Is there a pending flush? If so, execute the flush and go back to the top to check if we must terminate now.
		if (CheckForFlush())
		{
			continue;
		}

		// Because of the different paths this loop can take there is a possibility that it may go very fast and not wait for any resources.
		// To prevent this from becoming a tight loop we make sure to sleep at least some time  here to throttle down.
		int64 TimeNow = MEDIAutcTime::CurrentMSec();
		int64 elapsedMS = TimeNow - TimeLast;
		if (elapsedMS < MinLoopSleepTimeMsec)
		{
			FMediaRunnable::SleepMilliseconds(MinLoopSleepTimeMsec - elapsedMS);
		}
		else
		{
			FPlatformProcess::YieldThread();
		}
		TimeLast = TimeNow;
		MinLoopSleepTimeMsec = kDefaultMinLoopSleepTimeMS;

		// Create decoder if necessary.
		if (!DecoderInstance.IsValid())
		{
			if (!InternalDecoderCreate())
			{
				bError = true;
			}
		}

		if (!bError)
		{
			// Get the next access unit to decode.
			EAUChangeFlags NewAUFlags = GetAndPrepareInputAU();

			// Did the codec specific data change?
			if ((NewAUFlags & EAUChangeFlags::CSDChanged) != EAUChangeFlags::None)
			{
				// If the decoder is not adaptive, ask it how we have to handle the change.
				if (!bIsAdaptiveDecoder)
				{
					IElectraDecoder::ECSDCompatibility Compatibility = IsCompatibleWith();
					if (Compatibility == IElectraDecoder::ECSDCompatibility::Drain || Compatibility == IElectraDecoder::ECSDCompatibility::DrainAndReset)
					{
						StartDraining(Compatibility == IElectraDecoder::ECSDCompatibility::Drain ? EDecodingState::NormalDecoding : EDecodingState::NeedsReset);
					}
				}
			}
			// Is there a discontinuity that requires us to drain the decoder, including a switch to dummy-decoding?
			else if ((NewAUFlags & EAUChangeFlags::Discontinuity) != EAUChangeFlags::None)
			{
				StartDraining(EDecodingState::NormalDecoding);
			}
			// Upcoming codec change?
			else if ((NewAUFlags & EAUChangeFlags::CodecChange) != EAUChangeFlags::None)
			{
				StartDraining(EDecodingState::CodecChange);
			}

			// When draining the decoder we get all the output that we can.
			if (CurrentDecodingState == EDecodingState::Draining)
			{
				IElectraDecoder::EOutputStatus OS = HandleOutput();
				if (OS == IElectraDecoder::EOutputStatus::Error)
				{
					bError = true;
				}
				else if (OS == IElectraDecoder::EOutputStatus::TryAgainLater)
				{
				}
				else if (OS == IElectraDecoder::EOutputStatus::EndOfData ||
						 OS == IElectraDecoder::EOutputStatus::NeedInput)
				{
					// All output has been retrieved
					InDecoderInput.Empty();
					// Continue with next state.
					CurrentDecodingState = NextDecodingStateAfterDrain;
				}
			}

			// Codec change?
			if (CurrentDecodingState == EDecodingState::CodecChange)
			{
				// We are done. Leave the decode loop.
				break;
			}

			// Does the decoder need to be reset?
			if (CurrentDecodingState == EDecodingState::NeedsReset)
			{
				if (DecoderInstance.IsValid())
				{
					if (!DecoderInstance->ResetToCleanStart())
					{
						InternalDecoderDestroy();
					}
				}
				CurrentDecodingState = EDecodingState::NormalDecoding;
			}

			// Handle decoding replay data?
			if (CurrentDecodingState == EDecodingState::ReplayDecoding)
			{
				HandleOutput();
				ENextDecodingState NextState = HandleReplaying();
				if (NextState != ENextDecodingState::ReplayDecoding)
				{
					CurrentDecodingState = EDecodingState::NormalDecoding;
				}
			}
			// Handle decoding of either regular or dummy data.
			if (CurrentDecodingState == EDecodingState::NormalDecoding)
			{
				HandleOutput();
				ENextDecodingState NextState = HandleDecoding();
				if (NextState == ENextDecodingState::ReplayDecoding)
				{
					// We hold on to the current access unit, but we need to replay old data first.
					CurrentDecodingState = EDecodingState::ReplayDecoding;
				}
				else
				{
					// If this access unit requires us to drain the decoder we do it now.
					if (bDrainAfterDecode)
					{
						StartDraining(EDecodingState::NormalDecoding);
					}

					// Is the buffer at EOD?
					if (NextAccessUnits.ReachedEOD())
					{
						if (!bIsDecoderClean)
						{
							StartDraining(EDecodingState::NormalDecoding);
						}
						else
						{
							NotifyReadyBufferListener(true);
						}
					}
				}
			}
		}
		else
		{
			// In case of an error spend some time sleeping. If we have an access unit use its duration, otherwise some reasonable time.
			if (CurrentAccessUnit.IsValid() && CurrentAccessUnit->AccessUnit->Duration.IsValid())
			{
				FMediaRunnable::SleepMicroseconds(CurrentAccessUnit->AccessUnit->Duration.GetAsMicroseconds());
			}
			else
			{
				FMediaRunnable::SleepMilliseconds(10);
			}
			CurrentAccessUnit.Reset();
		}
	}

	ReturnUnusedOutputBuffer();
	// Close the decoder.
	InternalDecoderDestroy();
	DestroyDecoderOutputPool();

	DecoderFactory.Reset();
	DecoderFactoryAddtlCfg.Empty();

	// Flush any remaining input data.
	NextAccessUnits.Empty();
	InDecoderInput.Empty();
	CurrentSequenceIndex.Reset();
	NextExpectedDTSHNS.Reset();
	CurrentActiveCSD.Empty();
	CurrentColorimetry.Reset();
	CurrentHDR.Reset();
	ReplayAccessUnit.Reset();
	ReplayAccessUnits.Empty();
	ReplayingAccessUnits.Empty();

	// On a pending codec change notify the player that we are done.
	if (bDrainForCodecChange)
	{
		// Notify the player that we have finished draining.
		SessionServices->SendMessageToPlayer(FDecoderMessage::Create(FDecoderMessage::EReason::DrainingFinished, this, EStreamType::Video));
		// We need to wait to get terminated. Also check if flushing is requested and acknowledge if it is.
		while(!TerminateThreadSignal.IsSignaled())
		{
			if (FlushDecoderSignal.WaitTimeoutAndReset(1000 * 10))
			{
				DecoderFlushedSignal.Signal();
			}
		}
	}
}

} // namespace Electra

