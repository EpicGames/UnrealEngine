// Copyright Epic Games, Inc. All Rights Reserved.

#include "Demuxer/ParserISO13818-1.h"
#include "Utilities/ElectraBitstream.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utilities/StringHelpers.h"

DECLARE_LOG_CATEGORY_EXTERN(LogElectraMPEGTSParser, Log, All);
DEFINE_LOG_CATEGORY(LogElectraMPEGTSParser);


namespace Electra
{

class FParserISO13818_1 : public IParserISO13818_1
{
public:
	FParserISO13818_1();
	virtual ~FParserISO13818_1();

	EParseState BeginParsing(IPlayerSessionServices* InPlayerSession, IGenericDataReader* InDataReader, EParserFlags InParseFlags, const FSourceInfo& InSourceInfo) override;
	EParseState Parse(IPlayerSessionServices* InPlayerSession, IGenericDataReader* InDataReader) override;
	TSharedPtrTS<const FProgramTable> GetCurrentProgramTable() override
	{ return CurrentProgramTable; }
	void SelectProgramStreams(int32 InProgramNumber, const TArray<int32>& InProgramStreamPIDsToEnable) override;
	TSharedPtrTS<FPESData> GetPESPacket() override;
	FErrorDetail GetLastError() const override
	{ return ErrorDetail; }

	EPESPacketResult ParsePESPacket(TArray<FESPacket>& OutPackets, TSharedPtrTS<FPESData> InPESPacket) override;
	bool ParseCSD(FStreamCodecInformation& OutParsedCSD, const FESPacket& InFromPESPacket) override;

private:
	class FStaticInitSegReader : public IGenericDataReader
	{
	public:

		int64 ReadData(void* InDestinationBuffer, int64 InNumBytesToRead, int64 InFromOffset) override
		{
			check(InFromOffset == -1 || InFromOffset == Offset);
			int64 nb = Utils::Min(InNumBytesToRead, Buffer->Num() - Offset);
			if (InDestinationBuffer && nb > 0)
			{
				FMemory::Memcpy(InDestinationBuffer, Buffer->GetData() + Offset, nb);
			}
			Offset += nb;
			return nb;
		}
		int64 GetCurrentOffset() const override
		{ return Offset; }
		int64 GetTotalSize() const override
		{ return Buffer->Num(); }
		bool HasReadBeenAborted() const override
		{ return false; }
		bool HasReachedEOF() const override
		{ return Offset >= Buffer->Num(); }

		TSharedPtr<const TArray<uint8>, ESPMode::ThreadSafe> Buffer;
		int64 Offset = 0;
	};


	// Current block values
	struct FCurrent
	{
		void Reset()
		{
			AdaptationFieldSize = -1;
			AdaptationFieldFirstByte = -1;
			BytesSkippedUntilPayload = 0;
			PID = -1;
			ContinuityCounter = -1;
			bIsStart = false;
			bErrorIndicator = false;
			bRandomAccessIndicator = false;
			PCR.Reset();
			CurrentPIDCC = -1;
			ExpectedCC = -1;
			FileOffset = -1;
		}
		uint8 DataBlock[256];
		int32 AdaptationFieldSize = -1;
		int32 AdaptationFieldFirstByte = -1;
		int32 BytesSkippedUntilPayload = 0;
		int32 PID = -1;
		int32 ContinuityCounter = -1;
		bool bIsStart = false;
		bool bErrorIndicator = false;
		bool bRandomAccessIndicator = false;
		TOptional<uint64> PCR;
		int32 CurrentPIDCC = -1;
		int32 ExpectedCC = -1;
		int64 FileOffset = -1;
	};

	enum class EPayloadType
	{
		Continue,
		PSI,
		PES
	};

	struct FPayload
	{
		EPayloadType Type;
		TSharedPtrTS<TArray<uint8>> Data;
		TSharedPtrTS<FPESData> PESData;
	};

	struct FPSITable
	{
		FPSITable(int32 InPID) : PID(InPID)
		{ }
		int32 TransportOrProgramStreamId = -1;
		int16 VersionNumber = -1;
		int16 CurrentNext = -1;
		int32 SectionNumber = -1;
		int32 LastSectionNumber = -1;
		int32 PID = -1;
	};

	struct FPIDStream
	{
		enum class EType
		{
			Section,
			PES
		};

		struct FSectionGathering
		{
			void Reset()
			{
				TotalSize = -1;
				bIsOpen = false;
				bRandomAccessIndicator = false;
				PCR.Reset();
			}
			int32 TotalSize = -1;
			bool bIsOpen = false;
			bool bRandomAccessIndicator = false;
			TOptional<uint64> PCR;
		};

		EType Type = EType::Section;
		TSharedPtrTS<TArray<uint8>> PacketDataBuffer;
		int32 ContinuityCounter = -1;
		FSectionGathering GatheringSection;
		int32 PID = -1;
		int32 ProgramID = -1;

		// Same as in the FPESStream, here for convenience.
		FProgramStreamInfo StreamInfo;

		// Enabled?
		bool bIsEnabled = false;

		FPIDStream()
		{
			PacketDataBuffer = MakeSharedTS<TArray<uint8>>();
		}
		bool ProcessPayload(TArray<FPayload>& OutPayloadResults, FBitstreamReader& InOutBR, const FCurrent& InCurrent);
		void FinishCurrentPESPacket(TSharedPtrTS<FPESData>& OutPESPacket);
	private:
		void ExtractValidSections(TArray<TSharedPtrTS<TArray<uint8>>>& OutDataSections);
		void ExtractValidPESPackets(TArray<TSharedPtrTS<FPESData>>& OutPESSections, const FCurrent& InCurrent, int32 InNumBytesAddedNow);
	};

	struct FProgramAssociation
	{
		int32 VersionNumber = -1;
		int32 NetworkPID = -1;
		TMap<uint16, int32> ProgramPIDMap;
	};

	struct FPESStream
	{
		int32 ProgramMapPID = -1;
		int32 ProgramNumber = -1;
		int32 PESPID = -1;
		FProgramStreamInfo StreamInfo;
	};

	struct FProgramMap
	{
		int32 Program = -1;
		int32 ProgramStreamPID = -1;
		int32 VersionNumber = -1;
		int32 PCRPid = -1;
		TMap<int32, FPESStream> PESPIDStream;
	};

	enum class ETableResult
	{
		Continue,
		NewProgram
	};

	struct FUserProgramSelection
	{
		int32 ProgramNumber = -1;
		TArray<int32> SelectedStreamPIDs;
	};


	struct FDTSPTS
	{
		TOptional<uint64> DTS;
		TOptional<uint64> PTS;
	};

	struct FResidualPESData
	{
		FDTSPTS PreviousDTSPTS;
		TArray<uint8> RemainingData;
		int32 PID = -1;
	};

	bool SetError(FString Message)
	{
		ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MPEGTSParser).SetCode(1).SetMessage(Message);
		ParseState = EParseState::Failed;
		return false;
	}

	EParseState ParseNextPacket(IPlayerSessionServices* InPlayerSession, IGenericDataReader* InDataReader);

	ETableResult HandlePSITable(const FPayload& InTablePayload);
	bool ProcessPAT(const FPSITable& InTableInfo, FBitstreamReader& InOutBR);
	bool ProcessPMT(const FPSITable& InTableInfo, FBitstreamReader& InOutBR);
	void ProcessSDT(const FPSITable& InTableInfo, FBitstreamReader& InOutBR);
	void ProcessDescriptor(FBitstreamReader& InOutBR, int32 InStreamType, FStreamCodecInformation& OutCodecInfo);

	void DeselectAllPESStreams();
	void ActivateUserStreamSelection();

	EPESPacketResult ParseADTSAAC(TArray<FESPacket>& OutPackets, FBitstreamReader& InOutBR, const FDTSPTS& InDTSPTS, FResidualPESData* InResidualData, bool bFlushResiduals);
	EPESPacketResult ParseMPEGAudio(TArray<FESPacket>& OutPackets, FBitstreamReader& InOutBR, const FDTSPTS& InDTSPTS, FResidualPESData* InResidualData, bool bFlushResiduals);
	EPESPacketResult ParseAVC(TArray<FESPacket>& OutPackets, FBitstreamReader& InOutBR, const FDTSPTS& InDTSPTS, FResidualPESData* InResidualData, bool bFlushResiduals);
	EPESPacketResult ParseHEVC(TArray<FESPacket>& OutPackets, FBitstreamReader& InOutBR, const FDTSPTS& InDTSPTS, FResidualPESData* InResidualData, bool bFlushResiduals);

	TUniquePtr<FStaticInitSegReader> InitSegReader;

	EParserFlags ParseFlags = EParserFlags::ParseFlag_Default;

	TSharedPtrTS<FProgramAssociation> CurrentProgramAssociation;
	TSharedPtrTS<FProgramMap> CurrentProgramMap;
	TMap<int32,	TSharedPtrTS<FProgramMap>> ProgramMap;
	TMap<uint32, TSharedPtrTS<FPIDStream>> PIDStreamData;
	FCurrent Current;

	TSharedPtrTS<FProgramTable> CurrentProgramTable;

	TOptional<FUserProgramSelection> PendingUserProgramSelection;
	TArray<TSharedPtrTS<FPESData>> AvailablePESPackets;

	TMap<int32, FResidualPESData> ResidualPESDataMap;

	FErrorDetail ErrorDetail;
	EParseState ParseState = EParseState::Continue;
	int64 FileOffset = 0;
	uint64 TimestampOffset = 0;


	static const uint32 CRCTable[256];
};

const uint32 FParserISO13818_1::CRCTable[256] = {
	0x00000000, 0x04c11db7, 0x09823b6e, 0x0d4326d9,	0x130476dc, 0x17c56b6b, 0x1a864db2, 0x1e475005,
	0x2608edb8, 0x22c9f00f, 0x2f8ad6d6, 0x2b4bcb61,	0x350c9b64, 0x31cd86d3, 0x3c8ea00a, 0x384fbdbd,
	0x4c11db70, 0x48d0c6c7, 0x4593e01e, 0x4152fda9,	0x5f15adac, 0x5bd4b01b, 0x569796c2, 0x52568b75,
	0x6a1936c8, 0x6ed82b7f, 0x639b0da6, 0x675a1011,	0x791d4014, 0x7ddc5da3, 0x709f7b7a, 0x745e66cd,
	0x9823b6e0, 0x9ce2ab57, 0x91a18d8e, 0x95609039,	0x8b27c03c, 0x8fe6dd8b, 0x82a5fb52, 0x8664e6e5,
	0xbe2b5b58, 0xbaea46ef, 0xb7a96036, 0xb3687d81,	0xad2f2d84, 0xa9ee3033, 0xa4ad16ea, 0xa06c0b5d,
	0xd4326d90, 0xd0f37027, 0xddb056fe, 0xd9714b49,	0xc7361b4c, 0xc3f706fb, 0xceb42022, 0xca753d95,
	0xf23a8028, 0xf6fb9d9f, 0xfbb8bb46, 0xff79a6f1,	0xe13ef6f4, 0xe5ffeb43, 0xe8bccd9a, 0xec7dd02d,
	0x34867077, 0x30476dc0, 0x3d044b19, 0x39c556ae,	0x278206ab, 0x23431b1c, 0x2e003dc5, 0x2ac12072,
	0x128e9dcf, 0x164f8078, 0x1b0ca6a1, 0x1fcdbb16,	0x018aeb13, 0x054bf6a4, 0x0808d07d, 0x0cc9cdca,
	0x7897ab07, 0x7c56b6b0, 0x71159069, 0x75d48dde,	0x6b93dddb, 0x6f52c06c, 0x6211e6b5, 0x66d0fb02,
	0x5e9f46bf, 0x5a5e5b08, 0x571d7dd1, 0x53dc6066,	0x4d9b3063, 0x495a2dd4, 0x44190b0d, 0x40d816ba,
	0xaca5c697, 0xa864db20, 0xa527fdf9, 0xa1e6e04e,	0xbfa1b04b, 0xbb60adfc, 0xb6238b25, 0xb2e29692,
	0x8aad2b2f, 0x8e6c3698, 0x832f1041, 0x87ee0df6,	0x99a95df3, 0x9d684044, 0x902b669d, 0x94ea7b2a,
	0xe0b41de7, 0xe4750050, 0xe9362689, 0xedf73b3e,	0xf3b06b3b, 0xf771768c, 0xfa325055, 0xfef34de2,
	0xc6bcf05f, 0xc27dede8, 0xcf3ecb31, 0xcbffd686,	0xd5b88683, 0xd1799b34, 0xdc3abded, 0xd8fba05a,
	0x690ce0ee, 0x6dcdfd59, 0x608edb80, 0x644fc637,	0x7a089632, 0x7ec98b85, 0x738aad5c, 0x774bb0eb,
	0x4f040d56, 0x4bc510e1, 0x46863638, 0x42472b8f,	0x5c007b8a, 0x58c1663d, 0x558240e4, 0x51435d53,
	0x251d3b9e, 0x21dc2629, 0x2c9f00f0, 0x285e1d47,	0x36194d42, 0x32d850f5, 0x3f9b762c, 0x3b5a6b9b,
	0x0315d626, 0x07d4cb91, 0x0a97ed48, 0x0e56f0ff,	0x1011a0fa, 0x14d0bd4d, 0x19939b94, 0x1d528623,
	0xf12f560e, 0xf5ee4bb9, 0xf8ad6d60, 0xfc6c70d7,	0xe22b20d2, 0xe6ea3d65, 0xeba91bbc, 0xef68060b,
	0xd727bbb6, 0xd3e6a601, 0xdea580d8, 0xda649d6f,	0xc423cd6a, 0xc0e2d0dd, 0xcda1f604, 0xc960ebb3,
	0xbd3e8d7e, 0xb9ff90c9, 0xb4bcb610, 0xb07daba7,	0xae3afba2, 0xaafbe615, 0xa7b8c0cc, 0xa379dd7b,
	0x9b3660c6, 0x9ff77d71, 0x92b45ba8, 0x9675461f,	0x8832161a, 0x8cf30bad, 0x81b02d74, 0x857130c3,
	0x5d8a9099, 0x594b8d2e, 0x5408abf7, 0x50c9b640,	0x4e8ee645, 0x4a4ffbf2, 0x470cdd2b, 0x43cdc09c,
	0x7b827d21, 0x7f436096, 0x7200464f, 0x76c15bf8,	0x68860bfd, 0x6c47164a, 0x61043093, 0x65c52d24,
	0x119b4be9, 0x155a565e, 0x18197087, 0x1cd86d30,	0x029f3d35, 0x065e2082, 0x0b1d065b, 0x0fdc1bec,
	0x3793a651, 0x3352bbe6, 0x3e119d3f, 0x3ad08088,	0x2497d08d, 0x2056cd3a, 0x2d15ebe3, 0x29d4f654,
	0xc5a92679, 0xc1683bce, 0xcc2b1d17, 0xc8ea00a0,	0xd6ad50a5, 0xd26c4d12, 0xdf2f6bcb, 0xdbee767c,
	0xe3a1cbc1, 0xe760d676, 0xea23f0af, 0xeee2ed18,	0xf0a5bd1d, 0xf464a0aa, 0xf9278673, 0xfde69bc4,
	0x89b8fd09, 0x8d79e0be, 0x803ac667, 0x84fbdbd0, 0x9abc8bd5, 0x9e7d9662, 0x933eb0bb, 0x97ffad0c,
	0xafb010b1, 0xab710d06, 0xa6322bdf, 0xa2f33668,	0xbcb4666d, 0xb8757bda, 0xb5365d03, 0xb1f740b4
};


FParserISO13818_1::FParserISO13818_1()
{
}

FParserISO13818_1::~FParserISO13818_1()
{
}

IParserISO13818_1::EParseState FParserISO13818_1::BeginParsing(IPlayerSessionServices* InPlayerSession, IGenericDataReader* InDataReader, EParserFlags InParseFlags, const FSourceInfo& InSourceInfo)
{
	ParseFlags = InParseFlags;
	CurrentProgramAssociation.Reset();
	CurrentProgramMap.Reset();
	ProgramMap.Empty();
	PIDStreamData.Empty();
	CurrentProgramTable.Reset();
	PendingUserProgramSelection.Reset();
	AvailablePESPackets.Empty();
	Current.Reset();
	ErrorDetail.Clear();
	ParseState = EParseState::Continue;
	FileOffset = 0;
	TimestampOffset = InSourceInfo.TimestampOffset;

	if (InSourceInfo.InitSegmentData.IsValid() && InSourceInfo.InitSegmentData->Num())
	{
		// If provided the init data must be a multiple of packet size.
		if ((InSourceInfo.InitSegmentData->Num() % 188) == 0)
		{
			if (InSourceInfo.InitSegmentData->operator[](0) == 0x47)
			{
				InitSegReader = MakeUnique<FStaticInitSegReader>();
				InitSegReader->Buffer = InSourceInfo.InitSegmentData;
			}
			else
			{
				SetError(TEXT("MPEG TS sync_byte not found in init data."));
				ParseState = EParseState::Failed;
			}
		}
		else
		{
			SetError(TEXT("Init data is not a multiple of TS packet size."));
			ParseState = EParseState::Failed;
		}
	}
	return ParseState;
}

IParserISO13818_1::EParseState FParserISO13818_1::ParseNextPacket(IPlayerSessionServices* InPlayerSession, IGenericDataReader* InDataReader)
{
	EParseState NextParseState = EParseState::Continue;

	// Get the next 188 bytes. Either from static init data or from the provided reader.
	int64 NumBytesRead = (InitSegReader.IsValid() ? InitSegReader.Get() : InDataReader)->ReadData(Current.DataBlock, 188, FileOffset);
	if (NumBytesRead == 188)
	{
		Current.FileOffset = FileOffset;
		FileOffset += 188;
		// Is the first byte the required sync byte?
		if (Current.DataBlock[0] != 0x47)
		{
			SetError(TEXT("MPEG TS sync_byte not found. Invalid packet."));
			NextParseState = EParseState::Failed;
			return NextParseState;
		}
		FBitstreamReader br(Current.DataBlock, 188, 1);
		const uint32 transport_error_indicator = br.GetBits(1);
		const uint32 payload_unit_start_indicator = br.GetBits(1);
		const uint32 transport_priority = br.GetBits(1);
		const uint32 PID = br.GetBits(13);
		const uint32 transport_scrambling_control = br.GetBits(2);
		const uint32 adaptation_field_control = br.GetBits(2);
		const uint32 continuity_counter = br.GetBits(4);

		// ISO/IEC 13818-1 decoders shall discard transport stream packets with the adaptation_field_control field set to a value of '00'.
		if (adaptation_field_control == 0)
		{
			return NextParseState;
		}
		// We could remember to skip processing of this PID, but at this point we rather fail.
		if (transport_scrambling_control != 0)
		{
			SetError(TEXT("Scrambled packets are not supported."));
			return NextParseState;
		}

		// Flag in the adaptation field signaling a discontinuity.
		bool bIsDiscontinuity = false;

		int32 BytesSkippedUntilPayload = 0;
		Current.AdaptationFieldSize = -1;
		Current.AdaptationFieldFirstByte = -1;
		Current.PCR.Reset();
		Current.bRandomAccessIndicator = false;

		// Adaptation field?
		if (adaptation_field_control == 2 || adaptation_field_control == 3)
		{
			const uint32 adaptation_field_length = br.GetBits(8);
			Current.AdaptationFieldSize = (int32) adaptation_field_length;
			if (adaptation_field_control == 3)
			{
				// Note: Technically, according to the specification, the field length must be within 0-182 to allow for 1 byte of payload.
				//       We have however seen files that do not adhere to this and use all 183 bytes for stuffing, leaving no payload!
				check(adaptation_field_length <= 183);
			}
			else if (adaptation_field_control == 2)
			{
				check(adaptation_field_length == 183);
			}
			int32 adaptation_field_length_remaining = (int32) adaptation_field_length;
			if (adaptation_field_length)
			{
				FBitstreamReader abr1(br);

				Current.AdaptationFieldFirstByte = (int32) abr1.PeekBits(8);

				const uint32 discontinuity_indicator = abr1.GetBits(1);
				const uint32 random_access_indicator = abr1.GetBits(1);
				const uint32 elementary_stream_priority_indicator = abr1.GetBits(1);
				const uint32 PCR_flag = abr1.GetBits(1);
				const uint32 OPCR_flag = abr1.GetBits(1);
				const uint32 splicing_point_flag = abr1.GetBits(1);
				const uint32 transport_private_data_flag = abr1.GetBits(1);
				const uint32 adaptation_field_extension_flag = abr1.GetBits(1);
				adaptation_field_length_remaining -= 1;
				if (PCR_flag)
				{
					uint64 program_clock_reference_base = abr1.GetBits64(33);
					abr1.SkipBits(6);	// reserved
					const uint32 program_clock_reference_extension = abr1.GetBits(9);
					adaptation_field_length_remaining -= 6;
					// Set the PCR for reference.
					Current.PCR = program_clock_reference_base * 300 + program_clock_reference_extension;
				}
				if (OPCR_flag)
				{
					const uint64 original_program_clock_reference_base = abr1.GetBits64(33);
					abr1.SkipBits(6);	// reserved
					const uint32 original_program_clock_reference_extension = abr1.GetBits(9);
					adaptation_field_length_remaining -= 6;
				}
				if (splicing_point_flag)
				{
					const uint32 splice_countdown = abr1.GetBits(8);
					adaptation_field_length_remaining -= 1;
				}
				if (transport_private_data_flag)
				{
					const uint32 transport_private_data_length = abr1.GetBits(8);
					for(uint32 i=0; i<transport_private_data_length; ++i)
					{
						const uint32 private_data_byte = abr1.GetBits(8);
					}
					adaptation_field_length_remaining -= transport_private_data_length;
				}
				if (adaptation_field_extension_flag)
				{
					const uint32 adaptation_field_extension_length = abr1.GetBits(8);
					adaptation_field_length_remaining -= 1;
					adaptation_field_length_remaining -= adaptation_field_extension_length;
					#if 0
						uint32 ltw_flag = abr1.GetBits(1);
						uint32 piecewise_rate_flag = abr1.GetBits(1);
						uint32 seamless_splice_flag = abr1.GetBits(1);
						uint32 af_descriptor_not_present_flag = abr1.GetBits(1);
						abr1.SkipBits(4);	// reserved
						if (ltw_flag)
						{
							uint32 ltw_valid_flag = abr1.GetBits(1);
							uint32 ltw_offset = abr1.GetBits(15);
						}
						if (piecewise_rate_flag)
						{
							abr1.SkipBits(2);	// reserved
							uint32 piecewise_rate = abr1.GetBits(22);
						}
						if (seamless_splice_flag)
						{
							uint32 Splice_type = abr1.GetBits(4);
							uint32 DTS_next_AU_32_30 = abr1.GetBits(3);
							uint32 marker_bit = abr1.GetBits(1);
							uint32 DTS_next_AU_29_15 = abr1.GetBits(15);
							marker_bit = abr1.GetBits(1);
							uint32 DTS_next_AU_14_0 = abr1.GetBits(15);
							marker_bit = abr1.GetBits(1);
						}
						if (af_descriptor_not_present_flag == 0)
						{
							uint32 N1 = 0;
							for(uint32 i=0; i<N1; ++i)
							{
								check(!"TODO");
							// af_descriptor();
							}
						}
						else
						{
							check(!"TODO");
							uint32 N2 = 0;
							for(uint32 i=0; i<N2; ++i)
							{
								abr1.SkipBits(8);	// reserved
							}
						}
					#endif
				}

				// Was a discontinuity signaled?
				bIsDiscontinuity = !!discontinuity_indicator;

				// How many more bytes until the payload?
				BytesSkippedUntilPayload = adaptation_field_length_remaining;

				Current.bRandomAccessIndicator = !!random_access_indicator;
			}
			else
			{
				BytesSkippedUntilPayload = 1;
			}

			// Skip over the entire adaptation field.
			br.SkipBytes(adaptation_field_length);
		}

		// Set the current package values
		Current.PID = (int32) PID;
		Current.ContinuityCounter = (int32) continuity_counter;
		Current.bIsStart = !!payload_unit_start_indicator;
		// We hope that if the error indicator is set the PID will not be affected. We check the TEI only in handling
		// of the respective PSI/PES to invalidate the data we collected for this PID, so it better be correct.
		Current.bErrorIndicator = !!transport_error_indicator;
		Current.CurrentPIDCC = -1;
		Current.ExpectedCC = -1;
		Current.BytesSkippedUntilPayload = BytesSkippedUntilPayload;

		// Skip reserved PID or Null PID
		if (!((PID >= 5 && PID <= 15) || (PID == 0x1fff)))
		{
			const bool bHasPayload = adaptation_field_control == 1 || adaptation_field_control == 3;

			TSharedPtrTS<FPIDStream>* PIDStreamPtr = PIDStreamData.Find(PID);
			TSharedPtrTS<FPIDStream> PIDStream = PIDStreamPtr ? *PIDStreamPtr : nullptr;
			Current.CurrentPIDCC = PIDStream.IsValid() ? PIDStream->ContinuityCounter : -1;
			Current.ExpectedCC = Current.CurrentPIDCC >= 0 ? (bHasPayload ? ((Current.CurrentPIDCC + 1) & 15) : Current.CurrentPIDCC) : -1;
			// Duplicate packet?
			if (bHasPayload && Current.ExpectedCC >= 0 && Current.CurrentPIDCC >= 0 && Current.CurrentPIDCC == Current.ContinuityCounter)
			{
				UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Dropping duplicate packet with `continuity_counter` %d in PID %u", Current.ContinuityCounter, PID);
			}
			else
			{
				// Payload
				if (bHasPayload)
				{
					check(br.IsByteAligned());
					uint32 PayloadSize = br.GetRemainingByteLength();
					if (PayloadSize)
					{
						// PIDs 0-4 and 16-31 do not require a PAT/PMT mapping.
						if (PID <= 4 || (PID >= 16 && PID <= 31))
						{
							if (!PIDStream.IsValid())
							{
								PIDStream = PIDStreamData.Emplace(PID, MakeSharedTS<FPIDStream>());
								PIDStream->PID = (int32) PID;
							}
						}
						else if (!PIDStream.IsValid())
						{
							UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Dropping packet for PID %u", PID);
						}
						if (PIDStream.IsValid())
						{
							TArray<FPayload> Payloads;
							if (!PIDStream->ProcessPayload(Payloads, br, Current))
							{
								ResidualPESDataMap.Remove((int32) PID);
							}
							for(int32 i=0; i<Payloads.Num(); ++i)
							{
								switch(Payloads[i].Type)
								{
									case EPayloadType::PSI:
									{
										ETableResult TableResult = HandlePSITable(Payloads[i]);
										if (TableResult == ETableResult::NewProgram)
										{
											NextParseState = EParseState::NewProgram;
										}
										break;
									}
									case EPayloadType::PES:
									{
										AvailablePESPackets.Emplace(MoveTemp(Payloads[i].PESData));
										NextParseState = EParseState::HavePESPacket;
										break;
									}
									default:
									{
										break;
									}
								}
							}
						}
					}
				}
			}
			// Update the continuity counter for this PID
			if (PIDStream.IsValid())
			{
				PIDStream->ContinuityCounter = Current.ContinuityCounter;
			}
		}
	}
	else
	{
		if (NumBytesRead == 0)
		{
			// Did we read from a static init segment so far?
			if (InitSegReader.IsValid())
			{
				InitSegReader.Reset();
				FileOffset = 0;
			}
			else
			{
				NextParseState = EParseState::EOS;
			}
		}
		else
		{
			NextParseState = EParseState::ReadError;
		}
	}

	return NextParseState;
}

IParserISO13818_1::EParseState FParserISO13818_1::Parse(IPlayerSessionServices* InPlayerSession, IGenericDataReader* InDataReader)
{
	switch(ParseState)
	{
		// Continue parsing the next TS packets until there is a new data to act on.
		case EParseState::Continue:
		{
			EParseState NewState;
			do
			{
				NewState = ParseNextPacket(InPlayerSession, InDataReader);
			}
			while(NewState == EParseState::Continue);
			ParseState = NewState;
			// If we have reached EOS, we can't let the user know right away as we first need to deliver
			// any pending PES packets.
			if (NewState == EParseState::EOS)
			{
				return EParseState::Continue;
			}
			break;
		}
		case EParseState::NewProgram:
		{
			if (PendingUserProgramSelection.IsSet())
			{
				DeselectAllPESStreams();
				ActivateUserStreamSelection();
			}
			else
			{
				UE_LOGF(LogElectraMPEGTSParser, Log, "Received new program, but user did not handle it.");
			}
			ParseState = EParseState::Continue;
			break;
		}
		case EParseState::HavePESPacket:
		{
			if (AvailablePESPackets.IsEmpty())
			{
				ParseState = EParseState::Continue;
			}
			break;
		}
		case EParseState::Failed:
		{
			break;
		}
		case EParseState::EOS:
		{
			// Emit all the currently open PES packets.
			for(auto& pesIt : PIDStreamData)
			{
				TSharedPtrTS<FPIDStream> pidPes = pesIt.Value;
				TSharedPtrTS<FPESData> PESPacket;
				pidPes->FinishCurrentPESPacket(PESPacket);
				if (PESPacket.IsValid())
				{
					AvailablePESPackets.Emplace(MoveTemp(PESPacket));
					return EParseState::HavePESPacket;
				}
				else
				{
					// Are there residuals that, at least for video PES streams will most likely
					// contain the last frame? We assume the residuals to be a complete frame and
					// not partial data that carries over into the next segment.
					if (ResidualPESDataMap.Contains(pesIt.Key))
					{
						PESPacket = MakeShared<FPESData, ESPMode::ThreadSafe>();
						PESPacket->PID = pesIt.Key;
						PESPacket->StreamType = pidPes->StreamInfo.StreamType;
						AvailablePESPackets.Emplace(MoveTemp(PESPacket));
						return EParseState::HavePESPacket;
					}
				}
			}
			// Flush any residuals. We do not expect consecutive segments to need them.
			ResidualPESDataMap.Empty();
			break;
		}
		default:
		{
			break;
		}
	}
	return ParseState;
}

void FParserISO13818_1::DeselectAllPESStreams()
{
	for(auto& pid : PIDStreamData)
	{
		const TSharedPtrTS<FPIDStream>& ps = pid.Value;
		check(ps.IsValid());
		if (ps->Type == FPIDStream::EType::PES)
		{
			ps->PacketDataBuffer = MakeSharedTS<TArray<uint8>>();
			ps->GatheringSection.Reset();
			ps->bIsEnabled = false;
		}
	}
	ResidualPESDataMap.Empty();
}

void FParserISO13818_1::ActivateUserStreamSelection()
{
	if (PendingUserProgramSelection.IsSet())
	{
		const FUserProgramSelection& sel(PendingUserProgramSelection.GetValue());
		if (sel.ProgramNumber >= 0)
		{
			if (ProgramMap.Contains(sel.ProgramNumber))
			{
				TSharedPtrTS<FProgramMap> pm = ProgramMap[sel.ProgramNumber];
				for(int32 i=0; i<sel.SelectedStreamPIDs.Num(); ++i)
				{
					if (sel.SelectedStreamPIDs[i] >= 32)
					{
						if (pm->PESPIDStream.Contains(sel.SelectedStreamPIDs[i]))
						{
							if (PIDStreamData.Contains((uint32)sel.SelectedStreamPIDs[i]))
							{
								PIDStreamData[(uint32)sel.SelectedStreamPIDs[i]]->bIsEnabled = true;
							}
							else
							{
								UE_LOGF(LogElectraMPEGTSParser, Warning, "PID %d not present in stream map.", sel.SelectedStreamPIDs[i]);
							}
						}
						else
						{
							UE_LOGF(LogElectraMPEGTSParser, Log, "User selected PES PID %d which does not exist in the selected program.", sel.SelectedStreamPIDs[i]);
						}
					}
					else
					{
						UE_LOGF(LogElectraMPEGTSParser, Warning, "Invalid PID %d specified in program stream selection.", sel.SelectedStreamPIDs[i]);
					}
				}
			}
			else
			{
				UE_LOGF(LogElectraMPEGTSParser, Log, "User selected program %d which does not exist in the PAT.", sel.ProgramNumber);
			}
		}
		else
		{
			UE_LOGF(LogElectraMPEGTSParser, Log, "Not enabling any PES stream since user requested no program.");
		}
		PendingUserProgramSelection.Reset();
	}
}


void FParserISO13818_1::SelectProgramStreams(int32 InProgramNumber, const TArray<int32>& InProgramStreamPIDsToEnable)
{
	FUserProgramSelection sel;
	sel.ProgramNumber = InProgramNumber;
	sel.SelectedStreamPIDs = InProgramStreamPIDsToEnable;
	PendingUserProgramSelection = sel;
}

TSharedPtrTS<FParserISO13818_1::FPESData> FParserISO13818_1::GetPESPacket()
{
	check(AvailablePESPackets.Num());
	TSharedPtrTS<FParserISO13818_1::FPESData> Next(AvailablePESPackets[0]);
	AvailablePESPackets.RemoveAt(0);
	return MoveTemp(Next);
}


void FParserISO13818_1::FPIDStream::ExtractValidSections(TArray<TSharedPtrTS<TArray<uint8>>>& OutDataSections)
{
	check(PacketDataBuffer.IsValid());
	if (!PacketDataBuffer.IsValid())
	{
		return;
	}
	TArray<uint8>& AccumulationBuffer = *PacketDataBuffer;

	while(GatheringSection.bIsOpen)
	{
		// Get length if not known yet.
		if (GatheringSection.TotalSize < 0 && AccumulationBuffer.Num() >= 3)
		{
			GatheringSection.TotalSize = (int32) (((uint32)(AccumulationBuffer[1] & 3) << 8U) | AccumulationBuffer[2]) + 3;
		}
		// If we do not have enough bytes, leave.
		if (GatheringSection.TotalSize < 0 || AccumulationBuffer.Num() < GatheringSection.TotalSize)
		{
			return;
		}

		// Extract the section bytes. There may even be bytes from new section or stuffing (0xff) bytes following.

		bool bCheckCRC = false;
		// Validate the CRC for table_id's 0-3.
		if (AccumulationBuffer[0] <= 3)
		{
			bCheckCRC = true;
		}
		// ETSI EN 300 468 - 5.2.3 Service Description Table ?
		else if (PID == 17 && AccumulationBuffer[0] == 0x42)
		{
			bCheckCRC = true;
		}
		bool bIsGood = true;
		if (bCheckCRC)
		{
			uint32 crc = 0xffffffffU;
			const uint8* SectionData = AccumulationBuffer.GetData();
			for(int32 i=0; i<GatheringSection.TotalSize; ++i)
			{
				crc = (crc << 8) ^ FParserISO13818_1::CRCTable[(crc >> 24) ^ (*SectionData++)];
			}
			// CRC bad?
			if (crc)
			{
				UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Mismatching `CRC_32` in section, ignoring section!");
				bIsGood = false;
			}
		}
		if (bIsGood)
		{
			TSharedPtrTS<TArray<uint8>> NewSection = MakeSharedTS<TArray<uint8>>();
			NewSection->Append(AccumulationBuffer.GetData(), GatheringSection.TotalSize);
			OutDataSections.Emplace(MoveTemp(NewSection));
		}
		// Remove this section data.
		AccumulationBuffer.RemoveAt(0, GatheringSection.TotalSize, EAllowShrinking::No);
		// Is there additional data?
		if (AccumulationBuffer.Num() == 0 || AccumulationBuffer[0] == 0xff)
		{
			// No.
			GatheringSection.Reset();
			AccumulationBuffer.Empty();
		}
	}
}

void FParserISO13818_1::FPIDStream::ExtractValidPESPackets(TArray<TSharedPtrTS<FPESData>>& OutPESSections, const FCurrent& InCurrent, int32 InNumBytesAddedNow)
{
	check(PacketDataBuffer.IsValid());
	if (!PacketDataBuffer.IsValid())
	{
		return;
	}
	TArray<uint8>& AccumulationBuffer = *PacketDataBuffer;
	while(GatheringSection.bIsOpen)
	{
		// Get length if not known yet.
		if (GatheringSection.TotalSize < 0 && AccumulationBuffer.Num() >= 6)
		{
			// First check if this is actually a PES packet
			if (AccumulationBuffer[0] != 0 || AccumulationBuffer[1] != 0 || AccumulationBuffer[2] != 1)
			{
				UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Supposed PES packet does not start with 0x00 0x00 0x01, ignoring!");
				GatheringSection.Reset();
				AccumulationBuffer.Empty();
				return;
			}
			GatheringSection.TotalSize = (int32) (((uint32)AccumulationBuffer[4] << 8U) | AccumulationBuffer[5]);
			// If there is a known size, add the size of the header to it because the given size excludes it.
			GatheringSection.TotalSize += GatheringSection.TotalSize ? 6 : 0;
			// A size of 0 is only permitted with video streams
			if (GatheringSection.TotalSize == 0 && ((AccumulationBuffer[3] & 0xf0) != 0xe0))
			{
				UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "PES packet size given as 0 for a non-video stream type 0x(%02x), ignoring!", AccumulationBuffer[3]);
				GatheringSection.Reset();
				AccumulationBuffer.Empty();
				return;
			}
		}
		// If we do not have enough bytes to get the size (even if zero), leave.
		if ((GatheringSection.TotalSize < 0) || (GatheringSection.TotalSize > 0 && AccumulationBuffer.Num() < GatheringSection.TotalSize))
		{
			if (InNumBytesAddedNow < 0)
			{
				UE_LOGF(LogElectraMPEGTSParser, Verbose, "PES packet with given size of %d on PID %d is incomplete at end of stream, ignoring!", GatheringSection.TotalSize, PID);
			}
			return;
		}

		if (GatheringSection.TotalSize > 0)
		{
			// Move the entire accumulation buffer over.
			TSharedPtrTS<FPESData> NewPES = MakeSharedTS<FPESData>();
			NewPES->PID = PID;
			NewPES->StreamType = StreamInfo.StreamType;
			NewPES->PacketData = MoveTemp(PacketDataBuffer);
			NewPES->bRandomAccessIndicator = GatheringSection.bRandomAccessIndicator;
			NewPES->PCR = GatheringSection.PCR;
			OutPESSections.Emplace(MoveTemp(NewPES));
			GatheringSection.Reset();
			PacketDataBuffer = MakeSharedTS<TArray<uint8>>();
		}
		else
		{
			// End-of-stream packet flushing?
			if (InNumBytesAddedNow < 0)
			{
				InNumBytesAddedNow = 0;
			}

			// The size is unknown. This means that we collect data until we get a packet that has `payload_unit_start_indicator` set.
			if (InCurrent.bIsStart && AccumulationBuffer.Num() > InNumBytesAddedNow)
			{
				GatheringSection.TotalSize = AccumulationBuffer.Num() - InNumBytesAddedNow;
				TSharedPtrTS<FPESData> NewPES = MakeSharedTS<FPESData>();
				NewPES->PID = PID;
				NewPES->StreamType = StreamInfo.StreamType;
				NewPES->PacketData = MakeSharedTS<TArray<uint8>>();
				NewPES->PacketData->Append(AccumulationBuffer.GetData(), GatheringSection.TotalSize);
				NewPES->bRandomAccessIndicator = GatheringSection.bRandomAccessIndicator;
				NewPES->PCR = GatheringSection.PCR;
				OutPESSections.Emplace(MoveTemp(NewPES));
				AccumulationBuffer.RemoveAt(0, GatheringSection.TotalSize);
				GatheringSection.TotalSize = -1;
				// Make the current values active for the next packet now.
				GatheringSection.bRandomAccessIndicator = InCurrent.bRandomAccessIndicator;
				GatheringSection.PCR = InCurrent.PCR;
				continue;
			}
			return;
		}
	}
}


bool FParserISO13818_1::FPIDStream::ProcessPayload(TArray<FPayload>& OutPayloadResults, FBitstreamReader& InOutBR, const FCurrent& InCurrent)
{
	check(PacketDataBuffer.IsValid());
	if (!PacketDataBuffer.IsValid())
	{
		return false;
	}
	TArray<uint8>& AccumulationBuffer = *PacketDataBuffer;
	// PSI or PES?
	if (Type == EType::Section)
	{
		// Validate the continuity counter.
		if (InCurrent.ExpectedCC >= 0 && InCurrent.ExpectedCC != InCurrent.ContinuityCounter)
		{
			UE_LOGF(LogElectraMPEGTSParser, Verbose, "Mismatching `continuity_counter` in packet, dropping.");
			GatheringSection.Reset();
			return false;
		}
		if (InCurrent.bErrorIndicator)
		{
			UE_LOGF(LogElectraMPEGTSParser, Verbose, "`transport_error_indicator` set in PSI packet, dropping.");
			GatheringSection.Reset();
			return false;
		}

		// There can always only be a single section active per PID at any given time.
		// Before a new section can start the current section needs to have finished.

		// Does this packet contain a section start?
		if (InCurrent.bIsStart)
		{
			// Start packets have a pointer field. Get and validate it.
			uint32 pointer_field = InOutBR.GetBits(8);
			if (pointer_field > InOutBR.GetRemainingByteLength())
			{
				UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Section `pointer_field` points outside the packet, ignoring section!");
				GatheringSection.Reset();
				return false;
			}
			// The pointer field indicates where the section starts in the packet.
			// If we are not currently collecting section data, skip the remainder of the previous section.
			if (pointer_field && !GatheringSection.bIsOpen)
			{
				InOutBR.SkipBytes(pointer_field);
			}

			GatheringSection.bIsOpen = true;
			AccumulationBuffer.Append(reinterpret_cast<const uint8*>(InOutBR.GetRemainingData()), InOutBR.GetRemainingByteLength());
		}
		else
		{
			// Not a section start, continuation of the current section.
			if (GatheringSection.bIsOpen)
			{
				AccumulationBuffer.Append(reinterpret_cast<const uint8*>(InOutBR.GetRemainingData()), InOutBR.GetRemainingByteLength());
			}
		}
		TArray<TSharedPtrTS<TArray<uint8>>> DataSections;
		ExtractValidSections(DataSections);
		for(int32 i=0; i<DataSections.Num(); ++i)
		{
			FPayload& Payload = OutPayloadResults.Emplace_GetRef();
			Payload.Type = EPayloadType::PSI;
			Payload.Data = MoveTemp(DataSections[i]);
		}
	}
	else
	{
		// If this PES stream is not enabled then we do not need to handle it.
		if (!bIsEnabled)
		{
			return false;
		}

		// Validate the continuity counter.
		if (InCurrent.ExpectedCC >= 0 && InCurrent.ExpectedCC != InCurrent.ContinuityCounter)
		{
			UE_LOGF(LogElectraMPEGTSParser, Verbose, "Mismatching `continuity_counter` in packet, dropping.");
			GatheringSection.Reset();
			AccumulationBuffer.Empty();
			return false;
		}
		if (InCurrent.bErrorIndicator)
		{
			UE_LOGF(LogElectraMPEGTSParser, Verbose, "`transport_error_indicator` set in PES packet, dropping.");
			GatheringSection.Reset();
			AccumulationBuffer.Empty();
			return false;
		}

		int32 BytesAddedNow = 0;
		if (InCurrent.bIsStart)
		{
			// Remember values from the current packet only when they are in the start packet and the
			// gathering section is not already open. If it is, then we will probably be closing the
			// current packet during ExtractValidPESPackets() and these values apply only after that.
			if (!GatheringSection.bIsOpen)
			{
				GatheringSection.bRandomAccessIndicator = InCurrent.bRandomAccessIndicator;
				GatheringSection.PCR = InCurrent.PCR;
			}
			GatheringSection.bIsOpen = true;
		}
		if (GatheringSection.bIsOpen)
		{
			BytesAddedNow = InOutBR.GetRemainingByteLength();
			AccumulationBuffer.Append(reinterpret_cast<const uint8*>(InOutBR.GetRemainingData()), BytesAddedNow);
		}
		TArray<TSharedPtrTS<FPESData>> PESPackets;
		ExtractValidPESPackets(PESPackets, InCurrent, BytesAddedNow);
		for(int32 i=0; i<PESPackets.Num(); ++i)
		{
			FPayload& Payload = OutPayloadResults.Emplace_GetRef();
			Payload.Type = EPayloadType::PES;
			Payload.PESData = MoveTemp(PESPackets[i]);
		}
	}
	return true;
}

void FParserISO13818_1::FPIDStream::FinishCurrentPESPacket(TSharedPtrTS<FPESData>& OutPESPacket)
{
	if (Type == EType::PES)
	{
		FCurrent Final;
		Final.BytesSkippedUntilPayload = 188;
		TArray<TSharedPtrTS<FPESData>> PESPackets;
		ExtractValidPESPackets(PESPackets, Final, -1);
		check(PESPackets.Num() <= 1);
		if (PESPackets.Num())
		{
			OutPESPacket = MoveTemp(PESPackets[0]);
		}
	}
}


FParserISO13818_1::ETableResult FParserISO13818_1::HandlePSITable(const FPayload& InTablePayload)
{
	check(InTablePayload.Type == EPayloadType::PSI);
	check(InTablePayload.Data.IsValid());

	FBitstreamReader br(InTablePayload.Data->GetData(), InTablePayload.Data->Num());

	uint8 table_id = (uint8) br.GetBits(8);
	if (table_id == 0xff)
	{
		UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Section `table_id` is set to fobidden value, ignoring section!");
		return ETableResult::Continue;
	}

	const uint32 SectionMaxSize = table_id <= 3 ? 1021 : 4093;
	const bool bCheckSSI = table_id <= 2;
	uint32 section_syntax_indicator = br.GetBits(1);
	if (bCheckSSI && section_syntax_indicator == 0)
	{
		UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Section has `section_syntax_indicator` set to 0 for table_id %u, ignoring section!", table_id);
		return ETableResult::Continue;
	}
	uint32 Zero = br.GetBits(1);
	if ((table_id == 0 || table_id == 1 || table_id == 2 || table_id == 3) && Zero != 0)
	{
		UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Section has `0` set to 1 for table_id %u, ignoring section!", table_id);
		return ETableResult::Continue;
	}

	br.SkipBits(2);	// reserved
	uint32 section_length = br.GetBits(12);
	if (section_length > SectionMaxSize)
	{
		UE_LOGF(LogElectraMPEGTSParser, VeryVerbose, "Section has `section_length` %u exceeding %u, ignoring section!", section_length, SectionMaxSize);
		return ETableResult::Continue;
	}

	auto GetStandardTable = [](FPSITable& InOutTable, FBitstreamReader& InOutBR)
	{
		InOutTable.TransportOrProgramStreamId = (int32) InOutBR.GetBits(16);
		InOutBR.SkipBits(2);	// reserved
		InOutTable.VersionNumber = (int16) InOutBR.GetBits(5);
		InOutTable.CurrentNext = (int16) InOutBR.GetBits(1);
		InOutTable.SectionNumber = (int32) InOutBR.GetBits(8);
		InOutTable.LastSectionNumber = (int32) InOutBR.GetBits(8);
	};

	// 0x00 - program_association_section; 0x01 - conditional_access_section (CA_section); 0x02 - TS_program_map_section; 0x03 - TS_description_section ?
	if (table_id <= 3)
	{
		// Are we to ignore the PAT and PMT from the stream if we got a valid one from an init segment?
		if ((ParseFlags & EParserFlags::ParseFlag_IgnoreProgramStream) == EParserFlags::ParseFlag_IgnoreProgramStream && !InitSegReader.IsValid() && CurrentProgramAssociation.IsValid())
		{
			return ETableResult::Continue;
		}

		FPSITable Table(Current.PID);
		GetStandardTable(Table, br);
		if (table_id == 0)
		{
			ProcessPAT(Table, br);
		}
		else if (table_id == 2)
		{
			if (ProcessPMT(Table, br))
			{
				return ETableResult::NewProgram;
			}
		}
		return ETableResult::Continue;
	}
	// Rec. ITU-T H.222.0 | ISO/IEC 13818-1 reserved ?
	else if (table_id >= 0x0c && table_id <= 0x37)
	{
		return ETableResult::Continue;
	}
	// Defined in ISO/IEC 13818-6 ?
	else if (table_id >= 0x38 && table_id <= 0x3f)
	{
		return ETableResult::Continue;
	}
	// User private ?
	else if (table_id >= 0x40 && table_id <= 0xfe)
	{
		// We assume some user-private to be ETSI EN 300 468 (DVB) tables.

		// 5.2.3 Service Description Table ?
		if (Current.PID == 17 && table_id == 0x42)
		{
			FPSITable Table(Current.PID);
			GetStandardTable(Table, br);
			ProcessSDT(Table, br);
		}
		return ETableResult::Continue;
	}
	/*
	Already checked above.
		// Forbidden ?
		else if (table_id == 0xff)
		{
			return ETableResult::Continue;
		}
	*/
	else
	{
		/*
			One of:
				0x04 - ISO_IEC_14496_scene_description_section
				0x05 - ISO_IEC_14496_object_descriptor_section
				0x06 - Metadata_section
				0x07 - IPMP Control Information Section (defined in ISO/IEC 13818-11)
				0x08 - ISO_IEC_14496_section
				0x09 - ISO/IEC 23001-11 (Green access unit) section
				0x0A - ISO/IEC 23001-10 (Quality access unit) section
				0x0B - ISO/IEC 23001-13 (Media Orchestration access unit) section
		*/
		return ETableResult::Continue;
	}
}


bool FParserISO13818_1::ProcessPAT(const FPSITable& InTableInfo, FBitstreamReader& InOutBR)
{
	const uint32 kCRC32_ByteSize = 4;
	const uint32 kTableEntryByteSize = 4;
	const uint32 NumPrograms = (InOutBR.GetRemainingByteLength() - kCRC32_ByteSize) / kTableEntryByteSize;

	TSharedPtrTS<FProgramAssociation> NewProgramAssociation = MakeSharedTS<FProgramAssociation>();
	NewProgramAssociation->VersionNumber = InTableInfo.VersionNumber;

	for(uint32 i=0; i<NumPrograms; ++i)
	{
		uint16 program_number = (uint16) InOutBR.GetBits(16);
		InOutBR.SkipBits(3);	// reserved
		if (program_number == 0)
		{
			NewProgramAssociation->NetworkPID = (int32) InOutBR.GetBits(13);
		}
		else
		{
			NewProgramAssociation->ProgramPIDMap.Emplace(program_number, (int32) InOutBR.GetBits(13));
		}
	}

	// Activate the program association?
	if (!CurrentProgramAssociation.IsValid() || (CurrentProgramAssociation->VersionNumber != NewProgramAssociation->VersionNumber && InTableInfo.CurrentNext))
	{
		if (CurrentProgramAssociation.IsValid())
		{
			UE_LOGF(LogElectraMPEGTSParser, Warning, "PAT change detected. This is not supported at the moment. Ignoring new PAT.");
			// Do we need to clear current programs?
			return false;
		}

		// Add the PID of the program stream to the list of active PIDs
		for(auto& it : NewProgramAssociation->ProgramPIDMap)
		{
			TSharedPtrTS<FPIDStream> ProgramPIDStream = MakeSharedTS<FPIDStream>();
			ProgramPIDStream->Type = FPIDStream::EType::Section;
			ProgramPIDStream->PID = it.Value;
			ProgramPIDStream->ProgramID = it.Key;
			PIDStreamData.Emplace(ProgramPIDStream->PID, ProgramPIDStream);
		}

		CurrentProgramAssociation = NewProgramAssociation;
		return true;
	}
	return false;
}

bool FParserISO13818_1::ProcessPMT(const FPSITable& InTableInfo, FBitstreamReader& InOutBR)
{
	TSharedPtrTS<FProgramMap> NewProgramMap = MakeSharedTS<FProgramMap>();
	NewProgramMap->VersionNumber = InTableInfo.VersionNumber;
	NewProgramMap->Program = InTableInfo.TransportOrProgramStreamId;
	NewProgramMap->ProgramStreamPID = InTableInfo.PID;

	InOutBR.SkipBits(3);	// reserved
	NewProgramMap->PCRPid = (int32) InOutBR.GetBits(13);
	InOutBR.SkipBits(4);	// reserved
	uint32 program_info_length = InOutBR.GetBits(12) & 0x3ff;	// only 10 bits used, upper 2 bits must be zero, we ignore them.
	if (program_info_length)
	{
		FBitstreamReader br(InOutBR.GetRemainingData(), program_info_length);
		InOutBR.SkipBytes(program_info_length);
		while(br.GetRemainingByteLength())
		{
			uint32 descriptor_tag = br.GetBits(8);
			uint32 descriptor_length = br.GetBits(8);
			// TBD: Do something with useful descriptors?
			br.SkipBytes(descriptor_length);
		}
	}
	// Parse out the program map until the CRC_32 element.
	while(InOutBR.GetRemainingByteLength() > 4)
	{
		FStreamCodecInformation CodecInfo;
		uint32 stream_type = InOutBR.GetBits(8);
		InOutBR.SkipBits(3);	// reserved
		uint32 elementary_PID = InOutBR.GetBits(13);
		InOutBR.SkipBits(4);	// reserved
		uint32 ES_info_length = InOutBR.GetBits(12) & 0x3ff;	// only 10 bits used, upper 2 bits must be zero, we ignore them.
		// Descriptors?
		if (ES_info_length)
		{
			FBitstreamReader br(InOutBR.GetRemainingData(), ES_info_length);
			InOutBR.SkipBytes(ES_info_length);
			while(br.GetRemainingByteLength())
			{
				ProcessDescriptor(br, stream_type, CodecInfo);
			}
		}

		// Is this a supported stream type?
		switch(stream_type)
		{
			// AVC video
			case 0x1b:
			{
				CodecInfo.SetStreamType(EStreamType::Video);
				CodecInfo.SetCodec(FStreamCodecInformation::ECodec::H264);
				break;
			}
			// HEVC video
			case 0x24:
			{
				CodecInfo.SetStreamType(EStreamType::Video);
				CodecInfo.SetCodec(FStreamCodecInformation::ECodec::H265);
				break;
			}
			// ISO/IEC 11172-3 Audio
			case 0x03:
			{
				CodecInfo.SetStreamType(EStreamType::Audio);
				CodecInfo.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
				CodecInfo.SetMimeType(TEXT("audio/mpeg"));
				CodecInfo.SetCodec4CC(Utils::Make4CC('m','p','g','a'));
				CodecInfo.SetProfile(1);
				CodecInfo.SetCodecSpecifierRFC6381(TEXT("mp4a.6b"));
				break;
			}
			// ISO/IEC 13818-7 Audio with ADTS transport syntax
			case 0x0f:
			{
				CodecInfo.SetStreamType(EStreamType::Audio);
				CodecInfo.SetCodec(FStreamCodecInformation::ECodec::AAC);
				break;
			}
		}

		FPESStream& PES = NewProgramMap->PESPIDStream.Emplace((int32) elementary_PID);
		PES.ProgramMapPID = InTableInfo.PID;
		PES.ProgramNumber = InTableInfo.TransportOrProgramStreamId;
		PES.PESPID = (int32) elementary_PID;
		PES.StreamInfo.CodecInfo = CodecInfo;
		PES.StreamInfo.StreamType = stream_type;
	}

	// Activate the program map?
	if (!CurrentProgramMap.IsValid() || (CurrentProgramMap->VersionNumber != NewProgramMap->VersionNumber && InTableInfo.CurrentNext))
	{
		if (CurrentProgramMap.IsValid())
		{
			UE_LOGF(LogElectraMPEGTSParser, Warning, "PMT change detected. This is not supported at the moment. Ignoring new PMT.");
			// Do we need to clear current programs?
			return false;
		}

		// Perform some checks. These are only informational at the moment. We do not reject anything here yet.
		if (CurrentProgramAssociation.IsValid())
		{
			if (CurrentProgramAssociation->ProgramPIDMap.Contains(InTableInfo.TransportOrProgramStreamId))
			{
				if (CurrentProgramAssociation->ProgramPIDMap[InTableInfo.TransportOrProgramStreamId] != InTableInfo.PID)
				{
					UE_LOGF(LogElectraMPEGTSParser, Warning, "New PMT encountered for program %d on PID %d that was using PID %d until now.", InTableInfo.TransportOrProgramStreamId, InTableInfo.PID, CurrentProgramAssociation->ProgramPIDMap[InTableInfo.TransportOrProgramStreamId]);
				}
			}
			else
			{
				UE_LOGF(LogElectraMPEGTSParser, Warning, "New PMT encountered for program %d on PID %d that is not listed in the current PAT.", InTableInfo.TransportOrProgramStreamId, InTableInfo.PID);
			}
		}
		else
		{
			UE_LOGF(LogElectraMPEGTSParser, Warning, "New PMT encountered for program %d on PID %d without having an established PAT yet.", InTableInfo.TransportOrProgramStreamId, InTableInfo.PID);
		}

		// Add the PID of the program stream to the list of active PIDs
		for(auto& it : NewProgramMap->PESPIDStream)
		{
			TSharedPtrTS<FPIDStream> ProgramPIDStream = MakeSharedTS<FPIDStream>();
			ProgramPIDStream->Type = FPIDStream::EType::PES;
			ProgramPIDStream->PID = it.Value.PESPID;
			ProgramPIDStream->ProgramID = it.Value.ProgramNumber;
			ProgramPIDStream->StreamInfo = it.Value.StreamInfo;
			PIDStreamData.Emplace(ProgramPIDStream->PID, ProgramPIDStream);
		}

		// Set the current program to be this new one.
		CurrentProgramMap = NewProgramMap;
		// Add the new program to the table of programs for future reference.
		ProgramMap.Emplace(InTableInfo.TransportOrProgramStreamId, NewProgramMap);

		// Build the user-facing program table
		TSharedPtrTS<FProgramTable> NewProgramTable = MakeSharedTS<FProgramTable>();
		for(auto &patIt : CurrentProgramAssociation->ProgramPIDMap)
		{
			FProgramStream ps;
			int32 pmtPid = patIt.Value;
			for(auto &esIt : CurrentProgramMap->PESPIDStream)
			{
				ps.StreamTable.Emplace(esIt.Key, esIt.Value.StreamInfo);
			}
			NewProgramTable->ProgramTable.Emplace(patIt.Key, MoveTemp(ps));
		}
		CurrentProgramTable = NewProgramTable;
		return true;
	}
	return false;
}

// ETSI EN 300 468 - 5.2.3 Service Description Table
void FParserISO13818_1::ProcessSDT(const FPSITable& InTableInfo, FBitstreamReader& InOutBR)
{
	uint32 original_network_id = InOutBR.GetBits(16);
	InOutBR.SkipBits(8);	// reserved for future use
	// Parse table until we reach the CRC_32 element.
	while(InOutBR.GetRemainingByteLength() > 4)
	{
		uint32 service_id = InOutBR.GetBits(16);
		InOutBR.SkipBits(6);	// reserved_future_use
		uint32 EIT_schedule_flag = InOutBR.GetBits(1);
		uint32 EIT_present_following_flag = InOutBR.GetBits(1);
		uint32 running_status = InOutBR.GetBits(3);
		uint32 free_CA_mode = InOutBR.GetBits(1);
		uint32 descriptors_length = InOutBR.GetBits(12);
		// Descriptors?
		if (descriptors_length)
		{
			FBitstreamReader br(InOutBR.GetRemainingData(), descriptors_length);
			InOutBR.SkipBytes(descriptors_length);
			while(br.GetRemainingByteLength())
			{
				uint32 descriptor_tag = br.GetBits(8);
				uint32 descriptor_length = br.GetBits(8);
				// 6.2.33 Service descriptor ?
				if (descriptor_tag == 0x48)
				{
					FBitstreamReader dr(br.GetRemainingData(), descriptor_length);
					uint32 service_type = dr.GetBits(8);
					// no use for this right now
					(void)service_type;
					uint32 service_provider_name_length = dr.GetBits(8);
					// Skip provider
					dr.SkipBytes(service_provider_name_length);
					// Skip name
					uint32 service_name_length = dr.GetBits(8);
					dr.SkipBytes(service_name_length);
				}
				br.SkipBytes(descriptor_length);
			}
		}
	}
}

void FParserISO13818_1::ProcessDescriptor(FBitstreamReader& InOutBR, int32 InStreamType, FStreamCodecInformation& OutCodecInfo)
{
	uint32 descriptor_tag = InOutBR.GetBits(8);
	uint32 descriptor_length = InOutBR.GetBits(8);
	FBitstreamReader br(InOutBR.GetRemainingData(), descriptor_length);
	InOutBR.SkipBytes(descriptor_length);
	switch(descriptor_tag)
	{
		// ISO_639_language_descriptor()
		case 10:
		{
			while(br.GetRemainingByteLength() >= 4)
			{
				uint8 Lang[4];
				Lang[0] = (uint8) br.GetBits(8);
				Lang[1] = (uint8) br.GetBits(8);
				Lang[2] = (uint8) br.GetBits(8);
				Lang[3] = 0;
				uint8 audio_type = (uint8) br.GetBits(8);
				/*
					0x00 Undefined
					0x01 Clean effects
					0x02 Hearing impaired
					0x03 Visual impaired commentary
					0x04 .. 0x7F User Private
					0x80 Primary
					0x81 Native
					0x82 Emergency
					0x83 Primary commentary
					0x84 Alternate commentary
					0x85 .. 0xFF Reserved
				*/
				FString Language = StringHelpers::ISO_8859_1_ToFString(Lang, 3);
				BCP47::FLanguageTag LanguageTag;
				BCP47::ParseRFC5646Tag(LanguageTag, Language);
				OutCodecInfo.SetStreamLanguageTag(LanguageTag);
				if (audio_type == 0 || audio_type == 0x80)
				{
					break;
				}
			}
			break;
		}
		// HEVC_video_descriptor()
		case 56:
		{
			ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet sps;
			sps.profile_tier_level.general_profile_space = (uint8) br.GetBits(2);
			sps.profile_tier_level.general_tier_flag = (uint8) br.GetBits(1);
			sps.profile_tier_level.general_profile_idc = (uint8) br.GetBits(5);
			sps.profile_tier_level.general_profile_compatibility_flags = br.GetBits(32);
			sps.profile_tier_level.general_progressive_source_flag = (uint8) br.GetBits(1);
			sps.profile_tier_level.general_interlaced_source_flag = (uint8) br.GetBits(1);
			sps.profile_tier_level.general_non_packed_constraint_flag = (uint8) br.GetBits(1);
			sps.profile_tier_level.general_frame_only_constraint_flag = (uint8) br.GetBits(1);
			sps.profile_tier_level.general_constraint_indicator_flags = br.GetBits64(44);
			sps.profile_tier_level.general_level_idc = (uint8) br.GetBits(8);
			const uint32 temporal_layer_subset_flag = br.GetBits(1);
			const uint32 HEVC_still_present_flag = br.GetBits(1);
			const uint32 HEVC_24hr_picture_present_flag = br.GetBits(1);
			const uint32 sub_pic_hrd_params_not_present_flag = br.GetBits(1);
			br.SkipBits(2);	// reserved
			const uint32 HDR_WCG_idc = br.GetBits(2);
			if (temporal_layer_subset_flag)
			{
				const uint32 temporal_id_min = br.GetBits(3);
				br.SkipBits(5);	// reserved
				const uint32 temporal_id_max = br.GetBits(3);
				br.SkipBits(5);	// reserved
			}
			OutCodecInfo.SetStreamType(EStreamType::Video);
			OutCodecInfo.SetCodec(FStreamCodecInformation::ECodec::H265);
			OutCodecInfo.SetProfileSpace(sps.profile_tier_level.general_profile_space);
			OutCodecInfo.SetProfileTier(sps.profile_tier_level.general_tier_flag);
			OutCodecInfo.SetProfile(sps.profile_tier_level.general_profile_idc);
			OutCodecInfo.SetProfileLevel(sps.profile_tier_level.general_level_idc);
			OutCodecInfo.SetProfileConstraints(sps.GetConstraintFlags());
			OutCodecInfo.SetProfileCompatibilityFlags(sps.profile_tier_level.general_profile_compatibility_flags);
			OutCodecInfo.SetCodecSpecifierRFC6381(sps.GetRFC6381(TEXT("hvc1")));
			break;
		}
		// AAC_descriptor()
		case 124:
		{
			uint32 profile_and_level = br.GetBits(8);
			if (descriptor_length > 1)
			{
				uint32 AAC_type_flag = br.GetBits(1);
				uint32 SAOC_DE_flag = br.GetBits(1);
				br.SkipBits(6);	// reserved_zero_future_use
				if (AAC_type_flag == 1)
				{
					uint32 AAC_type = br.GetBits(8);
				}
				while(br.GetRemainingByteLength())
				{
					// additional_info_byte
					br.SkipBits(8);
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

IParserISO13818_1::EPESPacketResult FParserISO13818_1::ParsePESPacket(TArray<IParserISO13818_1::FESPacket>& OutPackets, TSharedPtrTS<IParserISO13818_1::FPESData> InPESPacket)
{
	if (!InPESPacket.IsValid() || (InPESPacket->PacketData.IsValid() && InPESPacket->PacketData->Num() < 6))
	{
		return EPESPacketResult::Invalid;
	}
	if (!InPESPacket->PacketData.IsValid() && ParseState != EParseState::EOS)
	{
		return EPESPacketResult::Invalid;
	}

	FDTSPTS DTSPTS;
	FBitstreamReader br;
	uint8 stream_id = 0xbe;
	bool bHandleStreamId = false;
	if (InPESPacket->PacketData.IsValid())
	{
		// Check that this packet has the proper start code.
		const TArray<uint8>& PESData = *(InPESPacket->PacketData);
		const uint8* Data = PESData.GetData();
		if (Data[0] != 0 || Data[1] != 0 || Data[2] != 1)
		{
			return EPESPacketResult::Invalid;
		}
		stream_id = Data[3];
		const int32 PES_packet_length = (int32) (((uint32)Data[4] << 8) | Data[5]);
		if (PES_packet_length && PES_packet_length+6 != PESData.Num())
		{
			// Packet size mismatch.
			return PES_packet_length+6 > PESData.Num() ? EPESPacketResult::Truncated : EPESPacketResult::Invalid;
		}
		if (stream_id != 0xbc /* program_stream_map */ &&
			stream_id != 0xbe /* padding_stream */ &&
			stream_id != 0xbf /* private_stream_2 */ &&
			stream_id != 0xf0 /* ECM */ &&
			stream_id != 0xf1 /* EMM */ &&
			stream_id != 0xff /* program_stream_directory */ &&
			stream_id != 0xf2 /* Rec. ITU-T H.222.0 | ISO/IEC 13818-1 Annex A or ISO/IEC 13818-6_DSMCC_stream */ &&
			stream_id != 0xf8 /* Rec. ITU-T H.222.1 type E */)
		{
			br.SetData(Data, PESData.NumBytes(), 6);
			const uint32 OneZero = br.GetBits(2);
			const uint32 PES_scrambling_control = br.GetBits(2);
			const uint32 PES_priority = br.GetBits(1);
			const uint32 data_alignment_indicator = br.GetBits(1);
			const uint32 copyright = br.GetBits(1);
			const uint32 original_or_copy = br.GetBits(1);
			const uint32 PTS_DTS_flags = br.GetBits(2);
			const uint32 ESCR_flag = br.GetBits(1);
			const uint32 ES_rate_flag = br.GetBits(1);
			const uint32 DSM_trick_mode_flag = br.GetBits(1);
			const uint32 additional_copy_info_flag = br.GetBits(1);
			const uint32 PES_CRC_flag = br.GetBits(1);
			const uint32 PES_extension_flag = br.GetBits(1);
			const uint32 PES_header_data_length = br.GetBits(8);
			FBitstreamReader opt(br);
			br.SkipBytes(PES_header_data_length);
			if (PTS_DTS_flags == 2)
			{
				const uint32 ZeroZeroOneZero = opt.GetBits(4);
				const uint32 PTS_32_30 = opt.GetBits(3);
				opt.SkipBits(1);
				const uint32 PTS_29_15 = opt.GetBits(15);
				opt.SkipBits(1);
				const uint32 PTS_14_0 = opt.GetBits(15);
				opt.SkipBits(1);
				DTSPTS.PTS = ((((uint64)PTS_32_30 << 30U) | ((uint64)PTS_29_15 << 15U) | (uint64)PTS_14_0) + TimestampOffset) & 0x1ffffffffULL;
			}
			else if (PTS_DTS_flags == 3)
			{
				const uint32 ZeroZeroOneOne = opt.GetBits(4);
				const uint32 PTS_32_30 = opt.GetBits(3);
				opt.SkipBits(1);
				const uint32 PTS_29_15 = opt.GetBits(15);
				opt.SkipBits(1);
				const uint32 PTS_14_0 = opt.GetBits(15);
				opt.SkipBits(1);
				const uint32 ZeroZeroZeroOne = opt.GetBits(4);
				const uint32 DTS_32_30 = opt.GetBits(3);
				opt.SkipBits(1);
				const uint32 DTS_29_15 = opt.GetBits(15);
				opt.SkipBits(1);
				const uint32 DTS_14_0 = opt.GetBits(15);
				opt.SkipBits(1);
				DTSPTS.PTS = ((((uint64)PTS_32_30 << 30U) | ((uint64)PTS_29_15 << 15U) | (uint64)PTS_14_0) + TimestampOffset) & 0x1ffffffffULL;
				DTSPTS.DTS = ((((uint64)DTS_32_30 << 30U) | ((uint64)DTS_29_15 << 15U) | (uint64)DTS_14_0) + TimestampOffset) & 0x1ffffffffULL;
			}
			if (ESCR_flag)
			{
				opt.SkipBits(2); // reserved
				const uint32 ESCR_base_32_30 = opt.GetBits(3);
				opt.SkipBits(1);
				const uint32 ESCR_base_29_15 = opt.GetBits(15);
				opt.SkipBits(1);
				const uint32 ESCR_base_14_0 = opt.GetBits(15);
				opt.SkipBits(1);
				const uint32 ESCR_extension = opt.GetBits(9);
				opt.SkipBits(1);
			}
			if (ES_rate_flag)
			{
				opt.SkipBits(1);
				const uint32 ES_rate = opt.GetBits(22);
				opt.SkipBits(1);
			}
			if (DSM_trick_mode_flag)
			{
				const uint32 trick_mode_control = opt.GetBits(3);
				// ...
				opt.SkipBits(5);
			}
			if (additional_copy_info_flag)
			{
				opt.SkipBits(1);
				const uint32 additional_copy_info = opt.GetBits(7);
			}
			if (PES_CRC_flag)
			{
				const uint32 previous_PES_packet_CRC = opt.GetBits(16);
			}

			bHandleStreamId = true;
		}
	}
	else
	{
		// This is called to process the residuals from a previous packet, so this needs to be handled.
		bHandleStreamId = true;
	}

	if (bHandleStreamId)
	{
		TArray<FESPacket> NewPackets;
		EPESPacketResult PESResult = EPESPacketResult::NotSupported;
		bool bSuccess = false;
		bool bFlushResiduals = ParseState == EParseState::EOS;
		check(br.IsByteAligned());	// even an unset bitstream reader is aligned, so this works when flushing residuals as well
		switch(InPESPacket->StreamType)
		{
			// MPEG audio (layer 1, 2 or 3)
			case 0x03:
			// MPEG audio in ADTS format (AAC)
			case 0x0f:
			// AVC
			case 0x1b:
			// HEVC
			case 0x24:
			{
				FResidualPESData* Residuals = ResidualPESDataMap.Find(InPESPacket->PID);
				if (InPESPacket->StreamType == 0x0f)
				{
					PESResult = ParseADTSAAC(NewPackets, br, DTSPTS, Residuals, bFlushResiduals);
				}
				else if (InPESPacket->StreamType == 0x03)
				{
					PESResult = ParseMPEGAudio(NewPackets, br, DTSPTS, Residuals, bFlushResiduals);
				}
				else if (InPESPacket->StreamType == 0x1b)
				{
					PESResult = ParseAVC(NewPackets, br, DTSPTS, Residuals, bFlushResiduals);
				}
				else
				{
					PESResult = ParseHEVC(NewPackets, br, DTSPTS, Residuals, bFlushResiduals);
				}

				if (PESResult == EPESPacketResult::Ok)
				{
					ResidualPESDataMap.Remove(InPESPacket->PID);
				}
				else if (PESResult == EPESPacketResult::Truncated && br.GetRemainingByteLength())
				{
					if (!Residuals)
					{
						Residuals = &ResidualPESDataMap.FindOrAdd(InPESPacket->PID);
						Residuals->PID = InPESPacket->PID;
						Residuals->PreviousDTSPTS = DTSPTS;
						Residuals->RemainingData.Append(reinterpret_cast<const uint8*>(br.GetRemainingData()), (int32) br.GetRemainingByteLength());
					}
					else
					{
						// If there were residuals before then the new data has been appended to the earlier data
						// so we have a single linear buffer to parse.
						// Anything that was parsed we remove.
						uint32 AtNow = br.GetBytePosition();
						if (AtNow)
						{
							Residuals->RemainingData.RemoveAt(0, AtNow);
						}
						// If we managed to use everything, drop the residuals.
						if (Residuals->RemainingData.IsEmpty())
						{
							ResidualPESDataMap.Remove(InPESPacket->PID);
						}
					}
					PESResult = EPESPacketResult::Ok;
				}
				if (bFlushResiduals)
				{
					ResidualPESDataMap.Remove(InPESPacket->PID);
				}
				break;
			}
			// Dolby
			case 0x87:
			{
				break;
			}
			default:
			{
				break;
			}
		}
		// Return new packets if successful. The incomplete packet is not included at this time.
		if (PESResult == EPESPacketResult::Ok)
		{
			// Set the stream type with each packet for convenience.
			for(auto& npit : NewPackets)
			{
				npit.StreamType = InPESPacket->StreamType;
			}
			OutPackets.Append(MoveTemp(NewPackets));
		}
		return PESResult;
	}
	else if (stream_id == 0xbe /* padding_stream */)
	{
		return EPESPacketResult::NotSupported;
	}
	else
	{
		return EPESPacketResult::NotSupported;
	}
}

FParserISO13818_1::EPESPacketResult FParserISO13818_1::ParseADTSAAC(TArray<FESPacket>& OutPackets, FBitstreamReader& InOutBR, const FDTSPTS& InDTSPTS, FResidualPESData* InResidualData, bool bFlushResiduals)
{
	FBitstreamReader br(InOutBR);
	FDTSPTS DTSPTS(InDTSPTS);

	// Are we dealing with residuals?
	if (InResidualData && InResidualData->RemainingData.Num())
	{
		InResidualData->RemainingData.Append(reinterpret_cast<const uint8*>(InOutBR.GetRemainingData()), (int32)InOutBR.GetRemainingByteLength());
		br.SetData(InResidualData->RemainingData.GetData(), InResidualData->RemainingData.Num());
		DTSPTS = InResidualData->PreviousDTSPTS;
	}

	for(int32 npkt=0; ; ++npkt)
	{
		if (br.GetRemainingByteLength() < 7)
		{
			if (!InResidualData)
			{
				UE_LOGF(LogElectraMPEGTSParser, Warning, "Remaining PES packet data too small to contain an ADTS header. Incorrect multiplex?");
			}
			return EPESPacketResult::Truncated;
		}
		if (br.GetBits(12) != 0xfff)
		{
			UE_LOGF(LogElectraMPEGTSParser, Warning, "Incorrect sync value in ADTS header. Incorrect multiplex?");
			return EPESPacketResult::Invalid;
		}
		const uint32 mpeg_version = br.GetBits(1);
		const uint32 layer = br.GetBits(2);
		const uint32 prot_absent = br.GetBits(1);
		const uint32 profile = br.GetBits(2);
		const uint32 sampling_frequency_index = br.GetBits(4);
		const uint32 private_bit = br.GetBits(1);
		const uint32 channel_configuration = br.GetBits(3);
		const uint32 originalty = br.GetBits(1);
		const uint32 home = br.GetBits(1);
		const uint32 copyright_id = br.GetBits(1);
		const uint32 copyright_id_start = br.GetBits(1);
		const uint32 frame_length = br.GetBits(13);
		const uint32 buffer_fullness = br.GetBits(11);
		const uint32 num_frames = br.GetBits(2);
		const uint32 crc = prot_absent ? 0 : br.GetBits(16);
		const int32 FrameSize = frame_length - (prot_absent ? 7 : 9);
		if (FrameSize < 0 || br.GetRemainingByteLength() < FrameSize)
		{
			return EPESPacketResult::Truncated;
		}
		if (num_frames > 0)
		{
			UE_LOGF(LogElectraMPEGTSParser, Warning, "Multiple RDBs in ADTS frame is not supported!");
			return EPESPacketResult::Invalid;
		}
		if (channel_configuration == 0)
		{
			UE_LOGF(LogElectraMPEGTSParser, Warning, "Channel configuration 0 is not supported!");
			return EPESPacketResult::Invalid;
		}
		FESPacket& pkt = OutPackets.Emplace_GetRef();
		pkt.bIsSyncFrame = true;
		pkt.SubPacketNum = npkt;
		pkt.DTS = InDTSPTS.DTS;
		pkt.PTS = InDTSPTS.PTS;
		// Create the CSD
		pkt.CSD = MakeSharedTS<TArray<uint8>>();
#if 1
		pkt.CSD->SetNumUninitialized(2);
		uint32 csd = (profile + 1) << 11;
		csd |= sampling_frequency_index << 7;
		csd |= channel_configuration << 3;
		pkt.CSD->operator[](0) = (uint8)(csd >> 8);
		pkt.CSD->operator[](1) = (uint8)(csd & 255);
#else
		ElectraDecodersUtil::FElectraBitstreamWriter bw;
		bw.PutBits(profile+1, 5);
		bw.PutBits(sampling_frequency_index, 4);
		bw.PutBits(channel_configuration, 4);
		bw.PutBits(0, 3);
		check(bw.IsByteAligned());
		bw.AlignToBytes(0);
		bw.GetArray(*pkt.CSD);
#endif
		pkt.Data = MakeSharedTS<TArray<uint8>>();
		pkt.Data->Append(reinterpret_cast<const uint8*>(br.GetRemainingData()), FrameSize);
		br.SkipBytes(FrameSize);
		InOutBR = br;
		DTSPTS = InDTSPTS;
		// Update the DTS and PTS in the residual data
		if (InResidualData)
		{
			InResidualData->PreviousDTSPTS = InDTSPTS;
		}

		if (br.GetRemainingByteLength() == 0)
		{
			break;
		}
	}
	return EPESPacketResult::Ok;
}

FParserISO13818_1::EPESPacketResult FParserISO13818_1::ParseMPEGAudio(TArray<FESPacket>& OutPackets, FBitstreamReader& InOutBR, const FDTSPTS& InDTSPTS, FResidualPESData* InResidualData, bool bFlushResiduals)
{
	FBitstreamReader br(InOutBR);
	FDTSPTS DTSPTS(InDTSPTS);

	// Are we dealing with residuals?
	if (InResidualData && InResidualData->RemainingData.Num())
	{
		InResidualData->RemainingData.Append(reinterpret_cast<const uint8*>(InOutBR.GetRemainingData()), (int32)InOutBR.GetRemainingByteLength());
		br.SetData(InResidualData->RemainingData.GetData(), InResidualData->RemainingData.Num());
		DTSPTS = InResidualData->PreviousDTSPTS;
	}

	uint32 MPEGHeaderMask = 0xfffe0c00;
	uint32 MPEGHeaderExpectedValue = 0;

	const uint8* InData = reinterpret_cast<const uint8*>(br.GetRemainingData());
	uint64 InDataLength = br.GetRemainingByteLength();
	int32 NumBytesRemaining = (int32) InDataLength;
	auto GetUINT32BE = [](const uint8* InData) -> uint32
	{
		return (static_cast<uint32>(InData[0]) << 24) | (static_cast<uint32>(InData[1]) << 16) | (static_cast<uint32>(InData[2]) << 8) | static_cast<uint32>(InData[3]);
	};
	for(int32 npkt=0; ; ++npkt)
	{
		if (NumBytesRemaining < 4)
		{
			if (!InResidualData)
			{
				UE_LOGF(LogElectraMPEGTSParser, Warning, "Remaining PES packet data too small to contain an MPEG audio frame header. Incorrect multiplex?");
			}
			return EPESPacketResult::Truncated;
		}
		// Check that the first byte is 0xff (the first 8 bits of the 11 bit sync marker)
		if (NumBytesRemaining >= 3 &&
			InData[0] == 0xff && (InData[1] & 0xe0) == 0xe0 &&		// sync marker (11 1-bits)
			((InData[1] >> 3) & 3) >= 2	&&							// audio version 1 or 2 (2.5 not supported)
			((InData[1] >> 1) & 3) != 0 &&							// layer index 1, 2 or 3
			(InData[2] >> 4) != 15 &&								// bitrate index not 15
			(InData[2] & 0x0c) != 0x0c)								// sample rate index not 3
		{
			const uint32 HeaderValue = GetUINT32BE(InData);
			if (MPEGHeaderExpectedValue == 0)
			{
				MPEGHeaderExpectedValue = HeaderValue & MPEGHeaderMask;
			}
			if ((HeaderValue & MPEGHeaderMask) != MPEGHeaderExpectedValue)
			{
				// Mismatching header between consecutive packets?
				UE_LOGF(LogElectraMPEGTSParser, Warning, "Mismatching frame header between consecutive audio frames. Incorrect multiplex?");
				return EPESPacketResult::Invalid;
			}
			const uint32 FrameSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetFrameSize(HeaderValue);
			if (NumBytesRemaining < (int32)FrameSize)
			{
				if (!InResidualData)
				{
					UE_LOGF(LogElectraMPEGTSParser, Warning, "Remaining PES packet data too small for a complete MPEG audio frame. Incorrect multiplex?");
				}
				return EPESPacketResult::Truncated;
			}

			FESPacket& pkt = OutPackets.Emplace_GetRef();
			pkt.bIsSyncFrame = true;
			pkt.SubPacketNum = npkt;
			pkt.DTS = InDTSPTS.DTS;
			pkt.PTS = InDTSPTS.PTS;
			// Create the CSD, which is just the audio header.
			pkt.CSD = MakeSharedTS<TArray<uint8>>();
			pkt.CSD->SetNumUninitialized(4);
			*reinterpret_cast<uint32*>(pkt.CSD->GetData()) = MEDIA_TO_BIG_ENDIAN(MPEGHeaderExpectedValue);
			pkt.Data = MakeSharedTS<TArray<uint8>>();
			pkt.Data->Append(InData, FrameSize);
			br.SkipBytes(FrameSize);
			InOutBR = br;
			DTSPTS = InDTSPTS;
			// Update the DTS and PTS in the residual data
			if (InResidualData)
			{
				InResidualData->PreviousDTSPTS = InDTSPTS;
			}
			InData += FrameSize;
			NumBytesRemaining -= FrameSize;
			if (NumBytesRemaining <= 0)
			{
				break;
			}
		}
		else
		{
			// Not locked to the sync marker
			UE_LOGF(LogElectraMPEGTSParser, Warning, "Incorrect sync value in MPEG audio frame header. Incorrect multiplex?");
			return EPESPacketResult::Invalid;
		}
	}
	return EPESPacketResult::Ok;
}

FParserISO13818_1::EPESPacketResult FParserISO13818_1::ParseAVC(TArray<FESPacket>& OutPackets, FBitstreamReader& InOutBR, const FDTSPTS& InDTSPTS, FResidualPESData* InResidualData, bool bFlushResiduals)
{
	FBitstreamReader br(InOutBR);
	FDTSPTS DTSPTS(InDTSPTS);

	// Are we dealing with residuals?
	if (InResidualData && InResidualData->RemainingData.Num())
	{
		if (!bFlushResiduals)
		{
			InResidualData->RemainingData.Append(reinterpret_cast<const uint8*>(InOutBR.GetRemainingData()), (int32)InOutBR.GetRemainingByteLength());
		}
		br.SetData(InResidualData->RemainingData.GetData(), InResidualData->RemainingData.Num());
		DTSPTS = InResidualData->PreviousDTSPTS;
	}

	// Deal with potentially multiple frames
	for(int32 npkt=0; ; ++npkt)
	{
		TArray<ElectraDecodersUtil::MPEG::H264::FNaluInfo> NALUs;
		const uint8* InData = reinterpret_cast<const uint8*>(br.GetRemainingData());
		uint64 InDataLength = br.GetRemainingByteLength();
		if (InDataLength == 0)
		{
			break;
		}
		// We get an Annex-B stream here which we need to decompose, remove the AUD NALU and separate the SPS and PPS NALUs.
		if (ElectraDecodersUtil::MPEG::H264::ParseBitstreamForNALUs(NALUs, InData, InDataLength))
		{
			// Because of some streams splitting video across multiple PES packets of smaller sizes (instead of 0)
			// and therefore several start flag and PES headers, we need to reassemble the packets.
			// In order to do this we need to take data enclosed between two AUD NALUs and thus need to have
			// an additional frame (AUD denotes the start of a frame, not the end).
			int32 NumAUDNalus = 0;
			for(int32 i=0; i<NALUs.Num(); ++i)
			{
				NumAUDNalus += NALUs[i].Type == 9 ? 1 : 0;
			}
			if (NumAUDNalus >= (bFlushResiduals ? 1 : 2))
			{
				int32 FirstNALUIndex = -1;
				int32 LastNALUIndex = -1;
				const int32 kSizeOfSizeField = 4;
				int32 SizeCSD = 0;
				int32 SizeData = 0;
				bool bIsIDR = false;
				// In a first pass calculate the size of the final data.
				for(int32 i=0; i<NALUs.Num(); ++i)
				{
					// First NALU we need must be AUD. If not, skip it.
					if (FirstNALUIndex < 0 && NALUs[i].Type != 9)
					{
						continue;
					}
					// Take note of the indices of the first and second AUD.
					if (NALUs[i].Type == 9)
					{
						FirstNALUIndex = FirstNALUIndex < 0 ? i : FirstNALUIndex;
						LastNALUIndex = LastNALUIndex < 0 && i > FirstNALUIndex ? i : LastNALUIndex;
						if (LastNALUIndex > 0)
						{
							break;
						}
						continue;
					}

					if (NALUs[i].Type == 7 || NALUs[i].Type == 8)
					{
						// SPS or PPS
						SizeCSD += NALUs[i].Size + kSizeOfSizeField;
					}
					else if (NALUs[i].Type == 12)
					{
						// Filler data
					}
					else
					{
						// Other
						SizeData += NALUs[i].Size + kSizeOfSizeField;
						if (NALUs[i].Type == 5)
						{
							bIsIDR = true;
						}
					}
				}
				if (FirstNALUIndex != 0)
				{
					UE_LOGF(LogElectraMPEGTSParser, Warning, "First NALU in AVC packet is not an AUD");
				}

				FESPacket& pkt = OutPackets.Emplace_GetRef();
				pkt.bIsSyncFrame = bIsIDR;
				pkt.SubPacketNum = npkt;
				pkt.DTS = DTSPTS.DTS;
				pkt.PTS = DTSPTS.PTS;
				pkt.CSD = MakeSharedTS<TArray<uint8>>();
				pkt.CSD->SetNumUninitialized(SizeCSD);
				pkt.Data = MakeSharedTS<TArray<uint8>>();
				pkt.Data->SetNumUninitialized(SizeData);
				uint8* CSDPtr = pkt.CSD->GetData();
				uint8* DataPtr = pkt.Data->GetData();

				// Second pass, copy the data.
				for(int32 i=FirstNALUIndex, iMax=bFlushResiduals?NALUs.Num():LastNALUIndex; i<iMax; ++i)
				{
					if (NALUs[i].Type == 9 || NALUs[i].Type == 12)
					{
						// Skip AUD and filler data
						continue;
					}
					const bool bCSD = NALUs[i].Type == 7 || NALUs[i].Type == 8;
					uint8** DstPtr = bCSD ? &CSDPtr : &DataPtr;
					uint32 Size = (uint32)NALUs[i].Size;
					*((uint32*)(*DstPtr)) = bCSD ? MEDIA_TO_BIG_ENDIAN((uint32)1) : MEDIA_TO_BIG_ENDIAN(Size);
					(*DstPtr) += 4;
					uint32 Pos = NALUs[i].Offset + NALUs[i].UnitLength;
					FMemory::Memcpy((*DstPtr), InData + Pos, Size);
					(*DstPtr) += Size;
				}
				// Remove the data we processed from the residuals
				int32 ConsumedSize = bFlushResiduals ? (int32) br.GetRemainingByteLength() : (int32) NALUs[LastNALUIndex].Offset;
				br.SkipBytes(ConsumedSize);
				InOutBR = br;
				// Update the DTS and PTS in the residual data
				if (InResidualData)
				{
					InResidualData->PreviousDTSPTS = InDTSPTS;
				}
			}
			else
			{
				// Not enough data yet. Need an additional AUD NALU.
				InOutBR = br;
				return EPESPacketResult::Truncated;
			}
		}
		else
		{
			UE_LOGF(LogElectraMPEGTSParser, Warning, "Failed to parse the AVC packet for NALUs");
			return EPESPacketResult::Invalid;
		}
	}
	return EPESPacketResult::Ok;
}

FParserISO13818_1::EPESPacketResult FParserISO13818_1::ParseHEVC(TArray<FESPacket>& OutPackets, FBitstreamReader& InOutBR, const FDTSPTS& InDTSPTS, FResidualPESData* InResidualData, bool bFlushResiduals)
{
	FBitstreamReader br(InOutBR);
	FDTSPTS DTSPTS(InDTSPTS);

	// Are we dealing with residuals?
	if (InResidualData && InResidualData->RemainingData.Num())
	{
		if (!bFlushResiduals)
		{
			InResidualData->RemainingData.Append(reinterpret_cast<const uint8*>(InOutBR.GetRemainingData()), (int32)InOutBR.GetRemainingByteLength());
		}
		br.SetData(InResidualData->RemainingData.GetData(), InResidualData->RemainingData.Num());
		DTSPTS = InResidualData->PreviousDTSPTS;
	}

	// Deal with potentially multiple frames
	for(int32 npkt=0; ; ++npkt)
	{
		TArray<ElectraDecodersUtil::MPEG::H265::FNaluInfo> NALUs;
		const uint8* InData = reinterpret_cast<const uint8*>(br.GetRemainingData());
		uint64 InDataLength = br.GetRemainingByteLength();
		if (InDataLength == 0)
		{
			break;
		}
		// We get an Annex-B stream here which we need to decompose, remove the AUD NUT and separate the VPS, SPS and PPS NUTs.
		if (ElectraDecodersUtil::MPEG::H265::ParseBitstreamForNALUs(NALUs, InData, InDataLength))
		{
			// Because of some streams splitting video across multiple PES packets of smaller sizes (instead of 0)
			// and therefore several start flag and PES headers, we need to reassemble the packets.
			// In order to do this we need to take data enclosed between two AUD NUTs and thus need to have
			// an additional frame (AUD denotes the start of a frame, not the end).
			int32 NumAUDNalus = 0;
			for(int32 i=0; i<NALUs.Num(); ++i)
			{
				NumAUDNalus += NALUs[i].Type == 35 ? 1 : 0;
			}
			if (NumAUDNalus >= (bFlushResiduals ? 1 : 2))
			{
				int32 FirstNALUIndex = -1;
				int32 LastNALUIndex = -1;
				const int32 kSizeOfSizeField = 4;
				int32 SizeCSD = 0;
				int32 SizeData = 0;
				bool bIsSync = false;
				// In a first pass calculate the size of the final data.
				for(int32 i=0; i<NALUs.Num(); ++i)
				{
					// First NUT we need must be AUD. If not, skip it.
					if (FirstNALUIndex < 0 && NALUs[i].Type != 35)
					{
						continue;
					}
					// Take note of the indices of the first and second AUD.
					if (NALUs[i].Type == 35)
					{
						FirstNALUIndex = FirstNALUIndex < 0 ? i : FirstNALUIndex;
						LastNALUIndex = LastNALUIndex < 0 && i > FirstNALUIndex ? i : LastNALUIndex;
						if (LastNALUIndex > 0)
						{
							break;
						}
						continue;
					}

					if (NALUs[i].Type == 32 || NALUs[i].Type == 33 || NALUs[i].Type == 34)
					{
						// VPS, SPS or PPS
						SizeCSD += NALUs[i].Size + kSizeOfSizeField;
					}
					else if (NALUs[i].Type == 38)
					{
						// Filler data
					}
					else
					{
						// Other
						SizeData += NALUs[i].Size + kSizeOfSizeField;
						// IDR, CRA or BLA frame?
						if (NALUs[i].Type >= 16 && NALUs[i].Type <= 21)
						{
							bIsSync = true;
						}
					}
				}
				if (FirstNALUIndex != 0)
				{
					UE_LOGF(LogElectraMPEGTSParser, Warning, "First NUT in HEVC packet is not an AUD");
				}

				FESPacket& pkt = OutPackets.Emplace_GetRef();
				pkt.bIsSyncFrame = bIsSync;
				pkt.SubPacketNum = npkt;
				pkt.DTS = DTSPTS.DTS;
				pkt.PTS = DTSPTS.PTS;
				pkt.CSD = MakeSharedTS<TArray<uint8>>();
				pkt.CSD->SetNumUninitialized(SizeCSD);
				pkt.Data = MakeSharedTS<TArray<uint8>>();
				pkt.Data->SetNumUninitialized(SizeData);
				uint8* CSDPtr = pkt.CSD->GetData();
				uint8* DataPtr = pkt.Data->GetData();

				// Second pass, copy the data.
				for(int32 i=FirstNALUIndex, iMax=bFlushResiduals?NALUs.Num():LastNALUIndex; i<iMax; ++i)
				{
					if (NALUs[i].Type == 35 || NALUs[i].Type == 38)
					{
						// Skip AUD and filler data
						continue;
					}
					const bool bCSD = NALUs[i].Type == 32 || NALUs[i].Type == 33 || NALUs[i].Type == 34;
					uint8** DstPtr = bCSD ? &CSDPtr : &DataPtr;
					uint32 Size = (uint32)NALUs[i].Size;
					*((uint32*)(*DstPtr)) = bCSD ? MEDIA_TO_BIG_ENDIAN((uint32)1) : MEDIA_TO_BIG_ENDIAN(Size);
					(*DstPtr) += 4;
					uint32 Pos = NALUs[i].Offset + NALUs[i].UnitLength;
					FMemory::Memcpy((*DstPtr), InData + Pos, Size);
					(*DstPtr) += Size;
				}
				// Remove the data we processed from the residuals
				int32 ConsumedSize = bFlushResiduals ? (int32) br.GetRemainingByteLength() : (int32) NALUs[LastNALUIndex].Offset;
				br.SkipBytes(ConsumedSize);
				InOutBR = br;
				// Update the DTS and PTS in the residual data
				if (InResidualData)
				{
					InResidualData->PreviousDTSPTS = InDTSPTS;
				}
			}
			else
			{
				// Not enough data yet. Need an additional AUD NALU.
				InOutBR = br;
				return EPESPacketResult::Truncated;
			}
		}
		else
		{
			UE_LOGF(LogElectraMPEGTSParser, Warning, "Failed to parse the HEVC packet for NUTs");
			return EPESPacketResult::Invalid;
		}
	}
	return EPESPacketResult::Ok;
}



bool FParserISO13818_1::ParseCSD(FStreamCodecInformation& OutParsedCSD, const FESPacket& InFromPESPacket)
{
	auto GetUINT32BE = [](const uint8* InData) -> uint32
	{
		return (static_cast<uint32>(InData[0]) << 24) | (static_cast<uint32>(InData[1]) << 16) | (static_cast<uint32>(InData[2]) << 8) | static_cast<uint32>(InData[3]);
	};

	switch(InFromPESPacket.StreamType)
	{
		// MPEG audio (layer 1, 2 or 3)
		case 0x03:
		{
			if (InFromPESPacket.CSD.IsValid() && InFromPESPacket.CSD->Num() == 4)
			{
				const uint32 HeaderValue = GetUINT32BE(InFromPESPacket.CSD->GetData());
				OutParsedCSD.SetStreamType(EStreamType::Audio);
				OutParsedCSD.SetMimeType(TEXT("audio/mpeg"));
				OutParsedCSD.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
				OutParsedCSD.SetCodec4CC(Utils::Make4CC('m','p','g','a'));
				OutParsedCSD.SetProfile(ElectraDecodersUtil::MPEG::UtilsMPEG123::GetVersion(HeaderValue));
				OutParsedCSD.SetProfileLevel(ElectraDecodersUtil::MPEG::UtilsMPEG123::GetLayer(HeaderValue));
				OutParsedCSD.SetCodecSpecifierRFC6381(TEXT("mp4a.6b"));	// alternatively "mp4a.40.34"
				OutParsedCSD.SetSamplingRate(ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplingRate(HeaderValue));
				OutParsedCSD.SetNumberOfChannels(ElectraDecodersUtil::MPEG::UtilsMPEG123::GetChannelCount(HeaderValue));
				OutParsedCSD.GetExtras().Set(StreamCodecInformationOptions::SamplesPerBlock, FVariantValue((int64)ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplesPerFrame(HeaderValue)));
				return true;
			}
			break;
		}
		// MPEG audio in ADTS format (AAC)
		case 0x0f:
		{
			if (InFromPESPacket.CSD.IsValid() && InFromPESPacket.CSD->Num())
			{
				ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord AudioSpecificConfig;
				if (AudioSpecificConfig.ParseFrom(InFromPESPacket.CSD->GetData(), InFromPESPacket.CSD->Num()))
				{
					OutParsedCSD.SetStreamType(EStreamType::Audio);
					OutParsedCSD.SetMimeType(TEXT("audio/mp4"));
					OutParsedCSD.SetCodec(FStreamCodecInformation::ECodec::AAC);
					OutParsedCSD.SetCodecSpecificData(AudioSpecificConfig.GetCodecSpecificData());
					OutParsedCSD.SetCodecSpecifierRFC6381(FString::Printf(TEXT("mp4a.40.%d"), AudioSpecificConfig.ExtAOT ? AudioSpecificConfig.ExtAOT : AudioSpecificConfig.AOT));
					OutParsedCSD.SetSamplingRate(AudioSpecificConfig.ExtSamplingFrequency ? AudioSpecificConfig.ExtSamplingFrequency : AudioSpecificConfig.SamplingRate);
					OutParsedCSD.SetChannelConfiguration(AudioSpecificConfig.ChannelConfiguration);
					OutParsedCSD.SetNumberOfChannels(ElectraDecodersUtil::MPEG::AACUtils::GetNumberOfChannelsFromChannelConfiguration(AudioSpecificConfig.ChannelConfiguration));
					// We assume that all platforms can decode PS (parametric stereo). As such we change the channel count from mono to stereo
					// to convey the _decoded_ format, not the source format.
					if (AudioSpecificConfig.ChannelConfiguration == 1 && AudioSpecificConfig.PSSignal > 0)
					{
						OutParsedCSD.SetNumberOfChannels(2);
					}
					const int32 NumDecodedSamplesPerBlock = AudioSpecificConfig.SBRSignal > 0 ? 2048 : 1024;
					OutParsedCSD.GetExtras().Set(StreamCodecInformationOptions::SamplesPerBlock, FVariantValue((int64)NumDecodedSamplesPerBlock));
					return true;
				}
			}
			break;
		}
		// AVC
		case 0x1b:
		{
			if (InFromPESPacket.CSD.IsValid() && InFromPESPacket.CSD->Num())
			{
				TArray<ElectraDecodersUtil::MPEG::H264::FNaluInfo> NALUs;
				if (ElectraDecodersUtil::MPEG::H264::ParseBitstreamForNALUs(NALUs, InFromPESPacket.CSD->GetData(), InFromPESPacket.CSD->Num()))
				{
					// Is there an SPS NALU?
					for(int32 nn=0; nn<NALUs.Num(); ++nn)
					{
						TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet> spsmap;
						if (NALUs[nn].Type == 7 && ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(spsmap, InFromPESPacket.CSD->GetData() + NALUs[nn].Offset + NALUs[nn].UnitLength, NALUs[nn].Size))
						{
							OutParsedCSD.SetStreamType(EStreamType::Video);
							OutParsedCSD.SetMimeType(TEXT("video/mp4"));
							OutParsedCSD.SetCodec(FStreamCodecInformation::ECodec::H264);
							OutParsedCSD.SetCodecSpecificData(*InFromPESPacket.CSD);

							int32 CropL, CropR, CropT, CropB;
							int32 arW, arH;
							const ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet& sps = spsmap[spsmap.CreateConstIterator()->Key];
							sps.GetCrop(CropL, CropR, CropT, CropB);
							OutParsedCSD.SetResolution(FStreamCodecInformation::FResolution(sps.GetWidth() - CropL - CropR, sps.GetHeight() - CropT - CropB));
							OutParsedCSD.SetCrop(FStreamCodecInformation::FCrop(CropL, CropT, CropR, CropB));
							sps.GetAspect(arW, arH);
							OutParsedCSD.SetAspectRatio(FStreamCodecInformation::FAspectRatio(arW, arH));
							OutParsedCSD.SetFrameRate(sps.GetTiming().Denom ? FTimeFraction(sps.GetTiming().Num, sps.GetTiming().Denom) : FTimeFraction());
							OutParsedCSD.SetProfile(sps.profile_idc);
							OutParsedCSD.SetProfileLevel(sps.level_idc);
							uint8 Constraints = (sps.constraint_set0_flag << 7) | (sps.constraint_set1_flag << 6) | (sps.constraint_set2_flag << 5) | (sps.constraint_set3_flag << 4) | (sps.constraint_set4_flag << 3) | (sps.constraint_set5_flag << 2);
							OutParsedCSD.SetProfileConstraints(Constraints);
							OutParsedCSD.SetCodecSpecifierRFC6381(FString::Printf(TEXT("avc1.%02x%02x%02x"), sps.profile_idc, Constraints, sps.level_idc));
							return true;
						}
					}
				}
			}
			break;
		}
		// HEVC
		case 0x24:
		{
			if (InFromPESPacket.CSD.IsValid() && InFromPESPacket.CSD->Num())
			{
				TArray<ElectraDecodersUtil::MPEG::H265::FNaluInfo> NALUs;
				if (ElectraDecodersUtil::MPEG::H265::ParseBitstreamForNALUs(NALUs, InFromPESPacket.CSD->GetData(), InFromPESPacket.CSD->Num()))
				{
					// Is there an SPS NALU?
					for(int32 nn=0; nn<NALUs.Num(); ++nn)
					{
						TMap<uint32, ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet> spsmap;
						if (NALUs[nn].Type == 33 && ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(spsmap, InFromPESPacket.CSD->GetData() + NALUs[nn].Offset + NALUs[nn].UnitLength, NALUs[nn].Size))
						{
							OutParsedCSD.SetStreamType(EStreamType::Video);
							OutParsedCSD.SetMimeType(TEXT("video/mp4"));
							OutParsedCSD.SetCodec(FStreamCodecInformation::ECodec::H265);
							OutParsedCSD.SetCodecSpecificData(*InFromPESPacket.CSD);

							int32 CropL, CropR, CropT, CropB;
							int32 arW, arH;
							const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet& sps = spsmap[spsmap.CreateConstIterator()->Key];
							sps.GetCrop(CropL, CropR, CropT, CropB);
							OutParsedCSD.SetResolution(FStreamCodecInformation::FResolution(sps.GetWidth() - CropL - CropR, sps.GetHeight() - CropT - CropB));
							OutParsedCSD.SetCrop(FStreamCodecInformation::FCrop(CropL, CropT, CropR, CropB));
							sps.GetAspect(arW, arH);
							OutParsedCSD.SetAspectRatio(FStreamCodecInformation::FAspectRatio(arW, arH));
							OutParsedCSD.SetFrameRate(sps.GetTiming().Denom ? FTimeFraction(sps.GetTiming().Num, sps.GetTiming().Denom) : FTimeFraction());
							OutParsedCSD.SetProfileSpace(sps.profile_tier_level.general_profile_space);
							OutParsedCSD.SetProfileTier(sps.profile_tier_level.general_tier_flag);
							OutParsedCSD.SetProfile(sps.profile_tier_level.general_profile_idc);
							OutParsedCSD.SetProfileLevel(sps.profile_tier_level.general_level_idc);
							OutParsedCSD.SetProfileConstraints(sps.GetConstraintFlags());
							OutParsedCSD.SetProfileCompatibilityFlags(sps.profile_tier_level.general_profile_compatibility_flags);
							OutParsedCSD.SetCodecSpecifierRFC6381(sps.GetRFC6381(TEXT("hvc1")));

							return true;
						}
					}
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}
	return false;
}


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

TSharedPtrTS<IParserISO13818_1> IParserISO13818_1::CreateParser()
{
	return MakeSharedTS<FParserISO13818_1>();
}


} // namespace Electra
