// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "StreamTypes.h"
#include "ParameterDictionary.h"
#include "ErrorDetail.h"
#include "ElectraEncryptedSampleInfo.h"
#include "BufferedDataReader.h"
#include "StreamDataBuffer.h"

namespace Electra
{
	//
	// Forward declarations
	//
	class IPlayerSessionServices;


	/**
	 * Interface for parsing an MPEG transport stream (ISO/IEC 13818-1)
	 */
	class IParserISO13818_1
	{
	public:
		virtual ~IParserISO13818_1() = default;

		static TSharedPtrTS<IParserISO13818_1> CreateParser();

		enum class EParserFlags : uint8
		{
			ParseFlag_Default = 0,
			// Ignore PAT/PMT from the actual stream and rely on info from the init segment (if present)
			ParseFlag_IgnoreProgramStream = 1 << 0,
		};

		struct FSourceInfo
		{
			TSharedPtr<const TArray<uint8>, ESPMode::ThreadSafe> InitSegmentData;
			int64 InFirstFileByteOffset = -1;
			int64 InLastFileByteOffset = -1;
			uint64 TimestampOffset = 0;
		};

		enum class EParseState
		{
			// Continue parsing. Need more input data.
			Continue,
			// A new program was activated.
			NewProgram,
			// A new PES packet has been assembled. Call to get it.
			HavePESPacket,
			// Failed
			Failed,
			// Read error
			ReadError,
			// Reached the end of the stream.
			EOS
		};

		struct FProgramStreamInfo
		{
			// The program stream codec information is filled only _very_ rudimentary. There are no details on
			// the codec (like profile, level, resolution, sample rate, etc.) given in the program table.
			FStreamCodecInformation CodecInfo;
			// The ISO 13818-1 / ITU Rec H.220.0 `stream_type` value from Table 2-34
			uint8 StreamType = 0;
		};

		struct FProgramStream
		{
			// This is a map of the program with the elementary stream PID as key and the stream type as value.
			TSortedMap<int32, FProgramStreamInfo> StreamTable;
		};

		struct FProgramTable
		{
			// This is a map with the program number as key and the program stream information as value.
			TMap<int32, FProgramStream> ProgramTable;
		};

		virtual EParseState BeginParsing(IPlayerSessionServices* InPlayerSession, IGenericDataReader* InDataReader, EParserFlags InParseFlags, const FSourceInfo& InSourceInfo) = 0;

		/**
		 * Call this method to demultiplex the transport stream and get program information and PES packets.
		 * Internally this method keeps reading transport stream packets until it has assembled the program
		 * map and program mapping table, after which it returns `NewProgram`.
		 * You then need to call GetCurrentProgramTable() followed by SelectProgramStreams() to select the
		 * elementary streams you wish to demultiplex, then continue calling Parse().
		 * When the state returns `HavePESPacket` a new packet can be retrieved by calling GetPESPacket().
		 * Repeat this process until `Failed` or `EOS` is returned.
		 */
		virtual EParseState Parse(IPlayerSessionServices* InPlayerSession, IGenericDataReader* InDataReader) = 0;

		/**
		 * Returns the current program table as defined by the transport stream.
		 * Do this when the parse state returns `NewProgram`, then enable the elementary streams you want to
		 * demultiplex by calling SelectProgramStreams().
		 */
		virtual TSharedPtrTS<const FProgramTable> GetCurrentProgramTable() = 0;

		/**
		 * Selects the program's individual elementary streams to demultiplex.
		 * Do this *ONLY* when the parse state returns `NewProgram` and you get the program table by calling
		 * GetCurrentProgramTable().
		 * Then select the streams you wish to demultiplex and receive the PES packets for.
		 * All streams whose PID is NOT in the list will be discarded.
		 * While you could call this method at any time on a running stream it is not recommended.
		 */
		virtual void SelectProgramStreams(int32 InProgramNumber, const TArray<int32>& InProgramStreamPIDsToEnable) = 0;

		struct FPESData
		{
			int32 PID;
			TSharedPtrTS<TArray<uint8>> PacketData;
			TOptional<uint64> PCR;
			// The `random_access_indicator` flag from the adaptation_field. This may or may not be set.
			bool bRandomAccessIndicator = false;
			// The ISO 13818-1 / ITU Rec H.220.0 `stream_type` value from Table 2-34
			uint8 StreamType = 0;
		};
		/**
		 * Call this to obtain the most recently demultiplexed PES packet when the
		 * parse state returns `HavePESPacket`.
		 * If you do not get the packet and continue to call Parse() the packet will
		 * be dropped.
		 */
		virtual TSharedPtrTS<FPESData> GetPESPacket() = 0;

		/**
		 * Returns the last error.
		 */
		virtual FErrorDetail GetLastError() const = 0;


		enum class EPESPacketResult
		{
			Ok,
			Invalid,
			Truncated,
			NotSupported
		};

		struct FESPacket
		{
			TOptional<uint64> DTS;
			TOptional<uint64> PTS;
			TSharedPtrTS<TArray<uint8>> CSD;
			TSharedPtrTS<TArray<uint8>> Data;
			int32 SubPacketNum = 0;
			bool bIsSyncFrame = false;
			uint8 StreamType = 0;
		};
		virtual EPESPacketResult ParsePESPacket(TArray<FESPacket>& OutPackets, TSharedPtrTS<FPESData> InPESPacket) = 0;

		virtual bool ParseCSD(FStreamCodecInformation& OutParsedCSD, const FESPacket& InFromPESPacket) = 0;
	};
	ENUM_CLASS_FLAGS(IParserISO13818_1::EParserFlags);

} // namespace Electra
