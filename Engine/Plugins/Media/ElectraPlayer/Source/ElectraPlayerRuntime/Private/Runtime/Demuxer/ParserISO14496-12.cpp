// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Build.h"
#include "PlayerCore.h"
#include "Containers/ChunkedArray.h"
#include "Misc/Optional.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Math/BigInt.h"

#include "Demuxer/ParserISO14496-12.h"

#include "Utilities/Utilities.h"
#include "Utilities/UtilsMP4.h"
#include "Utilities/UtilsMPEG.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utilities/ISO639-Map.h"
#include "InfoLog.h"
#include "Player/AdaptivePlayerMetadataKeynames.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/PlayerSessionServices.h"
#include "Player/PlayerStreamFilter.h"
#include "Player/PlaylistReader.h"
#include "Decoder/SubtitleDecoder.h"

#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"


#if !UE_BUILD_SHIPPING
#define MEDIA_DEBUG_HAS_BOX_NAMES 1
#endif

#ifndef MEDIA_DEBUG_HAS_BOX_NAMES
#define MEDIA_DEBUG_HAS_BOX_NAMES 0
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogElectraMP4Parser, Log, All);
DEFINE_LOG_CATEGORY(LogElectraMP4Parser);


namespace Electra
{
	namespace Utils
	{
		template <typename T>
		T ValueFromBigEndian(const T value)
		{
			return MEDIA_FROM_BIG_ENDIAN(value);
		}

		template <typename T>
		T ValueToBigEndian(const T value)
		{
			return MEDIA_TO_BIG_ENDIAN(value);
		}

		FString Printable4CC(const uint32 In4CC)
		{
			FString Out;
			// Not so much just printable as alphanumeric.
			for(uint32 i=0, Atom=In4CC; i<4; ++i, Atom<<=8)
			{
				int32 v = Atom >> 24;
				if ((v >= 'A' && v <= 'Z') || (v >= 'a' && v <= 'z') || (v >= '0' && v <= '9') || v == '_' || v == '.')
				{
					Out.AppendChar(v);
				}
				else
				{
					// Not alphanumeric, return it as a hex string.
					return FString::Printf(TEXT("%08x"), In4CC);
				}
			}
			return Out;
		}
	}

	class FMP4Box;

	static const uint16 kAllOnes16 = uint16(0xffff);
	static const uint32 kAllOnes32 = uint32(0xffffffff);
	static const uint64 kAllOnes64 = uint64(0xffffffffffffffffL);

	class FMP4BoxReader : private TMediaNoncopyable<FMP4BoxReader>
	{
	public:
		FMP4BoxReader(IGenericDataReader* InDataReader, IParserISO14496_12::IBoxCallback* InBoxParseCallback)
			: DataReader(InDataReader)
			, BoxParseCallback(InBoxParseCallback)
			, DataDuplicationBuffer(nullptr)
		{
			check(DataReader);
		}

		~FMP4BoxReader()
		{
		}

		bool IsAtEOF() const
		{
			return DataReader->HasReachedEOF();
		}

		int64 GetCurrentReadOffset() const
		{
			return DataReader->GetCurrentOffset();
		}

		template <typename T>
		UEMediaError Read(T& value)
		{
			T Temp = 0;
			int64 NumToRead = sizeof(T);
			int64 NumRead = DataReader->ReadData(&Temp, NumToRead, -1);
			if (NumRead == NumToRead)
			{
				if (DataDuplicationBuffer)
				{
					DataDuplicationBuffer->Append((const uint8*)&Temp, NumToRead);
				}
				value = Utils::ValueFromBigEndian(Temp);
				return UEMEDIA_ERROR_OK;
			}
			else if (DataReader->HasReadBeenAborted())
			{
				return UEMEDIA_ERROR_ABORTED;
			}
			else if (NumRead < NumToRead || DataReader->HasReachedEOF())
			{
				return UEMEDIA_ERROR_INSUFFICIENT_DATA;
			}
			return UEMEDIA_ERROR_READ_ERROR;
		}

		UEMediaError ReadString(FString& OutString, uint32 MaxBytes, bool bIsUTF8=true)
		{
			TArray<uint8> Buf;
			Buf.Reserve(MaxBytes);

			auto ConvFromBuf = [](const TArray<uint8>& SrcBuf, bool bIsUTF8) -> FString
			{
				if (bIsUTF8)
				{
					FUTF8ToTCHAR cnv((const ANSICHAR*)SrcBuf.GetData(), SrcBuf.Num());
					return FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
				}
				else
				{
					return FString::ConstructFromPtrSize((const char*)SrcBuf.GetData(), SrcBuf.Num());
				}
			};

			OutString.Empty();
			if (MaxBytes == 0)
			{
				return UEMEDIA_ERROR_OK;
			}
			uint8 FirstByte = 0;
			for(uint32 i = 0; i < MaxBytes; ++i)
			{
				uint8 NextChar;
				UEMediaError Error = Read(NextChar);
				if (Error != UEMEDIA_ERROR_OK)
				{
					return Error;
				}
				if (NextChar == 0)
				{
					OutString = ConvFromBuf(Buf, bIsUTF8);
					return UEMEDIA_ERROR_OK;
				}
				if (i == 0)
				{
					FirstByte = NextChar;
				}
				Buf.Add(NextChar);
			}
			// NOTE: This is mostly for the 'qt' brand. ISO files should not be using such strings.
			// FIXME: Do this only for brands that may require this?
			// Do a special test here to check if the string is perhaps a Pascal string (first byte is length, not NUL terminated)
			if (FirstByte + 1 == MaxBytes)
			{
				Buf.RemoveAt(0);
				OutString = ConvFromBuf(Buf, bIsUTF8);
				return UEMEDIA_ERROR_OK;
			}
			return UEMEDIA_ERROR_INSUFFICIENT_DATA;	// Did not find a NUL char to terminate the string in the maximum number of characters allowed to read.
		}

		UEMediaError ReadBytes(void* Buffer, int64 NumBytes)
		{
			if (NumBytes == 0)
			{
				return UEMEDIA_ERROR_OK;
			}

			int64 NumRead = 0;
			if (!Buffer && DataDuplicationBuffer)
			{
				int32 BufferSizeNow = DataDuplicationBuffer->Num();
				DataDuplicationBuffer->AddUninitialized((int32)NumBytes);
				NumRead = DataReader->ReadData(DataDuplicationBuffer->GetData() + BufferSizeNow, NumBytes, -1);
			}
			else
			{
				NumRead = DataReader->ReadData(Buffer, NumBytes, -1);
				if (NumRead == NumBytes && DataDuplicationBuffer)
				{
					DataDuplicationBuffer->Append((const uint8*)Buffer, (int32)NumBytes);
				}
			}
			if (NumRead == NumBytes)
			{
				return UEMEDIA_ERROR_OK;
			}
			else if (DataReader->HasReadBeenAborted())
			{
				return UEMEDIA_ERROR_ABORTED;
			}
			else if (DataReader->HasReachedEOF())
			{
				return UEMEDIA_ERROR_INSUFFICIENT_DATA;
			}
			return UEMEDIA_ERROR_READ_ERROR;
		}

		IParserISO14496_12::IBoxCallback::EParseContinuation NotifyStartOfBox(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset)
		{
			if (BoxParseCallback)
			{
				return BoxParseCallback->OnFoundBox(InBoxType, InBoxSize, InStartOffset, InDataOffset);
			}
			return IParserISO14496_12::IBoxCallback::EParseContinuation::Stop;
		}

		IParserISO14496_12::IBoxCallback::EParseContinuation NotifyEndOfBox(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset)
		{
			if (BoxParseCallback)
			{
				return BoxParseCallback->OnEndOfBox(InBoxType, InBoxSize, InStartOffset, InDataOffset);
			}
			return IParserISO14496_12::IBoxCallback::EParseContinuation::Stop;
		}


		void SetDataDuplicationBuffer(TArray<uint8>* InDataDuplicationBuffer)
		{
			DataDuplicationBuffer = InDataDuplicationBuffer;
		}

		class ScopedDataDuplication : private TMediaNoncopyable<ScopedDataDuplication>
		{
		public:
			explicit ScopedDataDuplication(FMP4BoxReader* InBoxReader, TArray<uint8>& DuplicationBuffer)
				: BoxReader(InBoxReader)
			{
				BoxReader->SetDataDuplicationBuffer(&DuplicationBuffer);
			}
			~ScopedDataDuplication()
			{
				BoxReader->SetDataDuplicationBuffer(nullptr);
			}
		private:
			ScopedDataDuplication();
			FMP4BoxReader* BoxReader;
		};

	private:
		FMP4BoxReader() = delete;
		FMP4BoxReader(const FMP4BoxReader&) = delete;

		IGenericDataReader* DataReader = nullptr;
		IParserISO14496_12::IBoxCallback* BoxParseCallback = nullptr;

		TArray<uint8>* DataDuplicationBuffer = nullptr;
	};


	class FDataBufferReader : public IGenericDataReader
	{
	public:
		FDataBufferReader(const TArray<uint8>& InDataBufferToReadFrom, int64 InBaseOffset=0) : DataBufferRef(InDataBufferToReadFrom), BaseOffset(InBaseOffset)
		{}
		virtual ~FDataBufferReader() = default;
		int64 ReadData(void* IntoBuffer, int64 NumBytesToRead, int64 InFromOffset) override
		{
			int64 NumAvail = DataBufferRef.Num() - CurrentOffset;
			if (NumAvail >= NumBytesToRead)
			{
				if (IntoBuffer)
				{
					FMemory::Memcpy(IntoBuffer, DataBufferRef.GetData() + CurrentOffset, NumBytesToRead);
				}
				CurrentOffset += NumBytesToRead;
				return NumBytesToRead;
			}
			return -1;
		}
		bool HasReachedEOF() const override
		{ return CurrentOffset >= DataBufferRef.Num(); }
		bool HasReadBeenAborted() const override
		{ return false;	}
		int64 GetCurrentOffset() const override
		{ return CurrentOffset + BaseOffset; }
		int64 GetTotalSize() const override
		{ check(!"this should not be called"); return -1; }

	private:
		const TArray<uint8>& DataBufferRef;
		int64 CurrentOffset = 0;
		int64 BaseOffset = 0;
	};


#define RETURN_IF_ERROR(expr)											\
	if ((Error = expr) != UEMEDIA_ERROR_OK)								\
	{																	\
		return Error;													\
	}																	\

	/**
	 * Parse data helper.
	 */
	struct FMP4ParseInfo : private TMediaNoncopyable<FMP4ParseInfo>
	{
	public:
		FMP4ParseInfo()
		{ }

		virtual ~FMP4ParseInfo();

		UEMediaError Parse(FMP4BoxReader* Reader, IPlayerSessionServices* PlayerSession, const FMP4ParseInfo* InOptionalMoovParseInfo);

		/**
		 * Reads the type and size of the next box. If the box is an uuid box the 16 byte uuid is stored in the provided buffer.
		 *
		 * @param OutBoxType - Receives the type of the next box
		 * @param OutBoxSize - Receives the size of the next box
		 * @param InOutUUID - A 16 byte buffer to put the UUID into if the box is of type 'uuid'
		 */
		UEMediaError ReadBoxTypeAndSize(IParserISO14496_12::FBoxType& OutBoxType, int64& OutBoxSize, uint8* InOutUUID);

		FMP4BoxReader* Reader()
		{
			return BoxReader;
		}

		void PushReader(FMP4BoxReader* InNewReader)
		{
			BoxReaders.Push(BoxReader);
			BoxReader = InNewReader;
		}

		void PopReader()
		{
			BoxReader = BoxReaders.Pop();
		}

		FMP4Box* GetCurrentFtypBox() const
		{
			return CurrentFtypBox;
		}

		FMP4Box* GetCurrentStypBox() const
		{
			return CurrentStypBox;
		}

		FMP4Box* GetCurrentMoovBox() const
		{
			return CurrentMoovBox;
		}

		FMP4Box* GetCurrentMoofBox() const
		{
			return CurrentMoofBox;
		}

		FMP4Box* GetCurrentSidxBox() const
		{
			return CurrentSidxBox;
		}

		int32 GetNumberOfEMSGBoxes() const
		{
			return EMSGBoxes.Num();
		}
		FMP4Box* GetEMSGBox(int32 Index) const
		{
			return Index < EMSGBoxes.Num() ? EMSGBoxes[Index] : nullptr;
		}

		FMP4Box* GetCurrentTrackBox() const
		{
			return CurrentTrackBox;
		}

		FMP4Box* GetCurrentHandlerBox() const
		{
			return CurrentHandlerBox;
		}

		FMP4Box* GetCurrentMediaHandlerBox() const
		{
			return CurrentMediaHandlerBox;
		}

		UEMediaError ReadAndParseNextBox(FMP4Box* ParentBox, int64 BoxEndByteOffset=-1);

		const FMP4Box* GetRootBox() const
		{
			return RootBox;
		}

		void LogMessage(IInfoLog::ELevel InLevel, const FString& InMessage)
		{
			if (PlayerSession)
			{
				PlayerSession->PostLog(Facility::EFacility::MP4Parser, InLevel, InMessage);
			}
		}

		bool ShouldParseTimecodeInfo() const
		{
			return bShouldParseTimecodeInfo;
		}
	private:
		FMP4ParseInfo(const FMP4ParseInfo&) = delete;

		void SetCurrentFtypBox(FMP4Box* FtypBox)
		{
			CurrentFtypBox = FtypBox;
		}

		void SetCurrentStypBox(FMP4Box* StypBox)
		{
			CurrentStypBox = StypBox;
		}

		void SetCurrentMoovBox(FMP4Box* MoovBox)
		{
			CurrentMoovBox = MoovBox;
		}

		void SetCurrentMoofBox(FMP4Box* MoofBox)
		{
			CurrentMoofBox = MoofBox;
		}

		void SetCurrentSidxBox(FMP4Box* SidxBox)
		{
			CurrentSidxBox = SidxBox;
		}

		void AddEMSGBox(FMP4Box* EmsgBox)
		{
			EMSGBoxes.Emplace(EmsgBox);
		}

		void SetCurrentTrackBox(FMP4Box* TrackBox)
		{
			CurrentTrackBox = TrackBox;
		}

		void SetCurrentHandlerBox(FMP4Box* HandlerBox)
		{
			CurrentHandlerBox = HandlerBox;
		}

		void SetCurrentMediaHandlerBox(FMP4Box* MediaHandlerBox)
		{
			CurrentMediaHandlerBox = MediaHandlerBox;
		}

		void IncreaseNestingLevel()
		{
			++BoxNestingLevel;
		}
		void DecreaseNestingLevel()
		{
			--BoxNestingLevel;
		}
		int32 GetNestingLevel() const
		{
			return BoxNestingLevel;
		}

		const FMP4ParseInfo* OptionalMoovParseInfo = nullptr;

		FMP4Box* RootBox = nullptr;					//!< A root box that is not an actual file box but a container representing the file itself.
		TArray<FMP4BoxReader*> BoxReaders;
		FMP4BoxReader* BoxReader = nullptr;			//!< Instance of the box reader we were given. This is not ours so we must not delete it!
		int32 BoxNestingLevel = 0;					//!< Box tree depth

		FMP4Box* CurrentFtypBox = nullptr;
		FMP4Box* CurrentStypBox = nullptr;
		FMP4Box* CurrentMoovBox = nullptr;			//!< moov being parsed at the moment.
		FMP4Box* CurrentMoofBox = nullptr;			//!< moof being parsed at the moment.
		FMP4Box* CurrentSidxBox = nullptr;			//!< sidx being parsed at the moment.
		FMP4Box* CurrentTrackBox = nullptr;			//!< trak/traf being parsed at the moment.
		FMP4Box* CurrentHandlerBox = nullptr;		//!< hdlr being parsed at the moment.
		FMP4Box* CurrentMediaHandlerBox = nullptr;	//!< media handler being parsed at the moment ('vmhd', 'smhd', 'sthd', nmhd')

		TArray<FMP4Box*> EMSGBoxes;

		IPlayerSessionServices* PlayerSession = nullptr;
		int32 NumTotalBoxesParsed = 0;
		bool bShouldParseTimecodeInfo = false;
	};



	/**
	 * MP4 Box base class
	 */
	class FMP4Box : private TMediaNoncopyable<FMP4Box>
	{
	public:
		FMP4Box(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: ParentBox(nullptr)
			, BoxType(InBoxType)
			, BoxSize(InBoxSize)
			, StartOffset(InStartOffset)
			, DataOffset(InDataOffset)
			, bIsLeafBox(bInIsLeafBox)
			, FillerData(nullptr)
		{
#if MEDIA_DEBUG_HAS_BOX_NAMES
			* ((uint32*)DbgBoxName) = MEDIA_TO_BIG_ENDIAN(BoxType);
#endif
		}

		virtual ~FMP4Box()
		{
			for(int32 i = 0, iMax = ChildBoxes.Num(); i < iMax; ++i)
			{
				delete ChildBoxes[i];
			}
			ChildBoxes.Empty();
			delete FillerData;
		}

		IParserISO14496_12::FBoxType GetType() const
		{
			return BoxType;
		}

#if MEDIA_DEBUG_HAS_BOX_NAMES
		static FString GetBoxNameString(const char InBoxName[4])
		{
			return FString::Printf(TEXT("%c%c%c%c"), TCHAR(InBoxName[0]?InBoxName[0]:'?'), TCHAR(InBoxName[1]?InBoxName[1]:'?'), TCHAR(InBoxName[2]?InBoxName[2]:'?'), TCHAR(InBoxName[3]?InBoxName[3]:'?'));
		}

		FString GetBoxName() const
		{
			return GetBoxNameString(DbgBoxName);
		}
#endif

		bool IsLeafBox() const
		{
			return bIsLeafBox;
		}

		int64 GetDataSize() const
		{
			return BoxSize - (DataOffset - StartOffset);
		}

		int64 GetStartOffset() const
		{
			return StartOffset;
		}

		int64 GetDataOffset() const
		{
			return DataOffset;
		}

		int64 GetSize() const
		{
			return BoxSize;
		}

		const FMP4Box* GetParentBox() const
		{
			return ParentBox;
		}

		void AddChildBox(FMP4Box* Child)
		{
			Child->ParentBox = this;
			ChildBoxes.Push(Child);
		}

		int32 GetNumberOfChildren() const
		{
			return ChildBoxes.Num();
		}
		const FMP4Box* GetChildBox(int32 Index) const
		{
			return Index >= 0 && Index < ChildBoxes.Num() ? ChildBoxes[Index] : nullptr;
		}

		const FMP4Box* FindBox(IParserISO14496_12::FBoxType BoxTypeIn, int32 MaxDepth = 32) const
		{
			// First sweep, check children
			for(int32 i=0, iMax=ChildBoxes.Num(); i<iMax; ++i)
			{
				if (ChildBoxes[i]->GetType() == BoxTypeIn)
				{
					return ChildBoxes[i];
				}
			}
			// Second sweep, check children recursively
			if (MaxDepth > 0)
			{
				for(int32 i=0, iMax=ChildBoxes.Num(); i<iMax; ++i)
				{
					const FMP4Box* Box = ChildBoxes[i]->FindBox(BoxTypeIn, MaxDepth - 1);
					if (Box)
					{
						return Box;
					}
				}
			}
			return nullptr;
		}

		void GetAllBoxInstances(TArray<const FMP4Box*>& OutAllBoxes, IParserISO14496_12::FBoxType BoxTypeIn) const
		{
			// First sweep, check children
			for(int32 i = 0, iMax = ChildBoxes.Num(); i < iMax; ++i)
			{
				if (ChildBoxes[i]->GetType() == BoxTypeIn)
				{
					OutAllBoxes.Add(ChildBoxes[i]);
				}
			}
		}

		const FMP4Box* GetBoxPath(IParserISO14496_12::FBoxType P1) const
		{
			return FindBox(P1, 0);
		}
		const FMP4Box* GetBoxPath(IParserISO14496_12::FBoxType P1, IParserISO14496_12::FBoxType P2) const
		{
			const FMP4Box* Box = GetBoxPath(P1);
			return Box ? Box->GetBoxPath(P2) : nullptr;
		}
		const FMP4Box* GetBoxPath(IParserISO14496_12::FBoxType P1, IParserISO14496_12::FBoxType P2, IParserISO14496_12::FBoxType P3) const
		{
			const FMP4Box* Box = GetBoxPath(P1, P2);
			return Box ? Box->GetBoxPath(P3) : nullptr;
		}
		const FMP4Box* GetBoxPath(IParserISO14496_12::FBoxType P1, IParserISO14496_12::FBoxType P2, IParserISO14496_12::FBoxType P3, IParserISO14496_12::FBoxType P4) const
		{
			const FMP4Box* Box = GetBoxPath(P1, P2, P3);
			return Box ? Box->GetBoxPath(P4) : nullptr;
		}
		const FMP4Box* GetBoxPath(IParserISO14496_12::FBoxType P1, IParserISO14496_12::FBoxType P2, IParserISO14496_12::FBoxType P3, IParserISO14496_12::FBoxType P4, IParserISO14496_12::FBoxType P5) const
		{
			const FMP4Box* Box = GetBoxPath(P1, P2, P3, P4);
			return Box ? Box->GetBoxPath(P5) : nullptr;
		}



		struct FFillerData
		{
			~FFillerData()
			{
				FMemory::Free(Data);
			}
			void* Data = nullptr;
			int64	Size = 0;
			int64	StartOffset = 0;
		};

		UEMediaError ReadFillerData(FMP4ParseInfo* ParseInfo, int64 Size)
		{
			if (Size == 0)
			{
				return UEMEDIA_ERROR_OK;
			}
			else if (Size < 0 || Size > BoxSize || Size > 8 * 1024 * 1024)	// Additional check if data exceeds 8MiB. While not illegal, strictly speaking, we know we do not have any such large boxes.
			{
				ParseInfo->LogMessage(IInfoLog::ELevel::Error, FString::Printf(TEXT("Invalid filler data size of 0x%llx to read at offset 0x%llx in box 0x%08x (size 0x%llx, offset 0x%llx, dataoffset 0x%llx)"), (long long int)Size, (long long int)ParseInfo->Reader()->GetCurrentReadOffset(), BoxType, (long long int)BoxSize, (long long int)StartOffset, (long long int)DataOffset));
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}
			UEMediaError Error = UEMEDIA_ERROR_OK;
			if (FillerData)
			{
				delete FillerData;
				FillerData = nullptr;
				ParseInfo->LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Encountered addition filler data size of 0x%llx to read at offset 0x%llx in box 0x%08x (size 0x%llx, offset 0x%llx, dataoffset 0x%llx)"), (long long int)Size, (long long int)ParseInfo->Reader()->GetCurrentReadOffset(), BoxType, (long long int)BoxSize, (long long int)StartOffset, (long long int)DataOffset));
			}
			FillerData = new FFillerData;
			FillerData->Data = FMemory::Malloc(Size);
			FillerData->Size = Size;
			FillerData->StartOffset = ParseInfo->Reader()->GetCurrentReadOffset();
			return ParseInfo->Reader()->ReadBytes(FillerData->Data, FillerData->Size);
		}

		const FFillerData* GetFillerData() const
		{
			return FillerData;
		}

		virtual UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) = 0;

		// constexpr doesn't work here
	//	#define MAKE_BOX_ATOM(a,b,c,d) (IParserISO14496_12::FBoxType)((uint32)a << 24) | ((uint32)b << 16) | ((uint32)c << 8) | ((uint32)d)

		/**
		 * Box atoms
		 */
		static const IParserISO14496_12::FBoxType kBox_VOID = MAKE_BOX_ATOM('V', 'O', 'I', 'D');		// This is not an actual box. We use it for malformed 0-size boxes.
		static const IParserISO14496_12::FBoxType kBox_ftyp = MAKE_BOX_ATOM('f', 't', 'y', 'p');
		static const IParserISO14496_12::FBoxType kBox_styp = MAKE_BOX_ATOM('s', 't', 'y', 'p');
		static const IParserISO14496_12::FBoxType kBox_uuid = MAKE_BOX_ATOM('u', 'u', 'i', 'd');
		static const IParserISO14496_12::FBoxType kBox_free = MAKE_BOX_ATOM('f', 'r', 'e', 'e');
		static const IParserISO14496_12::FBoxType kBox_skip = MAKE_BOX_ATOM('s', 'k', 'i', 'p');
		static const IParserISO14496_12::FBoxType kBox_wide = MAKE_BOX_ATOM('w', 'i', 'd', 'e');
		static const IParserISO14496_12::FBoxType kBox_iods = MAKE_BOX_ATOM('i', 'o', 'd', 's');
		static const IParserISO14496_12::FBoxType kBox_sidx = MAKE_BOX_ATOM('s', 'i', 'd', 'x');
		static const IParserISO14496_12::FBoxType kBox_moov = MAKE_BOX_ATOM('m', 'o', 'o', 'v');
		static const IParserISO14496_12::FBoxType kBox_mvhd = MAKE_BOX_ATOM('m', 'v', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_trak = MAKE_BOX_ATOM('t', 'r', 'a', 'k');
		static const IParserISO14496_12::FBoxType kBox_tkhd = MAKE_BOX_ATOM('t', 'k', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_edts = MAKE_BOX_ATOM('e', 'd', 't', 's');
		static const IParserISO14496_12::FBoxType kBox_elst = MAKE_BOX_ATOM('e', 'l', 's', 't');
		static const IParserISO14496_12::FBoxType kBox_mdia = MAKE_BOX_ATOM('m', 'd', 'i', 'a');
		static const IParserISO14496_12::FBoxType kBox_mdhd = MAKE_BOX_ATOM('m', 'd', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_hdlr = MAKE_BOX_ATOM('h', 'd', 'l', 'r');
		static const IParserISO14496_12::FBoxType kBox_minf = MAKE_BOX_ATOM('m', 'i', 'n', 'f');
		static const IParserISO14496_12::FBoxType kBox_nmhd = MAKE_BOX_ATOM('n', 'm', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_vmhd = MAKE_BOX_ATOM('v', 'm', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_smhd = MAKE_BOX_ATOM('s', 'm', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_sthd = MAKE_BOX_ATOM('s', 't', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_gmhd = MAKE_BOX_ATOM('g', 'm', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_dinf = MAKE_BOX_ATOM('d', 'i', 'n', 'f');
		static const IParserISO14496_12::FBoxType kBox_dref = MAKE_BOX_ATOM('d', 'r', 'e', 'f');
		static const IParserISO14496_12::FBoxType kBox_url  = MAKE_BOX_ATOM('u', 'r', 'l', ' ');
		static const IParserISO14496_12::FBoxType kBox_urn  = MAKE_BOX_ATOM('u', 'r', 'n', ' ');
		static const IParserISO14496_12::FBoxType kBox_tref = MAKE_BOX_ATOM('t', 'r', 'e', 'f');
		static const IParserISO14496_12::FBoxType kBox_stbl = MAKE_BOX_ATOM('s', 't', 'b', 'l');
		static const IParserISO14496_12::FBoxType kBox_stsd = MAKE_BOX_ATOM('s', 't', 's', 'd');
		static const IParserISO14496_12::FBoxType kBox_btrt = MAKE_BOX_ATOM('b', 't', 'r', 't');
		static const IParserISO14496_12::FBoxType kBox_pasp = MAKE_BOX_ATOM('p', 'a', 's', 'p');
		static const IParserISO14496_12::FBoxType kBox_tapt = MAKE_BOX_ATOM('t', 'a', 'p', 't');
		static const IParserISO14496_12::FBoxType kBox_fiel = MAKE_BOX_ATOM('f', 'i', 'e', 'l');
		static const IParserISO14496_12::FBoxType kBox_avcC = MAKE_BOX_ATOM('a', 'v', 'c', 'C');
		static const IParserISO14496_12::FBoxType kBox_hvcC = MAKE_BOX_ATOM('h', 'v', 'c', 'C');
		static const IParserISO14496_12::FBoxType kBox_esds = MAKE_BOX_ATOM('e', 's', 'd', 's');
		static const IParserISO14496_12::FBoxType kBox_dac3 = MAKE_BOX_ATOM('d', 'a', 'c', '3');
		static const IParserISO14496_12::FBoxType kBox_dec3 = MAKE_BOX_ATOM('d', 'e', 'c', '3');
		static const IParserISO14496_12::FBoxType kBox_stts = MAKE_BOX_ATOM('s', 't', 't', 's');
		static const IParserISO14496_12::FBoxType kBox_ctts = MAKE_BOX_ATOM('c', 't', 't', 's');
		static const IParserISO14496_12::FBoxType kBox_stss = MAKE_BOX_ATOM('s', 't', 's', 's');
		static const IParserISO14496_12::FBoxType kBox_stsc = MAKE_BOX_ATOM('s', 't', 's', 'c');
		static const IParserISO14496_12::FBoxType kBox_stsz = MAKE_BOX_ATOM('s', 't', 's', 'z');
		static const IParserISO14496_12::FBoxType kBox_stco = MAKE_BOX_ATOM('s', 't', 'c', 'o');
		static const IParserISO14496_12::FBoxType kBox_co64 = MAKE_BOX_ATOM('c', 'o', '6', '4');
		static const IParserISO14496_12::FBoxType kBox_sdtp = MAKE_BOX_ATOM('s', 'd', 't', 'p');
		static const IParserISO14496_12::FBoxType kBox_udta = MAKE_BOX_ATOM('u', 'd', 't', 'a');
		static const IParserISO14496_12::FBoxType kBox_meta = MAKE_BOX_ATOM('m', 'e', 't', 'a');
		static const IParserISO14496_12::FBoxType kBox_mdat = MAKE_BOX_ATOM('m', 'd', 'a', 't');
		static const IParserISO14496_12::FBoxType kBox_mvex = MAKE_BOX_ATOM('m', 'v', 'e', 'x');
		static const IParserISO14496_12::FBoxType kBox_mehd = MAKE_BOX_ATOM('m', 'e', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_trex = MAKE_BOX_ATOM('t', 'r', 'e', 'x');
		static const IParserISO14496_12::FBoxType kBox_moof = MAKE_BOX_ATOM('m', 'o', 'o', 'f');
		static const IParserISO14496_12::FBoxType kBox_mfhd = MAKE_BOX_ATOM('m', 'f', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_traf = MAKE_BOX_ATOM('t', 'r', 'a', 'f');
		static const IParserISO14496_12::FBoxType kBox_tfhd = MAKE_BOX_ATOM('t', 'f', 'h', 'd');
		static const IParserISO14496_12::FBoxType kBox_tfdt = MAKE_BOX_ATOM('t', 'f', 'd', 't');
		static const IParserISO14496_12::FBoxType kBox_trun = MAKE_BOX_ATOM('t', 'r', 'u', 'n');
		static const IParserISO14496_12::FBoxType kBox_trep = MAKE_BOX_ATOM('t', 'r', 'e', 'p');
		static const IParserISO14496_12::FBoxType kBox_colr = MAKE_BOX_ATOM('c', 'o', 'l', 'r');
		static const IParserISO14496_12::FBoxType kBox_elng = MAKE_BOX_ATOM('e', 'l', 'n', 'g');
		static const IParserISO14496_12::FBoxType kBox_prft = MAKE_BOX_ATOM('p', 'r', 'f', 't');
		static const IParserISO14496_12::FBoxType kBox_emsg = MAKE_BOX_ATOM('e', 'm', 's', 'g');
		static const IParserISO14496_12::FBoxType kBox_sbgp = MAKE_BOX_ATOM('s', 'b', 'g', 'p');
		static const IParserISO14496_12::FBoxType kBox_sgpd = MAKE_BOX_ATOM('s', 'g', 'p', 'd');
		static const IParserISO14496_12::FBoxType kBox_saiz = MAKE_BOX_ATOM('s', 'a', 'i', 'z');
		static const IParserISO14496_12::FBoxType kBox_saio = MAKE_BOX_ATOM('s', 'a', 'i', 'o');
		static const IParserISO14496_12::FBoxType kBox_wave = MAKE_BOX_ATOM('w', 'a', 'v', 'e');
		static const IParserISO14496_12::FBoxType kBox_chan = MAKE_BOX_ATOM('c', 'h', 'a', 'n');
		static const IParserISO14496_12::FBoxType kBox_enda = MAKE_BOX_ATOM('e', 'n', 'd', 'a');
		static const IParserISO14496_12::FBoxType kBox_mdcv = MAKE_BOX_ATOM('m', 'd', 'c', 'v');
		static const IParserISO14496_12::FBoxType kBox_clli = MAKE_BOX_ATOM('c', 'l', 'l', 'i');
		static const IParserISO14496_12::FBoxType kBox_coll = MAKE_BOX_ATOM('c', 'o', 'l', 'l');

		/**
		 * Encryption related boxes
		 */
		static const IParserISO14496_12::FBoxType kBox_pssh = MAKE_BOX_ATOM('p', 's', 's', 'h');
		static const IParserISO14496_12::FBoxType kBox_senc = MAKE_BOX_ATOM('s', 'e', 'n', 'c');
		static const IParserISO14496_12::FBoxType kBox_tenc = MAKE_BOX_ATOM('t', 'e', 'n', 'c');
		static const IParserISO14496_12::FBoxType kBox_sinf = MAKE_BOX_ATOM('s', 'i', 'n', 'f');
		static const IParserISO14496_12::FBoxType kBox_frma = MAKE_BOX_ATOM('f', 'r', 'm', 'a');
		static const IParserISO14496_12::FBoxType kBox_schm = MAKE_BOX_ATOM('s', 'c', 'h', 'm');
		static const IParserISO14496_12::FBoxType kBox_schi = MAKE_BOX_ATOM('s', 'c', 'h', 'i');

		/**
		 * Track handler types
		 */
		static const IParserISO14496_12::FBoxType kTrack_vide = MAKE_BOX_ATOM('v', 'i', 'd', 'e');
		static const IParserISO14496_12::FBoxType kTrack_soun = MAKE_BOX_ATOM('s', 'o', 'u', 'n');
		static const IParserISO14496_12::FBoxType kTrack_subt = MAKE_BOX_ATOM('s', 'u', 'b', 't');
		static const IParserISO14496_12::FBoxType kTrack_sbtl = MAKE_BOX_ATOM('s', 'b', 't', 'l');
		static const IParserISO14496_12::FBoxType kTrack_subp = MAKE_BOX_ATOM('s', 'u', 'b', 'p');
		static const IParserISO14496_12::FBoxType kTrack_hint = MAKE_BOX_ATOM('h', 'i', 'n', 't');
		static const IParserISO14496_12::FBoxType kTrack_meta = MAKE_BOX_ATOM('m', 'e', 't', 'a');
		static const IParserISO14496_12::FBoxType kTrack_text = MAKE_BOX_ATOM('t', 'e', 'x', 't');

		/**
		 * Sample types
		 */
		static const IParserISO14496_12::FBoxType kSample_avc1 = MAKE_BOX_ATOM('a', 'v', 'c', '1');
		static const IParserISO14496_12::FBoxType kSample_avc3 = MAKE_BOX_ATOM('a', 'v', 'c', '3');
		static const IParserISO14496_12::FBoxType kSample_hvc1 = MAKE_BOX_ATOM('h', 'v', 'c', '1');
		static const IParserISO14496_12::FBoxType kSample_hev1 = MAKE_BOX_ATOM('h', 'e', 'v', '1');
		static const IParserISO14496_12::FBoxType kSample_mp4a = MAKE_BOX_ATOM('m', 'p', '4', 'a');
		static const IParserISO14496_12::FBoxType kSample_stpp = MAKE_BOX_ATOM('s', 't', 'p', 'p');
		static const IParserISO14496_12::FBoxType kSample_sbtt = MAKE_BOX_ATOM('s', 'b', 't', 't');
		static const IParserISO14496_12::FBoxType kSample_tx3g = MAKE_BOX_ATOM('t', 'x', '3', 'g');
		static const IParserISO14496_12::FBoxType kSample_wvtt = MAKE_BOX_ATOM('w', 'v', 't', 't');
		static const IParserISO14496_12::FBoxType kSample_enca = MAKE_BOX_ATOM('e', 'n', 'c', 'a');
		static const IParserISO14496_12::FBoxType kSample_encv = MAKE_BOX_ATOM('e', 'n', 'c', 'v');
		static const IParserISO14496_12::FBoxType kSample_tmcd = MAKE_BOX_ATOM('t', 'm', 'c', 'd');

		/**
		 * Track Reference types - https://developer.apple.com/documentation/quicktime-file-format/track_reference_type_atom
		 */
		static const IParserISO14496_12::FBoxType kTrackRef_tmcd = MAKE_BOX_ATOM('t', 'm', 'c', 'd');

		/**
		 * Sample grouping types
		 */
		//static const IParserISO14496_12::FBoxType kGrouping_seig = MAKE_BOX_ATOM('s', 'e', 'i', 'g');
		//static const IParserISO14496_12::FBoxType kGrouping_rap  = MAKE_BOX_ATOM('r', 'a', 'p', ' ');

		/**
		 * Encryption scheme types
		 */
		static const IParserISO14496_12::FBoxType kEncryptionScheme_cenc = MAKE_BOX_ATOM('c', 'e', 'n', 'c');
		static const IParserISO14496_12::FBoxType kEncryptionScheme_cbcs = MAKE_BOX_ATOM('c', 'b', 'c', 's');

		/**
		 * Brands
		 */
		static const IParserISO14496_12::FBoxType kBrand_QT = MAKE_BOX_ATOM('q', 't', ' ', ' ');

		/**
		 * Known UUIDs for PIFF boxes before they were added to ISO/IEC 14496-12
		 */
		static const uint8 UUID_atom[16];
		static const uint8 UUID_pssh[16];
		static const uint8 UUID_tenc[16];
		static const uint8 UUID_senc[16];

		//	#undef MAKE_BOX_ATOM

	private:
		FMP4Box() = delete;
		FMP4Box(const FMP4Box&) = delete;
	protected:
		FMP4Box*						ParentBox;
		TArray<FMP4Box*>				ChildBoxes;
		IParserISO14496_12::FBoxType	BoxType;
		int64							BoxSize;
		int64							StartOffset;
		int64							DataOffset;
		bool							bIsLeafBox;
		FFillerData* FillerData;

#if MEDIA_DEBUG_HAS_BOX_NAMES
		char							DbgBoxName[4];
#endif
	};

	const uint8 FMP4Box::UUID_atom[16] = { 0x00,0x00,0x00,0x00,0x00,0x11,0x00,0x10,0x80,0x00,0x00,0xAA,0x00,0x38,0x9B,0x71 };
	const uint8 FMP4Box::UUID_pssh[16] = { 0xD0,0x8A,0x4F,0x18,0x10,0xF3,0x4A,0x82,0xB6,0xC8,0x32,0xD8,0xAB,0xA1,0x83,0xD3 };
	const uint8 FMP4Box::UUID_tenc[16] = { 0x89,0x74,0xDB,0xCE,0x7B,0xE7,0x4C,0x51,0x84,0xF9,0x71,0x48,0xF9,0x88,0x25,0x54 };
	const uint8 FMP4Box::UUID_senc[16] = { 0xA2,0x39,0x4F,0x52,0x5A,0x9B,0x4F,0x14,0xA2,0x44,0x6C,0x42,0x7C,0x64,0x8D,0xF4 };


	/**
	 * A basic container box that has no attributes of its own.
	 * This should be the base class for all boxes.
	 */
	class FMP4BoxBasic : public FMP4Box
	{
	public:
		FMP4BoxBasic(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4Box(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxBasic()
		{
		}

	private:
		FMP4BoxBasic() = delete;
		FMP4BoxBasic(const FMP4BoxBasic&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			// The basic box is merely a container with no attributes.
			return UEMEDIA_ERROR_OK;
		}
	};


	/**
	 * A "full" box. This is not so much an actual box but a box that has a version number and flag fields.
	 */
	class FMP4BoxFull : public FMP4BoxBasic
	{
	public:
		FMP4BoxFull(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, Flags(0), Version(0)
		{
		}

		uint8 GetBoxVersion() const
		{
			return Version;
		}

		uint32 GetFlags() const
		{
			return Flags;
		}

		virtual ~FMP4BoxFull()
		{
		}

	private:
		FMP4BoxFull() = delete;
		FMP4BoxFull(const FMP4BoxFull&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint32 VersionAndFlags = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(VersionAndFlags));
			Version = VersionAndFlags >> 24;
			Flags = VersionAndFlags & 0x00ffffff;
			return Error;
		}

		UEMediaError ReadAndParseAttributes(FMP4BoxReader& Reader)
		{
			uint32 VersionAndFlags = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(Reader.Read(VersionAndFlags));
			Version = VersionAndFlags >> 24;
			Flags = VersionAndFlags & 0x00ffffff;
			return Error;
		}

		uint32		Flags;
		uint8		Version;
	};


	/**
	 * A box we either do not know or do not care about.
	 */
	class FMP4BoxIgnored : public FMP4BoxBasic
	{
	public:
		FMP4BoxIgnored(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxIgnored()
		{
		}

	private:
		FMP4BoxIgnored() = delete;
		FMP4BoxIgnored(const FMP4BoxIgnored&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			// Read the remaining bytes from this box.
			return ReadFillerData(ParseInfo, BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
		}
	};


	/**
	 * 'mdat' box.
	 * This box has no parseable data. If we are asked to parse it then this is probably in locating a 'moov' box of
	 * a file that is not fast-startable and we will actually skip over it.
	 */
	class FMP4BoxMDAT : public FMP4BoxBasic
	{
	public:
		FMP4BoxMDAT(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxMDAT()
		{
		}

	private:
		FMP4BoxMDAT() = delete;
		FMP4BoxMDAT(const FMP4BoxMDAT&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			// Read the data from this box to NULL.
			return ParseInfo->Reader()->ReadBytes(nullptr, BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
		}
	};


	/**
	 * 'ftyp' box.
	 * ISO/IEC 14496-12:2014 - 4.3 - File Type Box
	 */
	class FMP4BoxFTYP : public FMP4BoxBasic
	{
	public:
		FMP4BoxFTYP(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, MajorBrand(0), MinorVersion(0)
		{
		}

		virtual ~FMP4BoxFTYP()
		{
		}

		int32 GetNumberOfBrands() const
		{
			return 1 + CompatibleBrands.Num();
		}

		IParserISO14496_12::FBrandType GetBrand(int32 Index) const
		{
			if (Index > 0 && Index <= CompatibleBrands.Num())
			{
				return CompatibleBrands[Index - 1];
			}
			return MajorBrand;
		}
	private:
		FMP4BoxFTYP() = delete;
		FMP4BoxFTYP(const FMP4BoxFTYP&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			uint32 Major;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Major));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(MinorVersion));
			MajorBrand = (IParserISO14496_12::FBrandType) Major;
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				uint32 NumCompatibleBrands = BytesRemaining / sizeof(uint32);
				if (NumCompatibleBrands)
				{
					CompatibleBrands.Reserve(NumCompatibleBrands);
					for(uint32 i = 0; i < NumCompatibleBrands; ++i)
					{
						uint32 CompatibleBrand = 0;
						RETURN_IF_ERROR(ParseInfo->Reader()->Read(CompatibleBrand));
						CompatibleBrands.Push((IParserISO14496_12::FBrandType) CompatibleBrand);
					}
				}
			}
			return Error;
		}

	private:
		IParserISO14496_12::FBrandType			MajorBrand;
		uint32									MinorVersion;
		TArray<IParserISO14496_12::FBrandType>	CompatibleBrands;
	};


	/**
	 * 'styp' box.
	 * ISO/IEC 14496-12:2014 - 8.16.2 - Segment Type Box
	 *
	 * Note: The 'styp' box is the same as the 'ftyp' box. We make a distinction regardless
	 *       for clearness and disambiguation.
	 */
	class FMP4BoxSTYP : public FMP4BoxFTYP
	{
	public:
		FMP4BoxSTYP(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFTYP(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxSTYP()
		{
		}

	private:
		FMP4BoxSTYP() = delete;
		FMP4BoxSTYP(const FMP4BoxSTYP&) = delete;

	protected:

	private:
	};



	/**
	 * 'mvhd' box.
	 * ISO/IEC 14496-12:2014 - 8.2.2 - Movie Header Box
	 */
	class FMP4BoxMVHD : public FMP4BoxFull
	{
	public:
		FMP4BoxMVHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, Duration(0), Timescale(0)
		{
		}

		virtual ~FMP4BoxMVHD()
		{
		}

		FTimeFraction GetDuration() const
		{
			// Set the duration only if it fits into a signed value. If duration is not specified
			// it will be set to ~0.
			if (Duration <= 0x7fffffffffffffffLL)
			{
				return FTimeFraction((int64)Duration, Timescale);
			}
			return FTimeFraction::GetInvalid();
		}

		uint32 GetTimescale() const
		{
			return Timescale;
		}

	private:
		FMP4BoxMVHD() = delete;
		FMP4BoxMVHD(const FMP4BoxMVHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint64  		Value64 = 0;
			uint32  		Value32 = 0;
			UEMediaError	Error = UEMEDIA_ERROR_OK;

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			// Which version of the box is this?
			if (Version == 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value64));			// creation_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value64));			// modification_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Timescale));			// timescale
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Duration));			// duration
			}
			else // Version == 0 (NOTE: Yes, this is how the standard defines version checking)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// creation_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// modification_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Timescale));			// timescale
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// duration
				Duration = (Value32 == kAllOnes32) ? kAllOnes64 : Value32;
			}
			// Read and ignore: rate, volume, reserved, reserved, matrix, pre_defined (old QuickTime values) and next_track_ID
			for(int32 i = 0; i < 20; ++i)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));
			}

			return Error;
		}

	private:
		uint64			Duration;
		uint32			Timescale;
	};


	/**
	 * 'mehd' box.
	 * ISO/IEC 14496-12:2014 - 8.8.2 - Movie Extends Header Box
	 */
	class FMP4BoxMEHD : public FMP4BoxFull
	{
	public:
		FMP4BoxMEHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, Duration(0)
		{
		}

		virtual ~FMP4BoxMEHD()
		{
		}

	private:
		FMP4BoxMEHD() = delete;
		FMP4BoxMEHD(const FMP4BoxMEHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint32  		Value32 = 0;
			UEMediaError	Error = UEMEDIA_ERROR_OK;

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			// Which version of the box is this?
			if (Version == 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Duration));			// duration
			}
			else // Version == 0 (NOTE: Yes, this is how the standard defines version checking)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// duration
				Duration = (Value32 == kAllOnes32) ? kAllOnes64 : Value32;
			}
			return Error;
		}

	private:
		uint64			Duration;
	};


	/**
	 * 'tkhd' box.
	 * ISO/IEC 14496-12:2014 - 8.3.2 - Track Header Box
	 */
	class FMP4BoxTKHD : public FMP4BoxFull
	{
	public:
		FMP4BoxTKHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, Duration(0), TrackID(0), Width(0), Height(0), Layer(0), AlternateGroup(0)
		{
		}

		virtual ~FMP4BoxTKHD()
		{
		}

		int64 GetDuration() const
		{
			return (int64)Duration;
		}

		uint32 GetTrackID() const
		{
			return TrackID;
		}

		const double* GetMatrix() const
		{
			return Matrix;
		}

		double GetMatrixValue(int32 Index) const
		{
			return (Index >= 0 && Index < 9) ? Matrix[Index] : 0.0;
		}

		uint16 GetWidth() const
		{
			return Width >> 16;
		}

		uint16 GetHeight() const
		{
			return Height >> 16;
		}


	private:
		FMP4BoxTKHD() = delete;
		FMP4BoxTKHD(const FMP4BoxTKHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint64 Value64 = 0;
			uint32 Value32 = 0;
			uint16 Value16 = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			// Which version of the box is this?
			if (Version == 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value64));			// creation_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value64));			// modification_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(TrackID));			// track_ID
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// reserved
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Duration));			// duration
			}
			else // Version == 0 (NOTE: Yes, this is how the standard defines version checking)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// creation_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// modification_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(TrackID));			// track_ID
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// reserved
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// duration
				Duration = (Value32 == kAllOnes32) ? kAllOnes64 : Value32;
			}
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));				// reserved[0]
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));				// reserved[1]
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Layer));					// layer
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(AlternateGroup));			// alternate_group
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));				// volume
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));				// reserved
			// Matrix
			for(int32 i=0; i<9; ++i)
			{
				/*
					"All the values in a matrix are stored as 16.16 fixed‐point values, except for u, v and w, which are stored
					as 2.30 fixed‐point values. The values in the matrix are stored in the order {a,b,u, c,d,v, x,y,w}."
				*/
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));
				if (i == 2 || i ==5 || i == 8)
				{
					Matrix[i] = (int32)Value32 / 1073741824.0;
				}
				else
				{
					Matrix[i] = (int32)Value32 / 65536.0;
				}
			}
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Width));					// width (16.16 fixed point). If flags & 8 then indicates aspect ratio.
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Height));					// height (16.16 fixed point). If flags & 8 then indicates aspect ratio.

			// Track IDs cannot be zero.
			if (TrackID == 0)
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}
			return Error;
		}

	private:
		uint64		Duration;
		uint32		TrackID;
		uint32		Width;
		uint32		Height;
		int16		Layer;
		int16		AlternateGroup;
		double		Matrix[9];
	};


	/**
	 * 'trex' box.
	 * ISO/IEC 14496-12:2014 - 8.8.3 - Track Extends Box
	 */
	class FMP4BoxTREX : public FMP4BoxFull
	{
	public:
		FMP4BoxTREX(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, TrackID(0), DefaultSampleDescriptionIndex(0), DefaultSampleDuration(0), DefaultSampleSize(0), DefaultSampleFlags(0)
		{
		}

		virtual ~FMP4BoxTREX()
		{
		}

		uint32 GetTrackID() const
		{
			return TrackID;
		}

		uint32 GetDefaultSampleDescriptionIndex() const
		{
			return DefaultSampleDescriptionIndex;
		}

		uint32 GetDefaultSampleDuration() const
		{
			return DefaultSampleDuration;
		}

		uint32 GetDefaultSampleSize() const
		{
			return DefaultSampleSize;
		}

		uint32 GetDefaultSampleFlags() const
		{
			return DefaultSampleFlags;
		}


	private:
		FMP4BoxTREX() = delete;
		FMP4BoxTREX(const FMP4BoxTREX&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(TrackID));						// track_ID
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleDescriptionIndex));	// default_sample_description_index
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleDuration));			// default_sample_duration
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleSize));				// default_sample_size
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleFlags));				// default_sample_flags

			// Track IDs cannot be zero.
			if (TrackID == 0)
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}
			return Error;
		}

	private:
		uint32		TrackID;
		uint32		DefaultSampleDescriptionIndex;
		uint32		DefaultSampleDuration;
		uint32		DefaultSampleSize;
		uint32		DefaultSampleFlags;
	};


	/**
	 * 'mvex' box.
	 * ISO/IEC 14496-12:2014 - 8.8.1 - Movie Extends Box
	 */
	class FMP4BoxMVEX : public FMP4BoxBasic
	{
	public:
		FMP4BoxMVEX(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxMVEX()
		{
		}

		const FMP4BoxTREX* FindTREXForTrackID(uint32 TrackID) const
		{
			for(int32 i = 0, iMax = GetNumberOfChildren(); i < iMax; ++i)
			{
				const FMP4Box* Box = GetChildBox(i);
				if (Box->GetType() == FMP4Box::kBox_trex)
				{
					const FMP4BoxTREX* TREXBox = static_cast<const FMP4BoxTREX*>(Box);
					if (TREXBox->GetTrackID() == TrackID)
					{
						return TREXBox;
					}
				}
			}
			return nullptr;
		}

	private:
		FMP4BoxMVEX() = delete;
		FMP4BoxMVEX(const FMP4BoxMVEX&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			// The mvex box is merely a container with no attributes.
			return UEMEDIA_ERROR_OK;
		}
	};




	/**
	 * 'mdhd' box.
	 * ISO/IEC 14496-12:2014 - 8.4.2 - Media Header Box
	 */
	class FMP4BoxMDHD : public FMP4BoxFull
	{
	public:
		FMP4BoxMDHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, Duration(0), Timescale(0)
		{
			Language[0] = Language[1] = Language[2] = 0;
		}

		virtual ~FMP4BoxMDHD()
		{
		}

		uint64 GetDuration() const
		{
			return Duration;
		}

		uint32 GetTimescale() const
		{
			return Timescale;
		}

		const BCP47::FLanguageTag& GetLanguageTag() const
		{
			return LanguageTag;
		}

	private:
		FMP4BoxMDHD() = delete;
		FMP4BoxMDHD(const FMP4BoxMDHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint64 Value64 = 0;
			uint32 Value32 = 0;
			uint16 Value16 = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			// Which version of the box is this?
			if (Version == 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value64));			// creation_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value64));			// modification_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Timescale));			// timescale
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Duration));			// duration
			}
			else // Version == 0 (NOTE: Yes, this is how the standard defines version checking)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// creation_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// modification_time (seconds since 1/1/1904 00:00 UTC)
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Timescale));			// timescale
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// duration
				Duration = (Value32 == kAllOnes32) ? kAllOnes64 : Value32;
			}
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));				// padbit + language (ISO-639-2/T)
			Language[0] = (char)(0x60 + ((Value16 & 0x7c00) >> 10));
			Language[1] = (char)(0x60 + ((Value16 & 0x03e0) >> 5));
			Language[2] = (char)(0x60 + (Value16 & 0x001f));
			// Try to map the ISO-639-2T language code to the shorter ISO-639-1 code if possible.
			/*bool bSuccess =*/ BCP47::ParseRFC5646Tag(LanguageTag, FString::ConstructFromPtrSize(Language, FMEDIA_STATIC_ARRAY_COUNT(Language)));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));				// pre_defined (in QuickTime this held 'Quality')
			return Error;
		}

	private:
		uint64		Duration;
		uint32		Timescale;
		char		Language[3];
		BCP47::FLanguageTag LanguageTag;
	};


	/**
	 * 'hdlr' box.
	 * ISO/IEC 14496-12:2014 - 8.4.3 - Handler Reference Box
	 */
	class FMP4BoxHDLR : public FMP4BoxFull
	{
	public:
		FMP4BoxHDLR(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, HandlerType(0)
		{
		}

		virtual ~FMP4BoxHDLR()
		{
		}

		uint32 GetHandlerType() const
		{
			return HandlerType;
		}

		uint32 GetReservedValue(int32 Index) const
		{
			return Index >=0 && Index <= 2 ? Reserved[Index] : 0;
		}

		FString GetHandlerName() const
		{
			return NameUTF8;
		}

	private:
		FMP4BoxHDLR() = delete;
		FMP4BoxHDLR(const FMP4BoxHDLR&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint32 Value32 = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));				// pre_defined (in QuickTime this was "Component Type")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(HandlerType));			// handler_type
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Reserved[0]));			// reserved[0] (In QuickTime this was "Component Manufacturer")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Reserved[1]));			// reserved[1] (In QuickTime this was "Component Flags")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Reserved[2]));			// reserved[2] (In QuickTime this was "Component Flags Mask")

			int32 BytesRemaining = (int32)(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(NameUTF8, BytesRemaining));

			// Ideally this has now all bytes of this box consumed. That's not required though and we will consume anything that's left.
			BytesRemaining = (int32)(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			check(BytesRemaining >= 0);
			if (BytesRemaining > 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, BytesRemaining));
			}
			return Error;
		}

	private:
		FString			NameUTF8;
		uint32			HandlerType;
		uint32			Reserved[3];
	};



	/**
	 * 'elst' box.
	 * ISO/IEC 14496-12:2014 - 8.6.6 - Edit List Box
	 */
	class FMP4BoxELST : public FMP4BoxFull
	{
	public:
		FMP4BoxELST(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxELST()
		{
		}

		struct FEntry
		{
			uint64		SegmentDuration;
			int64		MediaTime;
			int16		MediaRateInteger;
			int16		MediaRateFraction;
		};

		int32 GetNumberOfEntries() const
		{
			return Entries.Num();
		}

		const FEntry& GetEntry(int32 EntryIndex) const
		{
			return Entries[EntryIndex];
		}


	private:
		FMP4BoxELST() = delete;
		FMP4BoxELST(const FMP4BoxELST&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint32	NumEntries = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumEntries));					// entry_count
			Entries.Reserve(NumEntries);
			// Which version of the box is this?
			if (Version == 1)
			{
				uint64	SegmentDuration = 0;
				int64	MediaTime = 0;
				int16	MediaRateInteger = 0;
				int16	MediaRateFraction = 0;
				for(uint32 i = 0; i < NumEntries; ++i)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(SegmentDuration));	// segment_duration
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(MediaTime));			// media_time
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(MediaRateInteger));	// media_rate_integer
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(MediaRateFraction));	// media_rate_fraction
					FEntry& e = Entries.AddDefaulted_GetRef();
					e.SegmentDuration = SegmentDuration;
					e.MediaTime = MediaTime;
					e.MediaRateInteger = MediaRateInteger;
					e.MediaRateFraction = MediaRateFraction;
				}
			}
			else // Version == 0 (NOTE: Yes, this is how the standard defines version checking)
			{
				uint32	SegmentDuration = 0;
				int32	MediaTime = 0;
				int16	MediaRateInteger = 0;
				int16	MediaRateFraction = 0;
				for(uint32 i = 0; i < NumEntries; ++i)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(SegmentDuration));	// segment_duration
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(MediaTime));			// media_time
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(MediaRateInteger));	// media_rate_integer
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(MediaRateFraction));	// media_rate_fraction
					FEntry& e = Entries.AddDefaulted_GetRef();
					e.SegmentDuration = SegmentDuration;
					e.MediaTime = MediaTime;
					e.MediaRateInteger = MediaRateInteger;
					e.MediaRateFraction = MediaRateFraction;
				}
			}
			return Error;
		}

	private:
		TArray<FEntry>		Entries;
	};


	/**
	 * 'nmhd' box.
	 * ISO/IEC 14496-12:2014 - 8.4.5.2 - Null Media Header Box
	 */
	class FMP4BoxNMHD : public FMP4BoxFull
	{
	public:
		FMP4BoxNMHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxNMHD()
		{
		}

	private:
		FMP4BoxNMHD() = delete;
		FMP4BoxNMHD(const FMP4BoxNMHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			return Error;
		}

	private:
	};


	/**
	 * 'vmhd' box.
	 * ISO/IEC 14496-12:2014 - 12.1.2 - Video media header
	 */
	class FMP4BoxVMHD : public FMP4BoxFull
	{
	public:
		FMP4BoxVMHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxVMHD()
		{
		}

	private:
		FMP4BoxVMHD() = delete;
		FMP4BoxVMHD(const FMP4BoxVMHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint16 Value16 = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// graphicsmode
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// opcolor[0]
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// opcolor[1]
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// opcolor[2]

			return Error;
		}

	private:
	};


	/**
	 * 'smhd' box.
	 * ISO/IEC 14496-12:2014 - 12.2.2 - Sound media header
	 */
	class FMP4BoxSMHD : public FMP4BoxFull
	{
	public:
		FMP4BoxSMHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxSMHD()
		{
		}

	private:
		FMP4BoxSMHD() = delete;
		FMP4BoxSMHD(const FMP4BoxSMHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint16 Value16 = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// balance (fixed point 8.8 value to place mono in stereo space with 0 being center, -1.0 left and 1.0 right)
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// reserved

			return Error;
		}

	private:
	};


	/**
	 * 'sthd' box.
	 * ISO/IEC 14496-12:2014 - 12.6.2 - Subtitle media header
	 */
	class FMP4BoxSTHD : public FMP4BoxFull
	{
	public:
		FMP4BoxSTHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxSTHD()
		{
		}

	private:
		FMP4BoxSTHD() = delete;
		FMP4BoxSTHD(const FMP4BoxSTHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			return Error;
		}

	private:
	};


	/**
	 * 'dref' box.
	 * ISO/IEC 14496-12:2014 - 8.7.2 - Data Reference Box
	 */
	class FMP4BoxDREF : public FMP4BoxFull
	{
	public:
		FMP4BoxDREF(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, NumEntries(0)
		{
		}

		virtual ~FMP4BoxDREF()
		{
		}

	private:
		FMP4BoxDREF() = delete;
		FMP4BoxDREF(const FMP4BoxDREF&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			// Now that we parse this box we clear its leaf status. We call back into the generic box reader
			// for which this flag must be clear to continue with the next boxes.
			bIsLeafBox = false;
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumEntries));					// entry_count
			for(uint32 i = 0; i < NumEntries; ++i)
			{
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this));
			}
			return Error;
		}

	private:
		uint32		NumEntries;
	};


	/**
	 * 'url ' box.
	 * ISO/IEC 14496-12:2014 - 8.7.2 - Data Reference Box
	 */
	class FMP4BoxURL : public FMP4BoxFull
	{
	public:
		FMP4BoxURL(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxURL()
		{
		}

	private:
		FMP4BoxURL() = delete;
		FMP4BoxURL(const FMP4BoxURL&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			if ((Flags & 1) == 0)
			{
				uint32 LocationLen = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(LocationUTF8, LocationLen));
			}
			/*
				Ideally this has now all bytes of this box consumed. That's not a requirement but should be the case.
				There are malformed `url ` boxes out there that violate the standard by providing a location despite
				the flag being 1 as per 8.7.2.3:
						one flag is defined (x000001) which means that the media data
						is in the same file as the Box containing this data reference. If this flag is set, the DataEntryUrlBox
						shall be used and no string is present; the box terminates with the entry-flags field.
			*/
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, BytesRemaining));
			}
			return Error;
		}

	private:
		FString		LocationUTF8;
	};


	/**
	 * 'urn ' box.
	 * ISO/IEC 14496-12:2014 - 8.7.2 - Data Reference Box
	 */
	class FMP4BoxURN : public FMP4BoxFull
	{
	public:
		FMP4BoxURN(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxURN()
		{
		}

	private:
		FMP4BoxURN() = delete;
		FMP4BoxURN(const FMP4BoxURN&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			if ((Flags & 1) == 0)
			{
				check(!"TODO");
				//uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
				//RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(LocationUTF8, BytesRemaining));
			}
			// Ideally this has now all bytes of this box consumed. That's not a requirement but should be the case.
			check(0 == BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			return Error;
		}

	private:
		FString	NameUTF8;
		FString	LocationUTF8;
	};


	/**
	 * Base class of a sample entry (within an 'stsd' box; 8.5.2 - Sample Description Box)
	 */
	class FMP4BoxSampleEntry : public FMP4BoxBasic
	{
	public:
		FMP4BoxSampleEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, DataReferenceIndex(0)
		{
		}

		virtual ~FMP4BoxSampleEntry()
		{
		}

	private:
		FMP4BoxSampleEntry() = delete;
		FMP4BoxSampleEntry(const FMP4BoxSampleEntry&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, 6));			// reserved
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DataReferenceIndex));			// data_reference_index
			return Error;
		}
		uint16		DataReferenceIndex;
	};


	/**
	 * Visual sample entry (12.1.3 - Sample Entry)
	 */
	class FMP4BoxVisualSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		FMP4BoxVisualSampleEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxSampleEntry(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, HorizontalResolutionDPI(0), VerticalResolutionDPI(0), Width(0), Height(0), FrameCount(0), Depth(0)
		{
			FMemory::Memzero(CompressorName);
		}

		virtual ~FMP4BoxVisualSampleEntry()
		{
		}

		int32 GetWidth() const
		{
			return Width;
		}

		int32 GetHeight() const
		{
			return Height;
		}

	private:
		FMP4BoxVisualSampleEntry() = delete;
		FMP4BoxVisualSampleEntry(const FMP4BoxVisualSampleEntry&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			uint32 Value32 = 0;
			uint16 Value16 = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxSampleEntry::ReadAndParseAttributes(ParseInfo));
			// Skip/read over reserved values that have no meaning in the specification any more.
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// pre_defined (in QuickTime this was "Version")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// reserved (in QuickTime this was "Revision Level")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));					// pre_defined[0] (in QuickTime this was "Vendor")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));					// pre_defined[1] (in QuickTime this was "Temporal quality")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));					// pre_defined[2] (in QuickTime this was "Spatial quality")

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Width));						// width
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Height));						// height
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(HorizontalResolutionDPI));	// horizresolution (16.16 fixed point; typically 0x00480000 for 72 DPI)
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(VerticalResolutionDPI));		// vertresolution (16.16 fixed point; typically 0x00480000 for 72 DPI)

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));					// reserved (in QuickTime this was "Data size")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(FrameCount));					// frame_count
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(CompressorName, 32));	// compressorname
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Depth));						// depth (eg. 24, bits per pixel)
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// pre_defined (in QuickTime this was "Color table ID")
			// There must not be a color table here. The value must be set to -1
			if (Value16 != kAllOnes16)
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}
			// There can now be additional boxes following, most notably 'pasp' and 'clap'.
			int64 NowAt = ParseInfo->Reader()->GetCurrentReadOffset();
			int32 BytesRemaining = (int32)(BoxSize - (NowAt - StartOffset));
			if (BytesRemaining > 0)
			{
				bIsLeafBox = false;
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this, NowAt + BytesRemaining));
				BytesRemaining = (int32)(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
				check(BytesRemaining >= 0);
				if (BytesRemaining > 0)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, BytesRemaining));
				}
			}
			return Error;
		}

		uint8		CompressorName[32];
		uint32		HorizontalResolutionDPI;
		uint32		VerticalResolutionDPI;
		uint16		Width;
		uint16		Height;
		uint16		FrameCount;
		uint16		Depth;
	};


	/**
	 * Audio sample entry (12.2.3 - Sample Entry)
	 */
	class FMP4BoxAudioSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		FMP4BoxAudioSampleEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox, int32 InSTSDBoxVersion)
			: FMP4BoxSampleEntry(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, STSDBoxVersion(InSTSDBoxVersion)
		{
		}

		virtual ~FMP4BoxAudioSampleEntry()
		{
		}

		uint32 GetSampleRate() const
		{
			return SampleRate;
		}
		int32 GetChannelCount() const
		{
			return ChannelCount;
		}
		int32 GetSampleSize() const
		{
			return SampleSize;
		}

		bool HaveFormatSpecificFlags() const
		{
			return Version == 2;
		}
		int32 GetFormatSpecificFlags() const
		{
			return FormatSpecificFlags;
		}
		uint32 GetConstBytesPerAudioPacket() const
		{
			return ConstBytesPerAudioPacket;
		}
		uint32 GetConstLPCMFramesPerAudioPacket() const
		{
			return ConstLPCMFramesPerAudioPacket;
		}

	private:
		FMP4BoxAudioSampleEntry() = delete;
		FMP4BoxAudioSampleEntry(const FMP4BoxAudioSampleEntry&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			// The audio sample entry can be version 0 or version 1.
			// In ISO/IEC 14496-12 a version 1 sample is required to be inside a version 1 'stsd' box while in QuickTime a
			// version 1 sample was allowed in a version 0 'stsd'. The version 1 sample is not identical between ISO and QT
			// so we do some special handling based on version numbers.
			uint32 Value32 = 0;
			uint16 Value16 = 0;
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxSampleEntry::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Version));					// entry_version for v1, reserved for v0 (in QuickTime this was "Version")
			// Skip/read over reserved values that have no meaning in the specification any more.
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// reserved (in QuickTime this was "Revision Level")
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));					// reserved (in QuickTime this was "Vendor")

			// As defined by ISO/IEC
			if (Version == 0 || Version == 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(ChannelCount));				// channelcount
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleSize));					// samplesize
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// pre_defined (in QuickTime this was "Compression ID")
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// reserved (in QuickTime this was "Packet size")
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleRate));					// samplerate (16.16 fixed point; must be 1<<16 for ISO v1 with additional 'srat' box following)

				// Handle a special case of version 1 QuickTime box. In ISO/IEC 14496-12 the SampleRate must be set to 0x00010000 (1 Hz).
				// If this is not the case we assume this to be a QuickTime box which adds 4 additional fields *before* any optional boxes.
				if (STSDBoxVersion == 0 && Version == 1 && SampleRate != (1 << 16))
				{
					// Assume QT and read the following elements
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));				// Samples per packet
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));				// Bytes per packet
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));				// Bytes per frame
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));				// Bytes per Sample
				}
				SampleRate >>= 16;
			}
			// As defined by: https://developer.apple.com/library/archive/documentation/QuickTime/QTFF/QTFFChap3/qtff3.html
			else if (Version == 2)
			{
				int32 SizeOfStructOnly = 0;
				int32 NumChannels = 0;
				int32 ConstBitsPerChannel = 0;
				union Int64toDouble
				{
					uint64 ui64;
					double dbl64;
				};
				Int64toDouble SampleRateCnv;
				SampleRateCnv.ui64 = 0;
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// always 3
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// always 16
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// always 0xfffe
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));					// always 0
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));					// always 65536
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(SizeOfStructOnly));
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleRateCnv.ui64));
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumChannels));
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));					// always 0x7f000000
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(ConstBitsPerChannel));
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(FormatSpecificFlags));
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(ConstBytesPerAudioPacket));
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(ConstLPCMFramesPerAudioPacket));

				SampleRate = (uint32) SampleRateCnv.dbl64;
				ChannelCount = (uint16) NumChannels;
				SampleSize = (uint16) ConstBitsPerChannel;
			}
			else
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}

			// There can now be additional boxes following (channel layout, downmix, DRC, sample rate)
			int64 NowAt = ParseInfo->Reader()->GetCurrentReadOffset();
			int32 BytesRemaining = (int32)(BoxSize - (NowAt - StartOffset));
			if (BytesRemaining > 0)
			{
				bIsLeafBox = false;
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this, NowAt + BytesRemaining));
				BytesRemaining = (int32)(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
				check(BytesRemaining >= 0);
				if (BytesRemaining > 0)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, BytesRemaining));
				}
			}
			return Error;
		}

		int32		STSDBoxVersion = 0;
		uint16		Version = 0;
		uint32		SampleRate = 0;
		int32		FormatSpecificFlags = 0;
		uint32		ConstBytesPerAudioPacket = 0;
		uint32		ConstLPCMFramesPerAudioPacket = 0;
		uint16		ChannelCount = 0;
		uint16		SampleSize = 0;
	};

	/**
	 * Timecode sample entry
	 * Reference: https://developer.apple.com/documentation/quicktime-file-format/timecode_sample_description
	 */
	class FMP4BoxTimecodeSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		FMP4BoxTimecodeSampleEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxSampleEntry(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxTimecodeSampleEntry() override = default;

		struct FFlags
		{
			/** Indicates whether the timecode is drop frame. Set it to 1 if the timecode is drop frame. */
			static constexpr uint32 DropFrame = 0x0001;
			/** Indicates whether the timecode wraps after 24 hours. Set it to 1 if the timecode wraps. */
			static constexpr uint32 Max24Hour = 0x0002;
			/** Indicates whether negative time values are allowed. Set it to 1 if the timecode supports negative values. */
			static constexpr uint32 AllowNegativeTimes = 0x0004;
			/** Indicates whether the time value corresponds to a tape counter value. Set it to 1 if the timecode values are tape counter values.*/
			static constexpr uint32 Counter = 0x0008;
		};

		uint32 GetFlags() const { return Flags; }
		uint32 GetTimeScale() const { return TimeScale; }
		uint32 GetFrameDuration() const { return FrameDuration; }
		uint32 GetNumberOfFrames() const { return NumberOfFrames; }

	private:
		FMP4BoxTimecodeSampleEntry() = delete;
		FMP4BoxTimecodeSampleEntry(const FMP4BoxTimecodeSampleEntry&) = delete;

	protected:
		virtual UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxSampleEntry::ReadAndParseAttributes(ParseInfo));

			uint32 Reserved32 = 0;
			uint8 Reserved8 = 0;

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Reserved32));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Flags));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(TimeScale));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(FrameDuration));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumberOfFrames));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Reserved8));

			// Next would be (optional) source reference atom.
			const int64 NowAt = ParseInfo->Reader()->GetCurrentReadOffset();
			int32 BytesRemaining = static_cast<int32>(BoxSize - (NowAt - StartOffset));
			if (BytesRemaining > 0)
			{
				bIsLeafBox = false;
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this, NowAt + BytesRemaining));
				BytesRemaining = static_cast<int32>(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
				check(BytesRemaining >= 0);
				if (BytesRemaining > 0)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, BytesRemaining));
				}
			}
			return Error;
		}

		uint32 Flags = 0;
		uint32 TimeScale = 0;
		uint32 FrameDuration = 0;
		uint8 NumberOfFrames = 0;
	};

	/**
	 * Plain Text sample entry (12.5.3 - Sample Entry)
	 */
	class FMP4BoxPlainTextSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		FMP4BoxPlainTextSampleEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox, int32 InSTSDBoxVersion)
			: FMP4BoxSampleEntry(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, STSDBoxVersion(InSTSDBoxVersion)
		{
		}

		virtual ~FMP4BoxPlainTextSampleEntry()
		{
		}

	private:
		FMP4BoxPlainTextSampleEntry() = delete;
		FMP4BoxPlainTextSampleEntry(const FMP4BoxPlainTextSampleEntry&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxSampleEntry::ReadAndParseAttributes(ParseInfo));

			// There can now be additional boxes following
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				// Now that there are additional boxes we ourselves are no longer a leaf box.
				bIsLeafBox = false;
				// This will read all the boxes in here and add them as children.
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this));
			}

			return Error;
		}

		int32		STSDBoxVersion;
	};


	/**
	 * 3GPP / TX3G Text sample entry (ETSI TS 126 245 V11.0.0 - 5.16 Sample Description Format)
	 */
	class FMP4BoxTX3GSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		FMP4BoxTX3GSampleEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox, int32 InSTSDBoxVersion)
			: FMP4BoxSampleEntry(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, STSDBoxVersion(InSTSDBoxVersion)
		{
		}

		virtual ~FMP4BoxTX3GSampleEntry()
		{
		}

		struct FBoxRecord
		{
			int16 Top = 0;
			int16 Left = 0;
			int16 Bottom = 0;
			int16 Right = 0;
		};

		struct FStyleRecord
		{
			uint16 StartChar = 0;
			uint16 EndChar = 0;
			uint16 FontID = 0;
			uint8 FaceStyleFlags = 0;
			uint8 FontSize = 0;
			uint8 TextColorRGBA[4] { 0 };
		};

		struct FFontRecord
		{
			uint16 FontID = 0;
			FString FontName;
		};

		const TArray<uint8> GetRawBoxData() const
		{
			return BoxData;
		}

		int32 GetTranslationX() const
		{
			return TranslationX;
		}

		int32 GetTranslationY() const
		{
			return TranslationY;
		}

		uint16 GetWidth() const
		{
			return Width;
		}

		uint16 GetHeight() const
		{
			return Height;
		}

		uint32 GetTimescale() const
		{
			return Timescale;
		}

	private:
		FMP4BoxTX3GSampleEntry() = delete;
		FMP4BoxTX3GSampleEntry(const FMP4BoxTX3GSampleEntry&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;

			// Duplicate all the data we are reading into a buffer that we need to provide to the subtitle plugin.
			FMP4BoxReader::ScopedDataDuplication DuplicateBoxData(ParseInfo->Reader(), BoxData);

			RETURN_IF_ERROR(FMP4BoxSampleEntry::ReadAndParseAttributes(ParseInfo));

			const FMP4BoxTKHD* TKHD = ParseInfo->GetCurrentTrackBox() ? static_cast<const FMP4BoxTKHD*>(ParseInfo->GetCurrentTrackBox()->GetBoxPath(FMP4Box::kBox_tkhd)) : nullptr;
			if (TKHD)
			{
				Width = TKHD->GetWidth();
				Height = TKHD->GetHeight();
				// Translation must not be using fractional digits and can thus be treated like integers.
				TranslationX = (int32) TKHD->GetMatrixValue(6);
				TranslationY = (int32) TKHD->GetMatrixValue(7);
			}

			const FMP4BoxMDHD* MDHD = ParseInfo->GetCurrentTrackBox() ? static_cast<const FMP4BoxMDHD*>(ParseInfo->GetCurrentTrackBox()->FindBox(FMP4Box::kBox_mdhd)) : nullptr;
			if (MDHD)
			{
				Timescale = MDHD->GetTimescale();
			}

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DisplayFlags));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(HorizontalJustification));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(VerticalJustification));
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(BackgroundColorRGBA, 4));

			// BoxRecord
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(BoxRecord.Top));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(BoxRecord.Left));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(BoxRecord.Bottom));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(BoxRecord.Right));

			// Style record
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(StyleRecord.StartChar));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(StyleRecord.EndChar));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(StyleRecord.FontID));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(StyleRecord.FaceStyleFlags));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(StyleRecord.FontSize));
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(StyleRecord.TextColorRGBA, 4));

			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			// The font table is stored in a box. Not sure if there can be more than one though.
			// We skip over the box itself and parse it in place. Any additional boxes we ignore.
			if (BytesRemaining < 10)
			{
				UE_LOGF(LogElectraMP4Parser, Error, "ERROR: ElectraPlayer/FMP4BoxTX3GSampleEntry::ReadAndParseAttributes(): Not enough data for the 'ftab' box");
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, 8));

			// FontTableBox
			uint16 EntryCount;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(EntryCount));
			FontRecords.Reserve(EntryCount);
			for(int32 nEntry=0; nEntry<EntryCount; ++nEntry)
			{
				FFontRecord& fr = FontRecords.AddDefaulted_GetRef();
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(fr.FontID));
				uint8 FontNameLength;
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(FontNameLength));
				if (FontNameLength)
				{
					uint8 FontName[256] {0};
					RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(FontName, FontNameLength));
					if (FontNameLength >= 2 && ((FontName[0] == 0xff && FontName[1] == 0xfe) || (FontName[0] == 0xfe && FontName[1] == 0xff)))
					{
						UE_LOGF(LogElectraMP4Parser, Error, "ERROR: ElectraPlayer/FMP4BoxTX3GSampleEntry::ReadAndParseAttributes(): Font name uses UTF16 which is not supported");
						return UEMEDIA_ERROR_FORMAT_ERROR;
					}
					FUTF8ToTCHAR cnv((const ANSICHAR*)FontName, FontNameLength);
					fr.FontName = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
				}
			}

			// There could now be additional boxes following
			BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				// Now that there are additional boxes we ourselves are no longer a leaf box.
				bIsLeafBox = false;
				// This will read all the boxes in here and add them as children.
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this));
			}

			return Error;
		}

		TArray<uint8>		BoxData;

		int32				STSDBoxVersion;
		uint32				Timescale = 0;
		uint16				Width = 0;
		uint16				Height = 0;
		int32				TranslationX = 0;
		int32				TranslationY = 0;
		uint32				DisplayFlags = 0;
		int8				HorizontalJustification = 0;
		int8				VerticalJustification = 0;
		uint8				BackgroundColorRGBA[4] {0};
		FBoxRecord			BoxRecord;
		FStyleRecord		StyleRecord;
		TArray<FFontRecord>	FontRecords;
	};


	/**
	 * WebVTT sample entry (ISO/IEC 14496-30 Section 7.5 Sample entry format)
	 */
	class FMP4BoxWVTTSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		FMP4BoxWVTTSampleEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox, int32 InSTSDBoxVersion)
			: FMP4BoxSampleEntry(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, STSDBoxVersion(InSTSDBoxVersion)
		{
		}

		virtual ~FMP4BoxWVTTSampleEntry()
		{
		}

		const TArray<uint8> GetRawBoxData() const
		{
			return BoxData;
		}

		int32 GetTranslationX() const
		{
			return TranslationX;
		}

		int32 GetTranslationY() const
		{
			return TranslationY;
		}

		uint16 GetWidth() const
		{
			return Width;
		}

		uint16 GetHeight() const
		{
			return Height;
		}

		uint32 GetTimescale() const
		{
			return Timescale;
		}

	private:
		FMP4BoxWVTTSampleEntry() = delete;
		FMP4BoxWVTTSampleEntry(const FMP4BoxWVTTSampleEntry&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;

			// Duplicate all the data we are reading into a buffer that we need to provide to the subtitle plugin.
			FMP4BoxReader::ScopedDataDuplication DuplicateBoxData(ParseInfo->Reader(), BoxData);

			RETURN_IF_ERROR(FMP4BoxSampleEntry::ReadAndParseAttributes(ParseInfo));

			const FMP4BoxTKHD* TKHD = ParseInfo->GetCurrentTrackBox() ? static_cast<const FMP4BoxTKHD*>(ParseInfo->GetCurrentTrackBox()->GetBoxPath(FMP4Box::kBox_tkhd)) : nullptr;
			if (TKHD)
			{
				Width = TKHD->GetWidth();
				Height = TKHD->GetHeight();
				// Translation must not be using fractional digits and can thus be treated like integers.
				TranslationX = (int32) TKHD->GetMatrixValue(6);
				TranslationY = (int32) TKHD->GetMatrixValue(7);
			}

			const FMP4BoxMDHD* MDHD = ParseInfo->GetCurrentTrackBox() ? static_cast<const FMP4BoxMDHD*>(ParseInfo->GetCurrentTrackBox()->FindBox(FMP4Box::kBox_mdhd)) : nullptr;
			if (MDHD)
			{
				Timescale = MDHD->GetTimescale();
			}

			// There ought to be child boxes here now, at least a 'vttC' box.
			int32 BytesRemaining = (int32) (BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			if (BytesRemaining > 0)
			{
				// Now that there are additional boxes we ourselves are no longer a leaf box.
				bIsLeafBox = false;
				// This will read all the boxes in here and add them as children.
				for(; BytesRemaining>0; BytesRemaining = (int32) (BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset)))
				{
					IParserISO14496_12::FBoxType	ChildBoxType;
					int64							ChildBoxSize;
					uint8							ChildBoxUUID[16];

					int64 BoxStartOffset = ParseInfo->Reader()->GetCurrentReadOffset();
					RETURN_IF_ERROR(ParseInfo->ReadBoxTypeAndSize(ChildBoxType, ChildBoxSize, ChildBoxUUID));
#if MEDIA_DEBUG_HAS_BOX_NAMES
					char boxName[4];
					*((uint32*)&boxName) = MEDIA_TO_BIG_ENDIAN(ChildBoxType);
#endif
					int64 BoxDataOffset = ParseInfo->Reader()->GetCurrentReadOffset();
					FMP4Box* NextBox = new FMP4BoxIgnored(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);

					AddChildBox(NextBox);
					RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(NextBox));
				}
			}
			// There needs to be the 'vttC' config box now.
			static const IParserISO14496_12::FBoxType kWebVTT_vttC = MAKE_BOX_ATOM('v', 't', 't', 'C');
			if (!FindBox(kWebVTT_vttC, 1))
			{
				UE_LOGF(LogElectraMP4Parser, Error, "ERROR: ElectraPlayer/FMP4BoxWVTTSampleEntry::ReadAndParseAttributes(): Required 'vttC' box not found!");
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}
			return Error;
		}

		TArray<uint8>		BoxData;
		int32				STSDBoxVersion;
		uint32				Timescale = 0;
		uint16				Width = 0;
		uint16				Height = 0;
		int32				TranslationX = 0;
		int32				TranslationY = 0;
	};


	/**
	 * XMLSubtitleSampleEntry sample entry (ISO/IEC 14496-12 Section 12.6.3 Sample entry)
	 */
	class FMP4BoxXMLSubtitleSampleEntry : public FMP4BoxSampleEntry
	{
	public:
		FMP4BoxXMLSubtitleSampleEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox, int32 InSTSDBoxVersion)
			: FMP4BoxSampleEntry(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, STSDBoxVersion(InSTSDBoxVersion)
		{
		}

		virtual ~FMP4BoxXMLSubtitleSampleEntry()
		{
		}

		const TArray<uint8> GetRawBoxData() const
		{
			return BoxData;
		}

		int32 GetTranslationX() const
		{
			return TranslationX;
		}

		int32 GetTranslationY() const
		{
			return TranslationY;
		}

		uint16 GetWidth() const
		{
			return Width;
		}

		uint16 GetHeight() const
		{
			return Height;
		}

		uint32 GetTimescale() const
		{
			return Timescale;
		}

		const FString& GetNamespace() const
		{
			return Namespace;
		}
		const FString& GetSchemaLocation() const
		{
			return SchemaLocation;
		}
		const TArray<FString>& GetAuxiliaryMimeTypes() const
		{
			return AuxiliaryMimeTypes;
		}

	private:
		FMP4BoxXMLSubtitleSampleEntry() = delete;
		FMP4BoxXMLSubtitleSampleEntry(const FMP4BoxXMLSubtitleSampleEntry&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;

			// Duplicate all the data we are reading into a buffer that we need to provide to the subtitle plugin.
			FMP4BoxReader::ScopedDataDuplication DuplicateBoxData(ParseInfo->Reader(), BoxData);

			RETURN_IF_ERROR(FMP4BoxSampleEntry::ReadAndParseAttributes(ParseInfo));

			const FMP4BoxTKHD* TKHD = ParseInfo->GetCurrentTrackBox() ? static_cast<const FMP4BoxTKHD*>(ParseInfo->GetCurrentTrackBox()->GetBoxPath(FMP4Box::kBox_tkhd)) : nullptr;
			if (TKHD)
			{
				Width = TKHD->GetWidth();
				Height = TKHD->GetHeight();
				// Translation must not be using fractional digits and can thus be treated like integers.
				TranslationX = (int32) TKHD->GetMatrixValue(6);
				TranslationY = (int32) TKHD->GetMatrixValue(7);
			}

			const FMP4BoxMDHD* MDHD = ParseInfo->GetCurrentTrackBox() ? static_cast<const FMP4BoxMDHD*>(ParseInfo->GetCurrentTrackBox()->FindBox(FMP4Box::kBox_mdhd)) : nullptr;
			if (MDHD)
			{
				Timescale = MDHD->GetTimescale();
			}

			int32 BytesRemaining = (int32) (BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(Namespace, BytesRemaining));
			BytesRemaining = (int32) (BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(SchemaLocation, BytesRemaining));
			FString AuxMimes;
			BytesRemaining = (int32) (BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(AuxMimes, BytesRemaining));
			AuxMimes.ParseIntoArrayWS(AuxiliaryMimeTypes);
			BytesRemaining = (int32) (BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));

			// There may be child boxes here now, more precisely a 'btrt' box.
			if (BytesRemaining > 0)
			{
				// Now that there are additional boxes we ourselves are no longer a leaf box.
				bIsLeafBox = false;
				// This will read all the boxes in here and add them as children.
				for(; BytesRemaining>0; BytesRemaining = (int32) (BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset)))
				{
					IParserISO14496_12::FBoxType	ChildBoxType;
					int64							ChildBoxSize;
					uint8							ChildBoxUUID[16];

					int64 BoxStartOffset = ParseInfo->Reader()->GetCurrentReadOffset();
					RETURN_IF_ERROR(ParseInfo->ReadBoxTypeAndSize(ChildBoxType, ChildBoxSize, ChildBoxUUID));
#if MEDIA_DEBUG_HAS_BOX_NAMES
					char boxName[4];
					*((uint32*)&boxName) = MEDIA_TO_BIG_ENDIAN(ChildBoxType);
#endif
					int64 BoxDataOffset = ParseInfo->Reader()->GetCurrentReadOffset();
					FMP4Box* NextBox = new FMP4BoxIgnored(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);

					AddChildBox(NextBox);
					RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(NextBox));
				}
			}
			return Error;
		}

		TArray<uint8>		BoxData;
		FString				Namespace;
		FString				SchemaLocation;
		TArray<FString>		AuxiliaryMimeTypes;
		int32				STSDBoxVersion;
		uint32				Timescale = 0;
		uint16				Width = 0;
		uint16				Height = 0;
		int32				TranslationX = 0;
		int32				TranslationY = 0;
	};


	/**
	 * AVC Decoder Configuration box (ISO/IEC 14496-15:2014 - 5.4.2.1.2)
	 */
	class FMP4BoxAVCC : public FMP4BoxBasic
	{
	public:
		FMP4BoxAVCC(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxAVCC()
		{
		}

		const TOptional<ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord>& GetDecoderConfigurationRecord() const
		{
			return ConfigurationRecord;
		}

	private:
		FMP4BoxAVCC() = delete;
		FMP4BoxAVCC(const FMP4BoxAVCC&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				uint8* TempBuffer = (uint8*)FMemory::Malloc(BytesRemaining);
				if (!TempBuffer)
				{
					return UEMEDIA_ERROR_OOM;
				}
				Error = ParseInfo->Reader()->ReadBytes(TempBuffer, BytesRemaining);
				if (Error == UEMEDIA_ERROR_OK)
				{
					ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord dcr;
					if (dcr.Parse(MakeConstArrayView(TempBuffer, BytesRemaining)))
					{
						ConfigurationRecord = MoveTemp(dcr);
					}
				}
				FMemory::Free(TempBuffer);
			}
			return Error;
		}

		TOptional<ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord> ConfigurationRecord;
	};

	/**
	 * HEVC Decoder Configuration box (ISO/IEC 14496-15:2014 - 8.3.3.1.2)
	 */
	class FMP4BoxHVCC : public FMP4BoxBasic
	{
	public:
		FMP4BoxHVCC(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxHVCC()
		{
		}

		const TOptional<ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord>& GetDecoderConfigurationRecord() const
		{
			return ConfigurationRecord;
		}

	private:
		FMP4BoxHVCC() = delete;
		FMP4BoxHVCC(const FMP4BoxHVCC&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				uint8* TempBuffer = (uint8*)FMemory::Malloc(BytesRemaining);
				if (!TempBuffer)
				{
					return UEMEDIA_ERROR_OOM;
				}
				Error = ParseInfo->Reader()->ReadBytes(TempBuffer, BytesRemaining);
				if (Error == UEMEDIA_ERROR_OK)
				{
					ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord dcr;
					if (dcr.Parse(MakeConstArrayView(TempBuffer, BytesRemaining)))
					{
						ConfigurationRecord = MoveTemp(dcr);
					}
				}
				FMemory::Free(TempBuffer);
			}
			return Error;
		}

		TOptional<ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord> ConfigurationRecord;
	};


	/**
	 * MPEG-4 Sample Description Box containing an ISO/IEC 14496-1 8.6.5 ES_Descriptor (ISO/IEC 14496-14:2013 - 5.6)
	 */
	class FMP4BoxESDS : public FMP4BoxFull
	{
	public:
		FMP4BoxESDS(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxESDS()
		{
		}

		const MPEG::FESDescriptor& GetESDescriptor() const
		{
			return ESDescriptor;
		}


	private:
		FMP4BoxESDS() = delete;
		FMP4BoxESDS(const FMP4BoxESDS&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				void* TempBuffer = FMemory::Malloc(BytesRemaining);
				if (!TempBuffer)
				{
					return UEMEDIA_ERROR_OOM;
				}
				Error = ParseInfo->Reader()->ReadBytes(TempBuffer, BytesRemaining);
				if (Error == UEMEDIA_ERROR_OK)
				{
					ESDescriptor.SetRawData(TempBuffer, BytesRemaining);
				}
				FMemory::Free(TempBuffer);
			}
			return Error;
		}

		MPEG::FESDescriptor	ESDescriptor;
	};


	/**
	 * ETSI TS 102 366 V1.3.1 (2014-08) - F.4 AC3SpecificBox
	 */
	class FMP4BoxDAC3 : public FMP4BoxBasic
	{
	public:
		FMP4BoxDAC3(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxDAC3()
		{
		}

	private:
		FMP4BoxDAC3() = delete;
		FMP4BoxDAC3(const FMP4BoxDAC3&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;

			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				AC3SpecificBox.SetNumUninitialized(BytesRemaining);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(&AC3SpecificBox[0], BytesRemaining));
			}
			return Error;
		}

		TArray<uint8>		AC3SpecificBox;
	};


	/**
	 * ETSI TS 102 366 V1.3.1 (2014-08) - F.6 EC3SpecificBox
	 */
	class FMP4BoxDEC3 : public FMP4BoxBasic
	{
	public:
		FMP4BoxDEC3(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxDEC3()
		{
		}

	private:
		FMP4BoxDEC3() = delete;
		FMP4BoxDEC3(const FMP4BoxDEC3&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;

			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				EC3SpecificBox.SetNumUninitialized(BytesRemaining);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(&EC3SpecificBox[0], BytesRemaining));
			}
			return Error;
		}

		TArray<uint8>		EC3SpecificBox;
	};


	/**
	 * 'btrt' box.
	 * ISO/IEC 14496-12:2014 - 8.5.2.2 - Bitrate box (within 'stsd' box)
	 */
	class FMP4BoxBTRT : public FMP4BoxBasic
	{
	public:
		FMP4BoxBTRT(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, BufferSizeDB(0), MaxBitrate(0), AverageBitrate(0)
		{
		}

		virtual ~FMP4BoxBTRT()
		{
		}

		uint32 GetBufferSizeDB() const
		{
			return BufferSizeDB;
		}
		uint32 GetMaxBitrate() const
		{
			return MaxBitrate;
		}
		uint32 GetAverageBitrate() const
		{
			return AverageBitrate;
		}

	private:
		FMP4BoxBTRT() = delete;
		FMP4BoxBTRT(const FMP4BoxBTRT&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(BufferSizeDB));				// bufferSizeDB
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(MaxBitrate));					// maxBitrate
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(AverageBitrate));				// avgBitrate
			return Error;
		}

	private:
		uint32		BufferSizeDB;
		uint32		MaxBitrate;
		uint32		AverageBitrate;
	};


	/**
	 * 'pasp' box.
	 * ISO/IEC 14496-12:2014 - 12.1.4 - Pixel Aspect Ratio box (within 'stsd' box)
	 */
	class FMP4BoxPASP : public FMP4BoxBasic
	{
	public:
		FMP4BoxPASP(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, HSpacing(0), VSpacing(0)
		{
		}

		virtual ~FMP4BoxPASP()
		{
		}
		uint32 GetHSpacing() const
		{
			return HSpacing;
		}
		uint32 GetVSpacing() const
		{
			return VSpacing;
		}

	private:
		FMP4BoxPASP() = delete;
		FMP4BoxPASP(const FMP4BoxPASP&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(HSpacing));				// hSpacing
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(VSpacing));				// vSpacing
			return Error;
		}

	private:
		uint32		HSpacing;
		uint32		VSpacing;
	};


	/**
	 * 'elng' box.
	 * ISO/IEC 14496-12:2015 - 8.4.6 - Extended language tag (within 'mdia' box)
	 */
	class FMP4BoxELNG : public FMP4BoxFull
	{
	public:
		FMP4BoxELNG(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxELNG()
		{
		}

		const BCP47::FLanguageTag& GetLanguageTag() const
		{
			return LanguageTag;
		}

	private:
		FMP4BoxELNG() = delete;
		FMP4BoxELNG(const FMP4BoxELNG&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(ExtendedLanguage, BytesRemaining, true));
			/*bool bOk =*/ BCP47::ParseRFC5646Tag(LanguageTag, ExtendedLanguage);
			return Error;
		}

	private:
		BCP47::FLanguageTag LanguageTag;
		FString ExtendedLanguage;
	};


	/**
	 * 'prft' box.
	 * ISO/IEC 14496-12:2015 - 8.16.5 - Producer Reference Time Box
	 */
	class FMP4BoxPRFT : public FMP4BoxFull
	{
	public:
		FMP4BoxPRFT(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, ReferenceTrackID(0), MediaTime(0)
		{
		}

		virtual ~FMP4BoxPRFT()
		{
		}

		uint32 GetReferenceTrackID() const
		{
			return ReferenceTrackID;
		}

		uint64 GetNtpTimestamp() const
		{
			return NtpTimestamp;
		}

		uint64 GetMediaTime() const
		{
			return MediaTime;
		}

	private:
		FMP4BoxPRFT() = delete;
		FMP4BoxPRFT(const FMP4BoxPRFT&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(ReferenceTrackID));		// reference_track_ID
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NtpTimestamp));			// ntp_timestamp
			// Which version of the box is this?
			if (Version == 0)
			{
				uint32 Value32 = 0;
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));			// media_time
				MediaTime = Value32;
			}
			else
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(MediaTime));			// media_time
			}
			return Error;
		}

	private:
		uint32 ReferenceTrackID;
		uint64 NtpTimestamp;
		uint64 MediaTime;
	};


	/**
	 * 'emsg' box.
	 * ISO/IEC 23009-1:2019 - 5.10.3.3 Event Message Box
	 */
	class FMP4BoxEMSG : public FMP4BoxFull, public IParserISO14496_12::IEventMessage
	{
	public:
		FMP4BoxEMSG(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, Timescale(0), PresentationTimeDelta(0), PresentationTime(0), EventDuration(0), ID(0)
		{
		}

		virtual ~FMP4BoxEMSG()
		{
		}

		int32 GetVersion() const override
		{
			return Version;
		}
		const FString& GetSchemeIdUri() const override
		{
			return SchemeIdUri;
		}
		const FString& GetValue() const override
		{
			return Value;
		}
		uint32 GetTimescale() const override
		{
			return Timescale;
		}
		uint32 GetPresentationTimeDelta() const override
		{
			return PresentationTimeDelta;
		}
		uint64 GetPresentationTime() const override
		{
			return PresentationTime;
		}
		uint32 GetEventDuration() const override
		{
			return EventDuration;
		}
		uint32 GetID() const override
		{
			return ID;
		}
		const TArray<uint8>& GetMessageData() const override
		{
			return MessageData;
		}

	private:
		FMP4BoxEMSG() = delete;
		FMP4BoxEMSG(const FMP4BoxEMSG&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			// Which version of the box is this?
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (Version == 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(SchemeIdUri, BytesRemaining));						// scheme_id_uri
				BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(Value, BytesRemaining));							// value
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Timescale));												// timescale
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(PresentationTimeDelta));									// presentation_time_delta
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(EventDuration));											// event_duration
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(ID));														// id
			}
			else if (Version == 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Timescale));												// timescale
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(PresentationTime));										// presentation_time
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(EventDuration));											// event_duration
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(ID));														// id
				BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(SchemeIdUri, BytesRemaining));						// scheme_id_uri
				BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(Value, BytesRemaining));							// value
			}
			else
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}
			BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				MessageData.AddUninitialized(BytesRemaining);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(MessageData.GetData(), BytesRemaining));				// message_data
			}
			return Error;
		}
	private:
		TArray<uint8> MessageData;
		FString SchemeIdUri;
		FString Value;
		uint32 Timescale;
		uint32 PresentationTimeDelta;
		uint64 PresentationTime;
		uint32 EventDuration;
		uint32 ID;
	};


	/**
	 * 'stsd' box.
	 * ISO/IEC 14496-12:2014 - 8.5.2 - Sample Description Box
	 */
	class FMP4BoxSTSD : public FMP4BoxFull
	{
	public:
		FMP4BoxSTSD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxSTSD()
		{
		}

		enum class ESampleType
		{
			Video,
			Audio,
			Subtitles,
			Timecode,
			Ignored
		};

		ESampleType GetSampleType() const
		{
			return SampleType;
		}

		bool IsSupportedFormat() const
		{
			return SampleType != ESampleType::Ignored;
		}

	private:
		FMP4BoxSTSD() = delete;
		FMP4BoxSTSD(const FMP4BoxSTSD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			// Now that we parse this box we clear its leaf status. We call back into the generic box reader
			// for which this flag must be clear to continue with the next boxes.
			bIsLeafBox = false;
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			// To parse this box correctly we need to know the track or media handler.
			if (ParseInfo->GetCurrentHandlerBox() == nullptr && ParseInfo->GetCurrentMediaHandlerBox() == nullptr)
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}
			uint32 EntryCount;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(EntryCount));					// entry_count

			SampleType = ESampleType::Ignored;
			// Check for media handler first.
			if (ParseInfo->GetCurrentMediaHandlerBox())
			{
				switch(ParseInfo->GetCurrentMediaHandlerBox()->GetType())
				{
					case FMP4Box::kBox_vmhd:
						SampleType = ESampleType::Video;
						break;
					case FMP4Box::kBox_smhd:
						SampleType = ESampleType::Audio;
						break;
					case FMP4Box::kBox_nmhd:
					case FMP4Box::kBox_sthd:
						SampleType = ESampleType::Subtitles;
						break;
					case FMP4Box::kBox_gmhd:
						// Remark: Could parse gmhd box, check if it has 'tmcd' box to make sure it is a timecode.
						// For now assume it is a timecode, but will expect 'tmcd' sample description below (and ignore if not).
						SampleType = ParseInfo->ShouldParseTimecodeInfo() ? ESampleType::Timecode : ESampleType::Ignored;
						break;
					default:
						SampleType = ESampleType::Ignored;
						break;
				}
			}
			else
			{
				const FMP4BoxHDLR* CurrentHandler = static_cast<FMP4BoxHDLR*>(ParseInfo->GetCurrentHandlerBox());
				switch(CurrentHandler->GetHandlerType())
				{
					case FMP4Box::kTrack_vide:
						SampleType = ESampleType::Video;
						break;
					case FMP4Box::kTrack_soun:
						SampleType = ESampleType::Audio;
						break;
					default:
						SampleType = ESampleType::Ignored;
						break;
				}
			}

			for(uint32 i=0; i<EntryCount; ++i)
			{
				IParserISO14496_12::FBoxType	ChildBoxType;
				int64							ChildBoxSize;
				uint8							ChildBoxUUID[16];

				int64 BoxStartOffset = ParseInfo->Reader()->GetCurrentReadOffset();
				RETURN_IF_ERROR(ParseInfo->ReadBoxTypeAndSize(ChildBoxType, ChildBoxSize, ChildBoxUUID));
#if MEDIA_DEBUG_HAS_BOX_NAMES
				char boxName[4];
				*((uint32*)&boxName) = MEDIA_TO_BIG_ENDIAN(ChildBoxType);
#endif
				int64 BoxDataOffset = ParseInfo->Reader()->GetCurrentReadOffset();
				FMP4Box* NextBox = nullptr;
				switch(SampleType)
				{
					case ESampleType::Video:
					{
						NextBox = new FMP4BoxVisualSampleEntry(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);
						break;
					}
					case ESampleType::Audio:
					{
						NextBox = new FMP4BoxAudioSampleEntry(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true, Version);
						break;
					}
					case ESampleType::Subtitles:
					{
						if (ChildBoxType == FMP4Box::kSample_tx3g)
						{
							NextBox = new FMP4BoxTX3GSampleEntry(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true, Version);
						}
						else if (ChildBoxType == FMP4Box::kSample_wvtt)
						{
							NextBox = new FMP4BoxWVTTSampleEntry(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true, Version);
						}
						else if (ChildBoxType == FMP4Box::kSample_stpp)
						{
							NextBox = new FMP4BoxXMLSubtitleSampleEntry(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true, Version);
						}
						else
						{
							NextBox = new FMP4BoxIgnored(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);
						}
						break;
						/*
						// Subtitles, captions and subpictures (bitmap subtitles) are ignored for now.
						case FMP4Box::kTrack_subt:
						case FMP4Box::kTrack_sbtl:
						case FMP4Box::kTrack_subp:
							NextBox = new FMP4BoxIgnored(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						// Hint, metadata and text (ie. chapter names) are ignored.
						case FMP4Box::kTrack_hint:
						case FMP4Box::kTrack_meta:
						case FMP4Box::kTrack_text:
							NextBox = new FMP4BoxIgnored(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						*/
					}
					case ESampleType::Timecode:
					{
						if (ChildBoxType == FMP4Box::kSample_tmcd)
						{
							NextBox = new FMP4BoxTimecodeSampleEntry(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);
						}
						else
						{
							SampleType = ESampleType::Ignored; // Ignoring non-tmcd samples.
							NextBox = new FMP4BoxIgnored(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);
						}
						break;
					}
					default:
					{
						NextBox = new FMP4BoxIgnored(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);
						break;
					}
				}
				AddChildBox(NextBox);
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(NextBox));
				// Where are we at now?
				int64 BoxOffsetAfterParse = ParseInfo->Reader()->GetCurrentReadOffset();
				int64 BoxBytesParsed = BoxOffsetAfterParse - BoxStartOffset;
				int64 BoxBytesNotParsed = ChildBoxSize - BoxBytesParsed;
				if (BoxBytesNotParsed > 0)
				{
					RETURN_IF_ERROR(ReadFillerData(ParseInfo, BoxBytesNotParsed));
				}
				else if (BoxBytesNotParsed < 0)
				{
					// We somehow consumed more bytes than the box was set to have.
					check(!"How did this happen?");
					return UEMEDIA_ERROR_FORMAT_ERROR;
				}

				if (BoxOffsetAfterParse >= GetStartOffset() + GetSize())
				{
					return Error;
				}
			}

			return Error;
		}

	private:
		ESampleType SampleType = ESampleType::Ignored;
	};


	/**
	 * 'stts' box.
	 * ISO/IEC 14496-12:2014 - 8.6.1.2 - Decoding Time to Sample Box
	 */
	class FMP4BoxSTTS : public FMP4BoxFull
	{
	public:
		FMP4BoxSTTS(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, NumTotalSamples(0), TotalDuration(0)
		{
		}

		virtual ~FMP4BoxSTTS()
		{
		}

		struct FEntry
		{
			uint32		SampleCount;
			uint32		SampleDelta;
		};

		int64 GetNumTotalSamples() const
		{
			return NumTotalSamples;
		}

		int32 GetNumberOfEntries() const
		{
			return Entries.Num();
		}

		const FEntry& GetEntry(int32 EntryIndex) const
		{
			return Entries[EntryIndex];
		}

	private:
		FMP4BoxSTTS() = delete;
		FMP4BoxSTTS(const FMP4BoxSTTS&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			uint32 NumEntries = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumEntries));					// entry_count
			if (NumEntries)
			{
				// FIXME: To conserve a bit of memory we could separate 'sample_count' and 'sample_delta' into two arrays
				//        that hold either 8, 16 or 32 bit values. The sample delta tends be be small since it is measured
				//        in media timescale units.
				Entries.Reserve(NumEntries);
				uint32 c = 0, d = 0;
				for(uint32 i = 0; i < NumEntries; ++i)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(c));					// sample_count
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(d));					// sample_delta
					FEntry& e = Entries.AddDefaulted_GetRef();
					e.SampleCount = c;
					e.SampleDelta = d;
					// Update total counts for convenience.
					NumTotalSamples += c;
					TotalDuration += d * c;
				}
			}
			return Error;
		}

	private:
		TArray<FEntry>					Entries;
		int64							NumTotalSamples;
		int64							TotalDuration;
	};


	/**
	 * 'ctts' box.
	 * ISO/IEC 14496-12:2014 - 8.6.1.3 - Composition Time to Sample Box
	 */
	class FMP4BoxCTTS : public FMP4BoxFull
	{
	public:
		FMP4BoxCTTS(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, NumTotalSamples(0)
		{
		}

		virtual ~FMP4BoxCTTS()
		{
		}

		struct FEntry
		{
			uint32		SampleCount;
			int64		SampleOffset;
		};

		int32 GetNumberOfEntries() const
		{
			return Entries.Num();
		}

		const FEntry GetEntry(int32 EntryIndex) const
		{
			FEntry e;
			e.SampleCount = Entries[EntryIndex].SampleCount;
			// For box version 0 return the unsigned value, otherwise convert the unsigned to signed (first to 32, then to 64 bits)
//			e.SampleOffset = Version == 0 ? (int64)Entries[EntryIndex].SampleOffset : (int64)((int32)Entries[EntryIndex].SampleOffset);
		/*
			Note: We treat this box unconditionally as if it were version 1 using signed composition offsets.
			      This addresses the issue that under the QuickTime brand this box is always using signed integers
				  even in version 0.
				  It stands to reason that no video will actually require unsigned values >0x7fffffff even if using
				  HNS as timescale for which 0x7fffffff would still give about 214 seconds...
		*/
			e.SampleOffset = (int64)((int32)Entries[EntryIndex].SampleOffset);
			return e;
		}

	private:
		FMP4BoxCTTS() = delete;
		FMP4BoxCTTS(const FMP4BoxCTTS&) = delete;

	protected:
		struct InternalEntry
		{
			uint32		SampleCount;
			uint32		SampleOffset;
		};
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			uint32 NumEntries = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumEntries));					// entry_count
			if (NumEntries)
			{
				// FIXME: To conserve a bit of memory we could separate 'sample_count' and 'sample_offset' into two arrays
				//        that hold either 8, 16 or 32 bit values.
				//        The sample_offset here however is not in media track timescale units and can get large.
				//        NOTE: Care must be taken though with version 1 of this box that allows for negative offsets.
				//              Compression into fewer than 32 bits needs to account for signedness!
				Entries.Reserve(NumEntries);
				uint32 c = 0;
				uint32 o = 0;
				for(uint32 i = 0; i < NumEntries; ++i)
				{
					// NOTE: We always read the offset as unsigned as if the box is always version 0.
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(c));					// sample_count
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(o));					// sample_offset
					InternalEntry& e = Entries.AddDefaulted_GetRef();
					e.SampleCount = c;
					e.SampleOffset = o;
					// Update total count for convenience.
					NumTotalSamples += c;
				}
			}
			return Error;
		}

	private:
		TArray<InternalEntry>					Entries;
		int64									NumTotalSamples;
	};


	/**
	 * 'stss' box.
	 * ISO/IEC 14496-12:2014 - 8.6.2 - Sync Sample Box
	 */
	class FMP4BoxSTSS : public FMP4BoxFull
	{
	public:
		FMP4BoxSTSS(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxSTSS()
		{
		}

		int32 GetNumberOfEntries() const
		{
			return Entries.Num();
		}

		uint32 GetEntry(int32 EntryIndex) const
		{
			return Entries[EntryIndex];
		}
	private:
		FMP4BoxSTSS() = delete;
		FMP4BoxSTSS(const FMP4BoxSTSS&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			uint32 NumEntries = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumEntries));					// entry_count
			if (NumEntries)
			{
				Entries.Reserve(NumEntries);
				uint32 n = 0;
				for(uint32 i = 0; i < NumEntries; ++i)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(n));					// sample_number
					Entries.Push(n);
				}
			}
			return Error;
		}

	private:
		TArray<uint32>	Entries;
	};


	/**
	 * 'stsc' box.
	 * ISO/IEC 14496-12:2014 - 8.7.4 - Sample to Chunk Box
	 */
	class FMP4BoxSTSC : public FMP4BoxFull
	{
	public:
		FMP4BoxSTSC(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxSTSC()
		{
		}

		struct FEntry
		{
			uint32		FirstChunk;
			uint32		SamplesPerChunk;
			uint32		SampleDescriptionIndex;
		};

		int32 GetNumberOfEntries() const
		{
			return Entries.Num();
		}

		const FEntry& GetEntry(int32 EntryIndex) const
		{
			return Entries[EntryIndex];
		}

	private:
		FMP4BoxSTSC() = delete;
		FMP4BoxSTSC(const FMP4BoxSTSC&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			uint32 NumEntries = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumEntries));					// entry_count
			if (NumEntries)
			{
				// FIXME: To conserve a bit of memory we could separate the entry elements into three separate arrays
				//        that hold either 8, 16 or 32 bit values. The SampleDescriptionIndex tends to be constant and
				//        is otherwise a small number since it is the index into the stsd table of the sample format
				//        which is unlikely to contain a large number.
				//        SamplesPerChunk is usually a small number. FirstChunk cannot exceed the number of entries
				//        in stco/co64 but that box comes later. Can be a large number though. These two depend on
				//        the sample interleaving strategy of the muxer and the relative sample durations.
				Entries.Reserve(NumEntries);
				uint32 FirstChunk = 0, SamplesPerChunk = 0, DescriptionIndex = 0;
				for(uint32 i = 0; i < NumEntries; ++i)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(FirstChunk));			// first_chunk
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(SamplesPerChunk));	// samples_per_chunk
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(DescriptionIndex));	// sample_description_index
					FEntry& e = Entries.AddDefaulted_GetRef();
					e.FirstChunk = FirstChunk;
					e.SamplesPerChunk = SamplesPerChunk;
					e.SampleDescriptionIndex = DescriptionIndex;
				}
			}
			return Error;
		}

	private:
		TArray<FEntry>	Entries;
	};


	/**
	 * 'stsz' box.
	 * ISO/IEC 14496-12:2014 - 8.7.3.2 - Sample Size Box
	 */
	class FMP4BoxSTSZ : public FMP4BoxFull
	{
	public:
		FMP4BoxSTSZ(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, NumEntries(0), SampleSize(0)
		{
		}

		virtual ~FMP4BoxSTSZ()
		{
		}

		int32 GetNumTotalSamples() const
		{
			// It is extremely unlikely there'd be more than 2^31-1 entries.
			check(NumEntries <= 0x7fffffff);
			return (int32)NumEntries;
		}

		uint32 GetSampleSize(int32 SampleIndex) const
		{
			return SampleSize == 0 ? Entries[SampleIndex] : SampleSize;
		}

	private:
		FMP4BoxSTSZ() = delete;
		FMP4BoxSTSZ(const FMP4BoxSTSZ&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleSize));					// sample_size
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumEntries));					// sample_count
			if (SampleSize == 0 && NumEntries != 0)
			{
				Entries.Reserve(NumEntries);
				uint32 n = 0;
				for(uint32 i = 0; i < NumEntries; ++i)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(n));					// entry_size
					Entries.Push(n);
				}
			}
			return Error;
		}

	private:
		TArray<uint32>					Entries;
		uint32							NumEntries;
		uint32							SampleSize;
	};


	/**
	 * 'stco' box, 'co64' box.
	 * ISO/IEC 14496-12:2014 - 8.7.5 - Chunk Offset Box
	 */
	class FMP4BoxSTCO : public FMP4BoxFull
	{
	public:
		FMP4BoxSTCO(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, NumEntries(0)
		{
		}

		virtual ~FMP4BoxSTCO()
		{
		}

		int32 GetNumberOfEntries() const
		{
			return (int32)NumEntries;
		}

		int64 GetEntry(int32 EntryIndex) const
		{
			return Entries32.Num() ? (int64)Entries32[EntryIndex] : (int64)Entries64[EntryIndex];
		}

	private:
		FMP4BoxSTCO() = delete;
		FMP4BoxSTCO(const FMP4BoxSTCO&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(NumEntries));					// entry_count
			if (NumEntries != 0)
			{
				if (GetType() == FMP4Box::kBox_stco)
				{
					Entries32.Reserve(NumEntries);
					uint32 n = 0;
					for(uint32 i = 0; i < NumEntries; ++i)
					{
						RETURN_IF_ERROR(ParseInfo->Reader()->Read(n));				// chunk_offset
						Entries32.Push(n);
					}
				}
				else if (GetType() == FMP4Box::kBox_co64)
				{
					Entries64.Reserve(NumEntries);
					uint64 n = 0;
					for(uint32 i = 0; i < NumEntries; ++i)
					{
						RETURN_IF_ERROR(ParseInfo->Reader()->Read(n));				// chunk_offset
						Entries64.Push(n);
					}
				}
				else
				{
					return UEMEDIA_ERROR_FORMAT_ERROR;
				}
			}
			return Error;
		}

	private:
		TArray<uint64>					Entries64;
		TArray<uint32>					Entries32;
		uint32							NumEntries;
	};


	/**
	 * 'sidx' box.
	 * ISO/IEC 14496-12:2014 - 8.16.3 - Segment Index Box
	 */
	class FMP4BoxSIDX : public FMP4BoxFull, public IParserISO14496_12::ISegmentIndex
	{
	public:
		FMP4BoxSIDX(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, EarliestPresentationTime(0), FirstOffset(0), ReferenceID(0), Timescale(0)
		{
		}

		virtual ~FMP4BoxSIDX()
		{
		}

		// ----------------------------------------------------------------
		// Methods from IParserISO14496_12::ISegmentIndex
		//
		uint64 GetEarliestPresentationTime() const override
		{ return EarliestPresentationTime; }
		uint64 GetFirstOffset() const override
		{ return FirstOffset; }
		uint32 GetReferenceID() const override
		{ return ReferenceID; }
		uint32 GetTimescale() const override
		{ return Timescale; }
		int32 GetNumEntries() const override
		{ return Entries.Num(); }
		virtual const FEntry& GetEntry(int32 Index) const
		{
			static FEntry EmptyEntry = {0};
			return Index < Entries.Num() ? Entries[Index] : EmptyEntry;
		}

	private:
		FMP4BoxSIDX() = delete;
		FMP4BoxSIDX(const FMP4BoxSIDX&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(ReferenceID));					// reference_ID
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Timescale));						// timescale
			if (Version == 0)
			{
				uint32	ept = 0, fo = 0;
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(ept));						// earliest_presentation_time
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(fo));							// first_offset
				EarliestPresentationTime = ept;
				FirstOffset = fo;
			}
			else // Version != 0 (NOTE: Yes, this is how the standard defines version checking)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(EarliestPresentationTime));	// earliest_presentation_time
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(FirstOffset));				// first_offset
			}

			uint16 Value16 = 0, ReferenceCount = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value16));						// reserved
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(ReferenceCount));					// reference_count

			if (ReferenceCount)
			{
				Entries.Reserve(ReferenceCount);
				for(uint32 i = 0; i < ReferenceCount; ++i)
				{
					uint32 ReferenceTypeAndSize;
					uint32 SubSegmentDuration;
					uint32 SAPStartAndTypeAndDeltaTime;
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(ReferenceTypeAndSize));			// 31: reference_type, 30-0: referenced size
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(SubSegmentDuration));				// subsegment_duration
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(SAPStartAndTypeAndDeltaTime));	// 31: starts_with_SAP, 30-28: SAP_type; 27-0: SAP_delta_time

					IParserISO14496_12::ISegmentIndex::FEntry e;
					e.SubSegmentDuration = SubSegmentDuration;
					e.IsReferenceType = ReferenceTypeAndSize >> 31;
					e.Size = ReferenceTypeAndSize & 0x7fffffff;
					e.StartsWithSAP  = SAPStartAndTypeAndDeltaTime >> 31;
					e.SAPType = (SAPStartAndTypeAndDeltaTime >> 28) & 7;
					e.SAPDeltaTime = SAPStartAndTypeAndDeltaTime & 0x0fffffffU;
					Entries.AddElement(MoveTemp(e));
				}
			}
			return Error;
		}

	private:
		// At 12 bytes per entry we get 250 per chunk.
		TChunkedArray<IParserISO14496_12::ISegmentIndex::FEntry, 3000>	Entries;
		uint64															EarliestPresentationTime;
		uint64															FirstOffset;
		uint32															ReferenceID;
		uint32															Timescale;
	};


	/**
	 * 'mfhd' box. ISO/IEC 14496-12:2014 - 8.8.5 - Movie Fragment Header Box
	 */
	class FMP4BoxMFHD : public FMP4BoxFull
	{
	public:
		FMP4BoxMFHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, SequenceNumber(0)
		{
		}

		virtual ~FMP4BoxMFHD()
		{
		}

	private:
		FMP4BoxMFHD() = delete;
		FMP4BoxMFHD(const FMP4BoxMFHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(SequenceNumber));				// sequence_number
			return Error;
		}

	private:
		uint32		SequenceNumber;
	};


	/**
	 * 'tfhd' box. ISO/IEC 14496-12:2014 - 8.8.7 - Track Fragment Header Box
	 */
	class FMP4BoxTFHD : public FMP4BoxFull
	{
	public:
		FMP4BoxTFHD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, BaseDataOffset(0), TrackID(0), SampleDescriptionIndex(0), DefaultSampleDuration(0), DefaultSampleSize(0), DefaultSampleFlags(0)
		{
		}

		virtual ~FMP4BoxTFHD()
		{
		}

		uint32 GetTrackID() const
		{
			return TrackID;
		}

		bool HasBaseDataOffset() const
		{
			return (Flags & 0x000001) != 0;
		}
		uint64 GetBaseDataOffset() const
		{
			return BaseDataOffset;
		}

		bool HasSampleDescriptionIndex() const
		{
			return (Flags & 0x000002) != 0;
		}
		uint32 GetSampleDescriptionIndex() const
		{
			return SampleDescriptionIndex;
		}

		bool HasDefaultSampleDuration() const
		{
			return (Flags & 0x000008) != 0;
		}
		uint32 GetDefaultSampleDuration() const
		{
			return DefaultSampleDuration;
		}

		bool HasDefaultSampleSize() const
		{
			return (Flags & 0x000010) != 0;
		}
		uint32 GetDefaultSampleSize() const
		{
			return DefaultSampleSize;
		}

		bool HasDefaultSampleFlags() const
		{
			return (Flags & 0x000020) != 0;
		}
		uint32 GetDefaultSampleFlags() const
		{
			return DefaultSampleFlags;
		}

		bool IsDurationEmpty() const
		{
			return (Flags & 0x010000) != 0;
		}

		bool IsMoofDefaultBase() const
		{
			return (Flags & 0x020000) != 0;
		}

	private:
		FMP4BoxTFHD() = delete;
		FMP4BoxTFHD(const FMP4BoxTFHD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(TrackID));					// track_ID
			if ((Flags & 1) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(BaseDataOffset));			// base_data_offset
			}
			if ((Flags & 2) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleDescriptionIndex));	// sample_description_index
			}
			if ((Flags & 8) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleDuration));	// default_sample_duration
			}
			if ((Flags & 16) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleSize));		// default_sample_size
			}
			if ((Flags & 32) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleFlags));		// default_sample_flags
			}
			return Error;
		}

	private:
		uint64		BaseDataOffset;
		uint32		TrackID;
		uint32		SampleDescriptionIndex;
		uint32		DefaultSampleDuration;
		uint32		DefaultSampleSize;
		uint32		DefaultSampleFlags;
	};


	/**
	 * 'tfdt' box. ISO/IEC 14496-12:2014 - 8.8.12 - Track Fragment decode time
	 */
	class FMP4BoxTFDT : public FMP4BoxFull
	{
	public:
		FMP4BoxTFDT(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, BaseMediaDecodeTime(0)
		{
		}

		virtual ~FMP4BoxTFDT()
		{
		}

		uint64 GetBaseMediaDecodeTime() const
		{
			return BaseMediaDecodeTime;
		}

	private:
		FMP4BoxTFDT() = delete;
		FMP4BoxTFDT(const FMP4BoxTFDT&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			if (Version == 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(BaseMediaDecodeTime));	// baseMediaDecodeTime
			}
			else
			{
				uint32 Value32 = 0;
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));				// baseMediaDecodeTime
				BaseMediaDecodeTime = Value32;
			}
			return Error;
		}

	private:
		uint64		BaseMediaDecodeTime;
	};


	/**
	 * 'trun' box. ISO/IEC 14496-12:2014 - 8.8.8 - Track Fragment Run Box
	 */
	class FMP4BoxTRUN : public FMP4BoxFull
	{
	public:
		FMP4BoxTRUN(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, SampleCount(0), DataOffset(0), FirstSampleFlags(0)
		{
		}

		virtual ~FMP4BoxTRUN()
		{
		}

		uint32 GetNumberOfSamples() const
		{
			return SampleCount;
		}

		bool HasSampleOffset() const
		{
			return (Flags & 0x000001) != 0;
		}
		int32 GetSampleOffset() const
		{
			return DataOffset;
		}

		bool HasFirstSampleFlags() const
		{
			return (Flags & 0x000004) != 0;
		}
		uint32 GetFirstSampleFlags() const
		{
			return FirstSampleFlags;
		}

		bool HasSampleDurations() const
		{
			return (Flags & 0x000100) != 0;
		}
		const TArray<uint32>& GetSampleDurations() const
		{
			return SampleDurations;
		}

		bool HasSampleSizes() const
		{
			return (Flags & 0x000200) != 0;
		}
		const TArray<uint32>& GetSampleSizes() const
		{
			return SampleSizes;
		}

		bool HasSampleFlags() const
		{
			return (Flags & 0x000400) != 0;
		}
		const TArray<uint32>& GetSampleFlags() const
		{
			return SampleFlags;
		}

		bool HasSampleCompositionTimeOffsets() const
		{
			return (Flags & 0x000800) != 0;
		}
		const TArray<int32>& GetSampleCompositionTimeOffsets() const
		{
			return SampleCompositionTimeOffsets;
		}

	private:
		FMP4BoxTRUN() = delete;
		FMP4BoxTRUN(const FMP4BoxTRUN&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleCount));				// sample_count
			if ((Flags & 1) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(DataOffset));				// data_offset
			}
			if ((Flags & 4) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(FirstSampleFlags));		// first_sample_flags
			}
			if (SampleCount)
			{
				if (Flags & 0x100)
				{
					SampleDurations.SetNumUninitialized(SampleCount);
				}
				if (Flags & 0x200)
				{
					SampleSizes.SetNumUninitialized(SampleCount);
				}
				if (Flags & 0x400)
				{
					SampleFlags.SetNumUninitialized(SampleCount);
				}
				if (Flags & 0x800)
				{
					SampleCompositionTimeOffsets.SetNumUninitialized(SampleCount);
				}

				for(uint32 i = 0; i < SampleCount; ++i)
				{
					if (Flags & 0x100)
					{
						RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleDurations[i]));	// sample_duration
					}
					if (Flags & 0x200)
					{
						RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleSizes[i]));		// sample_size
					}
					if (Flags & 0x400)
					{
						RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleFlags[i]));		// sample_flags
					}
					if (Flags & 0x800)
					{
						if (Version == 0)
						{
							uint32 Value32 = 0;
							RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));		// sample_composition_time_offset
							// FIXME: Because we want to handle only signed time offsets we check if the value can actually be
							//        presented as such. If not then the first question would be why the value is that large,
							//        which could indicate a bad file. If that is legitimate then we would need to change
							//        our SampleCompositionTimeOffsets to be an int64 table.
							if (Value32 > 0x7fffffffU)
							{
								UE_LOGF(LogElectraMP4Parser, Error, "WARNING: ElectraPlayer/FMP4BoxTRUN::ReadAndParseAttributes(): Version 0 time value cannot be represented as a signed value");
								return UEMEDIA_ERROR_FORMAT_ERROR;
							}
							SampleCompositionTimeOffsets[i] = (int32)Value32;
						}
						else
						{
							RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleCompositionTimeOffsets[i]));	// sample_composition_time_offset
						}
					}
				}
			}
			return Error;
		}

	private:
		uint32							SampleCount;
		int32							DataOffset;
		uint32							FirstSampleFlags;
		TArray<uint32>					SampleDurations;
		TArray<uint32>					SampleSizes;
		TArray<uint32>					SampleFlags;
		TArray<int32>					SampleCompositionTimeOffsets;
	};




	/**
	 * 'pssh' box. ISO/IEC 23001-7:2016 - 8.1 - Protection system specific header box
	 */
	class FMP4BoxPSSH : public FMP4BoxFull
	{
	public:
		FMP4BoxPSSH(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxPSSH()
		{
		}

		const TArray<uint8>& GetBoxData()  const
		{
			return BoxData;
		}
		const TArray<uint8>& GetSystemID() const
		{
			return SystemID;
		}
		const TArray<TArray<uint8>>& GetKIDs() const
		{
			return KIDs;
		}
		const TArray<uint8>& GetData() const
		{
			return Data;
		}
	private:
		FMP4BoxPSSH() = delete;
		FMP4BoxPSSH(const FMP4BoxPSSH&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			// We need to retain the box as a whole since some DRM systems need it to be passed in verbatim.
			// In order to do that we read it into a buffer first, which can be handed out later if needed
			// and parse the box from the buffer.
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			BoxData.AddUninitialized(8 + BytesRemaining);
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(BoxData.GetData() + 8, BytesRemaining));
			uint32* RawBox = (uint32*)BoxData.GetData();
			RawBox[0] = Utils::ValueToBigEndian((uint32)(BytesRemaining+8));
			RawBox[1] = Utils::ValueToBigEndian(kBox_pssh);

			FDataBufferReader BufferReader(BoxData);
			FMP4BoxReader Reader(&BufferReader, nullptr);
			Reader.ReadBytes(nullptr, 8);

			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(Reader));

			SystemID.AddUninitialized(16);
			RETURN_IF_ERROR(Reader.ReadBytes(SystemID.GetData(), 16));		// SystemID
			if (Version > 0)
			{
				uint32 KID_count = 0;
				RETURN_IF_ERROR(Reader.Read(KID_count));					// KID_count
				for(uint32 i=0; i<KID_count; ++i)
				{
					TArray<uint8>& kid = KIDs.AddDefaulted_GetRef();
					kid.AddUninitialized(16);
					RETURN_IF_ERROR(Reader.ReadBytes(kid.GetData(), 16));	// KID
				}
			}
			uint32 DataSize = 0;
			RETURN_IF_ERROR(Reader.Read(DataSize));							// DataSize
			if (DataSize)
			{
				Data.AddUninitialized(DataSize);
				RETURN_IF_ERROR(Reader.ReadBytes(Data.GetData(), DataSize));// Data
			}
			return Error;
		}
	private:
		TArray<uint8>			BoxData;
		TArray<uint8>			SystemID;
		TArray<TArray<uint8>>	KIDs;
		TArray<uint8>			Data;
	};


	/**
	 * 'sinf' box. ISO/IEC 14496-12:2015 - 8.12.1 Protection Scheme Information Box
	 */
	class FMP4BoxSINF : public FMP4BoxBasic
	{
	public:
		FMP4BoxSINF(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxSINF()
		{
		}

	private:
		FMP4BoxSINF() = delete;
		FMP4BoxSINF(const FMP4BoxSINF&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			// Now that we parse this box we clear its leaf status. We call back into the generic box reader
			// for which this flag must be clear to continue with the next boxes.
			bIsLeafBox = false;
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this));
			return Error;
		}
	};


	/**
	 * 'frma' box. ISO/IEC 14496-12:2015 - 8.12.2 Original Format Box
	 */
	class FMP4BoxFRMA : public FMP4BoxBasic
	{
	public:
		FMP4BoxFRMA(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, DataFormat(0)
		{
		}

		virtual ~FMP4BoxFRMA()
		{
		}

		uint32 GetDataFormat() const
		{
			return DataFormat;
		}

	private:
		FMP4BoxFRMA() = delete;
		FMP4BoxFRMA(const FMP4BoxFRMA&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DataFormat));						// data_format
			return Error;
		}

	private:
		uint32 DataFormat;
	};


	/**
	 * 'schm' box. ISO/IEC 14496-12:2015 - 8.12.5 Scheme Type Box
	 */
	class FMP4BoxSCHM : public FMP4BoxFull
	{
	public:
		FMP4BoxSCHM(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, SchemeType(0), SchemeVersion(0)
		{
		}

		virtual ~FMP4BoxSCHM()
		{
		}

		uint32 GetSchemeType() const
		{
			return SchemeType;
		}

	private:
		FMP4BoxSCHM() = delete;
		FMP4BoxSCHM(const FMP4BoxSCHM&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(SchemeType));									// scheme_type
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(SchemeVersion));								// scheme_version
			if ((Flags & 1) != 0)
			{
				uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
				if (BytesRemaining)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->ReadString(SchemeURI, BytesRemaining));	// scheme_uri
				}
			}
			return Error;
		}

	private:
		uint32 SchemeType;
		uint32 SchemeVersion;
		FString SchemeURI;
	};


	/**
	 * 'tenc' box. ISO/IEC 23001-7:2016 - 8.2 Track Encryption box
	 */
	class FMP4BoxTENC : public FMP4BoxFull
	{
	public:
		FMP4BoxTENC(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, DefaultCryptByteBlock(0), DefaultSkipByteBlock(0), DefaultIsProtected(0), DefaultPerSampleIVSize(0)
		{
		}

		virtual ~FMP4BoxTENC()
		{
		}

		bool HasDefaultCryptBlockValues() const
		{
			return Version != 0;
		}
		uint8 GetDefaultCryptByteBlock() const
		{
			return DefaultCryptByteBlock;
		}
		uint8 GetDefaultSkipByteBlock() const
		{
			return DefaultSkipByteBlock;
		}
		uint8 GetDefaultIsProtected() const
		{
			return DefaultIsProtected;
		}
		uint8 GetDefaultPerSampleIVSize() const
		{
			return DefaultPerSampleIVSize;
		}
		const TArray<uint8>& GetDefaultKID() const
		{
			return DefaultKID;
		}
		const TArray<uint8>& GetDefaultConstantIV() const
		{
			return DefaultConstantIV;
		}

	private:
		FMP4BoxTENC() = delete;
		FMP4BoxTENC(const FMP4BoxTENC&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			uint8 Value8 = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value8));									// reserved
			if (Version == 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value8));								// reserved
			}
			else
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value8));
				DefaultCryptByteBlock = Value8 >> 4;											// default_crypt_byte_block
				DefaultSkipByteBlock = Value8 & 0x0f;											// default_skip_byte_block
			}
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultIsProtected));						// default_isProtected
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultPerSampleIVSize));					// default_Per_Sample_IV_Size
			DefaultKID.AddUninitialized(16);
			RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(DefaultKID.GetData(), 16));			// default_KID
			if (DefaultIsProtected == 1 && DefaultPerSampleIVSize == 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value8));								// default_constant_IV_size
				DefaultConstantIV.AddUninitialized(Value8);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(DefaultConstantIV.GetData(), Value8));// default_constant_IV
			}
			return Error;
		}

	private:
		uint8	DefaultCryptByteBlock;
		uint8	DefaultSkipByteBlock;
		uint8	DefaultIsProtected;
		uint8	DefaultPerSampleIVSize;
		TArray<uint8> DefaultKID;
		TArray<uint8> DefaultConstantIV;
	};


	/**
	 * 'saiz' box. ISO/IEC 14496-12:2015 - 8.7.8 Sample Auxiliary Information Sizes Box
	 */
	class FMP4BoxSAIZ : public FMP4BoxFull
	{
	public:
		FMP4BoxSAIZ(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, AuxInfoType(0), AuxInfoTypeParameter(0), DefaultSampleInfoSize(0)
		{
		}

		virtual ~FMP4BoxSAIZ()
		{
		}

	private:
		FMP4BoxSAIZ() = delete;
		FMP4BoxSAIZ(const FMP4BoxSAIZ&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			if ((Flags & 1) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(AuxInfoType));						// aux_info_type
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(AuxInfoTypeParameter));				// aux_info_type_parameter
			}

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleInfoSize));					// default_sample_info_size
			uint32 SampleCount = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleCount));							// sample_count
			if (DefaultSampleInfoSize == 0 && SampleCount != 0)
			{
				SampleInfoSize.AddUninitialized(SampleCount);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(SampleInfoSize.GetData(), SampleCount));// sample_info_size
			}
			return Error;
		}

	private:
		uint32	AuxInfoType;
		uint32	AuxInfoTypeParameter;
		uint8	DefaultSampleInfoSize;
		TArray<uint8> SampleInfoSize;
	};


	/**
	 * 'saio' box. ISO/IEC 14496-12:2015 - 8.7.9 Sample Auxiliary Information Offsets Box
	 */
	class FMP4BoxSAIO : public FMP4BoxFull
	{
	public:
		FMP4BoxSAIO(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, AuxInfoType(0), AuxInfoTypeParameter(0), DefaultSampleInfoSize(0)
		{
		}

		virtual ~FMP4BoxSAIO()
		{
		}

	private:
		FMP4BoxSAIO() = delete;
		FMP4BoxSAIO(const FMP4BoxSAIO&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			if ((Flags & 1) != 0)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(AuxInfoType));						// aux_info_type
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(AuxInfoTypeParameter));				// aux_info_type_parameter
			}
			uint32 EntryCount = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(EntryCount));								// entry_count
			if (EntryCount)
			{
				Offsets.AddUninitialized(EntryCount);
				uint64* Offs = Offsets.GetData();
				if (Version == 0)
				{
					uint32 Value32 = 0;
					for(uint32 i=0; i<EntryCount; ++i)
					{
						RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value32));					// offset
						*Offs++ = Value32;
					}
				}
				else
				{
					uint64 Value64 = 0;
					for(uint32 i=0; i<EntryCount; ++i)
					{
						RETURN_IF_ERROR(ParseInfo->Reader()->Read(Value64));					// offset
						*Offs++ = Value64;
					}
				}
			}
			return Error;
		}

	private:
		uint32	AuxInfoType;
		uint32	AuxInfoTypeParameter;
		uint8	DefaultSampleInfoSize;
		TArray<uint64> Offsets;
	};


	/**
	 * 'senc' box. ISO/IEC 23001-7:2016 - 7.2.1 Sample Encryption Box
	 */
	class FMP4BoxSENC : public FMP4BoxFull
	{
	public:
		FMP4BoxSENC(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxSENC()
		{
		}

		struct FEntry
		{
			TArray<uint8> IV;
			TArray<ElectraCDM::FMediaCDMSampleInfo::FSubSample> SubSamples;
		};

		const TArray<FEntry>& GetEntries() const
		{
			return Entries;
		}

		UEMediaError Parse(const FMP4BoxTENC* TENCBox, const FMP4BoxSAIZ* SAIZBox, const FMP4BoxSAIO* SAIOBox)
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			// We need to have the tenc box here. Right now the saiz and saio are not relevant.
			if (!TENCBox)
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}

			int32 PerSampleIVSize = TENCBox->GetDefaultPerSampleIVSize();

			FDataBufferReader BufferReader(BoxData);
			FMP4BoxReader Reader(&BufferReader, nullptr);

			if (SampleCount)
			{
				Entries.Empty();
				Entries.Reserve(SampleCount);
				for(uint32 i=0; i<SampleCount; ++i)
				{
					FEntry& Entry = Entries.AddDefaulted_GetRef();
					if (PerSampleIVSize)
					{
						Entry.IV.AddUninitialized(PerSampleIVSize);
						RETURN_IF_ERROR(Reader.ReadBytes(Entry.IV.GetData(), PerSampleIVSize));
					}
					if ((Flags & 2) != 0)
					{
						uint16 SubSampleCount = 0;
						RETURN_IF_ERROR(Reader.Read(SubSampleCount));
						for(int32 j=0; j<SubSampleCount; ++j)
						{
							ElectraCDM::FMediaCDMSampleInfo::FSubSample& SubSample = Entry.SubSamples.AddDefaulted_GetRef();
							RETURN_IF_ERROR(Reader.Read(SubSample.NumClearBytes));
							RETURN_IF_ERROR(Reader.Read(SubSample.NumEncryptedBytes));
						}
					}
				}
			}
			return UEMEDIA_ERROR_OK;
		}

	private:
		FMP4BoxSENC() = delete;
		FMP4BoxSENC(const FMP4BoxSENC&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			// Check for a pre-PIFF 1.3 box that has elements of the 'tenc' box here when flags&1.
			if ((Flags & 1) != 0)
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(SampleCount));
			// The contents of this box cannot be parsed without information from another box
			// containing the size of the IV.
			// We read the data of this box and parse it later when we need its data.
			uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
			if (BytesRemaining)
			{
				BoxData.AddUninitialized(BytesRemaining);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(BoxData.GetData(), BytesRemaining));
			}
			return Error;
		}

	private:
		uint32 SampleCount = 0;
		TArray<uint8> BoxData;
		TArray<FEntry> Entries;
	};


	/**
	 * 'sbgp' box. ISO/IEC 14496-12:2015 - 8.9.2 Sample to Group Box
	 */
	class FMP4BoxSBGP : public FMP4BoxFull
	{
	public:
		FMP4BoxSBGP(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, GroupingType(0), GroupingTypeParameter(0)
		{
		}

		virtual ~FMP4BoxSBGP()
		{
		}

		struct FEntry
		{
			uint32 SampleCount;
			uint32 GroupDescriptionIndex;
		};

		uint32 GetGroupingType() const
		{ return GroupingType; }

		uint32 GetGroupingTypeParameter() const
		{ return GroupingTypeParameter; }

		int32 GetNumberOfEntries() const
		{ return Entries.Num(); }

		const FEntry& GetEntry(int32 Index) const
		{ return Entries[Index]; }

	private:
		FMP4BoxSBGP() = delete;
		FMP4BoxSBGP(const FMP4BoxSBGP&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(GroupingType));						// grouping_type
			if (Version == 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(GroupingTypeParameter));			// grouping_type_parameter
			}
			uint32 EntryCount = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(EntryCount));							// entry_count
			for(uint32 i=0; i<EntryCount; ++i)
			{
				FEntry& Entry = Entries.AddDefaulted_GetRef();
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Entry.SampleCount));				// sample_count
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(Entry.GroupDescriptionIndex));	// group_description_index
			}
			return Error;
		}

	private:
		uint32	GroupingType;
		uint32	GroupingTypeParameter;
		TArray<FEntry> Entries;
	};



	/**
	 * Base class of a sample group description entry. ISO/IEC 14496-12:2016 - 8.9.3.2 Sample Group Description Box, Syntax
	 */
	struct FSampleGroupDescriptionEntry
	{
		TArray<uint8> Data;
	};

	/**
	 * 'seig' group type entry. ISO/IEC 23007-1:2016 - 6 Encryption parameters shared by groups of samples
	 */
	struct FCencSampleEncryptionInformationGroupEntry : FSampleGroupDescriptionEntry
	{
		uint8 GetCryptByteBlock() const { return Data[1] >> 4; }
		uint8 GetSkipByteBlock() const { return Data[1] & 0x0f; }
		uint8 GetIsProtected() const { return Data[2]; }
		uint8 GetPerSampleIVSize() const { return Data[3]; }
		TArray<uint8> GetKID() const { return TArray<uint8>(Data.GetData() + 4, 16); }
		uint8 GetConstantIVSize() const { return GetIsProtected() && GetPerSampleIVSize() == 0 ? Data[20] : 0; }
		TArray<uint8> GetConstantIV() const { uint8 civSize = GetConstantIVSize(); return civSize == 0 ? TArray<uint8>() : TArray<uint8>(Data.GetData() + 21, civSize); }
	};

	/**
	 * 'sgpd' box. ISO/IEC 14496-12:2015 - 8.9.3 Sample Group Description Box
	 */
	class FMP4BoxSGPD : public FMP4BoxFull
	{
	public:
		FMP4BoxSGPD(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
			, GroupingType(0), DefaultLength(0), DefaultSampleDescriptionIndex(0)
		{
		}

		virtual ~FMP4BoxSGPD()
		{
		}

		uint32 GetGroupingType() const
		{ return GroupingType; }

		uint32 GetDefaultSampleDescriptionIndex() const
		{ return DefaultSampleDescriptionIndex; }

		int32 GetNumberOfEntries() const
		{ return Entries.Num(); }

		const FSampleGroupDescriptionEntry& GetEntry(int32 Index) const
		{ return Entries[Index]; }

	private:
		FMP4BoxSGPD() = delete;
		FMP4BoxSGPD(const FMP4BoxSGPD&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
			/*
				Flags field must be zero when this box is in a `traf` box, otherwise, if it appears in
				a `stbl` box flag bit values mean:
					Bit 0: when set to 1, this flag indicates that there are no SampleGroupDescriptionBoxes
						   of this grouping_type in any TrackFragmentBox of this track.
					Bit 1: when set to 1, this flag indicates that there are no SampleToGroupBoxes
						   of this grouping_type in this track (in neither the SampleTableBox nor any TrackFragmentBox of this
						   track); all samples therefore map to the default.
			*/

			// Version 0 of this box type has been deprecated.
			if (Version == 0)
			{
				// Skip over the contents of this box.
				uint32 BytesRemaining = BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset);
				if (BytesRemaining)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, BytesRemaining));
				}
				return Error;
			}

			RETURN_IF_ERROR(ParseInfo->Reader()->Read(GroupingType));							// grouping_type

			if (Version >= 1)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultLength));						// default_length
			}
			if (Version >= 2)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(DefaultSampleDescriptionIndex));		// default_sample_description_index
			}

			uint32 EntryCount = 0;
			RETURN_IF_ERROR(ParseInfo->Reader()->Read(EntryCount));								// entry_count
			for(uint32 i=0; i<EntryCount; ++i)
			{
				uint32 EntrySize = DefaultLength;
				if (Version == 1 && DefaultLength == 0)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->Read(EntrySize));						// description_length
				}

				// The actual sample group entry now follows.
				FSampleGroupDescriptionEntry& Entry = Entries.AddDefaulted_GetRef();
				Entry.Data.AddUninitialized(EntrySize);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(Entry.Data.GetData(), EntrySize));
			}
			return Error;
		}

	private:
		uint32 GroupingType;
		uint32 DefaultLength;
		uint32 DefaultSampleDescriptionIndex;
		TArray<FSampleGroupDescriptionEntry> Entries;
	};

	/**
	 * Track reference entry (within an 'tref' box; Track Reference Box)
	*  The box type is the reference type (See kTrackRef_...), along with a list of TrackIds for that relation type.
	 * https://developer.apple.com/documentation/quicktime-file-format/track_reference_type_atom
	 */
	class FMP4BoxTrackReferenceEntry : public FMP4BoxBasic
	{
	public:
		FMP4BoxTrackReferenceEntry(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxTrackReferenceEntry() override = default;

		const TArray<int32>& GetTrackIds() const
		{
			return TrackIds;
		}

	private:
		FMP4BoxTrackReferenceEntry() = delete;
		FMP4BoxTrackReferenceEntry(const FMP4BoxTrackReferenceEntry&) = delete;

	protected:
		virtual UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			const int64 NowAt = ParseInfo->Reader()->GetCurrentReadOffset();
			const int32 BytesRemaining = static_cast<int32>(BoxSize - (NowAt - StartOffset));
			const int32 NumTrackIds = BytesRemaining/sizeof(int32);
			TrackIds.AddZeroed(NumTrackIds);
			for(int32 i=0; i<NumTrackIds; ++i)
			{
				RETURN_IF_ERROR(ParseInfo->Reader()->Read(TrackIds[i]));
			}
			return Error;
		}

		TArray<int32> TrackIds;
	};

	/**
	 * 'tref' box. Track Reference Atom
	 *  https://developer.apple.com/documentation/quicktime-file-format/track_reference_atom
	 *
	 *  Contains a list of relational track references per type.
	 */
	class FMP4BoxTREF : public FMP4BoxBasic
	{
	public:
		FMP4BoxTREF(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxTREF() override = default;

		const TArray<int32>& GetTrackIdsForType(IParserISO14496_12::FBoxType InReferenceType) const
		{
			for(const FMP4Box* ChildBox : ChildBoxes)
			{
				if (ChildBox->GetType() == InReferenceType)
				{
					return static_cast<const FMP4BoxTrackReferenceEntry*>(ChildBox)->GetTrackIds();
				}
			}
			static const TArray<int32> Empty;
			return Empty;
		}

	private:
		FMP4BoxTREF() = delete;
		FMP4BoxTREF(const FMP4BoxTREF&) = delete;

	protected:
		virtual UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;

			// Read the list of TrackReference atoms
			int32 BytesRemaining = static_cast<int32>(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			while (BytesRemaining > 0)
			{
				bIsLeafBox = false;

				IParserISO14496_12::FBoxType	ChildBoxType;
				int64							ChildBoxSize;
				uint8							ChildBoxUUID[16];

				const int64 BoxStartOffset = ParseInfo->Reader()->GetCurrentReadOffset();
				RETURN_IF_ERROR(ParseInfo->ReadBoxTypeAndSize(ChildBoxType, ChildBoxSize, ChildBoxUUID));
				const int64 BoxDataOffset = ParseInfo->Reader()->GetCurrentReadOffset();
				FMP4Box* NextBox = new FMP4BoxTrackReferenceEntry(ChildBoxType, ChildBoxSize, BoxStartOffset, BoxDataOffset, true);
				AddChildBox(NextBox);
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(NextBox));

				// Where are we at now?
				BytesRemaining = static_cast<int32>(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			}
			return Error;
		}
	};

	/**
	 * 'udta' box. ISO/IEC 14496-12:2015 - 8.10.1 User Data Box
	 */
	class FMP4BoxUDTA : public FMP4BoxBasic
	{
	public:
		FMP4BoxUDTA(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxUDTA() = default;


		TMediaOptionalValue<FString> GetName() const
		{
			static const IParserISO14496_12::FBoxType kUDTA_name = MAKE_BOX_ATOM('n', 'a', 'm', 'e');
			const FMP4BoxIgnored* NameBox = static_cast<const FMP4BoxIgnored*>(FindBox(kUDTA_name, 1));
			const FFillerData* Filler = NameBox ? NameBox->GetFillerData() : nullptr;
			if (Filler)
			{
				FString name = FString::ConstructFromPtrSize(static_cast<const ANSICHAR*>(Filler->Data), Filler->Size);
				return TMediaOptionalValue<FString>(name);
			}
			return TMediaOptionalValue<FString>();
		}

	private:
		FMP4BoxUDTA() = delete;
		FMP4BoxUDTA(const FMP4BoxUDTA&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			int64 NowAt = ParseInfo->Reader()->GetCurrentReadOffset();
			int32 BytesRemaining = (int32)(BoxSize - (NowAt - StartOffset));
			if (BytesRemaining > 0)
			{
				bIsLeafBox = false;
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this, NowAt + BytesRemaining));
				BytesRemaining = (int32)(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
				check(BytesRemaining >= 0);
				if (BytesRemaining > 0)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, BytesRemaining));
				}
			}
			return Error;
		}
	};


	/**
	 * 'meta' box. ISO/IEC 14496-12:2015 - 8.11.1 Meta Box
	 */
	class FMP4BoxMETA : public FMP4BoxFull,	public IParserISO14496_12::IMetadata
	{
	public:
		FMP4BoxMETA(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxFull(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxMETA() = default;

	private:
		FMP4BoxMETA() = delete;
		FMP4BoxMETA(const FMP4BoxMETA&) = delete;


		uint32 GetHandler() const override
		{
			return Handler ? Handler->GetHandlerType() : 0;
		}
		uint32 GetReserved(int32 InIndex0to2) const override
		{
			return Handler ? Handler->GetReservedValue(InIndex0to2) : 0;
		}
		int32 GetNumChildBoxes() const override
		{
			return GetNumberOfChildren();
		}
		uint32 GetChildBoxType(int32 InChildIndex) const override
		{
			const FMP4Box* Child = GetChildBox(InChildIndex);
			return Child ? (uint32)Child->GetType() : 0;
		}
		uint32 GetChildBoxDataSize(int32 InChildIndex) const override
		{
			const FMP4Box* Child = GetChildBox(InChildIndex);
			return Child ? (uint32)Child->GetDataSize() : 0;
		}
		const void* GetChildBoxData(int32 InChildIndex) const override
		{
			const FMP4Box* Child = GetChildBox(InChildIndex);
			return Child ? Child->GetFillerData() ? Child->GetFillerData()->Data : nullptr : nullptr;
		}

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;

			// The 'meta' box is a full box under ISO and a basic box in QuickTime.
			// Some tools write a QT 'meta' box even for ISO which is then missing the version number and flags!
			// To work around this issue we read the box here as a whole and locate the mandatory 'hdlr' box
			// that has to be the first box and based on its position handle this as a basic or a full box.
			TArray<uint8> BoxData;
			uint32 BytesRemaining = (uint32)(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
			if (BytesRemaining)
			{
				BoxData.AddUninitialized(BytesRemaining);
				RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(BoxData.GetData(), BytesRemaining));
			}

			bool bIsFullBox = false;
			if (BytesRemaining >= 12)
			{
				bIsFullBox = BoxData[8]=='h' && BoxData[9]=='d' && BoxData[10]=='l' && BoxData[11]=='r';
			}

			FDataBufferReader BufferReader(BoxData, StartOffset);
			FMP4BoxReader Reader(&BufferReader, nullptr);
			struct FAutoPopReader
			{
				FAutoPopReader(FMP4ParseInfo* InParseInfo, FMP4BoxReader* InReader) : PInfo(InParseInfo)
				{ PInfo->PushReader(InReader); }
				~FAutoPopReader()
				{ Detach(); }
				void Detach()
				{
					if (PInfo)
					{
						PInfo->PopReader();
						PInfo = nullptr;
					}
				}
				FMP4ParseInfo* PInfo;
			};
			FAutoPopReader BoxReader(ParseInfo, &Reader);
			if (bIsFullBox)
			{
				RETURN_IF_ERROR(FMP4BoxFull::ReadAndParseAttributes(ParseInfo));
				BytesRemaining -= 4;
			}
			if (BytesRemaining > 0)
			{
				bIsLeafBox = false;
				UEMediaError Err = ParseInfo->ReadAndParseNextBox(this, ParseInfo->Reader()->GetCurrentReadOffset() + BytesRemaining);
				if (Err != UEMEDIA_ERROR_OK && Err != UEMEDIA_ERROR_END_OF_STREAM)
				{
					return Err;
				}
			}
			Handler = static_cast<const FMP4BoxHDLR*>(FindBox(kBox_hdlr, 0));
			return Error;
		}
	private:
		const FMP4BoxHDLR* Handler = nullptr;
	};


	/**
	 * 'wave' atom from Apple QuickTime. See https://developer.apple.com/library/archive/documentation/QuickTime/QTFF/QTFFChap3/qtff3.html
	 *
	 * | "This atom contains other atoms with audio decompressor settings and is a required extension to the sound sample description for MPEG-4 audio.
	 * |  A 'wave' chunk for 'mp4a' typically contains (in order) at least a 'frma' atom, an 'mp4a' atom, an 'esds' atom, and a Terminator Atom (0x00000000) atom."
	 */
	class FMP4BoxWAVE : public FMP4BoxBasic
	{
	public:
		FMP4BoxWAVE(IParserISO14496_12::FBoxType InBoxType, int64 InBoxSize, int64 InStartOffset, int64 InDataOffset, bool bInIsLeafBox)
			: FMP4BoxBasic(InBoxType, InBoxSize, InStartOffset, InDataOffset, bInIsLeafBox)
		{
		}

		virtual ~FMP4BoxWAVE() = default;

	private:
		FMP4BoxWAVE() = delete;
		FMP4BoxWAVE(const FMP4BoxWAVE&) = delete;

	protected:
		UEMediaError ReadAndParseAttributes(FMP4ParseInfo* ParseInfo) override
		{
			UEMediaError Error = UEMEDIA_ERROR_OK;
			int64 NowAt = ParseInfo->Reader()->GetCurrentReadOffset();
			int32 BytesRemaining = (int32)(BoxSize - (NowAt - StartOffset));
			if (BytesRemaining > 0)
			{
				bIsLeafBox = false;
				RETURN_IF_ERROR(ParseInfo->ReadAndParseNextBox(this, NowAt + BytesRemaining));
				BytesRemaining = (int32)(BoxSize - (ParseInfo->Reader()->GetCurrentReadOffset() - StartOffset));
				check(BytesRemaining >= 0);
				if (BytesRemaining > 0)
				{
					RETURN_IF_ERROR(ParseInfo->Reader()->ReadBytes(nullptr, BytesRemaining));
				}
			}
			return Error;
		}
	};


	FMP4ParseInfo::~FMP4ParseInfo()
	{
		delete RootBox;
	}

	UEMediaError FMP4ParseInfo::ReadBoxTypeAndSize(IParserISO14496_12::FBoxType& OutBoxType, int64& OutBoxSize, uint8* InOutUUID)
	{
		UEMediaError Error;
		// Read the box size, which is a big-endian 32 bit value
		uint32 Size32 = 0;
		RETURN_IF_ERROR(BoxReader->Read(Size32));

		// Read the box type (atom). This is a 32 bit 4CC value in big-endian.
		uint32 Type = 0;
		RETURN_IF_ERROR(BoxReader->Read(Type));
		OutBoxType = Type;

		// Size-check. If the 32 bit size value is 1 then the next 64 bits will be the actual size.
		// NOTE: A value of 0 is permitted and indicates the box extends to the end of the file.
		if (Size32 != 1)
		{
			OutBoxSize = Size32;
		}
		else
		{
			uint64 Size64 = 0;
			if ((Error = BoxReader->Read(Size64)) != UEMEDIA_ERROR_OK)
			{
				return Error;
			}
			OutBoxSize = (int64)Size64;
		}

		// If this box is an UUID box we need to read the 16 byte UUID now.
		if (OutBoxType == FMP4Box::kBox_uuid)
		{
			RETURN_IF_ERROR(BoxReader->ReadBytes(InOutUUID, 16));
		}

		return UEMEDIA_ERROR_OK;
	}


	UEMediaError FMP4ParseInfo::Parse(FMP4BoxReader* Reader, IPlayerSessionServices* InPlayerSession, const FMP4ParseInfo* InOptionalMoovParseInfo)
	{
		PlayerSession = InPlayerSession;
		OptionalMoovParseInfo = InOptionalMoovParseInfo;
		bShouldParseTimecodeInfo = InPlayerSession ? InPlayerSession->HaveOptionValue(Electra::OptionKeyParseTimecodeInfo) : false;

		// New parse or continuing a previous?
		if (RootBox == nullptr)
		{
			RootBox = new FMP4BoxBasic(MAKE_BOX_ATOM('*', '*', '*', '*'), 0, 0, 0, false);
			if (!RootBox)
			{
				return UEMEDIA_ERROR_OOM;
			}
		}

		// There is not really a reliable way to detect if this is a valid mp4 file.
		// Although the first box should be an 'ftyp' or 'styp' box but this is not always a given.
		// We will rely on the parser gracefully failing if something doesn't parse as expected.
		BoxReader = Reader;
		UEMediaError Error = ReadAndParseNextBox(RootBox);
		BoxReader = nullptr;
		return Error;
	}


	UEMediaError FMP4ParseInfo::ReadAndParseNextBox(FMP4Box* ParentBox, int64 BoxEndByteOffset)
	{
		IParserISO14496_12::FBoxType	BoxType;
		int64							BoxSize;
		uint8							BoxUUID[16];
		UEMediaError					Error = UEMEDIA_ERROR_FORMAT_ERROR;

		if (ParentBox->IsLeafBox())
		{
			return ParentBox->ReadAndParseAttributes(this);
		}
		else
		{
			while(1)
			{
				// Check before reading anything if we reached the end of the file.
				if (BoxReader->IsAtEOF())
				{
					return UEMEDIA_ERROR_END_OF_STREAM;
				}

				int64 BoxStartOffset = BoxReader->GetCurrentReadOffset();

				// See if we have reached the end of the optionally passed end offset.
				// If we do not have enough bytes to read the next box size and type we are done.
				if (BoxEndByteOffset >= 0 && BoxStartOffset+8 > BoxEndByteOffset)
				{
					return UEMEDIA_ERROR_OK;
				}

				// Get the type and size of the first box.
				Error = ReadBoxTypeAndSize(BoxType, BoxSize, BoxUUID);
				if (Error != UEMEDIA_ERROR_OK)
				{
					// If we didn't see and EOF above, did we hit it now?
					return Error == UEMEDIA_ERROR_INSUFFICIENT_DATA ? UEMEDIA_ERROR_END_OF_STREAM : Error;
				}

#if MEDIA_DEBUG_HAS_BOX_NAMES
				char boxName[4];
				*((uint32*)&boxName) = MEDIA_TO_BIG_ENDIAN(BoxType);
				//UE_LOGF(LogElectraMP4Parser, Log, "Next box: '%ls' %lld B @ %lld (Parent: '%ls')", *FMP4Box::GetBoxNameString(boxName), (long long int)BoxSize, (long long int)BoxStartOffset, ParentBox ? *ParentBox->GetBoxName() : TEXT("****"));
#endif

				int64 BoxDataOffset = BoxReader->GetCurrentReadOffset();
				// Check for 0-size boxes. The last box is allowed to have a size of 0 to indicate it extending to the end of the file.
				// Other boxes may not have zero size.
				if (BoxSize == 0)
				{
					// We can't know at this point if this is actually the last box. We also do not really support those so we will treat those as a 'void' box,
					// which we allow only within a parent container box but not a root box.
					if (GetNestingLevel() > 1)
					{
						// Where does the parent box end?
						int64 ParentBoxEndOffset = ParentBox->GetStartOffset() + ParentBox->GetSize();
						// How many bytes do we still have after having read the current type and possibly large size?
						int64 ParentBoxDataSizeRemaining = ParentBoxEndOffset - BoxDataOffset;

						if (ParentBoxDataSizeRemaining >= 0)
						{
							BoxType = FMP4Box::kBox_VOID;
							BoxSize = ParentBoxDataSizeRemaining + (BoxDataOffset - BoxStartOffset);
						}
						else
						{
							// Already overshot the parent box end.
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}
					}
					else
					{
						// Top-level boxes with 0 size are not supported at the moment.
						return UEMEDIA_ERROR_FORMAT_ERROR;
					}
				}

				// Check with user callback if parsing should stop with this box.
				// Do this for root-level boxes only.
				IParserISO14496_12::IBoxCallback::EParseContinuation ParseContinuation = GetNestingLevel() == 0 ? BoxReader->NotifyStartOfBox(BoxType, BoxSize, BoxStartOffset, BoxDataOffset) : IParserISO14496_12::IBoxCallback::EParseContinuation::Continue;
				if (ParseContinuation == IParserISO14496_12::IBoxCallback::EParseContinuation::Continue)
				{
					// Create a box of the appropriate type.
					FMP4Box* NextBox = nullptr;
					switch(BoxType)
					{
						case FMP4Box::kBox_mdat:
							NextBox = new FMP4BoxMDAT(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;

						case FMP4Box::kBox_moov:
							check(GetCurrentMoovBox() == nullptr);
							NextBox = new FMP4BoxBasic(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, false);
							SetCurrentMoovBox(NextBox);
							break;
						case FMP4Box::kBox_moof:
							NextBox = new FMP4BoxBasic(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, false);
							SetCurrentMoofBox(NextBox);
							break;
						case FMP4Box::kBox_edts:
						case FMP4Box::kBox_mdia:
						case FMP4Box::kBox_minf:
						case FMP4Box::kBox_dinf:
						case FMP4Box::kBox_stbl:
						case FMP4Box::kBox_mvex:
						case FMP4Box::kBox_schi:
							NextBox = new FMP4BoxBasic(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, false);
							break;
						case FMP4Box::kBox_trak:
						case FMP4Box::kBox_traf:
							NextBox = new FMP4BoxBasic(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, false);
							SetCurrentTrackBox(NextBox);
							break;
						case FMP4Box::kBox_ftyp:
							check(GetCurrentFtypBox() == nullptr);
							NextBox = new FMP4BoxFTYP(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							SetCurrentFtypBox(NextBox);
							break;
						case FMP4Box::kBox_styp:
							NextBox = new FMP4BoxSTYP(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							SetCurrentStypBox(NextBox);
							break;
						case FMP4Box::kBox_sidx:
							NextBox = new FMP4BoxSIDX(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							SetCurrentSidxBox(NextBox);
							break;
						case FMP4Box::kBox_mvhd:
							NextBox = new FMP4BoxMVHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_mehd:
							NextBox = new FMP4BoxMEHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_tkhd:
							NextBox = new FMP4BoxTKHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_trex:
							NextBox = new FMP4BoxTREX(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_elst:
							NextBox = new FMP4BoxELST(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_mdhd:
							NextBox = new FMP4BoxMDHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_hdlr:
							NextBox = new FMP4BoxHDLR(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_nmhd:
							NextBox = new FMP4BoxNMHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_vmhd:
							NextBox = new FMP4BoxVMHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_smhd:
							NextBox = new FMP4BoxSMHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_sthd:
							NextBox = new FMP4BoxSTHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_dref:
							NextBox = new FMP4BoxDREF(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);		// 'dref' is not a true leaf box, but its elements are a list that must be parsed.
							break;
						case FMP4Box::kBox_url:
							NextBox = new FMP4BoxURL(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_urn:
							NextBox = new FMP4BoxURN(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_stsd:
							NextBox = new FMP4BoxSTSD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);		// 'stsd' is not a true leaf box, but its elements are a list that must be parsed.
							break;
						case FMP4Box::kBox_avcC:
							NextBox = new FMP4BoxAVCC(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_hvcC:
							NextBox = new FMP4BoxHVCC(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_wave:
							NextBox = new FMP4BoxWAVE(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_btrt:
							NextBox = new FMP4BoxBTRT(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_pasp:
							NextBox = new FMP4BoxPASP(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_elng:
							NextBox = new FMP4BoxELNG(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_prft:
							NextBox = new FMP4BoxPRFT(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_emsg:
							NextBox = new FMP4BoxEMSG(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							AddEMSGBox(NextBox);
							break;
						case FMP4Box::kBox_iods:
						case FMP4Box::kBox_esds:
							NextBox = new FMP4BoxESDS(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_dac3:
							NextBox = new FMP4BoxDAC3(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_dec3:
							NextBox = new FMP4BoxDEC3(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_stts:
							NextBox = new FMP4BoxSTTS(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_ctts:
							NextBox = new FMP4BoxCTTS(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_stss:
							NextBox = new FMP4BoxSTSS(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_stsc:
							NextBox = new FMP4BoxSTSC(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_stsz:
							NextBox = new FMP4BoxSTSZ(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_stco:
						case FMP4Box::kBox_co64:
							NextBox = new FMP4BoxSTCO(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_mfhd:
							NextBox = new FMP4BoxMFHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_tfhd:
							NextBox = new FMP4BoxTFHD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_tfdt:
							NextBox = new FMP4BoxTFDT(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_trun:
							NextBox = new FMP4BoxTRUN(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_saiz:
							NextBox = new FMP4BoxSAIZ(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_saio:
							NextBox = new FMP4BoxSAIO(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_sbgp:
							NextBox = new FMP4BoxSBGP(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_sgpd:
							NextBox = new FMP4BoxSGPD(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_tref:
							NextBox = new FMP4BoxTREF(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						// Boxes we ignore for now.
						case FMP4Box::kBox_colr:
						case FMP4Box::kBox_fiel:
						case FMP4Box::kBox_trep:
						case FMP4Box::kBox_sdtp:
						case FMP4Box::kBox_gmhd:
						case FMP4Box::kBox_tapt:
						case FMP4Box::kBox_chan:
						case FMP4Box::kBox_enda:
						case FMP4Box::kBox_mdcv:
						case FMP4Box::kBox_clli:
						case FMP4Box::kBox_coll:
							NextBox = new FMP4BoxIgnored(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_udta:
							NextBox = new FMP4BoxUDTA(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_meta:
							NextBox = new FMP4BoxMETA(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_skip:
						case FMP4Box::kBox_wide:
						case FMP4Box::kBox_VOID:
							NextBox = new FMP4BoxIgnored(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;

						case FMP4Box::kBox_free:
							NextBox = new FMP4BoxIgnored(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;

						case FMP4Box::kBox_uuid:
							// Check if this is a long form of an atom. This is rare and not supported.
							if (FMemory::Memcmp(BoxUUID+4, FMP4Box::UUID_atom+4, 12) == 0)
							{
								// This is not handled.
								return UEMEDIA_ERROR_FORMAT_ERROR;
							}
							// Compare the UUID with known boxes.
							else if (FMemory::Memcmp(BoxUUID, FMP4Box::UUID_pssh, 16) == 0)
							{
								NextBox = new FMP4BoxPSSH(FMP4Box::kBox_pssh, BoxSize, BoxStartOffset, BoxDataOffset, true);
							}
							else if (FMemory::Memcmp(BoxUUID, FMP4Box::UUID_tenc, 16) == 0)
							{
								NextBox = new FMP4BoxTENC(FMP4Box::kBox_tenc, BoxSize, BoxStartOffset, BoxDataOffset, true);
							}
							else if (FMemory::Memcmp(BoxUUID, FMP4Box::UUID_senc, 16) == 0)
							{
								NextBox = new FMP4BoxSENC(FMP4Box::kBox_senc, BoxSize, BoxStartOffset, BoxDataOffset, true);
							}
							else
							{
								NextBox = new FMP4BoxIgnored(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							}
							break;

						// Encryption related boxes.
						case FMP4Box::kBox_pssh:
							NextBox = new FMP4BoxPSSH(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_sinf:
							NextBox = new FMP4BoxSINF(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);		// 'sinf' is not a true leaf box, but its elements are a list that must be parsed.
							break;
						case FMP4Box::kBox_frma:
							NextBox = new FMP4BoxFRMA(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_schm:
							NextBox = new FMP4BoxSCHM(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_tenc:
							NextBox = new FMP4BoxTENC(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
						case FMP4Box::kBox_senc:
							NextBox = new FMP4BoxSENC(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;

						default:
#if MEDIA_DEBUG_HAS_BOX_NAMES
//							UE_LOGF(LogElectraMP4Parser, Log, "Ignoring mp4 box: '%ls' %lld B @ %lld", *FMP4Box::GetBoxNameString(boxName), (long long int)BoxSize, (long long int)BoxStartOffset);
#endif
							NextBox = new FMP4BoxIgnored(BoxType, BoxSize, BoxStartOffset, BoxDataOffset, true);
							break;
					}
					if (NextBox)
					{
						++NumTotalBoxesParsed;
						ParentBox->AddChildBox(NextBox);
						IncreaseNestingLevel();
						Error = ReadAndParseNextBox(NextBox, BoxEndByteOffset);
						DecreaseNestingLevel();
						if (Error == UEMEDIA_ERROR_OK)
						{
							// Where are we at now?
							int64 BoxOffsetAfterParse = BoxReader->GetCurrentReadOffset();
							int64 BoxBytesParsed = BoxOffsetAfterParse - BoxStartOffset;
							int64 BoxBytesNotParsed = BoxSize - BoxBytesParsed;
							if (BoxBytesNotParsed > 0)
							{
								// There is more data in the box we did not parse. This can happen with some tools
								// or on purpose with a "hidden" 'senc' box. Because of that possibility we need
								// to read this excess data into a buffer and keep it.
								check(!TEXT("Verify this"));
								RETURN_IF_ERROR(NextBox->ReadFillerData(this, BoxBytesNotParsed));
							}
							else if (BoxBytesNotParsed < 0)
							{
								// We somehow consumed more bytes than the box was set to have.
								check(!TEXT("How did this happen?"));
								return UEMEDIA_ERROR_FORMAT_ERROR;
							}
							// Finished a 'trak' or 'traf' box?
							if (NextBox->GetType() == FMP4Box::kBox_trak || NextBox->GetType() == FMP4Box::kBox_traf)
							{
								SetCurrentTrackBox(nullptr);
							}
							// Finished a 'hdlr' box?
							else if (NextBox->GetType() == FMP4Box::kBox_hdlr)
							{
								// We need to remember the handler for the continued parsing of the 'stsd' box, but only if this is a handler
								// of a type we support.
								FMP4BoxHDLR* HdlrBox = static_cast<FMP4BoxHDLR*>(NextBox);
								switch(HdlrBox->GetHandlerType())
								{
									case FMP4Box::kTrack_vide:
									case FMP4Box::kTrack_soun:
									case FMP4Box::kTrack_subt:
									case FMP4Box::kTrack_sbtl:
									case FMP4Box::kTrack_subp:
									case FMP4Box::kTrack_hint:
									case FMP4Box::kTrack_meta:
									case FMP4Box::kTrack_text:
										SetCurrentHandlerBox(NextBox);
										break;
									default:
										break;
								}
							}
							// Finished a supported media handler box?
							else if (NextBox->GetType() == FMP4Box::kBox_vmhd ||
								NextBox->GetType() == FMP4Box::kBox_smhd ||
								NextBox->GetType() == FMP4Box::kBox_nmhd ||
								NextBox->GetType() == FMP4Box::kBox_sthd ||
								NextBox->GetType() == FMP4Box::kBox_gmhd)
							{
								SetCurrentMediaHandlerBox(NextBox);
							}
							// Finished a 'mdia' or 'meta' box?
							else if (NextBox->GetType() == FMP4Box::kBox_mdia || NextBox->GetType() == FMP4Box::kBox_meta)
							{
								SetCurrentHandlerBox(nullptr);
								SetCurrentMediaHandlerBox(nullptr);
							}
							// Finished parsing a list of child boxes (when called to parse a list of contained boxes)
							if (BoxOffsetAfterParse >= ParentBox->GetStartOffset() + ParentBox->GetSize() && GetNestingLevel())
							{
								return Error;
							}
						}
						else
						{
							return Error;
						}
					}
					ParseContinuation = GetNestingLevel() == 0 ? BoxReader->NotifyEndOfBox(BoxType, BoxSize, BoxStartOffset, BoxDataOffset) : IParserISO14496_12::IBoxCallback::EParseContinuation::Continue;
					if (ParseContinuation == IParserISO14496_12::IBoxCallback::EParseContinuation::Stop)
					{
						// Stop parsing.
						return UEMEDIA_ERROR_OK;
					}
				}
				else
				{
					// Stop parsing.
					return UEMEDIA_ERROR_OK;
				}
			}
		}

		return UEMEDIA_ERROR_OK;

	}


	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/


	/**
	 *
	 */
	class FParserISO14496_12 : public IParserISO14496_12
	{
	public:
		FParserISO14496_12();
		virtual ~FParserISO14496_12();
		UEMediaError ParseHeader(IGenericDataReader* DataReader, IBoxCallback* BoxParseCallback, IPlayerSessionServices* PlayerSession, const IParserISO14496_12* OptionalInitSegment) override;

		UEMediaError PrepareTracks(IPlayerSessionServices* PlayerSession, TSharedPtrTS<const IParserISO14496_12> OptionalMP4InitSegment) override;

		virtual UEMediaError ResolveTimecodeTracks(IPlayerSessionServices* InPlayerSession, FCancellationCheckDelegate InCancellationCheckDelegate) override;

		TMediaOptionalValue<FTimeFraction> GetMovieDuration() const override;
		int32 GetNumberOfTracks() const override;
		int32 GetNumberOfSegmentIndices() const override;
		int32 GetNumberOfEventMessages() const override;
		int32 GetNumberOfBrands() const override;
		FBrandType GetBrandByIndex(int32 Index) const override;
		bool HasBrand(const FBrandType InBrand) const override;
		const ITrack* GetTrackByIndex(int32 Index) const override;
		const ITrack* GetTrackByTrackID(int32 TrackID) const override;
		const ISegmentIndex* GetSegmentIndexByIndex(int32 Index) const override;
		const IEventMessage* GetEventMessageByIndex(int32 Index) const override;
		const IMetadata* GetMetadata(EBaseBoxType InFromBox) const override;

		TSharedPtrTS<IAllTrackIterator> CreateAllTrackIteratorByFilePos(int64 InFromFilePos) const override;
		TSharedPtrTS<IAllTrackIterator> CreateAllTrackIteratorForTrackIDs(const TArray<uint32>& InTrackIDs) const override;

	private:
		class FTrack;

		class FTrackIterator : public ITrackIterator
		{
		public:
			FTrackIterator();
			virtual ~FTrackIterator();

			const ITrack* GetTrack() const override;
			int64 GetBaseMediaDecodeTime() const override;
			bool IsAtEOS() const override;

			UEMediaError StartAtTime(const FTimeValue& AtTime, ESearchMode SearchMode, bool bNeedSyncSample) override;
			UEMediaError StartAtFirst(bool bNeedSyncSample) override;
			UEMediaError Next() override;

			uint32 GetSampleNumber() const override;
			int64 GetDTS() const override;
			int64 GetPTS() const override;
			int64 GetDuration() const override;
			uint32 GetTimescale() const override;
			bool IsSyncSample() const override;
			int64 GetSampleSize() const override;
			int64 GetSampleFileOffset() const override;
			int64 GetRawDTS() const override;
			int64 GetRawPTS() const override;
			int64 GetCompositionTimeEdit() const override;
			int64 GetEmptyEditOffset() const override;
			bool GetEncryptionInfo(ElectraCDM::FMediaCDMSampleInfo& OutSampleEncryptionInfo) const override;

			UEMediaError StartAtFirstInternal();
			void SetTrack(const FTrack* InTrack);
			bool IsValid() const
			{
				return Track != nullptr;
			}
		private:
			typedef uint32 FGroupType;
			#define MAKE_MP4_GROUPTYPE(a,b,c,d) (FGroupType)((uint32)a << 24) | ((uint32)b << 16) | ((uint32)c << 8) | ((uint32)d)
			enum EGroupType
			{
				Grouping_rap = MAKE_MP4_GROUPTYPE('r', 'a', 'p', ' '),
			};
			#undef MAKE_MP4_GROUPTYPE

			FTrackIterator& AssignFrom(const FTrackIterator& Other)
			{
				if (this != &Other)
				{
					Track = Other.Track;
					bIsFragmented = Other.bIsFragmented;
					CurrentSampleNumber = Other.CurrentSampleNumber;
					Timescale = Other.Timescale;
					InTRUNIndex = Other.InTRUNIndex;
					SampleNumberInTRUN = Other.SampleNumberInTRUN;
					RemainingSamplesInTRUN = Other.RemainingSamplesInTRUN;
					NumSamplesTotal = Other.NumSamplesTotal;
					NumSamplesInChunk = Other.NumSamplesInChunk;
					ChunkNumOfNextSampleChange = Other.ChunkNumOfNextSampleChange;
					CurrentSampleInChunk = Other.CurrentSampleInChunk;
					CurrentChunkIndex = Other.CurrentChunkIndex;
					STSCIndex = Other.STSCIndex;
					STTSCount = Other.STTSCount;
					STTSDelta = Other.STTSDelta;
					STTSIndex = Other.STTSIndex;
					CTTSCount = Other.CTTSCount;
					CTTSOffset = Other.CTTSOffset;
					CTTSIndex = Other.CTTSIndex;
					STSSNextSyncSampleNum = Other.STSSNextSyncSampleNum;
					STSSIndex = Other.STSSIndex;
					EmptyEditDurationInMediaTimeUnits = Other.EmptyEditDurationInMediaTimeUnits;
					CompositionTimeEditOffset = Other.CompositionTimeEditOffset;
					DataOffset = Other.DataOffset;
					SampleDTS = Other.SampleDTS;
					SamplePTS = Other.SamplePTS;
					SampleFlag = Other.SampleFlag;
					SampleSize = Other.SampleSize;
					SampleDuration = Other.SampleDuration;
					SampleDescriptionIndex = Other.SampleDescriptionIndex;
					bEOS = Other.bEOS;
					GroupIterators = Other.GroupIterators;
					bHaveRAPGroup = Other.bHaveRAPGroup;
				}
				return *this;
			}

			const FTrack*		Track = nullptr;

			bool				bIsFragmented = false;

			uint32				CurrentSampleNumber = 0;
			uint32				Timescale = 0;

			// Used in fragmented files only
			uint32				InTRUNIndex = 0;
			uint32				SampleNumberInTRUN = 0;
			uint32				RemainingSamplesInTRUN = 0;

			// Used in standard files only
			uint32				NumSamplesTotal = 0;
			uint32				NumSamplesInChunk = 0;
			uint32				ChunkNumOfNextSampleChange = 0;
			uint32				CurrentSampleInChunk = 0;
			uint32				CurrentChunkIndex = 0;
			int32				STSCIndex = 0;
			uint32				STTSCount = 0;
			uint32				STTSDelta = 0;
			int32				STTSIndex = 0;
			uint32				CTTSCount = 0;
			int64				CTTSOffset = 0;
			int32				CTTSIndex = 0;
			uint32				STSSNextSyncSampleNum = 0;
			int32				STSSIndex = 0;

			// Used in all cases
			int64				EmptyEditDurationInMediaTimeUnits = 0;
			int64				CompositionTimeEditOffset = 0;
			int64				DataOffset = 0;
			int64				SampleDTS = 0;
			int64				SamplePTS = 0;
			uint32				SampleFlag = 0;
			uint32				SampleSize = 0;
			uint32				SampleDuration = 0;
			uint32				SampleDescriptionIndex = 0;
			bool				bEOS = true;

			struct FSampleGroupIterator
			{
				FBoxType GroupingType = 0;
				const FMP4BoxSBGP* SBGPBox = nullptr;
				uint32 DefaultDescriptionIndex = 0;
				uint32 CurrentSampleIdxInEntry = 0;
				int32 CurrentEntryIndex = 0;
				uint32 CurrentGroupDescriptionIndex = 0;
				uint32 GetCurrentGroupDescriptionIndex() const
				{
					return CurrentGroupDescriptionIndex;
				}
				void SetFirst()
				{
					check(SBGPBox);
					CurrentSampleIdxInEntry = 0;
					CurrentEntryIndex = 0;
					CurrentGroupDescriptionIndex = SBGPBox->GetNumberOfEntries() ? SBGPBox->GetEntry(CurrentEntryIndex).GroupDescriptionIndex : DefaultDescriptionIndex;
				}
				void Advance()
				{
					if (CurrentEntryIndex < SBGPBox->GetNumberOfEntries())
					{
						// Need to move onto the next entry?
						if (++CurrentSampleIdxInEntry >= SBGPBox->GetEntry(CurrentEntryIndex).SampleCount)
						{
							CurrentSampleIdxInEntry = 0;
							if (++CurrentEntryIndex < SBGPBox->GetNumberOfEntries())
							{
								CurrentGroupDescriptionIndex = SBGPBox->GetEntry(CurrentEntryIndex).GroupDescriptionIndex;
							}
							else
							{
								CurrentGroupDescriptionIndex = DefaultDescriptionIndex;
							}
						}
					}
					else
					{
						CurrentGroupDescriptionIndex = DefaultDescriptionIndex;
					}
				}
			};
			TMap<FBoxType, FSampleGroupIterator> GroupIterators;
			bool bHaveRAPGroup = false;
		};


		class FTrack : public ITrack
		{
		public:
			FTrack();
			virtual ~FTrack()
			{
			}
			uint32 GetID() const override;
			FString GetName() const override;
			FString GetNameFromHandler() const override;
			FTimeFraction GetDuration() const override;
			ITrackIterator* CreateIterator() const override;
			const TArray<uint8> GetCodecSpecificData() const override;
			const TArray<uint8> GetCodecSpecificDataRAW() const override;
			const FStreamCodecInformation GetCodecInformation() const override;
			const FBitrateInfo GetBitrateInfo() const override;
			const BCP47::FLanguageTag& GetLanguageTag() const override;
			bool GetEncryptionInfo(FEncryptionInfo& OutEncryptionInfo) const override;
			void GetPSSHBoxes(TArray<TArray<uint8>>& OutBoxes, bool bFromMOOV, bool bFromMOOF) const override;
			void GetPRFTBoxes(TArray<ITrack::FProducerReferenceTime>& OutBoxes) const override;

			//private:
			const FMP4BoxMVHD* MVHDBox = nullptr;
			const FMP4BoxELST* ELSTBox = nullptr;
			const FMP4BoxTKHD* TKHDBox = nullptr;
			const FMP4Box*	   MDIABox = nullptr;
			const FMP4BoxMDHD* MDHDBox = nullptr;
			const FMP4BoxELNG* ELNGBox = nullptr;
			const FMP4BoxHDLR* HDLRBox = nullptr;
			const FMP4Box*	   STBLBox = nullptr;
			const FMP4BoxSTSD* STSDBox = nullptr;
			const FMP4BoxSTTS* STTSBox = nullptr;
			const FMP4BoxCTTS* CTTSBox = nullptr;
			const FMP4BoxSTSC* STSCBox = nullptr;
			const FMP4BoxSTSZ* STSZBox = nullptr;
			const FMP4BoxSTCO* STCOBox = nullptr;
			const FMP4BoxSTSS* STSSBox = nullptr;
			const FMP4BoxUDTA* UDTABox = nullptr;
			const FMP4BoxTREF* TREFBox = nullptr;
			// Groupings
			TArray<const FMP4BoxSGPD*> SGPDBoxes;
			TArray<const FMP4BoxSBGP*> SBGPBoxes;
			// Encryption
			TArray<const FMP4BoxPSSH*>	PSSHBoxesFromInit;
			TArray<const FMP4BoxPSSH*>	PSSHBoxes;
			const FMP4BoxSCHM* SCHMBox = nullptr;
			const FMP4BoxTENC* TENCBox = nullptr;
			const FMP4BoxSENC* SENCBox = nullptr;
			const FMP4BoxSAIZ* SAIZBox = nullptr;
			const FMP4BoxSAIO* SAIOBox = nullptr;
			// Fragmented track
			const FMP4Box*	   MOOFBox = nullptr;
			const FMP4BoxTREX* TREXBox = nullptr;
			const FMP4BoxTFHD* TFHDBox = nullptr;
			const FMP4BoxTFDT* TFDTBox = nullptr;
			TArray<const FMP4BoxTRUN*>	TRUNBoxes;
			TArray<ITrack::FProducerReferenceTime> PRFTBoxes;
			// Future boxes: sbgp, sgpd, subs

			FStreamCodecInformation CodecInformation;

			int64 EmptyEditDurationInMediaTimeUnits = 0;
			int64 CompositionTimeEditOffset = 0;

			ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord CodecSpecificDataAVC;
			ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord CodecSpecificDataHEVC;
			MPEG::FESDescriptor CodecSpecificDataMP4A;
			TArray<uint8> CodecSpecificDataRAW;
			FBitrateInfo BitrateInfo;
			TOptional<FTimecode> StartTimecode;
		};


		class FAllTrackIterator : public IAllTrackIterator
		{
		public:
			virtual ~FAllTrackIterator();
			//! Returns the iterator at the current file position.
			const ITrackIterator* Current() const override;
			//! Advance iterator to point to the next sample in sequence. Returns false if there are no more samples.
			bool Next() override;
			//! Returns a list of all tracks iterators that reached EOS while iterating since the most recent call to ClearNewEOSTracks().
			void GetNewEOSTracks(TArray<const ITrackIterator*>& OutTracksThatNewlyReachedEOS) const override;
			//! Clears the list of track iterators that have reached EOS.
			void ClearNewEOSTracks() override;
			//! Returns list of all iterators.
			void GetAllIterators(TArray<const ITrackIterator*>& OutIterators) const override;

			TArray<TSharedPtrTS<ITrackIterator>>		TrackIterators;
			TSharedPtrTS<ITrackIterator>				CurrentIterator;
			TArray<const ITrackIterator*>				NewlyReachedEOS;
		};




		class FTrackInfo
		{
		public:
			~FTrackInfo();
			void DeleteTrackList();
			void AddTrack(FTrack* Track);
			int32 GetNumberOfTracks() const
			{
				return TrackList.Num();
			}
			const FTrack* GetTrackByIndex(int32 Index) const
			{
				return Index >= 0 && Index < GetNumberOfTracks() ? TrackList[Index] : nullptr;
			}

			FTrack* GetTrackByIndexMutable(int32 Index) const
			{
				return Index >= 0 && Index < GetNumberOfTracks() ? TrackList[Index] : nullptr;
			}

			FTrack* GetTrackByID(uint32 TrackID) const
			{
				for(int32 i = 0; i < TrackList.Num(); ++i)
				{
					if (TrackList[i]->GetID() == TrackID)
					{
						return TrackList[i];
					}
				}
				return nullptr;
			}

			void SetMovieDuration(const FTimeFraction& Duration)
			{
				MovieDuration.Set(Duration);
			}
			const TMediaOptionalValue<FTimeFraction>& GetMovieDuration() const
			{
				return MovieDuration;
			}

		private:
			TMediaOptionalValue<FTimeFraction>	MovieDuration;
			TArray<FTrack*>						TrackList;
		};


		void AddExtraBoxToCodecInfo(FStreamCodecInformation& InOutCodecInformation, const FMP4Box* InBox, IParserISO14496_12::FBoxType InBoxType, const FName& InAs)
		{
			const FMP4BoxIgnored* Box = static_cast<const FMP4BoxIgnored*>(InBox->FindBox(InBoxType, 3));
			if (Box && Box->GetFillerData())
			{
				TArray<uint8> Data((const uint8*)Box->GetFillerData()->Data, Box->GetFillerData()->Size);
				InOutCodecInformation.GetExtras().Set(InAs, FVariantValue(Data));
			}
		}
		void AddExtraBoxToCodecInfo(FStreamCodecInformation& InOutCodecInformation, const FMP4Box* InBox, const FName& InAs)
		{
			const FMP4BoxIgnored* Box = static_cast<const FMP4BoxIgnored*>(InBox);
			if (Box && Box->GetFillerData())
			{
				TArray<uint8> Data((const uint8*)Box->GetFillerData()->Data, Box->GetFillerData()->Size);
				InOutCodecInformation.GetExtras().Set(InAs, FVariantValue(Data));
			}
		}

		UEMediaError ParseAVCxSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox, int32 InAvcType);
		UEMediaError ParseHVC1SampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox);
		UEMediaError ParseMP4ASampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox);
		UEMediaError ParseTX3GSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox);
		UEMediaError ParseWVTTSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox);
		UEMediaError ParseSTPPSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox);
		UEMediaError ParseTMCDSampleType(IPlayerSessionServices* InPlayerSession, FTrack* InTrack, const FMP4Box* InSampleBox);
		UEMediaError ReadTMCDSample(IPlayerSessionServices* InPlayerSession, FTrack* InTrack, const FMP4Box* InSampleBox, FCancellationCheckDelegate InCheckCancellationDelegate);
		UEMediaError ParseGenericSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4BoxSTSD* STSDBox, const FMP4BoxFRMA* FRMABox);

		FMP4ParseInfo* ParsedData;

		FTrackInfo* ParsedTrackInfo;

		IElectraCodecFactoryModule* DecoderFactoryModule;
	};


	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/

	FParserISO14496_12::FTrackInfo::~FTrackInfo()
	{
		DeleteTrackList();
	}

	void FParserISO14496_12::FTrackInfo::AddTrack(FTrack* Track)
	{
		TrackList.Push(Track);
	}

	void FParserISO14496_12::FTrackInfo::DeleteTrackList()
	{
		for(int32 i = 0, iMax = TrackList.Num(); i < iMax; ++i)
		{
			delete TrackList[i];
		}
		TrackList.Empty();
	}

	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/


	FParserISO14496_12::FTrack::FTrack()
	{
	}

	uint32 FParserISO14496_12::FTrack::GetID() const
	{
		return TKHDBox ? TKHDBox->GetTrackID() : 0;
	}

	FString FParserISO14496_12::FTrack::GetName() const
	{
		if (UDTABox)
		{
			TMediaOptionalValue<FString> Name = UDTABox->GetName();
			if (Name.IsSet())
			{
				return Name.Value();
			}
		}
		return FString();
	}

	FString FParserISO14496_12::FTrack::GetNameFromHandler() const
	{
		return HDLRBox ? HDLRBox->GetHandlerName() : FString();
	}

	FTimeFraction FParserISO14496_12::FTrack::GetDuration() const
	{
		// Fragmented track?
		if (TFHDBox == nullptr)
		{
			// Not fragmented.
			if (TKHDBox && MVHDBox)
			{
				return FTimeFraction(TKHDBox->GetDuration(), MVHDBox->GetTimescale());
			}
			else if (MDHDBox)
			{
				return FTimeFraction((int64)MDHDBox->GetDuration(), MDHDBox->GetTimescale());
			}
		}
		return FTimeFraction();
	}

	IParserISO14496_12::ITrackIterator* FParserISO14496_12::FTrack::CreateIterator() const
	{
		FTrackIterator* TrackIterator = new FTrackIterator;
		if (TrackIterator)
		{
			TrackIterator->SetTrack(this);
		}
		return TrackIterator;
	}

	const TArray<uint8> FParserISO14496_12::FTrack::GetCodecSpecificData() const
	{
		static TArray<uint8> EmptyCSD;
		switch(CodecInformation.GetCodec())
		{
			case FStreamCodecInformation::ECodec::H264:
			{
				return CodecSpecificDataAVC.GetCodecSpecificData();
			}
			case FStreamCodecInformation::ECodec::H265:
			{
				return CodecSpecificDataHEVC.GetCodecSpecificData();
			}
			case FStreamCodecInformation::ECodec::AAC:
			{
				return CodecSpecificDataMP4A.GetCodecSpecificData();
			}
			case FStreamCodecInformation::ECodec::TX3G:
			case FStreamCodecInformation::ECodec::WebVTT:
			{
				return CodecSpecificDataRAW;
			}
			case FStreamCodecInformation::ECodec::Audio4CC:
			{
				if (CodecInformation.GetCodec4CC() == Utils::Make4CC('O','p','u','s'))
				{
					FVariantValue dOps = CodecInformation.GetExtras().GetValue(StreamCodecInformationOptions::DOpsBox);
					if (dOps.IsType(FVariantValue::EDataType::TypeU8Array))
					{
						return dOps.GetArray();
					}
				}
				break;
			}
			default:
			{
				break;
			}
		}
		return EmptyCSD;
	}

	const TArray<uint8> FParserISO14496_12::FTrack::GetCodecSpecificDataRAW() const
	{
		static TArray<uint8> EmptyCSD;
		switch(CodecInformation.GetCodec())
		{
			case FStreamCodecInformation::ECodec::H264:
			{
				return CodecSpecificDataAVC.GetRawData();
			}
			case FStreamCodecInformation::ECodec::H265:
			{
				return CodecSpecificDataHEVC.GetRawData();
			}
			case FStreamCodecInformation::ECodec::AAC:
			{
				return CodecSpecificDataMP4A.GetRawData();
			}
			case FStreamCodecInformation::ECodec::TX3G:
			case FStreamCodecInformation::ECodec::WebVTT:
			{
				return CodecSpecificDataRAW;
			}
			default:
			{
				break;
			}
		}
		return EmptyCSD;
	}


	const FStreamCodecInformation FParserISO14496_12::FTrack::GetCodecInformation() const
	{
		return CodecInformation;
	}

	const FParserISO14496_12::ITrack::FBitrateInfo FParserISO14496_12::FTrack::GetBitrateInfo() const
	{
		return BitrateInfo;
	}

	const BCP47::FLanguageTag& FParserISO14496_12::FTrack::GetLanguageTag() const
	{
		static BCP47::FLanguageTag Undefined { .FullLanguage=FString(TEXT("und")), .PrimaryLanguage=FString(TEXT("und"))};
		return ELNGBox ? ELNGBox->GetLanguageTag() : MDHDBox ? MDHDBox->GetLanguageTag() : Undefined;
	}

	bool FParserISO14496_12::FTrack::GetEncryptionInfo(FEncryptionInfo& OutEncryptionInfo) const
	{
		if (TENCBox)
		{
			OutEncryptionInfo.DefaultKID = TENCBox->GetDefaultKID();
			OutEncryptionInfo.DefaultIVSize = TENCBox->GetDefaultPerSampleIVSize();
			OutEncryptionInfo.DefaultIV = TENCBox->GetDefaultConstantIV();
			if (SCHMBox)
			{
				OutEncryptionInfo.Scheme = SCHMBox->GetSchemeType();
			}
			for(int32 i=0; i<PSSHBoxesFromInit.Num(); ++i)
			{
				FEncryptionInfo::FCDMInfo& cdm = OutEncryptionInfo.CDMInfos.AddDefaulted_GetRef();
				cdm.SystemID = PSSHBoxesFromInit[i]->GetSystemID();
				cdm.KIDs = PSSHBoxesFromInit[i]->GetKIDs();
				cdm.Data = PSSHBoxesFromInit[i]->GetData();
			}
			return true;
		}
		return false;
	}

	void FParserISO14496_12::FTrack::GetPSSHBoxes(TArray<TArray<uint8>>& OutBoxes, bool bFromMOOV, bool bFromMOOF) const
	{
		if (bFromMOOV)
		{
			for(int32 i=0; i<PSSHBoxesFromInit.Num(); ++i)
			{
				OutBoxes.Add(PSSHBoxesFromInit[i]->GetBoxData());
			}
		}
		if (bFromMOOF)
		{
			for(int32 i=0; i<PSSHBoxes.Num(); ++i)
			{
				OutBoxes.Add(PSSHBoxes[i]->GetBoxData());
			}
		}
	}

	void FParserISO14496_12::FTrack::GetPRFTBoxes(TArray<ITrack::FProducerReferenceTime>& OutBoxes) const
	{
		OutBoxes = PRFTBoxes;
	}


	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/

	FParserISO14496_12::FTrackIterator::FTrackIterator()
	{
	}

	FParserISO14496_12::FTrackIterator::~FTrackIterator()
	{
	}

	void FParserISO14496_12::FTrackIterator::SetTrack(const FTrack* InTrack)
	{
		Track = InTrack;
	}

	const FParserISO14496_12::ITrack* FParserISO14496_12::FTrackIterator::GetTrack() const
	{
		return Track;
	}

	int64 FParserISO14496_12::FTrackIterator::GetBaseMediaDecodeTime() const
	{
		check(Track);
		if (Track)
		{
			if (Track->TFDTBox)
			{
				return (int64)Track->TFDTBox->GetBaseMediaDecodeTime();
			}
		}
		return 0;
	}

	bool FParserISO14496_12::FTrackIterator::IsAtEOS() const
	{
		return bEOS;
	}


	UEMediaError FParserISO14496_12::FTrackIterator::StartAtFirstInternal()
	{
		// NOTE: All boxes that are referenced here have been checked to exist so accessing them is safe.

		auto SetupRAPGroup = [&]() -> void
		{
			// Sample groups
			for(int32 i=0; i<Track->SBGPBoxes.Num(); ++i)
			{
				switch(Track->SBGPBoxes[i]->GetGroupingType())
				{
					// Random access
					case EGroupType::Grouping_rap:
					{
						// ISO/IEC 14496-15 disallows `rap ` boxes greater than version 0.
						if (Track->SBGPBoxes[i]->GetBoxVersion() == 0)
						{
							// There needs to be a matching entry
							const FMP4BoxSGPD* const* SGPDBox = Track->SGPDBoxes.FindByPredicate([](const FMP4BoxSGPD* In) { return In->GetGroupingType() == EGroupType::Grouping_rap; });
							if (SGPDBox && (*SGPDBox)->GetNumberOfEntries() > 0)
							{
								// We do not care about the actual `VisualRandomAccessEntry` for now, only the fact that there is one.
								FSampleGroupIterator& sgit = GroupIterators.Add(EGroupType::Grouping_rap);
								sgit.GroupingType = EGroupType::Grouping_rap;
								sgit.SBGPBox = Track->SBGPBoxes[i];
								sgit.DefaultDescriptionIndex = (*SGPDBox)->GetDefaultSampleDescriptionIndex();
								sgit.SetFirst();
								// If the sample group says this sample is a RAP then clear the `sample_is_non_sync_sample` flag.
								if (sgit.GetCurrentGroupDescriptionIndex() != 0)
								{
									SampleFlag &= ~0x10000U;
								}
								bHaveRAPGroup = true;
							}
						}
						break;
					}
					default:
					{
						break;
					}
				}
			}
		};

		check(Track);
		if (!Track)
		{
			return UEMEDIA_ERROR_INTERNAL;
		}

		bEOS = false;
		bIsFragmented = Track->TFHDBox != nullptr;

		// Get the relevant 'elst' values over from the track for convenience.
		CompositionTimeEditOffset = Track->CompositionTimeEditOffset;
		EmptyEditDurationInMediaTimeUnits = Track->EmptyEditDurationInMediaTimeUnits;

		if (bIsFragmented)
		{
			CurrentSampleNumber = 0;
			SampleNumberInTRUN = 0;
			RemainingSamplesInTRUN = 0;
			InTRUNIndex = 0;

			if (Track->TFHDBox->IsDurationEmpty() || !Track->TRUNBoxes.Num())
			{
				bEOS = true;
				return UEMEDIA_ERROR_END_OF_STREAM;
			}
			const FMP4BoxTRUN* TRUNBox = Track->TRUNBoxes[0];

			// The timescale comes from the track's mdhd box
			Timescale = Track->MDHDBox->GetTimescale();

			// Establish the base data offset for the first sample.
			DataOffset = 0;
			// Base data offset comes first.
			if (Track->TFHDBox->HasBaseDataOffset())
			{
				DataOffset = (int64)Track->TFHDBox->GetBaseDataOffset();
			}
			// Then, if base data offset is not set the 'default-base-is-moof' flag is considered.
			else if (Track->TFHDBox->IsMoofDefaultBase())
			{
				DataOffset = Track->MOOFBox->GetStartOffset();
			}
			// Lastly the offset is the first byte in the moof box. Which happens to be the same thing as if 'default-base-is-moof' were set.
			else
			{
				DataOffset = Track->MOOFBox->GetStartOffset();
			}
			if (TRUNBox->HasSampleOffset())
			{
				DataOffset += TRUNBox->GetSampleOffset();
			}

			// Set up the first sample's flags.
			SampleFlag = TRUNBox->HasFirstSampleFlags() ? TRUNBox->GetFirstSampleFlags() :
				TRUNBox->HasSampleFlags() && TRUNBox->GetNumberOfSamples() ? (TRUNBox->GetSampleFlags().GetData())[0] :
				Track->TFHDBox->HasDefaultSampleFlags() ? Track->TFHDBox->GetDefaultSampleFlags() :
				Track->TREXBox->GetDefaultSampleFlags();

			// Get the sample size
			SampleSize = TRUNBox->HasSampleSizes() && TRUNBox->GetSampleSizes().Num() ? (TRUNBox->GetSampleSizes().GetData())[0] :
				Track->TFHDBox->HasDefaultSampleSize() ? Track->TFHDBox->GetDefaultSampleSize() :
				Track->TREXBox->GetDefaultSampleSize();

			// Get the sample duration
			SampleDuration = TRUNBox->HasSampleDurations() && TRUNBox->GetSampleDurations().Num() ? (TRUNBox->GetSampleDurations().GetData())[0] :
				Track->TFHDBox->HasDefaultSampleDuration() ? Track->TFHDBox->GetDefaultSampleDuration() :
				Track->TREXBox->GetDefaultSampleDuration();

			// Get the DTS
			// TODO: first timestamp needs to come from somewhere. What to use if there is not tfdt box??
			check(Track->TFDTBox);
			SampleDTS = Track->TFDTBox ? (int64)Track->TFDTBox->GetBaseMediaDecodeTime() : 0;
			SamplePTS = SampleDTS;
			// Adjust PTS by a sample composition offset?
			if (TRUNBox->HasSampleCompositionTimeOffsets() && TRUNBox->GetSampleCompositionTimeOffsets().Num())
			{
				SamplePTS += (TRUNBox->GetSampleCompositionTimeOffsets().GetData())[0];
			}

			// Get the sample description index
			SampleDescriptionIndex = Track->TFHDBox->HasSampleDescriptionIndex() ? Track->TFHDBox->GetSampleDescriptionIndex() :
				Track->TREXBox->GetDefaultSampleDescriptionIndex();

			// Set up the `rap` group if it exists and update the first SampleFlag accordingly.
			SetupRAPGroup();

			RemainingSamplesInTRUN = TRUNBox->GetNumberOfSamples();
			check(RemainingSamplesInTRUN != 0);
			// Run could be empty!
			if (RemainingSamplesInTRUN)
			{
				--RemainingSamplesInTRUN;
				return UEMEDIA_ERROR_OK;
			}
			else
			{
				// With an empty run move on to the next sample right away.
				return Next();
			}
		}
		else
		{
			CurrentSampleNumber = 0;

			// The timescale comes from the track's mdhd box
			Timescale = Track->MDHDBox->GetTimescale();

			NumSamplesTotal = Track->STSZBox->GetNumTotalSamples();
			// Both the 'stts' and 'stsz' boxes have to have the same number of samples in them.
			if (NumSamplesTotal != Track->STTSBox->GetNumTotalSamples())
			{
				return UEMEDIA_ERROR_FORMAT_ERROR;
			}

			// If there are no samples this could be either the empty 'moov' box of an otherwise fragmented stream or
			// a stream that has no samples, whatever sense that makes. Either way, we are done iterating this.
			if (Track->STTSBox->GetNumTotalSamples() == 0 || Track->STCOBox->GetNumberOfEntries() == 0 || Track->STSCBox->GetNumberOfEntries() == 0)
			{
				bEOS = true;
				return UEMEDIA_ERROR_END_OF_STREAM;
			}

			// The data offset from the stco box is incremented by the sample size for a number of times before a new offset is set.
			// This is because of the interleaving of samples from the contained tracks. How many times the sample size can be added
			// is determined by the stsc box.
			STSCIndex = 0;
			const FMP4BoxSTSC::FEntry& stsc0 = Track->STSCBox->GetEntry(0);
			// Establish the file data offset for the first sample.
			CurrentChunkIndex = stsc0.FirstChunk;
			DataOffset = Track->STCOBox->GetEntry(CurrentChunkIndex - 1);
			NumSamplesInChunk = stsc0.SamplesPerChunk;
			// If there is another entry in the stsc box we get the index of the next chunk at which the sample count or type changes.
			// Otherwise the current entry repeats until the end.
			if (Track->STSCBox->GetNumberOfEntries() > 1)
			{
				ChunkNumOfNextSampleChange = Track->STSCBox->GetEntry(1).FirstChunk;
			}
			else
			{
				ChunkNumOfNextSampleChange = kAllOnes32;
			}
			CurrentSampleInChunk = 0;

			// Get the sample duration and the repeat count.
			STTSIndex = 0;
			STTSCount = Track->STTSBox->GetEntry(0).SampleCount;
			STTSDelta = Track->STTSBox->GetEntry(0).SampleDelta;

			// Get the composition time offset and repeat count if a 'ctts' box is present.
			// Otherwise set up an offset of 0 with an (almost) infinite repeat count.
			CTTSIndex = 0;
			CTTSOffset = 0;
			if (Track->CTTSBox)
			{
				CTTSCount = Track->CTTSBox->GetEntry(0).SampleCount;
				CTTSOffset = Track->CTTSBox->GetEntry(0).SampleOffset;
			}
			else
			{
				CTTSCount = kAllOnes32;
			}

			// Set up the current sample values
			SampleSize = Track->STSZBox->GetSampleSize(0);
			SampleDuration = STTSDelta;
			SampleDescriptionIndex = stsc0.SampleDescriptionIndex;

			// Set up the timestamps
			int64 BaseMediaDecodeTime = 0;
			SampleDTS = 0;
			SamplePTS = SampleDTS + CTTSOffset;

			// Set up sync sample information. If there is an 'stss' box it indicates which samples are sync samples.
			// Otherwise every sample is a sync sample.
				/*
				Note: For simplicities sake we set up the SampleFlag just like in the fragmented case												;
					bit(4) reserved=0;
					unsigned int(2) is_leading;
					unsigned int(2) sample_depends_on;
					unsigned int(2) sample_is_depended_on;
					unsigned int(2) sample_has_redundancy;
					bit(3) sample_padding_value;
					bit(1) sample_is_non_sync_sample;
					unsigned int(16) sample_degradation_priority;
				*/
			if (Track->STSSBox)
			{
				STSSIndex = 0;
				// If the box is empty then there are in fact NO sync samples
				if (Track->STSSBox->GetNumberOfEntries())
				{
					// Is the first entry a sync sample?
					STSSNextSyncSampleNum = Track->STSSBox->GetEntry(STSSIndex) - 1;
					if (STSSNextSyncSampleNum == 0)
					{
						SampleFlag = 0;
						// When comes the next sync sample?
						if (Track->STSSBox->GetNumberOfEntries() > STSSIndex + 1)
						{
							++STSSIndex;
							STSSNextSyncSampleNum = Track->STSSBox->GetEntry(STSSIndex) - 1;
						}
						else
						{
							// No more sync samples
							STSSNextSyncSampleNum = kAllOnes32;
						}
					}
					else
					{
						SampleFlag = 0x10000;					// NOT a sync sample
					}
				}
				else
				{
					SampleFlag = 0x10000;						// NOT a sync sample
					STSSNextSyncSampleNum = kAllOnes32;			// there will never be a sync sample!
				}
			}
			else
			{
				STSSIndex = -1;
				SampleFlag = 0;			// If 0 then the sample *IS* a sync sample!
			}

			// Set up the `rap` group if it exists and update the first SampleFlag accordingly.
			SetupRAPGroup();

			return UEMEDIA_ERROR_OK;
		}
	}

	UEMediaError FParserISO14496_12::FTrackIterator::Next()
	{
		check(Track);
		if (!Track)
		{
			return UEMEDIA_ERROR_INTERNAL;
		}

		if (bIsFragmented)
		{
			if (Track->TFHDBox->IsDurationEmpty() || !Track->TRUNBoxes.Num())
			{
				bEOS = true;
				return UEMEDIA_ERROR_END_OF_STREAM;
			}
			const FMP4BoxTRUN* TRUNBox = Track->TRUNBoxes[InTRUNIndex];

			// Check if any samples remaining in this TRUN. If not, move on to the next TRUN.
			if (RemainingSamplesInTRUN == 0)
			{
				// Because a run can be empty we go into a loop here to find the next run that is not empty.
				while(1)
				{
					// Another TRUN box to move into?
					if (InTRUNIndex + 1 >= (uint32)Track->TRUNBoxes.Num())
					{
						bEOS = true;
						return UEMEDIA_ERROR_END_OF_STREAM;
					}
					// Move to next TRUN
					TRUNBox = Track->TRUNBoxes[++InTRUNIndex];
					// Does this TRUN have a data offset?
					if (TRUNBox->HasSampleOffset())
					{
						// Yes it does. We need to get the base offset back first.
						DataOffset = 0;
						if (Track->TFHDBox->IsMoofDefaultBase())
						{
							DataOffset = Track->MOOFBox->GetStartOffset();
						}
						else if (Track->TFHDBox->HasBaseDataOffset())
						{
							DataOffset = (int64)Track->TFHDBox->GetBaseDataOffset();
						}
						DataOffset += TRUNBox->GetSampleOffset();
					}
					else
					{
						// Samples are consecutive with no padding so we can just advance the data offset by the previous sample's byte size.
						DataOffset += SampleSize;
						SampleSize = 0;	// in case the run is empty and we loop.
					}
					SampleNumberInTRUN = 0;
					RemainingSamplesInTRUN = TRUNBox->GetNumberOfSamples();
					// Run could be empty!
					if (RemainingSamplesInTRUN)
					{
						--RemainingSamplesInTRUN;
						break;
					}
				}
			}
			else
			{
				--RemainingSamplesInTRUN;
				++SampleNumberInTRUN;
				// Samples are consecutive with no padding so we can just advance the data offset by the previous sample's byte size.
				DataOffset += SampleSize;
			}
			++CurrentSampleNumber;

			// Sample DTS advances by the previous sample's duration.
			SampleDTS += SampleDuration;

			// Set up the sample flags.
			SampleFlag = SampleNumberInTRUN == 0 && TRUNBox->HasFirstSampleFlags() ? TRUNBox->GetFirstSampleFlags() :
				TRUNBox->HasSampleFlags() && TRUNBox->GetNumberOfSamples() ? (TRUNBox->GetSampleFlags().GetData())[SampleNumberInTRUN] :
				Track->TFHDBox->HasDefaultSampleFlags() ? Track->TFHDBox->GetDefaultSampleFlags() :
				Track->TREXBox->GetDefaultSampleFlags();

			// Get the sample size
			SampleSize = TRUNBox->HasSampleSizes() && TRUNBox->GetSampleSizes().Num() ? (TRUNBox->GetSampleSizes().GetData())[SampleNumberInTRUN] :
				Track->TFHDBox->HasDefaultSampleSize() ? Track->TFHDBox->GetDefaultSampleSize() :
				Track->TREXBox->GetDefaultSampleSize();

			// Get the sample duration
			SampleDuration = TRUNBox->HasSampleDurations() && TRUNBox->GetSampleDurations().Num() ? (TRUNBox->GetSampleDurations().GetData())[SampleNumberInTRUN] :
				Track->TFHDBox->HasDefaultSampleDuration() ? Track->TFHDBox->GetDefaultSampleDuration() :
				Track->TREXBox->GetDefaultSampleDuration();

			// PTS
			SamplePTS = SampleDTS;
			// Adjust PTS by a sample composition offset?
			if (TRUNBox->HasSampleCompositionTimeOffsets() && TRUNBox->GetSampleCompositionTimeOffsets().Num())
			{
				SamplePTS += (TRUNBox->GetSampleCompositionTimeOffsets().GetData())[SampleNumberInTRUN];
			}

			// Advance the grouping iterators
			for(auto &grpit : GroupIterators)
			{
				grpit.Value.Advance();
			}
			// If we have a RAP grouping we need to look at it now
			if (bHaveRAPGroup)
			{
				// If the sample group says this sample is a RAP then clear the `sample_is_non_sync_sample` flag.
				if (GroupIterators[EGroupType::Grouping_rap].GetCurrentGroupDescriptionIndex() != 0)
				{
					SampleFlag &= ~0x10000U;
				}
			}

			return UEMEDIA_ERROR_OK;
		}
		else
		{
			if (CurrentSampleNumber + 1 >= NumSamplesTotal)
			{
				bEOS = true;
				return UEMEDIA_ERROR_END_OF_STREAM;
			}
			++CurrentSampleNumber;

			// Advance the data offset by the sample size until we reach a new chunk.
			DataOffset += SampleSize;

			// Did we reach the end of the current chunk?
			if (++CurrentSampleInChunk == NumSamplesInChunk)
			{
				// Next chunk run.
				CurrentSampleInChunk = 0;

				// Did we reach the next chunk
				if (++CurrentChunkIndex == ChunkNumOfNextSampleChange)
				{
					// Switch to next chunk.
					++STSCIndex;

					const FMP4BoxSTSC::FEntry& stsc = Track->STSCBox->GetEntry(STSCIndex);
					check(CurrentChunkIndex == stsc.FirstChunk);
					DataOffset = Track->STCOBox->GetEntry(CurrentChunkIndex - 1);
					NumSamplesInChunk = stsc.SamplesPerChunk;
					SampleDescriptionIndex = stsc.SampleDescriptionIndex;
					if (Track->STSCBox->GetNumberOfEntries() > STSCIndex + 1)
					{
						ChunkNumOfNextSampleChange = Track->STSCBox->GetEntry(STSCIndex + 1).FirstChunk;
					}
					else
					{
						ChunkNumOfNextSampleChange = kAllOnes32;
					}
				}
				else
				{
					// No change in both sample description and number of samples per chunk. Just move to the next chunk's data offset.
					check(Track->STCOBox->GetNumberOfEntries() >= (int32)CurrentChunkIndex);
					DataOffset = Track->STCOBox->GetEntry(CurrentChunkIndex - 1);
				}
			}

			// Get the next sample's size.
			SampleSize = Track->STSZBox->GetSampleSize(CurrentSampleNumber);

			// Advance the DTS by the sample's duration.
			SampleDTS += SampleDuration;

			// Get the next sample duration.
			check(STTSCount != 0);
			if (--STTSCount == 0)
			{
				// Repeat count exhausted. Move on to the next entry if there is one, otherwise we're done.
				++STTSIndex;
				check(STTSIndex < Track->STTSBox->GetNumberOfEntries());
				if (STTSIndex < Track->STTSBox->GetNumberOfEntries())
				{
					STTSCount = Track->STTSBox->GetEntry(STTSIndex).SampleCount;
					STTSDelta = Track->STTSBox->GetEntry(STTSIndex).SampleDelta;
				}
			}
			SampleDuration = STTSDelta;

			// Get the next composition time offset
			check(CTTSCount != 0);
			if (--CTTSCount == 0)
			{
				check(Track->CTTSBox);
				// Repeat count exhausted. Move on to the next entry if there is one, otherwise we're done.
				++CTTSIndex;
				check(CTTSIndex < Track->CTTSBox->GetNumberOfEntries());
				if (CTTSIndex < Track->CTTSBox->GetNumberOfEntries())
				{
					CTTSCount = Track->CTTSBox->GetEntry(CTTSIndex).SampleCount;
					CTTSOffset = Track->CTTSBox->GetEntry(CTTSIndex).SampleOffset;
				}
			}

			SamplePTS = SampleDTS + CTTSOffset;

			// Check for sync sample (if there is sync sample information)
			if (STSSIndex >= 0)
			{
				// Reached the next sync sample?
				if (CurrentSampleNumber == STSSNextSyncSampleNum)
				{
					SampleFlag = 0;			// IS a sync sample
					STSSNextSyncSampleNum = Track->STSSBox->GetNumberOfEntries() > STSSIndex + 1 ? Track->STSSBox->GetEntry(++STSSIndex) - 1 : kAllOnes32;
				}
				else
				{
					SampleFlag = 0x10000;	// NOT a sync sample
				}
			}

			// Advance the grouping iterators
			for(auto &grpit : GroupIterators)
			{
				grpit.Value.Advance();
			}
			// If we have a RAP grouping we need to look at it now
			if (bHaveRAPGroup)
			{
				// If the sample group says this sample is a RAP then clear the `sample_is_non_sync_sample` flag.
				if (GroupIterators[EGroupType::Grouping_rap].GetCurrentGroupDescriptionIndex() != 0)
				{
					SampleFlag &= ~0x10000U;
				}
			}

			return UEMEDIA_ERROR_OK;
		}
	}

	UEMediaError FParserISO14496_12::FTrackIterator::StartAtFirst(bool bNeedSyncSample)
	{
		UEMediaError err = StartAtFirstInternal();
		if (err == UEMEDIA_ERROR_OK)
		{
			for(; err == UEMEDIA_ERROR_OK; err = Next())
			{
				if (!bNeedSyncSample || (bNeedSyncSample && IsSyncSample()))
				{
					return UEMEDIA_ERROR_OK;
				}
			}
			err = UEMEDIA_ERROR_INSUFFICIENT_DATA;
		}
		return err;
	}

	UEMediaError FParserISO14496_12::FTrackIterator::StartAtTime(const FTimeValue& AtTime, FParserISO14496_12::ITrackIterator::ESearchMode SearchMode, bool bNeedSyncSample)
	{
		// Note: This code is perhaps not very optimal in that we're doing a linear search here. For unfragmented files we could look at the stss box
		//       to locate the sync sample and get its dts/pts relatively quickly from the stts/ctts boxes.
		//       If there's even a sidx box we could just look at that.
		//       Otherwise for fragmented files it's tricky to get at the sample flags to find the sync samples unless we would assume there is only
		//       a single sync sample at the start of the fragment.
		// So for now we take the shortcut to just iterate.
		UEMediaError err = StartAtFirstInternal();
		if (err == UEMEDIA_ERROR_OK)
		{
			FTrackIterator Best;

			// Get the local media time in its timescale units.
			int64 localTime = AtTime.GetAsTimebase(GetTimescale());

			for(; err == UEMEDIA_ERROR_OK; err = Next())
			{
				if (!bNeedSyncSample || (bNeedSyncSample && IsSyncSample()))
				{
					if (!Best.IsValid())
					{
						Best.AssignFrom(*this);
					}

					// As long as the time is before the local timestamp we're looking for we're taking it.
					int64 thisTime = GetPTS();
					if (thisTime < localTime)
					{
						Best.AssignFrom(*this);
					}
					else
					{
						// Otherwise, if we hit the time dead on or search for a time that's larger we're done.
						if (thisTime == localTime || SearchMode == FParserISO14496_12::ITrackIterator::ESearchMode::After)
						{
							return UEMEDIA_ERROR_OK;
						}
						else if (SearchMode == FParserISO14496_12::ITrackIterator::ESearchMode::Before)
						{
							// Is the best time we found so far less than or equal to what we're looking for?
							if (Best.GetPTS() <= localTime)
							{
								// Yes, get the best values back into this instance and return.
								AssignFrom(Best);
								return UEMEDIA_ERROR_OK;
							}
							// It is possible for the PTS to be offset. Check against the DTS.
							if (Best.GetDTS() <= localTime)
							{
								// Yes, get the best values back into this instance and return.
								AssignFrom(Best);
								return UEMEDIA_ERROR_OK;
							}
							// Didn't find what we're looking for.
							return UEMEDIA_ERROR_INSUFFICIENT_DATA;
						}
						else
						{
							// Should pick the closest one.
							if (localTime - Best.GetPTS() < GetPTS() - localTime)
							{
								AssignFrom(Best);
							}
							return UEMEDIA_ERROR_OK;
						}
					}
				}
			}
			// If there is only a single sync sample that was before the time we're looking for we get here.
			if (Best.IsValid())
			{
				if (SearchMode == FParserISO14496_12::ITrackIterator::ESearchMode::Before && Best.GetPTS() > localTime)
				{
					return UEMEDIA_ERROR_INSUFFICIENT_DATA;
				}
				AssignFrom(Best);
				return UEMEDIA_ERROR_OK;
			}
		}
		return err;
	}


	uint32 FParserISO14496_12::FTrackIterator::GetSampleNumber() const
	{
		return CurrentSampleNumber;
	}

	int64 FParserISO14496_12::FTrackIterator::GetDTS() const
	{
		return SampleDTS + EmptyEditDurationInMediaTimeUnits;
	}

	int64 FParserISO14496_12::FTrackIterator::GetPTS() const
	{
		return SamplePTS + EmptyEditDurationInMediaTimeUnits - CompositionTimeEditOffset;
	}

	int64 FParserISO14496_12::FTrackIterator::GetDuration() const
	{
		return SampleDuration;
	}

	uint32 FParserISO14496_12::FTrackIterator::GetTimescale() const
	{
		return Timescale ? Timescale : Track ? Track->MDHDBox->GetTimescale() : 0;
	}

	bool FParserISO14496_12::FTrackIterator::IsSyncSample() const
	{
		// Note: Depending on the codec an I-frame might also be a sync sample, in which
		//       case `sample_depends_on` (mask 0x03000000) would be set to 2.
		return (SampleFlag & 0x10000) == 0;
	}

	int64 FParserISO14496_12::FTrackIterator::GetSampleSize() const
	{
		return SampleSize;
	}

	int64 FParserISO14496_12::FTrackIterator::GetSampleFileOffset() const
	{
		return DataOffset;
	}

	int64 FParserISO14496_12::FTrackIterator::GetRawDTS() const
	{
		return SampleDTS;
	}

	int64 FParserISO14496_12::FTrackIterator::GetRawPTS() const
	{
		return SamplePTS;
	}

	int64 FParserISO14496_12::FTrackIterator::GetCompositionTimeEdit() const
	{
		return CompositionTimeEditOffset;
	}

	int64 FParserISO14496_12::FTrackIterator::GetEmptyEditOffset() const
	{
		return EmptyEditDurationInMediaTimeUnits;
	}

	bool FParserISO14496_12::FTrackIterator::GetEncryptionInfo(ElectraCDM::FMediaCDMSampleInfo& OutSampleEncryptionInfo) const
	{
		if (!Track || (!Track->SENCBox && !Track->TENCBox))
		{
			return false;
		}
		if (Track->TENCBox)
		{
			OutSampleEncryptionInfo.DefaultKID = Track->TENCBox->GetDefaultKID();
			OutSampleEncryptionInfo.IV = Track->TENCBox->GetDefaultConstantIV();
			if (Track->TENCBox->HasDefaultCryptBlockValues())
			{
				OutSampleEncryptionInfo.Pattern.PatternType = 1;
				OutSampleEncryptionInfo.Pattern.CryptByteBlock = Track->TENCBox->GetDefaultCryptByteBlock();
				OutSampleEncryptionInfo.Pattern.SkipByteBlock = Track->TENCBox->GetDefaultSkipByteBlock();
			}
		}
		if (Track->SENCBox)
		{
			const TArray<FMP4BoxSENC::FEntry>& SencEntries = Track->SENCBox->GetEntries();
			int32 Index = bIsFragmented ? SampleNumberInTRUN : CurrentSampleNumber;
			check((int32) Index < SencEntries.Num());
			if ((int32) Index >= SencEntries.Num())
			{
				return false;
			}
			if (SencEntries[Index].IV.Num())
			{
				OutSampleEncryptionInfo.IV = SencEntries[Index].IV;
			}
			OutSampleEncryptionInfo.SubSamples = SencEntries[Index].SubSamples;
		}
		OutSampleEncryptionInfo.Scheme4CC = Track->SCHMBox ? Track->SCHMBox->GetSchemeType() : 0;
		return true;
	}


	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/


	FParserISO14496_12::FParserISO14496_12()
		: ParsedData(nullptr)
		, ParsedTrackInfo(nullptr)
	{
		DecoderFactoryModule = static_cast<IElectraCodecFactoryModule*>(FModuleManager::Get().GetModule(TEXT("ElectraCodecFactory")));
	}

	FParserISO14496_12::~FParserISO14496_12()
	{
		delete ParsedTrackInfo;
		delete ParsedData;
	}

	UEMediaError FParserISO14496_12::ParseHeader(IGenericDataReader* InDataReader, IBoxCallback* InBoxParseCallback, IPlayerSessionServices* PlayerSession, const IParserISO14496_12* InOptionalInitSegment)
	{
		if (!InDataReader || !InBoxParseCallback)
		{
			return UEMEDIA_ERROR_BAD_ARGUMENTS;
		}

		// New parse or continuing a previous?
		if (ParsedData == nullptr)
		{
			ParsedData = new FMP4ParseInfo;
			if (!ParsedData)
			{
				return UEMEDIA_ERROR_OOM;
			}
		}

		const FParserISO14496_12* OptionalInitSegment = static_cast<const FParserISO14496_12*>(InOptionalInitSegment);
		FMP4BoxReader Reader(InDataReader, InBoxParseCallback);
		UEMediaError ParseError = ParsedData->Parse(&Reader, PlayerSession, OptionalInitSegment ? OptionalInitSegment->ParsedData : nullptr);
		return ParseError;
	}

	UEMediaError FParserISO14496_12::ParseAVCxSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox, int32 InAvcType)
	{
		check(InAvcType == 1 || InAvcType == 3);
		const FMP4BoxVisualSampleEntry* VisualSampleEntry = static_cast<const FMP4BoxVisualSampleEntry*>(SampleBox);
		check(VisualSampleEntry->GetNumberOfChildren() > 0);
		if (VisualSampleEntry->GetNumberOfChildren() > 0)
		{
			// There may be several entries. Usually there is an 'avcC' (required) and optional boxes like 'pasp', 'btrt', 'clap', etc.
			bool bGotVideoFormat = false;
			for(int32 j = 0, jMax = VisualSampleEntry->GetNumberOfChildren(); j < jMax; ++j)
			{
				const FMP4Box* SampleEntry = VisualSampleEntry->GetChildBox(j);
				switch(SampleEntry->GetType())
				{
					case FMP4Box::kBox_avcC:
					{
						const FMP4BoxAVCC* AVCCBox = static_cast<const FMP4BoxAVCC*>(SampleEntry);
						if (AVCCBox->GetDecoderConfigurationRecord().IsSet())
						{
							Track->CodecSpecificDataAVC = AVCCBox->GetDecoderConfigurationRecord().GetValue();
							Track->CodecInformation.SetStreamType(EStreamType::Video);
							Track->CodecInformation.SetCodec(FStreamCodecInformation::ECodec::H264);
							Track->CodecInformation.SetCodec4CC(InAvcType == 1 ? Utils::Make4CC('a','v','c','1') : Utils::Make4CC('a','v','c','3'));
							Track->CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());
							Track->CodecInformation.SetDecoderConfigRecord(Track->CodecSpecificDataAVC.GetRawData());
							Track->CodecInformation.SetCodecSpecificData(Track->CodecSpecificDataAVC.GetCodecSpecificData());
							if (Track->CodecSpecificDataAVC.GetSequenceParameterSets().IsEmpty())
							{
								if (InAvcType == 3)
								{
									Track->CodecInformation.SetProfile(Track->CodecSpecificDataAVC.GetAVCProfileIndication());
									Track->CodecInformation.SetProfileLevel(Track->CodecSpecificDataAVC.GetAVCLevelIndication());
									Track->CodecInformation.SetProfileConstraints(Track->CodecSpecificDataAVC.GetProfileCompatibility());
									Track->CodecInformation.SetCodecSpecifierRFC6381(Track->CodecSpecificDataAVC.GetCodecSpecifierRFC6381(TEXT("avc3")));
									if (Track->TKHDBox)
									{
										Track->CodecInformation.SetResolution(FStreamCodecInformation::FResolution(Track->TKHDBox->GetWidth(), Track->TKHDBox->GetHeight()));
										Track->CodecInformation.SetCrop(FStreamCodecInformation::FCrop(0, 0, Track->TKHDBox->GetWidth() % 16, Track->TKHDBox->GetHeight() % 16));
										Track->CodecInformation.SetAspectRatio(FStreamCodecInformation::FAspectRatio(1, 1));
									}

									bGotVideoFormat = true;
								}
								else
								{
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
							}
							else
							{
								ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet sps;
								if (ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(sps, Track->CodecSpecificDataAVC.GetSequenceParameterSets()[0]))
								{
									int32 CropL, CropR, CropT, CropB;
									sps.GetCrop(CropL, CropR, CropT, CropB);
									Track->CodecInformation.SetResolution(FStreamCodecInformation::FResolution(sps.GetWidth() - CropL - CropR, sps.GetHeight() - CropT - CropB));
									Track->CodecInformation.SetCrop(FStreamCodecInformation::FCrop(CropL, CropT, CropR, CropB));
									FStreamCodecInformation::FAspectRatio ar;
									sps.GetAspect(ar.Width, ar.Height);
									Track->CodecInformation.SetAspectRatio(ar);
									// Have a valid fixed framerate?
									if (sps.GetTiming().Denom)
									{
										Track->CodecInformation.SetFrameRate(FTimeFraction(sps.GetTiming().Num, sps.GetTiming().Denom));
									}
									else if (Track->MDHDBox && Track->STTSBox && Track->MDHDBox->GetTimescale() && Track->STTSBox->GetNumberOfEntries())
									{
										// We assume the frame rate to be constant.
										Track->CodecInformation.SetFrameRate(FTimeFraction(Track->MDHDBox->GetTimescale(), Track->STTSBox->GetEntry(0).SampleDelta));
									}
									else if (Track->MDHDBox && Track->STSZBox && Track->STSZBox->GetNumTotalSamples() && Track->MDHDBox->GetDuration() && Track->MDHDBox->GetTimescale())
									{
										// We assume the frame rate to be constant. Or at least the average to be close to what
										// a common constant framerate would be.
										TBigInt<128> n((int64)Track->STSZBox->GetNumTotalSamples() * (int32)Track->MDHDBox->GetTimescale() * 1000);
										TBigInt<128> d((int64)Track->MDHDBox->GetDuration());
										n /= d;
										Track->CodecInformation.SetFrameRate(FTimeFraction(n.ToInt(), 1000));
									}
									Track->CodecInformation.SetProfile(sps.profile_idc);
									Track->CodecInformation.SetProfileLevel(sps.level_idc);
									uint8 Constraints = (sps.constraint_set0_flag << 7) | (sps.constraint_set1_flag << 6) | (sps.constraint_set2_flag << 5) | (sps.constraint_set3_flag << 4) | (sps.constraint_set4_flag << 3) | (sps.constraint_set5_flag << 2);
									Track->CodecInformation.SetProfileConstraints(Constraints);
									Track->CodecInformation.SetCodecSpecifierRFC6381(FString::Printf(TEXT("avc%d.%02x%02x%02x"), InAvcType, sps.profile_idc, Constraints, sps.level_idc));
									bGotVideoFormat = true;
								}
								else
								{
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
							}
						}
						else
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}
						break;
					}
					case FMP4Box::kBox_btrt:
					{
						const FMP4BoxBTRT* BTRTBox = static_cast<const FMP4BoxBTRT*>(SampleEntry);
						Track->BitrateInfo.BufferSizeDB = BTRTBox->GetBufferSizeDB();
						Track->BitrateInfo.MaxBitrate = BTRTBox->GetMaxBitrate();
						Track->BitrateInfo.AvgBitrate = BTRTBox->GetAverageBitrate();
						break;
					}
					case FMP4Box::kBox_pasp:
					{
						break;
					}
					default:
					{
						break;
					}
				}
			}
			// Check that we got the required information
			if (bGotVideoFormat)
			{
				return UEMEDIA_ERROR_OK;
			}
		}
		return UEMEDIA_ERROR_FORMAT_ERROR;
	}

	UEMediaError FParserISO14496_12::ParseHVC1SampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox)
	{
		const FMP4BoxVisualSampleEntry* VisualSampleEntry = static_cast<const FMP4BoxVisualSampleEntry*>(SampleBox);
		check(VisualSampleEntry->GetNumberOfChildren() > 0);
		if (VisualSampleEntry->GetNumberOfChildren() > 0)
		{
			// There may be several entries. Usually there is an 'hvcC' (required) and optional boxes like 'pasp', 'btrt', 'clap', etc.
			bool bGotVideoFormat = false;
			for(int32 j = 0, jMax = VisualSampleEntry->GetNumberOfChildren(); j < jMax; ++j)
			{
				const FMP4Box* SampleEntry = VisualSampleEntry->GetChildBox(j);
				switch(SampleEntry->GetType())
				{
					case FMP4Box::kBox_hvcC:
					{
						const FMP4BoxHVCC* HVCCBox = static_cast<const FMP4BoxHVCC*>(SampleEntry);
						if (HVCCBox->GetDecoderConfigurationRecord().IsSet())
						{
							Track->CodecSpecificDataHEVC = HVCCBox->GetDecoderConfigurationRecord().GetValue();
							Track->CodecInformation.SetStreamType(EStreamType::Video);
							Track->CodecInformation.SetCodec(FStreamCodecInformation::ECodec::H265);
							Track->CodecInformation.SetCodec4CC(Utils::Make4CC('h','e','v','1'));
							Track->CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());
							Track->CodecInformation.SetProfileSpace(Track->CodecSpecificDataHEVC.GetGeneralProfileSpace());
							Track->CodecInformation.SetProfileTier(Track->CodecSpecificDataHEVC.GetGeneralTierFlag());
							Track->CodecInformation.SetProfile(Track->CodecSpecificDataHEVC.GetGeneralProfileIDC());
							Track->CodecInformation.SetProfileLevel(Track->CodecSpecificDataHEVC.GetGeneralLevelIDC());
							Track->CodecInformation.SetProfileConstraints(Track->CodecSpecificDataHEVC.GetGeneralConstraintIndicatorFlags());
							Track->CodecInformation.SetProfileCompatibilityFlags(Track->CodecSpecificDataHEVC.GetGeneralProfileCompatibilityFlags());
							Track->CodecInformation.SetDecoderConfigRecord(Track->CodecSpecificDataHEVC.GetRawData());
							Track->CodecInformation.SetCodecSpecificData(Track->CodecSpecificDataHEVC.GetCodecSpecificData());
							if (Track->CodecSpecificDataHEVC.GetSequenceParameterSets().IsEmpty())
							{
								if (Track->TKHDBox)
								{
									Track->CodecInformation.SetResolution(FStreamCodecInformation::FResolution(Track->TKHDBox->GetWidth(), Track->TKHDBox->GetHeight()));
									Track->CodecInformation.SetAspectRatio(FStreamCodecInformation::FAspectRatio(1, 1));
									// Crop values are not easily determined as there is no single macroblock (coding tree unit) size.
									// Do not set any.
										//Track->CodecInformation.SetCrop(FStreamCodecInformation::FCrop(0, 0, Track->TKHDBox->GetWidth() % 16, Track->TKHDBox->GetHeight() % 16));
									Track->CodecInformation.SetCodecSpecifierRFC6381(Track->CodecSpecificDataHEVC.GetCodecSpecifierRFC6381(TEXT("hev1")));
									bGotVideoFormat = true;
								}
								else
								{
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
							}
							else if (!Track->CodecSpecificDataHEVC.GetParsedSequenceParameterSets().IsEmpty())
							{
								const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet& sps = Track->CodecSpecificDataHEVC.GetParsedSequenceParameterSets()[0];
								int32 CropL, CropR, CropT, CropB;
								sps.GetCrop(CropL, CropR, CropT, CropB);
								Track->CodecInformation.SetResolution(FStreamCodecInformation::FResolution(sps.GetWidth() - CropL - CropR, sps.GetHeight() - CropT - CropB));
								Track->CodecInformation.SetCrop(FStreamCodecInformation::FCrop(CropL, CropT, CropR, CropB));
								FStreamCodecInformation::FAspectRatio ar;
								sps.GetAspect(ar.Width, ar.Height);
								Track->CodecInformation.SetAspectRatio(ar);
								Track->CodecInformation.SetFrameRate(sps.GetTiming().Denom ? FTimeFraction(sps.GetTiming().Num, sps.GetTiming().Denom) : FTimeFraction());
								Track->CodecInformation.SetCodecSpecifierRFC6381(sps.GetRFC6381(TEXT("hvc1")));
								Track->CodecInformation.SetCodec4CC(Utils::Make4CC('h','v','c','1'));
								bGotVideoFormat = true;
							}
							else
							{
								return UEMEDIA_ERROR_FORMAT_ERROR;
							}
						}
						else
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}
						break;
					}
					case FMP4Box::kBox_btrt:
					{
						const FMP4BoxBTRT* BTRTBox = static_cast<const FMP4BoxBTRT*>(SampleEntry);
						Track->BitrateInfo.BufferSizeDB = BTRTBox->GetBufferSizeDB();
						Track->BitrateInfo.MaxBitrate = BTRTBox->GetMaxBitrate();
						Track->BitrateInfo.AvgBitrate = BTRTBox->GetAverageBitrate();
						break;
					}
					case FMP4Box::kBox_pasp:
					{
						break;
					}
					default:
					{
						break;
					}
				}
			}
			// Check that we got the required information
			if (bGotVideoFormat)
			{
				return UEMEDIA_ERROR_OK;
			}
		}
		return UEMEDIA_ERROR_FORMAT_ERROR;
	}

	UEMediaError FParserISO14496_12::ParseMP4ASampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox)
	{
		auto HandleESDS = [PlayerSession, Track](const FMP4BoxAudioSampleEntry* AudioSampleEntry, const FMP4BoxESDS* ESDSBox, bool& bGotAudioFormat, bool& bIsSupported) -> bool
		{
			if (!ESDSBox)
			{
				bIsSupported = false;
				return false;
			}
			Track->CodecSpecificDataMP4A = ESDSBox->GetESDescriptor();
			bool bOk = Track->CodecSpecificDataMP4A.Parse();
			if (bOk)
			{
				// Is this AAC audio?
				if (Track->CodecSpecificDataMP4A.GetObjectTypeID() == MPEG::FESDescriptor::FObjectTypeID::MPEG4_Audio &&
					Track->CodecSpecificDataMP4A.GetStreamType() == MPEG::FESDescriptor::FStreamType::AudioStream)
				{
					Track->CodecInformation.SetStreamType(EStreamType::Audio);
					Track->CodecInformation.SetCodec(FStreamCodecInformation::ECodec::AAC);
					Track->CodecInformation.SetCodec4CC(Utils::Make4CC('m','p','4','a'));
					Track->CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());

					ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord ConfigRecord;
					if (ConfigRecord.ParseFrom(Track->CodecSpecificDataMP4A.GetCodecSpecificData().GetData(), Track->CodecSpecificDataMP4A.GetCodecSpecificData().Num()))
					{
						Track->CodecInformation.SetCodecSpecifierRFC6381(FString::Printf(TEXT("mp4a.40.%d"), ConfigRecord.ExtAOT ? ConfigRecord.ExtAOT : ConfigRecord.AOT));
						Track->CodecInformation.SetSamplingRate(ConfigRecord.ExtSamplingFrequency ? ConfigRecord.ExtSamplingFrequency : ConfigRecord.SamplingRate);
						Track->CodecInformation.SetChannelConfiguration(ConfigRecord.ChannelConfiguration);
						Track->CodecInformation.SetNumberOfChannels(ElectraDecodersUtil::MPEG::AACUtils::GetNumberOfChannelsFromChannelConfiguration(ConfigRecord.ChannelConfiguration));
						// We assume that all platforms can decode PS (parametric stereo). As such we change the channel count from mono to stereo
						// to convey the _decoded_ format, not the source format.
						if (ConfigRecord.ChannelConfiguration == 1 && ConfigRecord.PSSignal > 0)
						{
							Track->CodecInformation.SetNumberOfChannels(2);
						}
						Track->CodecInformation.GetExtras().Set(StreamCodecInformationOptions::SamplesPerBlock, FVariantValue(ConfigRecord.SBRSignal > 0 ? (int64)2048 : (int64)1024));
					}

					// Typically an mp4a track will not have a 'btrt' box because the bitrate is stored in the DecoderConfigDescriptor.
					Track->BitrateInfo.BufferSizeDB = Track->CodecSpecificDataMP4A.GetBufferSize();
					Track->BitrateInfo.MaxBitrate = Track->CodecSpecificDataMP4A.GetMaxBitrate();
					Track->BitrateInfo.AvgBitrate = Track->CodecSpecificDataMP4A.GetAvgBitrate();
					Track->CodecInformation.SetBitrate(Track->BitrateInfo.MaxBitrate);

					bGotAudioFormat = true;

					if (PlayerSession)
					{
						IPlayerStreamFilter* StreamFilter = PlayerSession->GetStreamFilter();
						if (StreamFilter && !StreamFilter->CanDecodeStream(Track->CodecInformation))
						{
							bIsSupported = false;
						}
					}
				}
				// Is this MPEG-1 audio?
				else if (Track->CodecSpecificDataMP4A.GetObjectTypeID() == MPEG::FESDescriptor::FObjectTypeID::MPEG1_Audio &&
					Track->CodecSpecificDataMP4A.GetStreamType() == MPEG::FESDescriptor::FStreamType::AudioStream)
				{
					Track->CodecInformation.SetStreamType(EStreamType::Audio);
					Track->CodecInformation.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
					Track->CodecInformation.SetCodec4CC(Utils::Make4CC('m','p','g','a'));
					Track->CodecInformation.SetProfile(1);
					Track->CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());
					Track->CodecInformation.SetCodecSpecifierRFC6381(FString::Printf(TEXT("mp4a.6b")));
					Track->CodecInformation.SetSamplingRate((int32) AudioSampleEntry->GetSampleRate());
					Track->CodecInformation.SetNumberOfChannels(AudioSampleEntry->GetChannelCount());

					// Typically an mp4a track will not have a 'btrt' box because the bitrate is stored in the DecoderConfigDescriptor.
					Track->BitrateInfo.BufferSizeDB = Track->CodecSpecificDataMP4A.GetBufferSize();
					Track->BitrateInfo.MaxBitrate = Track->CodecSpecificDataMP4A.GetMaxBitrate();
					Track->BitrateInfo.AvgBitrate = Track->CodecSpecificDataMP4A.GetAvgBitrate();
					Track->CodecInformation.SetBitrate(Track->BitrateInfo.MaxBitrate);

					bGotAudioFormat = true;

					if (PlayerSession)
					{
						IPlayerStreamFilter* StreamFilter = PlayerSession->GetStreamFilter();
						if (StreamFilter && !StreamFilter->CanDecodeStream(Track->CodecInformation))
						{
							bIsSupported = false;
						}
					}
				}
				else
				{
					// Not expected format. Ignore this track.
					bIsSupported = false;
					// Got the info, just not the format we want.
					bGotAudioFormat = true;
				}
			}
			return bOk;
		};


		const FMP4BoxAudioSampleEntry* AudioSampleEntry = static_cast<const FMP4BoxAudioSampleEntry*>(Track->STSDBox->GetChildBox(0));
		check(AudioSampleEntry->GetNumberOfChildren() > 0);
		if (AudioSampleEntry->GetNumberOfChildren() > 0)
		{
			// There may be several entries. Usually there is the required sample format box (eg 'mp4a') and optional boxes like 'btrt'
			bool bGotAudioFormat = false;
			bool bIsSupported = true;

			if (SampleBox && SampleBox->GetParentBox())
			{
				if (AudioSampleEntry->HaveFormatSpecificFlags())
				{
					Track->CodecInformation.GetExtras().Set(StreamCodecInformationOptions::FormatSpecificFlags, FVariantValue((int64) AudioSampleEntry->GetFormatSpecificFlags()));
				}
				AddExtraBoxToCodecInfo(Track->CodecInformation, SampleBox->GetParentBox(), FMP4Box::kBox_chan, StreamCodecInformationOptions::ChanBox);
			}

			for(int32 j=0, jMax=AudioSampleEntry->GetNumberOfChildren(); j<jMax; ++j)
			{
				const FMP4Box* SampleEntry = AudioSampleEntry->GetChildBox(j);
				switch(SampleEntry->GetType())
				{
					case FMP4Box::kBox_esds:
					{
						const FMP4BoxESDS* ESDSBox = static_cast<const FMP4BoxESDS*>(AudioSampleEntry->GetChildBox(0));
						bool bOk = HandleESDS(AudioSampleEntry, ESDSBox, bGotAudioFormat, bIsSupported);
						if (!bOk)
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}
						break;
					}
					case FMP4Box::kBox_wave:
					{
						const FMP4BoxESDS* ESDSBox = static_cast<const FMP4BoxESDS*>(AudioSampleEntry->FindBox(FMP4Box::kBox_esds, 1));
						bool bOk = HandleESDS(AudioSampleEntry, ESDSBox, bGotAudioFormat, bIsSupported);
						if (!bOk)
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}
						break;
					}
					case FMP4Box::kBox_btrt:
					{
						const FMP4BoxBTRT* BTRTBox = static_cast<const FMP4BoxBTRT*>(SampleEntry);
						Track->BitrateInfo.BufferSizeDB = BTRTBox->GetBufferSizeDB();
						Track->BitrateInfo.MaxBitrate = BTRTBox->GetMaxBitrate();
						Track->BitrateInfo.AvgBitrate = BTRTBox->GetAverageBitrate();
						break;
					}
					default:
					{
						break;
					}
				}
			}
			// Check that we got the required information
			if (bGotAudioFormat)
			{
				return bIsSupported ? UEMEDIA_ERROR_OK : UEMEDIA_ERROR_NOT_SUPPORTED;
			}
		}
		return UEMEDIA_ERROR_FORMAT_ERROR;
	}

	UEMediaError FParserISO14496_12::ParseTX3GSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox)
	{
		// Need to check if there is a subtitle decoder capable of decoding this.
		if (ISubtitleDecoder::IsSupported(TEXT(""), TEXT("tx3g")))
		{
			const FMP4BoxTX3GSampleEntry* SubtitleSampleEntry = static_cast<const FMP4BoxTX3GSampleEntry*>(Track->STSDBox->GetChildBox(0));

			Track->CodecSpecificDataRAW = SubtitleSampleEntry->GetRawBoxData();
			Track->CodecInformation.SetStreamType(EStreamType::Subtitle);
			Track->CodecInformation.SetCodec(FStreamCodecInformation::ECodec::TX3G);
			Track->CodecInformation.SetCodec4CC(Utils::Make4CC('t','x','3','g'));
			Track->CodecInformation.SetCodecSpecificData(Track->CodecSpecificDataRAW);
			Track->CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());
			Track->CodecInformation.SetCodecSpecifierRFC6381(TEXT("tx3g"));
			Track->CodecInformation.SetResolution(FStreamCodecInformation::FResolution(SubtitleSampleEntry->GetWidth(), SubtitleSampleEntry->GetHeight()));
			Track->CodecInformation.SetTranslation(FStreamCodecInformation::FTranslation(SubtitleSampleEntry->GetTranslationX(), SubtitleSampleEntry->GetTranslationY()));
			Track->CodecInformation.SetFrameRate(FTimeFraction(1, SubtitleSampleEntry->GetTimescale()));
			return UEMEDIA_ERROR_OK;
		}
		return UEMEDIA_ERROR_NOT_SUPPORTED;
	}

	UEMediaError FParserISO14496_12::ParseWVTTSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox)
	{
		// Need to check if there is a subtitle decoder capable of decoding this.
		if (ISubtitleDecoder::IsSupported(TEXT(""), TEXT("wvtt")))
		{
			const FMP4BoxWVTTSampleEntry* SubtitleSampleEntry = static_cast<const FMP4BoxWVTTSampleEntry*>(Track->STSDBox->GetChildBox(0));

			Track->CodecSpecificDataRAW = SubtitleSampleEntry->GetRawBoxData();
			Track->CodecInformation.SetStreamType(EStreamType::Subtitle);
			Track->CodecInformation.SetCodec(FStreamCodecInformation::ECodec::WebVTT);
			Track->CodecInformation.SetCodec4CC(Utils::Make4CC('w','v','t','t'));
			Track->CodecInformation.SetCodecSpecificData(Track->CodecSpecificDataRAW);
			Track->CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());
			Track->CodecInformation.SetCodecSpecifierRFC6381(TEXT("wvtt"));
			Track->CodecInformation.SetResolution(FStreamCodecInformation::FResolution(SubtitleSampleEntry->GetWidth(), SubtitleSampleEntry->GetHeight()));
			Track->CodecInformation.SetTranslation(FStreamCodecInformation::FTranslation(SubtitleSampleEntry->GetTranslationX(), SubtitleSampleEntry->GetTranslationY()));
			Track->CodecInformation.SetFrameRate(FTimeFraction(1, SubtitleSampleEntry->GetTimescale()));
			return UEMEDIA_ERROR_OK;
		}
		return UEMEDIA_ERROR_NOT_SUPPORTED;
	}

	UEMediaError FParserISO14496_12::ParseSTPPSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4Box* SampleBox)
	{
		// Need to check if there is a subtitle decoder capable of decoding this.
		if (ISubtitleDecoder::IsSupported(TEXT(""), TEXT("stpp")))
		{
			const FMP4BoxXMLSubtitleSampleEntry* SubtitleSampleEntry = static_cast<const FMP4BoxXMLSubtitleSampleEntry*>(Track->STSDBox->GetChildBox(0));

			Track->CodecSpecificDataRAW = SubtitleSampleEntry->GetRawBoxData();
			Track->CodecInformation.SetStreamType(EStreamType::Subtitle);
			Track->CodecInformation.SetCodec(FStreamCodecInformation::ECodec::TTML);
			Track->CodecInformation.SetCodec4CC(Utils::Make4CC('s','t','p','p'));
			Track->CodecInformation.SetCodecSpecificData(Track->CodecSpecificDataRAW);
			Track->CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());
			Track->CodecInformation.SetCodecSpecifierRFC6381(TEXT("stpp"));
			Track->CodecInformation.SetResolution(FStreamCodecInformation::FResolution(SubtitleSampleEntry->GetWidth(), SubtitleSampleEntry->GetHeight()));
			Track->CodecInformation.SetTranslation(FStreamCodecInformation::FTranslation(SubtitleSampleEntry->GetTranslationX(), SubtitleSampleEntry->GetTranslationY()));
			Track->CodecInformation.SetFrameRate(FTimeFraction(1, SubtitleSampleEntry->GetTimescale()));
			return UEMEDIA_ERROR_OK;
		}
		return UEMEDIA_ERROR_NOT_SUPPORTED;
	}

	UEMediaError FParserISO14496_12::ParseTMCDSampleType(IPlayerSessionServices* InPlayerSession, FTrack* InTrack, const FMP4Box* InSampleBox)
	{
		check(InSampleBox->GetType() == FMP4Box::kSample_tmcd);
		const FMP4BoxTimecodeSampleEntry* TMCDBox = static_cast<const FMP4BoxTimecodeSampleEntry*>(InSampleBox);

		const bool bDropFrame = TMCDBox->GetFlags() & FMP4BoxTimecodeSampleEntry::FFlags::DropFrame ? true : false;
		const FFrameRate FrameRate = bDropFrame ? FFrameRate(TMCDBox->GetTimeScale(), TMCDBox->GetFrameDuration()) : FFrameRate(TMCDBox->GetNumberOfFrames(), 1);

		InTrack->CodecInformation.SetFrameRate(FTimeFraction(FrameRate.Numerator, FrameRate.Denominator));
		InTrack->CodecInformation.SetCodec4CC(TMCDBox->GetType()); // Identify as 'tmcd' track, used to find again in second pass.
		return UEMEDIA_ERROR_OK;
	}

	UEMediaError FParserISO14496_12::ReadTMCDSample(IPlayerSessionServices* InPlayerSession, FTrack* InTrack, const FMP4Box* InSampleBox, FCancellationCheckDelegate InCheckCancellationDelegate)
	{
		if (!InPlayerSession || !InPlayerSession->GetManifestReader() || !InSampleBox)
		{
			return UEMEDIA_ERROR_NOT_SUPPORTED;
		}

		// We will need the stco, stsc and stsz boxes. We only need the first entry.
		if (!InTrack || !InTrack->STCOBox || InTrack->STCOBox->GetNumberOfEntries() == 0
			|| !InTrack->STSCBox || InTrack->STSCBox->GetNumberOfEntries() == 0
			|| !InTrack->STSZBox)
		{
			return UEMEDIA_ERROR_NOT_SUPPORTED;
		}

		check(InSampleBox->GetType() == FMP4Box::kSample_tmcd);
		const FMP4BoxTimecodeSampleEntry* TMCDBox = static_cast<const FMP4BoxTimecodeSampleEntry*>(InSampleBox);

		// Obtain the first sample only -> Fetch the first chunk and read the first sample.

		const int64 ChunkOffset = InTrack->STCOBox->GetEntry(0);
		//const uint32 SamplesPerChunk = InTrack->STSCBox->GetEntry(0).SamplesPerChunk;
		const uint32 SampleSizeInBytes = InTrack->STSZBox->GetSampleSize(0);
		// Note: since we need only the first sample, load only "sample size" to get the first one.
		const uint32 ChunkSize = SampleSizeInBytes; //SamplesPerChunk * SampleSizeInBytes;

		const FString ManifestURL = InPlayerSession->GetManifestReader()->GetURL();

		UtilsMP4::FMP4ChunkLoader ChunkLoader;

		// Propagate the worker thread's abort flag.
		const UtilsMP4::FMP4ChunkLoader::FCancellationCheckDelegate CancellationCheckDelegate
			= UtilsMP4::FMP4ChunkLoader::FCancellationCheckDelegate::CreateLambda([InCheckCancellationDelegate]()
			{
				return InCheckCancellationDelegate.Execute();
			});

		if (const TSharedPtrTS<FWaitableBuffer> ChunkData = ChunkLoader.LoadChunk(ChunkOffset, ChunkSize, InPlayerSession->GetHTTPManager(), InPlayerSession->GetHTTPResponseCache(), ManifestURL, CancellationCheckDelegate))
		{
			if (ChunkData.IsValid() && ChunkData->Num() >= SampleSizeInBytes)
			{
				const uint32* Data = reinterpret_cast<const uint32*>(ChunkData->GetLinearReadData());

				// Note: ignoring counter flag. Even if set to 0, which would mean QT int32 format (HHMMSSFF),
				// the timecode is still a frame number format (as if Counter set to 1).
				const FFrameNumber FrameNumber(static_cast<int32>(MEDIA_FROM_BIG_ENDIAN(Data[0])));

				const bool bDropFrame = TMCDBox->GetFlags() & FMP4BoxTimecodeSampleEntry::FFlags::DropFrame ? true : false;
				const bool bRollover = TMCDBox->GetFlags() & FMP4BoxTimecodeSampleEntry::FFlags::Max24Hour ? true : false;

				const FFrameRate FrameRate = bDropFrame ? FFrameRate(TMCDBox->GetTimeScale(), TMCDBox->GetFrameDuration()) : FFrameRate(TMCDBox->GetNumberOfFrames(), 1);

				// Need to convert to seconds first for rollover implementation below.
				const double TimecodeInSeconds = FrameRate.AsSeconds(FFrameTime(FrameNumber));

				// Convert to time code (apply roll over, etc).
				InTrack->StartTimecode = FTimecode(TimecodeInSeconds, FrameRate, bDropFrame, bRollover);

				// Expose the timecode in the track's metadata.
				const FVariantValue StartTimecodeValue(InTrack->StartTimecode->ToString());
				const FVariantValue StartTimecodeFrameRate(FrameRate.ToPrettyText().ToString());
				InTrack->CodecInformation.GetExtras().Set(MediaInfoStartTimecodeValue, StartTimecodeValue);
				InTrack->CodecInformation.GetExtras().Set(MediaInfoStartTimecodeFrameRate, StartTimecodeFrameRate);
				return UEMEDIA_ERROR_OK;
			}
		}
		return UEMEDIA_ERROR_NOT_SUPPORTED;
	}

	UEMediaError FParserISO14496_12::ParseGenericSampleType(IPlayerSessionServices* PlayerSession, FTrack* Track, const FMP4BoxSTSD* STSDBox, const FMP4BoxFRMA* FRMABox)
	{
		if (Track && STSDBox)
		{
			FStreamCodecInformation CodecInformation;
			TMap<FString, FVariant> Options;
			switch(STSDBox->GetSampleType())
			{
				case FMP4BoxSTSD::ESampleType::Video:
				{
					const FMP4BoxVisualSampleEntry* VisualSampleEntry = static_cast<const FMP4BoxVisualSampleEntry*>(STSDBox->GetChildBox(0));
					if (!VisualSampleEntry)
					{
						return UEMEDIA_ERROR_NOT_SUPPORTED;
					}
					CodecInformation.SetStreamType(EStreamType::Video);
					CodecInformation.SetCodec(FStreamCodecInformation::ECodec::Video4CC);
					CodecInformation.SetCodec4CC(FRMABox ? FRMABox->GetDataFormat() : VisualSampleEntry->GetType());
					CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());
					CodecInformation.SetResolution(FStreamCodecInformation::FResolution(VisualSampleEntry->GetWidth(), VisualSampleEntry->GetHeight()));
					CodecInformation.SetCodecSpecifierRFC6381(Utils::Printable4CC(CodecInformation.GetCodec4CC()));
					// Try to get the framerate
					if (Track->MDHDBox && Track->STTSBox && Track->STTSBox->GetNumTotalSamples())
					{
						FTimeFraction Rate(Track->MDHDBox->GetTimescale(), Track->STTSBox->GetEntry(0).SampleDelta);
						CodecInformation.SetFrameRate(Rate);
					}
					// Look at the additional boxes
					for(int32 i=0,iMax=VisualSampleEntry->GetNumberOfChildren(); i<iMax; ++i)
					{
						const FMP4Box* SampleEntry = VisualSampleEntry->GetChildBox(i);
						switch(SampleEntry->GetType())
						{
							case FMP4Box::kBox_pasp:
							{
								const FMP4BoxPASP* PASP = static_cast<const FMP4BoxPASP*>(SampleEntry);
								CodecInformation.SetAspectRatio(FStreamCodecInformation::FAspectRatio((int32)PASP->GetHSpacing(), (int32)PASP->GetVSpacing()));
								break;
							}
							case FMP4Box::kBox_btrt:
							{
								const FMP4BoxBTRT* BTRTBox = static_cast<const FMP4BoxBTRT*>(SampleEntry);
								Track->BitrateInfo.BufferSizeDB = BTRTBox->GetBufferSizeDB();
								Track->BitrateInfo.MaxBitrate = BTRTBox->GetMaxBitrate();
								Track->BitrateInfo.AvgBitrate = BTRTBox->GetAverageBitrate();
								break;
							}
							default:
							{
								// Note: adding the box via the filler data only works as long as these boxes are not explicitly handled in this module!
								FString BoxName = Utils::Printable4CC(SampleEntry->GetType());
								BoxName.Append(TEXT("_box"));
								AddExtraBoxToCodecInfo(CodecInformation, SampleEntry, FName(*BoxName));
								break;
							}
						}
					}
					CodecInformation.UpdateFromExtraBoxes();

					if (PlayerSession)
					{
						IPlayerStreamFilter* StreamFilter = PlayerSession->GetStreamFilter();
						if (StreamFilter && StreamFilter->CanDecodeStream(CodecInformation))
						{
							Track->CodecInformation = CodecInformation;
							return UEMEDIA_ERROR_OK;
						}
					}
					break;
				}
				case FMP4BoxSTSD::ESampleType::Audio:
				{
					const FMP4BoxAudioSampleEntry* AudioSampleEntry = static_cast<const FMP4BoxAudioSampleEntry*>(STSDBox->GetChildBox(0));
					if (!AudioSampleEntry)
					{
						return UEMEDIA_ERROR_NOT_SUPPORTED;
					}

					CodecInformation.SetStreamType(EStreamType::Audio);
					CodecInformation.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
					CodecInformation.SetCodec4CC(FRMABox ? FRMABox->GetDataFormat() : AudioSampleEntry->GetType());
					CodecInformation.SetStreamLanguageTag(Track->GetLanguageTag());
					CodecInformation.SetSamplingRate((int32) AudioSampleEntry->GetSampleRate());
					CodecInformation.SetNumberOfChannels(AudioSampleEntry->GetChannelCount());
					CodecInformation.SetCodecSpecifierRFC6381(Utils::Printable4CC(CodecInformation.GetCodec4CC()));

					CodecInformation.GetExtras().Set(StreamCodecInformationOptions::SampleSize, FVariantValue((int64)AudioSampleEntry->GetSampleSize()));
					if (AudioSampleEntry->HaveFormatSpecificFlags())
					{
						CodecInformation.GetExtras().Set(StreamCodecInformationOptions::FormatSpecificFlags, FVariantValue((int64) AudioSampleEntry->GetFormatSpecificFlags()));
						CodecInformation.GetExtras().Set(StreamCodecInformationOptions::ConstBytesPerAudioPacket, FVariantValue((int64)AudioSampleEntry->GetConstBytesPerAudioPacket()));
						CodecInformation.GetExtras().Set(StreamCodecInformationOptions::ConstLPCMFramesPerAudioPacket, FVariantValue((int64)AudioSampleEntry->GetConstLPCMFramesPerAudioPacket()));
					}

					// Look at the additional boxes
					for(int32 i=0,iMax=AudioSampleEntry->GetNumberOfChildren(); i<iMax; ++i)
					{
						const FMP4Box* SampleEntry = AudioSampleEntry->GetChildBox(i);
						switch (SampleEntry->GetType())
						{
							case FMP4Box::kBox_btrt:
							{
								const FMP4BoxBTRT* BTRTBox = static_cast<const FMP4BoxBTRT*>(SampleEntry);
								Track->BitrateInfo.BufferSizeDB = BTRTBox->GetBufferSizeDB();
								Track->BitrateInfo.MaxBitrate = BTRTBox->GetMaxBitrate();
								Track->BitrateInfo.AvgBitrate = BTRTBox->GetAverageBitrate();
								break;
							}
							default:
							{
								// Note: adding the box via the filler data only works as long as these boxes are not explicitly handled in this module!
								FString BoxName = Utils::Printable4CC(SampleEntry->GetType());
								BoxName.Append(TEXT("_box"));
								AddExtraBoxToCodecInfo(CodecInformation, SampleEntry, FName(*BoxName));
								break;
							}
						}
					}
					CodecInformation.UpdateFromExtraBoxes();

					if (PlayerSession)
					{
						IPlayerStreamFilter* StreamFilter = PlayerSession->GetStreamFilter();
						if (StreamFilter && StreamFilter->CanDecodeStream(CodecInformation))
						{
							Track->CodecInformation = CodecInformation;
							return UEMEDIA_ERROR_OK;
						}
					}
					break;
				}
				case FMP4BoxSTSD::ESampleType::Subtitles:
				{
					break;
				}
				default:
				{
					break;
				}
			}

		}
		return UEMEDIA_ERROR_NOT_SUPPORTED;
	}

	UEMediaError FParserISO14496_12::PrepareTracks(IPlayerSessionServices* PlayerSession, TSharedPtrTS<const IParserISO14496_12> OptionalMP4InitSegment)
	{
		delete ParsedTrackInfo;
		ParsedTrackInfo = nullptr;
		// With no parsed data we stop with an insufficient data error.
		if (!ParsedData || !ParsedData->GetRootBox())
		{
			return UEMEDIA_ERROR_INSUFFICIENT_DATA;
		}

		TUniquePtr<FTrackInfo> NewParsedTrackInfo(new FTrackInfo);
		if (!NewParsedTrackInfo)
		{
			return UEMEDIA_ERROR_OOM;
		}

		const FMP4Box* MOOVBox = nullptr;
		if (OptionalMP4InitSegment.IsValid())
		{
			const FParserISO14496_12* InitSegParser = static_cast<const FParserISO14496_12*>(OptionalMP4InitSegment.Get());
			check(InitSegParser->ParsedData);
			if (InitSegParser->ParsedData && InitSegParser->ParsedData->GetCurrentMoovBox())
			{
				MOOVBox = InitSegParser->ParsedData->GetCurrentMoovBox();
			}
			else
			{
				// FIXME: ignore this or throw an error?
			}
		}
		if (!MOOVBox)
		{
			MOOVBox = ParsedData->GetCurrentMoovBox();
		}
		if (MOOVBox)
		{
			// Get the mvhd box if it exists. We do this explicitly to get to the timescale value we need for
			// the trak boxes in case the boxes are not ordered.
			const FMP4BoxMVHD* MVHDBox = static_cast<const FMP4BoxMVHD*>(MOOVBox->FindBox(FMP4Box::kBox_mvhd, 0));
			uint32 Timescale = 0;
			if (MVHDBox)
			{
				NewParsedTrackInfo->SetMovieDuration(MVHDBox->GetDuration());
				Timescale = MVHDBox->GetTimescale();
			}

			// Is this a fragmented stream?
			const FMP4BoxMVEX* MVEXBox = static_cast<const FMP4BoxMVEX*>(MOOVBox->FindBox(FMP4Box::kBox_mvex, 0));

			for(int32 i=0, iMax=MOOVBox->GetNumberOfChildren(); i<iMax; ++i)
			{
				const FMP4Box* Box = MOOVBox->GetChildBox(i);
				switch(Box->GetType())
				{
					case FMP4Box::kBox_trak:
					{
						TUniquePtr<FTrack> Track(new FTrack);
						if (!Track)
						{
							return UEMEDIA_ERROR_OOM;
						}

						Track->MVHDBox = MVHDBox;
						Track->ELSTBox = static_cast<const FMP4BoxELST*>(Box->GetBoxPath(FMP4Box::kBox_edts, FMP4Box::kBox_elst));
						Track->TKHDBox = static_cast<const FMP4BoxTKHD*>(Box->GetBoxPath(FMP4Box::kBox_tkhd));
						Track->MDIABox = static_cast<const FMP4Box*>    (Box->GetBoxPath(FMP4Box::kBox_mdia));
						if (!Track->TKHDBox || !Track->MDIABox)
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}
						Track->MDHDBox = static_cast<const FMP4BoxMDHD*>(Track->MDIABox->GetBoxPath(FMP4Box::kBox_mdhd));
						Track->HDLRBox = static_cast<const FMP4BoxHDLR*>(Track->MDIABox->GetBoxPath(FMP4Box::kBox_hdlr));
						Track->STBLBox = static_cast<const FMP4Box*>    (Track->MDIABox->GetBoxPath(FMP4Box::kBox_minf, FMP4Box::kBox_stbl));
						if (!Track->MDHDBox || !Track->HDLRBox || !Track->STBLBox)
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}
						Track->ELNGBox = static_cast<const FMP4BoxELNG*>(Track->MDIABox->GetBoxPath(FMP4Box::kBox_elng));
						Track->UDTABox = static_cast<const FMP4BoxUDTA*>(Box->GetBoxPath(FMP4Box::kBox_udta));
						Track->TREFBox = static_cast<const FMP4BoxTREF*>(Box->GetBoxPath(FMP4Box::kBox_tref));
						// TODO: Check the dinf->dref->url box to not reference any external media!

						// Get the composition time offset and the empty edit from the edit list. Those are necessary to correct the PTS values from
						// positive values in the 'ctts' box as well as general mapping of the content.
						if (Track->ELSTBox)
						{
							bool bHaveEmptyEdit = false;
							bool bHaveEmptyEditWarned = false;
							bool bHaveOffsetEdit = false;
							bool bHaveOffsetEditWarned = false;
							for(int32 nEntry=0; nEntry<Track->ELSTBox->GetNumberOfEntries(); ++nEntry)
							{
								const FMP4BoxELST::FEntry& ee = Track->ELSTBox->GetEntry(nEntry);
								// As per 8.6.6.1 the entries we are interested in have a rate of 1, so we use that to narrow down the entries to look at.
								if (ee.MediaRateInteger == 1 && ee.MediaRateFraction == 0)
								{
									// Empty edit?
									if (ee.MediaTime == -1)
									{
										// An empty edit means to insert blank, nonexistent material into the timeline.
										if (!bHaveEmptyEdit)
										{
											// We can only have a single empty edit.
											bHaveEmptyEdit = true;
										// Removed for now since this is not correct. This only means to say that there is empty content on the timeline
										// but NOT that the content that does exist is delayed by that much.
										//	Track->EmptyEditDurationInMediaTimeUnits = Track->MDHDBox->GetTimescale() == Timescale ? ee.SegmentDuration : FTimeFraction(ee.SegmentDuration, Timescale).GetAsTimebase(Track->MDHDBox->GetTimescale());
										}
										else if (!bHaveEmptyEditWarned)
										{
											// Emit a warning.
											bHaveEmptyEditWarned = true;
											UE_LOGF(LogElectraMP4Parser, Warning, "Found another empty edit in 'elst' box, ignoring");
										}
									}
									else if (ee.MediaTime >= 0)
									{
										// This defines the part of the media that is mapped to the track timeline's 0 point.
										// This is commonly used to adjust the composition time offset and we treat it as such.
										if (!bHaveOffsetEdit)
										{
											bHaveOffsetEdit = true;
											Track->CompositionTimeEditOffset = ee.MediaTime;
										}
										else if (!bHaveOffsetEditWarned)
										{
											bHaveOffsetEditWarned = true;
											UE_LOGF(LogElectraMP4Parser, Warning, "Found another media edit in 'elst' box, ignoring");
										}
									}
								}
							}
						}


						// Fragmented track?
						if (MVEXBox)
						{
							Track->TREXBox = MVEXBox->FindTREXForTrackID(Track->TKHDBox->GetTrackID());
						}

						// Get the 'stbl' box to determine the type of track and codec.
						// We use the codec since this is already unique.
						Track->STSDBox = static_cast<const FMP4BoxSTSD*>(Track->STBLBox->GetBoxPath(FMP4Box::kBox_stsd));
						if (!Track->STSDBox)
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}

						// If this is not a format we support we do not add the track.
						if (!Track->STSDBox->IsSupportedFormat())
						{
							continue;
						}

						// Get the sample description boxes from the 'stbl'
						for(int32 j=0, jMax=Track->STBLBox->GetNumberOfChildren(); j<jMax; ++j)
						{
							const FMP4Box* SampleInfoBox = Track->STBLBox->GetChildBox(j);
							switch(SampleInfoBox->GetType())
							{
								case FMP4Box::kBox_stts:
									Track->STTSBox = static_cast<const FMP4BoxSTTS*>(SampleInfoBox);
									break;
								case FMP4Box::kBox_ctts:
									Track->CTTSBox = static_cast<const FMP4BoxCTTS*>(SampleInfoBox);
									break;
								case FMP4Box::kBox_stsc:
									Track->STSCBox = static_cast<const FMP4BoxSTSC*>(SampleInfoBox);
									break;
								case FMP4Box::kBox_stsz:
									Track->STSZBox = static_cast<const FMP4BoxSTSZ*>(SampleInfoBox);
									break;
								case FMP4Box::kBox_stco:
								case FMP4Box::kBox_co64:
									Track->STCOBox = static_cast<const FMP4BoxSTCO*>(SampleInfoBox);
									break;
								case FMP4Box::kBox_stss:
									Track->STSSBox = static_cast<const FMP4BoxSTSS*>(SampleInfoBox);
									break;
								default:
									break;
							}
						}
						// All but the 'ctts' and 'stss' boxes have to be there.
						if (!Track->STTSBox || !Track->STSCBox || !Track->STSZBox || !Track->STCOBox)
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}

						// For the time being we ask that only one entry is in the 'stsd' box.
						if (Track->STSDBox->GetNumberOfChildren() != 1)
						{
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}

						bool bIsSupported = true;
						const FMP4Box* STSDFirstChildBox = Track->STSDBox->GetChildBox(0);
						switch(STSDFirstChildBox->GetType())
						{
							case FMP4Box::kSample_encv:
							{
								const FMP4BoxFRMA* FRMABox = static_cast<const FMP4BoxFRMA*>(STSDFirstChildBox->GetBoxPath(FMP4Box::kBox_sinf, FMP4Box::kBox_frma));
								const FMP4BoxSCHM* SCHMBox = static_cast<const FMP4BoxSCHM*>(STSDFirstChildBox->GetBoxPath(FMP4Box::kBox_sinf, FMP4Box::kBox_schm));
								if (!FRMABox || !SCHMBox)
								{
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
								// Expecting encryption to be 'cenc' or 'cbcs'
								if (SCHMBox->GetSchemeType() != FMP4Box::kEncryptionScheme_cenc && SCHMBox->GetSchemeType() != FMP4Box::kEncryptionScheme_cbcs)
								{
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
								// Expecting avc1/avc3/hvc1/hev1 for video right now.
								UEMediaError Error = UEMEDIA_ERROR_FORMAT_ERROR;
								if (FRMABox->GetDataFormat() == FMP4Box::kSample_avc1)
								{
									Error = ParseAVCxSampleType(PlayerSession, Track.Get(), STSDFirstChildBox, 1);
								}
								else if (FRMABox->GetDataFormat() == FMP4Box::kSample_avc3)
								{
									Error = ParseAVCxSampleType(PlayerSession, Track.Get(), STSDFirstChildBox, 3);
								}
								else if (FRMABox->GetDataFormat() == FMP4Box::kSample_hvc1 || FRMABox->GetDataFormat() == FMP4Box::kSample_hev1)
								{
									Error = ParseHVC1SampleType(PlayerSession, Track.Get(), STSDFirstChildBox);
								}
								else
								{
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
								/*if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
								{
									bIsSupported = false;
								}
								else*/ if (Error != UEMEDIA_ERROR_OK)
								{
									return Error;
								}
								// Remember the encryption boxes.
								Track->SCHMBox = SCHMBox;
								Track->TENCBox = static_cast<const FMP4BoxTENC*>(STSDFirstChildBox->GetBoxPath(FMP4Box::kBox_sinf, FMP4Box::kBox_schi, FMP4Box::kBox_tenc));
								Track->SAIZBox = static_cast<const FMP4BoxSAIZ*>(Box->FindBox(FMP4Box::kBox_saiz, 4));
								Track->SAIOBox = static_cast<const FMP4BoxSAIO*>(Box->FindBox(FMP4Box::kBox_saio, 4));
								Track->SENCBox = static_cast<const FMP4BoxSENC*>(Box->FindBox(FMP4Box::kBox_senc, 4));
								if (Track->SENCBox)
								{
									Error = const_cast<FMP4BoxSENC*>(Track->SENCBox)->Parse(Track->TENCBox, Track->SAIZBox, Track->SAIOBox);
									if (Error != UEMEDIA_ERROR_OK)
									{
										return Error;
									}
								}
								TArray<const FMP4Box*> AllPSSHBoxes;
								MOOVBox->GetAllBoxInstances(AllPSSHBoxes, FMP4Box::kBox_pssh);
								for(int32 nPSSHs=0; nPSSHs<AllPSSHBoxes.Num(); ++nPSSHs)
								{
									Track->PSSHBoxesFromInit.Add(static_cast<const FMP4BoxPSSH*>(AllPSSHBoxes[nPSSHs]));
								}
								break;
							}
							case FMP4Box::kSample_enca:
							{
								const FMP4BoxFRMA* FRMABox = static_cast<const FMP4BoxFRMA*>(STSDFirstChildBox->GetBoxPath(FMP4Box::kBox_sinf, FMP4Box::kBox_frma));
								const FMP4BoxSCHM* SCHMBox = static_cast<const FMP4BoxSCHM*>(STSDFirstChildBox->GetBoxPath(FMP4Box::kBox_sinf, FMP4Box::kBox_schm));
								if (!FRMABox || !SCHMBox)
								{
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
								// Encryption needs to be 'cenc' or 'cbcs'
								if (SCHMBox->GetSchemeType() != FMP4Box::kEncryptionScheme_cenc && SCHMBox->GetSchemeType() != FMP4Box::kEncryptionScheme_cbcs)
								{
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
								UEMediaError Error = UEMEDIA_ERROR_FORMAT_ERROR;
								if (FRMABox->GetDataFormat() == FMP4Box::kSample_mp4a)
								{
									// Parse the sample
									Error = ParseMP4ASampleType(PlayerSession, Track.Get(), STSDFirstChildBox);
									if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
									{
										bIsSupported = false;
									}
									else if (Error != UEMEDIA_ERROR_OK)
									{
										return Error;
									}
								}
								else
								{
									Error = ParseGenericSampleType(PlayerSession, Track.Get(), Track->STSDBox, FRMABox);
									if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
									{
										bIsSupported = false;
									}
									else if (Error != UEMEDIA_ERROR_OK) //-V547
									{
										return Error;
									}
								}
								// Remember the encryption boxes.
								Track->SCHMBox = SCHMBox;
								Track->TENCBox = static_cast<const FMP4BoxTENC*>(STSDFirstChildBox->GetBoxPath(FMP4Box::kBox_sinf, FMP4Box::kBox_schi, FMP4Box::kBox_tenc));
								Track->SAIZBox = static_cast<const FMP4BoxSAIZ*>(Box->FindBox(FMP4Box::kBox_saiz, 4));
								Track->SAIOBox = static_cast<const FMP4BoxSAIO*>(Box->FindBox(FMP4Box::kBox_saio, 4));
								Track->SENCBox = static_cast<const FMP4BoxSENC*>(Box->FindBox(FMP4Box::kBox_senc, 4));
								if (Track->SENCBox)
								{
									Error = const_cast<FMP4BoxSENC*>(Track->SENCBox)->Parse(Track->TENCBox, Track->SAIZBox, Track->SAIOBox);
									if (Error != UEMEDIA_ERROR_OK)
									{
										return Error;
									}
								}
								TArray<const FMP4Box*> AllPSSHBoxes;
								MOOVBox->GetAllBoxInstances(AllPSSHBoxes, FMP4Box::kBox_pssh);
								for(int32 nPSSHs=0; nPSSHs<AllPSSHBoxes.Num(); ++nPSSHs)
								{
									Track->PSSHBoxesFromInit.Add(static_cast<const FMP4BoxPSSH*>(AllPSSHBoxes[nPSSHs]));
								}
								break;
							}
							case FMP4Box::kSample_avc1:
							case FMP4Box::kSample_avc3:
							{
								UEMediaError Error = ParseAVCxSampleType(PlayerSession, Track.Get(), STSDFirstChildBox, STSDFirstChildBox->GetType() == FMP4Box::kSample_avc1 ? 1 : 3);
								/*
								if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
								{
									bIsSupported = false;
								}
								else*/ if (Error != UEMEDIA_ERROR_OK)
								{
									return Error;
								}
								break;
							}
							case FMP4Box::kSample_hvc1:
							case FMP4Box::kSample_hev1:
							{
								UEMediaError Error = ParseHVC1SampleType(PlayerSession, Track.Get(), STSDFirstChildBox);
								/*
								if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
								{
									bIsSupported = false;
								}
								else*/ if (Error != UEMEDIA_ERROR_OK)
								{
									return Error;
								}
								break;
							}
							case FMP4Box::kSample_mp4a:
							{
								UEMediaError Error = ParseMP4ASampleType(PlayerSession, Track.Get(), STSDFirstChildBox);
								if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
								{
									bIsSupported = false;
								}
								else if (Error != UEMEDIA_ERROR_OK)
								{
									return Error;
								}
								break;
							}
							case FMP4Box::kSample_stpp:
							{
								UEMediaError Error = ParseSTPPSampleType(PlayerSession, Track.Get(), STSDFirstChildBox);
								if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
								{
									bIsSupported = false;
								}
								else if (Error != UEMEDIA_ERROR_OK) //-V547
								{
									return Error;
								}
								break;
							}
							case FMP4Box::kSample_tx3g:
							{
								UEMediaError Error = ParseTX3GSampleType(PlayerSession, Track.Get(), STSDFirstChildBox);
								if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
								{
									bIsSupported = false;
								}
								else if (Error != UEMEDIA_ERROR_OK) //-V547
								{
									return Error;
								}
								break;
							}
							case FMP4Box::kSample_wvtt:
							{
								UEMediaError Error = ParseWVTTSampleType(PlayerSession, Track.Get(), STSDFirstChildBox);
								if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
								{
									bIsSupported = false;
								}
								else if (Error != UEMEDIA_ERROR_OK) //-V547
								{
									return Error;
								}
								break;
							}
							case FMP4Box::kSample_tmcd:
							{
								UEMediaError Error = ParseTMCDSampleType(PlayerSession, Track.Get(), STSDFirstChildBox);
								if (Error == UEMEDIA_ERROR_NOT_SUPPORTED) //-V547
								{
									bIsSupported = false;
								}
								else if (Error != UEMEDIA_ERROR_OK) //-V547
								{
									return Error;
								}
								break;
							}
							default:
							{
								UEMediaError Error = ParseGenericSampleType(PlayerSession, Track.Get(), Track->STSDBox, nullptr);
								if (Error == UEMEDIA_ERROR_NOT_SUPPORTED)
								{
									bIsSupported = false;
								}
								else if (Error != UEMEDIA_ERROR_OK) //-V547
								{
									return Error;
								}
								break;
							}
						}

						// Locate all sample grouping boxes.
						TArray<const FMP4Box*> ListOfBoxes;
						Track->STBLBox->GetAllBoxInstances(ListOfBoxes, FMP4Box::kBox_sgpd);
						for(int32 nBox=0; nBox<ListOfBoxes.Num(); ++nBox)
						{
							Track->SGPDBoxes.Add(static_cast<const FMP4BoxSGPD*>(ListOfBoxes[nBox]));
						}
						ListOfBoxes.Empty();
						Track->STBLBox->GetAllBoxInstances(ListOfBoxes, FMP4Box::kBox_sbgp);
						for(int32 nBox=0; nBox<ListOfBoxes.Num(); ++nBox)
						{
							Track->SBGPBoxes.Add(static_cast<const FMP4BoxSBGP*>(ListOfBoxes[nBox]));
						}

						// So far all went well. Add the track to the map.
						if (bIsSupported)
						{
							NewParsedTrackInfo->AddTrack(Track.Release());
						}
						break;
					}
					default:
						break;
				}
			}
		}

		// Fragmented?
		const FMP4Box* MOOFBox = ParsedData->GetCurrentMoofBox();
		if (MOOFBox)
		{
			for(int32 i = 0, iMax = MOOFBox->GetNumberOfChildren(); i < iMax; ++i)
			{
				const FMP4Box* Box = MOOFBox->GetChildBox(i);
				switch(Box->GetType())
				{
					case FMP4Box::kBox_traf:
					{
						// FIXME: for encrypted content we may need to also handle: `subs`

						// Get the tfhd box.
						const FMP4BoxTFHD* TFHDBox = static_cast<const FMP4BoxTFHD*>(Box->FindBox(FMP4Box::kBox_tfhd, 0));
						check(TFHDBox);
						if (TFHDBox)
						{
							FTrack* Track = NewParsedTrackInfo->GetTrackByID(TFHDBox->GetTrackID());
							// We may not find it because the track is not of a supported type.
							if (Track)
							{
								// The fragmented track needs to have a trex box.
								check(Track->TREXBox);
								if (Track->TREXBox)
								{
									Track->MOOFBox = MOOFBox;		// need this to get to the absolute file position for default-base-is-moof addressing.
									Track->TFHDBox = TFHDBox;
									Track->TFDTBox = static_cast<const FMP4BoxTFDT*>(Box->FindBox(FMP4Box::kBox_tfdt, 0));
									TArray<const FMP4Box*> ListOfBoxes;
									// Locate all TRUN boxes.
									Box->GetAllBoxInstances(ListOfBoxes, FMP4Box::kBox_trun);
									for(int32 nBox=0; nBox<ListOfBoxes.Num(); ++nBox)
									{
										Track->TRUNBoxes.Add(static_cast<const FMP4BoxTRUN*>(ListOfBoxes[nBox]));
									}
									// Locate all sample grouping boxes.
									ListOfBoxes.Empty();
									Box->GetAllBoxInstances(ListOfBoxes, FMP4Box::kBox_sgpd);
									for(int32 nBox=0; nBox<ListOfBoxes.Num(); ++nBox)
									{
										Track->SGPDBoxes.Add(static_cast<const FMP4BoxSGPD*>(ListOfBoxes[nBox]));
									}
									ListOfBoxes.Empty();
									Box->GetAllBoxInstances(ListOfBoxes, FMP4Box::kBox_sbgp);
									for(int32 nBox=0; nBox<ListOfBoxes.Num(); ++nBox)
									{
										Track->SBGPBoxes.Add(static_cast<const FMP4BoxSBGP*>(ListOfBoxes[nBox]));
									}

									// Locate encryption related boxes. At present we do this only for fragmented streams.
									Track->SAIZBox = static_cast<const FMP4BoxSAIZ*>(Box->FindBox(FMP4Box::kBox_saiz, 0));
									Track->SAIOBox = static_cast<const FMP4BoxSAIO*>(Box->FindBox(FMP4Box::kBox_saio, 0));
									Track->SENCBox = static_cast<const FMP4BoxSENC*>(Box->FindBox(FMP4Box::kBox_senc, 0));

									// Get the PSSH boxes from the init segment.
									TArray<const FMP4Box*> AllPSSHBoxes;
									Track->PSSHBoxesFromInit.Empty();
									if (MOOVBox)
									{
										MOOVBox->GetAllBoxInstances(AllPSSHBoxes, FMP4Box::kBox_pssh);
										for(int32 nPSSHs=0; nPSSHs<AllPSSHBoxes.Num(); ++nPSSHs)
										{
											Track->PSSHBoxesFromInit.Add(static_cast<const FMP4BoxPSSH*>(AllPSSHBoxes[nPSSHs]));
										}
										AllPSSHBoxes.Empty();
									}

									// Get the PSSH boxes from the track fragment.
									MOOFBox->GetAllBoxInstances(AllPSSHBoxes, FMP4Box::kBox_pssh);
									for(int32 nPSSHs=0; nPSSHs<AllPSSHBoxes.Num(); ++nPSSHs)
									{
										Track->PSSHBoxes.Add(static_cast<const FMP4BoxPSSH*>(AllPSSHBoxes[nPSSHs]));
									}
									// Now that we have the information needed to parse the senc box, do it.
									if (Track->SENCBox)
									{
										UEMediaError Error = const_cast<FMP4BoxSENC*>(Track->SENCBox)->Parse(Track->TENCBox, Track->SAIZBox, Track->SAIOBox);
										if (Error != UEMEDIA_ERROR_OK)
										{
											return Error;
										}
									}

									// Get the PRFT boxes from the track fragment that apply to this track.
									TArray<const FMP4Box*> AllPRFTBoxes;
									MOOFBox->GetParentBox()->GetAllBoxInstances(AllPRFTBoxes, FMP4Box::kBox_prft);
									for(int32 nPRFTs=0; nPRFTs<AllPRFTBoxes.Num(); ++nPRFTs)
									{
										const FMP4BoxPRFT* PRFTBox = static_cast<const FMP4BoxPRFT*>(AllPRFTBoxes[nPRFTs]);
										if (PRFTBox->GetReferenceTrackID() == Track->GetID())
										{
											ITrack::FProducerReferenceTime prft;
											prft.NtpTimestamp = PRFTBox->GetNtpTimestamp();
											prft.MediaTime = PRFTBox->GetMediaTime();
											switch(PRFTBox->GetFlags() & (8 | 16))
											{
												case 0:
													prft.Reference = ITrack::FProducerReferenceTime::EReferenceType::Encoder;
													break;
												case 8|16:
													prft.Reference = ITrack::FProducerReferenceTime::EReferenceType::Captured;
													break;
												case 16:
													prft.Reference = ITrack::FProducerReferenceTime::EReferenceType::Application;
													break;
												default:
													prft.Reference = ITrack::FProducerReferenceTime::EReferenceType::Undefined;
													break;
											}
											Track->PRFTBoxes.Emplace(MoveTemp(prft));
										}
									}
								}
								else
								{
									// The trex box is mandatory!
									return UEMEDIA_ERROR_FORMAT_ERROR;
								}
							}
							else
							{
								// Either the track is not a supported type or the moov really does not specify
								// a track with that ID, which would be an error.
								// We assume that the file itself is ok and that the track is not supported.
							}
						}
						else
						{
							// tfhd is mandatory!
							return UEMEDIA_ERROR_FORMAT_ERROR;
						}
						break;
					}
					default:
						break;
				}
			}
		}

		// Set the new track information.
		ParsedTrackInfo = NewParsedTrackInfo.Release();

		// Need to have moov, moof or sidx.
		return MOOVBox || MOOFBox || ParsedData->GetCurrentSidxBox() ? UEMEDIA_ERROR_OK : UEMEDIA_ERROR_FORMAT_ERROR;
	}

	UEMediaError FParserISO14496_12::ResolveTimecodeTracks(IPlayerSessionServices* InPlayerSession, FCancellationCheckDelegate InCancellationCheckDelegate)
	{
		if (!ParsedData->ShouldParseTimecodeInfo())
		{
			return UEMEDIA_ERROR_OK;
		}

		// Read timecode sample data for timecode tracks.
		for(int32 TrackIndex = 0; TrackIndex < ParsedTrackInfo->GetNumberOfTracks(); ++TrackIndex)
		{
			FTrack* Track = ParsedTrackInfo->GetTrackByIndexMutable(TrackIndex);
			if (Track && Track->CodecInformation.GetCodec4CC() == FMP4Box::kSample_tmcd)
			{
				ReadTMCDSample(InPlayerSession, Track, Track->STSDBox->FindBox(FMP4Box::kSample_tmcd), InCancellationCheckDelegate);
			}
		}

		// Copy timecode meta data from tmcd tracks to the related video tracks.
		for(int32 TrackIndex = 0; TrackIndex < ParsedTrackInfo->GetNumberOfTracks(); ++TrackIndex)
		{
			FTrack* Track = ParsedTrackInfo->GetTrackByIndexMutable(TrackIndex);
			if (Track && Track->CodecInformation.IsVideoCodec() && Track->TREFBox)
			{
				const TArray<int32>& TimecodeTrackIds = Track->TREFBox->GetTrackIdsForType(FMP4Box::kTrackRef_tmcd);
				if (!TimecodeTrackIds.IsEmpty())
				{
					if (const FTrack* TimecodeTrack = ParsedTrackInfo->GetTrackByID(TimecodeTrackIds[0]))
					{
						if (TimecodeTrack->StartTimecode.IsSet())
						{
							Track->StartTimecode = TimecodeTrack->StartTimecode;
							FParamDict& TrackExtras = Track->CodecInformation.GetExtras();
							TrackExtras.SetValueFrom(MediaInfoStartTimecodeValue, TimecodeTrack->CodecInformation.GetExtras());
							TrackExtras.SetValueFrom(MediaInfoStartTimecodeFrameRate, TimecodeTrack->CodecInformation.GetExtras());
						}
					}
				}
			}
		}

		// Look for a tmcd track not referenced by any video track and export it globally.
		for(int32 TrackIndex = 0; TrackIndex < ParsedTrackInfo->GetNumberOfTracks(); ++TrackIndex)
		{
			const FTrack* Track = ParsedTrackInfo->GetTrackByIndex(TrackIndex);
			if (Track && Track->CodecInformation.GetCodec4CC() == FMP4Box::kSample_tmcd && Track->StartTimecode.IsSet())
			{
				auto IsTimecodeTrackReferenced = [](const FTrackInfo* InTrackInfo, int32 InTimecodeTrackId)
				{
					for(int32 TrackIndex = 0; TrackIndex < InTrackInfo->GetNumberOfTracks(); ++TrackIndex)
					{
						const FTrack* Track = InTrackInfo->GetTrackByIndex(TrackIndex);
						if (Track && Track->CodecInformation.IsVideoCodec() && Track->TREFBox)
						{
							const TArray<int32>& TimecodeTrackIds = Track->TREFBox->GetTrackIdsForType(FMP4Box::kTrackRef_tmcd);
							if (TimecodeTrackIds.Contains(InTimecodeTrackId))
							{
								return true;
							}
						}
					}
					return false;
				};

				if (!IsTimecodeTrackReferenced(ParsedTrackInfo, Track->GetID()))
				{
					InPlayerSession->GetMediaInfoDictionary().SetValueFrom(MediaInfoStartTimecodeValue, Track->CodecInformation.GetExtras());
					InPlayerSession->GetMediaInfoDictionary().SetValueFrom(MediaInfoStartTimecodeFrameRate, Track->CodecInformation.GetExtras());
					break;
				}
			}
		}
		return UEMEDIA_ERROR_OK;
	}

	TMediaOptionalValue<FTimeFraction> FParserISO14496_12::GetMovieDuration() const
	{
		return ParsedTrackInfo ? ParsedTrackInfo->GetMovieDuration() : TMediaOptionalValue<FTimeFraction>();
	}


	int32 FParserISO14496_12::GetNumberOfTracks() const
	{
		return ParsedTrackInfo ? ParsedTrackInfo->GetNumberOfTracks() : 0;
	}

	int32 FParserISO14496_12::GetNumberOfSegmentIndices() const
	{
		// For now there is only one or none.
		// See GetSegmentIndexByIndex()
		return ParsedData ? (ParsedData->GetCurrentSidxBox() ? 1 : 0) : 0;
	}

	int32 FParserISO14496_12::GetNumberOfEventMessages() const
	{
		return ParsedData ? ParsedData->GetNumberOfEMSGBoxes() : 0;
	}


	const FParserISO14496_12::ITrack* FParserISO14496_12::GetTrackByIndex(int32 Index) const
	{
		return ParsedTrackInfo ? ParsedTrackInfo->GetTrackByIndex(Index) : nullptr;
	}

	const FParserISO14496_12::ITrack* FParserISO14496_12::GetTrackByTrackID(int32 TrackID) const
	{
		return ParsedTrackInfo ? ParsedTrackInfo->GetTrackByID(TrackID) : nullptr;
	}

	const FParserISO14496_12::ISegmentIndex* FParserISO14496_12::GetSegmentIndexByIndex(int32 Index) const
	{
		// There is either none or one right now.
		// See GetNumberOfSegmentIndices()
		return Index < GetNumberOfSegmentIndices() ? static_cast<FMP4BoxSIDX*>(ParsedData->GetCurrentSidxBox()) : nullptr;
	}

	const FParserISO14496_12::IEventMessage* FParserISO14496_12::GetEventMessageByIndex(int32 Index) const
	{
		return ParsedData ? static_cast<FMP4BoxEMSG*>(ParsedData->GetEMSGBox(Index)) : nullptr;
	}

	const IParserISO14496_12::IMetadata* FParserISO14496_12::GetMetadata(IParserISO14496_12::EBaseBoxType InFromBox) const
	{
		// We need parsed data and a 'moov' box. We do not look at top-level 'meta' boxes here.
		if (!ParsedData || (InFromBox == EBaseBoxType::Moov && !ParsedData->GetCurrentMoovBox()) || (InFromBox == EBaseBoxType::Moof && !ParsedData->GetCurrentMoofBox()))
		{
			return nullptr;
		}
		// First try 'udta'->'meta', which is the most common location.
		const FMP4Box* Box = InFromBox == EBaseBoxType::Moov ? ParsedData->GetCurrentMoovBox() : ParsedData->GetCurrentMoofBox();
		const FMP4BoxMETA* Meta = Box ? static_cast<const FMP4BoxMETA*>(Box->GetBoxPath(FMP4Box::kBox_udta, FMP4Box::kBox_meta)) : nullptr;
		if (!Meta)
		{
			Meta = Box ? static_cast<const FMP4BoxMETA*>(Box->GetBoxPath(FMP4Box::kBox_meta)) : nullptr;
		}
		return Meta;
	}

	int32 FParserISO14496_12::GetNumberOfBrands() const
	{
		const FMP4BoxFTYP* Box = static_cast<const FMP4BoxFTYP*>(ParsedData->GetCurrentStypBox() ? ParsedData->GetCurrentStypBox() : ParsedData->GetCurrentFtypBox());
		if (Box)
		{
			return Box->GetNumberOfBrands();
		}
		return 0;
	}

	IParserISO14496_12::FBrandType FParserISO14496_12::GetBrandByIndex(int32 Index) const
	{
		const FMP4BoxFTYP* Box = static_cast<const FMP4BoxFTYP*>(ParsedData->GetCurrentStypBox() ? ParsedData->GetCurrentStypBox() : ParsedData->GetCurrentFtypBox());
		if (Box)
		{
			return Box->GetBrand(Index);
		}
		return MAKE_MP4_BRAND(0,0,0,0);
	}

	bool FParserISO14496_12::HasBrand(const IParserISO14496_12::FBrandType InBrand) const
	{
		for(int32 i=0,iMax=GetNumberOfBrands(); i<iMax; ++i)
		{
			if (GetBrandByIndex(i) == InBrand)
			{
				return true;
			}
		}
		return false;
	}





	FParserISO14496_12::FAllTrackIterator::~FAllTrackIterator()
	{
	}
	//! Returns the iterator at the current file position.
	const IParserISO14496_12::ITrackIterator* FParserISO14496_12::FAllTrackIterator::Current() const
	{
		return CurrentIterator.Get();
	}

	//! Advance iterator to point to the next sample in sequence. Returns false if there are no more samples.
	bool FParserISO14496_12::FAllTrackIterator::Next()
	{
		if (CurrentIterator.IsValid())
		{
			UEMediaError err = CurrentIterator->Next();
			if (err == UEMEDIA_ERROR_END_OF_STREAM)
			{
				NewlyReachedEOS.Add(CurrentIterator.Get());
			}
			CurrentIterator.Reset();
			int64 LowestFilePos = TNumericLimits<int64>::Max();
			for(int32 nTrk = 0; nTrk < TrackIterators.Num(); ++nTrk)
			{
				if (!TrackIterators[nTrk]->IsAtEOS() && TrackIterators[nTrk]->GetSampleFileOffset() < LowestFilePos)
				{
					CurrentIterator = TrackIterators[nTrk];
					LowestFilePos = CurrentIterator->GetSampleFileOffset();
				}
			}
		}
		return CurrentIterator.IsValid();
	}


	//! Returns a list of all tracks iterators that reached EOS while iterating since the most recent call to ClearNewEOSTracks().
	void FParserISO14496_12::FAllTrackIterator::GetNewEOSTracks(TArray<const ITrackIterator*>& OutTracksThatNewlyReachedEOS) const
	{
		OutTracksThatNewlyReachedEOS = NewlyReachedEOS;
	}

	//! Clears the list of track iterators that have reached EOS.
	void FParserISO14496_12::FAllTrackIterator::ClearNewEOSTracks()
	{
		NewlyReachedEOS.Empty();
	}

	//! Returns list of all iterators.
	void FParserISO14496_12::FAllTrackIterator::GetAllIterators(TArray<const ITrackIterator*>& OutIterators) const
	{
		OutIterators.Empty();
		for(int32 nTrk = 0; nTrk < TrackIterators.Num(); ++nTrk)
		{
			OutIterators.Add(TrackIterators[nTrk].Get());
		}
	}


	TSharedPtrTS<IParserISO14496_12::IAllTrackIterator> FParserISO14496_12::CreateAllTrackIteratorByFilePos(int64 InFromFilePos) const
	{
		TSharedPtrTS<FParserISO14496_12::FAllTrackIterator> ti = MakeShared<FParserISO14496_12::FAllTrackIterator, ESPMode::ThreadSafe>();

		int64 LowestFilePos = TNumericLimits<int64>::Max();
		for(int32 nTrk = 0, nTrkMax = GetNumberOfTracks(); nTrk < nTrkMax; ++nTrk)
		{
			const FTrack* Track = ParsedTrackInfo->GetTrackByIndex(nTrk);
			check(Track);
			FTrackIterator* TrkIt = static_cast<FTrackIterator*>(Track->CreateIterator());
			TSharedPtrTS<ITrackIterator> SafeTrkIt(TrkIt);
			ti->TrackIterators.Add(SafeTrkIt);
			UEMediaError err = TrkIt->StartAtFirstInternal();
			if (err == UEMEDIA_ERROR_OK)
			{
				while(TrkIt->GetSampleFileOffset() < InFromFilePos)
				{
					err = TrkIt->Next();
					if (err != UEMEDIA_ERROR_OK)
					{
						if (err == UEMEDIA_ERROR_END_OF_STREAM)
						{
							ti->NewlyReachedEOS.Add(TrkIt);
						}
						break;
					}
				}
				if (err == UEMEDIA_ERROR_OK && (!ti->CurrentIterator.IsValid() || TrkIt->GetSampleFileOffset() < LowestFilePos))
				{
					LowestFilePos = TrkIt->GetSampleFileOffset();
					ti->CurrentIterator = SafeTrkIt;
				}
			}
			else if (err == UEMEDIA_ERROR_END_OF_STREAM)
			{
				ti->NewlyReachedEOS.Add(TrkIt);
			}
		}
		return ti;
	}

	TSharedPtrTS<IParserISO14496_12::IAllTrackIterator> FParserISO14496_12::CreateAllTrackIteratorForTrackIDs(const TArray<uint32>& InTrackIDs) const
	{
		TSharedPtrTS<FParserISO14496_12::FAllTrackIterator> ti = MakeShared<FParserISO14496_12::FAllTrackIterator, ESPMode::ThreadSafe>();

		int64 LowestFilePos = TNumericLimits<int64>::Max();
		for(int32 nTrk=0, nTrkMax=InTrackIDs.Num(); nTrk<nTrkMax; ++nTrk)
		{
			const FTrack* Track = ParsedTrackInfo->GetTrackByID(InTrackIDs[nTrk]);
			if (!Track)
			{
				continue;
			}
			FTrackIterator* TrkIt = static_cast<FTrackIterator*>(Track->CreateIterator());
			TSharedPtrTS<ITrackIterator> SafeTrkIt(TrkIt);
			ti->TrackIterators.Add(SafeTrkIt);
			UEMediaError err = TrkIt->StartAtFirstInternal();
			if (err == UEMEDIA_ERROR_OK)
			{
				if (TrkIt->GetSampleFileOffset() < LowestFilePos)
				{
					LowestFilePos = TrkIt->GetSampleFileOffset();
					ti->CurrentIterator = SafeTrkIt;
				}
			}
			else if (err == UEMEDIA_ERROR_END_OF_STREAM)
			{
				ti->NewlyReachedEOS.Add(TrkIt);
			}
		}
		return ti;
	}


	/*********************************************************************************************************************/

	TSharedPtrTS<IParserISO14496_12> IParserISO14496_12::CreateParser()
	{
		return MakeSharedTS<FParserISO14496_12>();
	}



} // namespace Electra

#undef MEDIA_DEBUG_HAS_BOX_NAMES

