// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PlayerCore.h"

#include "PlayerTime.h"
#include "OptionalValue.h"
#include "StreamTypes.h"
#include "ParameterDictionary.h"
#include "BufferedDataReader.h"

#include "ElectraEncryptedSampleInfo.h"


namespace Electra
{
	//
	// Forward declarations
	//
	class IPlayerSessionServices;


	/**
	 * Interface for parsing an ISO/IEC 14496-12 file, commonly referred to as an "mp4" file.
	 */
	class IParserISO14496_12
	{
	public:
		virtual ~IParserISO14496_12() = default;

		/** Box type is a 32 bit value in an mp4 file. */
		typedef uint32 FBoxType;

		/**
		 * Commonly referenced boxes
		 */
		 // constexpr doesn't work here
			 /*
				 static inline constexpr FBoxType MakeBoxType(unsigned char v1, unsigned char v2, unsigned char v3, unsigned char v4)
				 {
					 return ((uint32)v1 << 24) | ((uint32)v2 << 16) | ((uint32)v3 << 8) | (uint32)v4;
				 }
			 */
#define MAKE_BOX_ATOM(a,b,c,d) (IParserISO14496_12::FBoxType)((uint32)a << 24) | ((uint32)b << 16) | ((uint32)c << 8) | ((uint32)d)
		static const FBoxType BoxType_ftyp = MAKE_BOX_ATOM('f', 't', 'y', 'p');
		static const FBoxType BoxType_styp = MAKE_BOX_ATOM('s', 't', 'y', 'p');
		static const FBoxType BoxType_moov = MAKE_BOX_ATOM('m', 'o', 'o', 'v');
		static const FBoxType BoxType_sidx = MAKE_BOX_ATOM('s', 'i', 'd', 'x');
		static const FBoxType BoxType_prft = MAKE_BOX_ATOM('p', 'r', 'f', 't');
		static const FBoxType BoxType_moof = MAKE_BOX_ATOM('m', 'o', 'o', 'f');
		static const FBoxType BoxType_mdat = MAKE_BOX_ATOM('m', 'd', 'a', 't');
		static const FBoxType BoxType_free = MAKE_BOX_ATOM('f', 'r', 'e', 'e');



		/**
		 * Interface for receiving parse notifications
		 */
		class IBoxCallback
		{
		public:
			virtual ~IBoxCallback() = default;

			/**
			 * Choices for continued parsing.
			 */
			enum class EParseContinuation
			{
				/** Continue parsing */
				Continue,
				/** Stop parsing */
				Stop
			};

			/**
			 * Notifies the caller of the start/end of a new box.
			 *
			 * The caller may continue to parse this box or ask to stop parsing.
			 *
			 * @param Box Type of the box that will be parsed next.
			 * @param BoxSizeInBytes Size of the box in bytes.
			 * @param FileDataOffset Offset of the box within the file. This points to the first byte of the box length field.
			 * @param BoxDataOffset Offset of the box' data. This points to the first payload byte of the box.
			 * @return Whether or not to continue parsing thix box.
			 */
			virtual EParseContinuation OnFoundBox(FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset) = 0;
			virtual EParseContinuation OnEndOfBox(FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset) = 0;
		};


		static TSharedPtrTS<IParserISO14496_12> CreateParser();

		/**
		 * Parses the header boxes (all non-MDAT boxes).
		 */
		virtual UEMediaError ParseHeader(IGenericDataReader* DataReader, IBoxCallback* BoxParseCallback, IPlayerSessionServices* PlayerSession, const IParserISO14496_12* OptionalInitSegment) = 0;


		/** A brand is a 32 bit value in an mp4 file. */
		typedef uint32 FBrandType;
		#define MAKE_MP4_BRAND(a,b,c,d) (IParserISO14496_12::FBrandType)((uint32)a << 24) | ((uint32)b << 16) | ((uint32)c << 8) | ((uint32)d)
		static const FBrandType BrandType_emsg = MAKE_MP4_BRAND('e', 'm', 's', 'g');
		static const FBrandType BrandType_lmsg = MAKE_MP4_BRAND('l', 'm', 's', 'g');

		virtual int32 GetNumberOfBrands() const = 0;
		virtual FBrandType GetBrandByIndex(int32 Index) const = 0;
		virtual bool HasBrand(const FBrandType InBrand) const = 0;



		/*******************************************************************************************************************/
		class ITrack;

		virtual UEMediaError PrepareTracks(IPlayerSessionServices* PlayerSession, TSharedPtrTS<const IParserISO14496_12> OptionalMP4InitSegment) = 0;

		// Cancellation delegate for potentially slow tasks.
		DECLARE_DELEGATE_RetVal(bool, FCancellationCheckDelegate);
		virtual UEMediaError ResolveTimecodeTracks(IPlayerSessionServices* InPlayerSession, FCancellationCheckDelegate InCancellationCheckDelegate) = 0;

		virtual TMediaOptionalValue<FTimeFraction> GetMovieDuration() const = 0;

		virtual int32 GetNumberOfTracks() const = 0;

		virtual int32 GetNumberOfSegmentIndices() const = 0;

		virtual int32 GetNumberOfEventMessages() const = 0;

		class IMetadata
		{
		public:
			virtual uint32 GetHandler() const = 0;
			virtual uint32 GetReserved(int32 InIndex0to2) const = 0;
			virtual int32 GetNumChildBoxes() const = 0;
			virtual uint32 GetChildBoxType(int32 InChildIndex) const = 0;
			virtual uint32 GetChildBoxDataSize(int32 InChildIndex) const = 0;
			virtual const void* GetChildBoxData(int32 InChildIndex) const = 0;
		};

		class ITrackIterator
		{
		public:
			virtual ~ITrackIterator() = default;

			enum class ESearchMode
			{
				Before,
				After,
				Closest
			};

			virtual UEMediaError StartAtTime(const FTimeValue& AtTime, ESearchMode SearchMode, bool bNeedSyncSample) = 0;
			virtual UEMediaError StartAtFirst(bool bNeedSyncSample) = 0;
			virtual UEMediaError Next() = 0;

			virtual bool IsAtEOS() const = 0;
			virtual const ITrack* GetTrack() const = 0;
			virtual int64 GetBaseMediaDecodeTime() const = 0;
			virtual uint32 GetSampleNumber() const = 0;
			virtual int64 GetDTS() const = 0;
			virtual int64 GetPTS() const = 0;
			virtual int64 GetDuration() const = 0;
			virtual uint32 GetTimescale() const = 0;
			virtual bool IsSyncSample() const = 0;
			virtual int64 GetSampleSize() const = 0;
			virtual int64 GetSampleFileOffset() const = 0;
			virtual int64 GetRawDTS() const = 0;
			virtual int64 GetRawPTS() const = 0;
			virtual int64 GetCompositionTimeEdit() const = 0;
			virtual int64 GetEmptyEditOffset() const = 0;

			virtual bool GetEncryptionInfo(ElectraCDM::FMediaCDMSampleInfo& OutSampleEncryptionInfo) const = 0;
		};

		class ITrack
		{
		public:
			// Bitrate box from either 'btrt' or the 'esds' DecoderConfigDescriptor
			struct FBitrateInfo
			{
				uint32	BufferSizeDB = 0;
				uint32	MaxBitrate = 0;
				uint32	AvgBitrate = 0;
			};

			// Producer reference time box ('prft') applicable to this track.
			struct FProducerReferenceTime
			{
				enum class EReferenceType
				{
					Undefined,
					Captured,
					Encoder,
					Application
				};
				uint64 NtpTimestamp = 0;
				uint64 MediaTime = 0;
				EReferenceType Reference = EReferenceType::Undefined;
			};

			struct FEncryptionInfo
			{
				TArray<uint8> DefaultKID;
				TArray<uint8> DefaultIV;
				int32 DefaultIVSize = 0;
				uint32 Scheme = 0;
				struct FCDMInfo
				{
					TArray<uint8> SystemID;
					TArray<TArray<uint8>> KIDs;
					TArray<uint8> Data;
				};
				TArray<FCDMInfo> CDMInfos;
			};

			virtual ~ITrack() = default;

			virtual uint32 GetID() const = 0;
			virtual FString GetName() const = 0;
			virtual FString GetNameFromHandler() const = 0;
			virtual FTimeFraction GetDuration() const = 0;

			virtual ITrackIterator* CreateIterator() const = 0;

			virtual const TArray<uint8> GetCodecSpecificData() const = 0;
			virtual const TArray<uint8> GetCodecSpecificDataRAW() const = 0;
			virtual const FStreamCodecInformation GetCodecInformation() const = 0;
			virtual const FBitrateInfo GetBitrateInfo() const = 0;
			virtual const BCP47::FLanguageTag& GetLanguageTag() const = 0;
			virtual bool GetEncryptionInfo(FEncryptionInfo& OutEncryptionInfo) const = 0;
			virtual void GetPSSHBoxes(TArray<TArray<uint8>>& OutBoxes, bool bFromMOOV, bool bFromMOOF) const = 0;
			virtual void GetPRFTBoxes(TArray<FProducerReferenceTime>& OutBoxes) const = 0;
		};


		class IAllTrackIterator
		{
		public:
			virtual ~IAllTrackIterator() = default;
			//! Returns the iterator at the current file position.
			virtual const ITrackIterator* Current() const = 0;
			//! Advance iterator to point to the next sample in sequence. Returns false if there are no more samples.
			virtual bool Next() = 0;
			//! Returns a list of all tracks iterators that reached EOS while iterating since the most recent call to ClearNewEOSTracks().
			virtual void GetNewEOSTracks(TArray<const ITrackIterator*>& OutTracksThatNewlyReachedEOS) const = 0;
			//! Clears the list of track iterators that have reached EOS.
			virtual void ClearNewEOSTracks() = 0;
			//! Returns list of all iterators.
			virtual void GetAllIterators(TArray<const ITrackIterator*>& OutIterators) const = 0;
		};


		class ISegmentIndex
		{
		public:
			struct FEntry
			{
				uint32		SubSegmentDuration;
				uint32		IsReferenceType : 1;
				uint32		Size : 31;
				uint32		StartsWithSAP : 1;
				uint32		SAPType : 3;
				uint32		SAPDeltaTime : 28;
			};

			virtual ~ISegmentIndex() = default;
			virtual uint64 GetEarliestPresentationTime() const = 0;
			virtual uint64 GetFirstOffset() const = 0;
			virtual uint32 GetReferenceID() const = 0;
			virtual uint32 GetTimescale() const = 0;
			virtual int32 GetNumEntries() const = 0;
			virtual const FEntry& GetEntry(int32 Index) const = 0;
		};

		class IEventMessage
		{
		public:
			virtual ~IEventMessage() = default;
			virtual int32 GetVersion() const = 0;
			virtual const FString& GetSchemeIdUri() const = 0;
			virtual const FString& GetValue() const = 0;
			virtual uint32 GetTimescale() const = 0;
			virtual uint32 GetPresentationTimeDelta() const = 0;
			virtual uint64 GetPresentationTime() const = 0;
			virtual uint32 GetEventDuration() const = 0;
			virtual uint32 GetID() const = 0;
			virtual const TArray<uint8>& GetMessageData() const = 0;
		};

		enum class EBaseBoxType
		{
			Moov,
			Moof
		};

		virtual const IMetadata* GetMetadata(EBaseBoxType InFromBox) const = 0;

		virtual TSharedPtrTS<IAllTrackIterator> CreateAllTrackIteratorByFilePos(int64 InFromFilePos) const = 0;

		virtual TSharedPtrTS<IAllTrackIterator> CreateAllTrackIteratorForTrackIDs(const TArray<uint32>& InTrackIDs) const = 0;

		virtual const ITrack* GetTrackByIndex(int32 Index) const = 0;
		virtual const ITrack* GetTrackByTrackID(int32 TrackID) const = 0;

		virtual const ISegmentIndex* GetSegmentIndexByIndex(int32 Index) const = 0;

		virtual const IEventMessage* GetEventMessageByIndex(int32 Index) const = 0;
	};

} // namespace Electra
