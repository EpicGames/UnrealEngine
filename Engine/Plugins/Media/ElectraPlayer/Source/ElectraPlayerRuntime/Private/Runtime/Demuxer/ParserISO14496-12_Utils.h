// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PlayerCore.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserISO14496-12.h"

namespace Electra
{

	class FMP4StaticDataReader : public IGenericDataReader
	{
	public:
		FMP4StaticDataReader() = default;
		virtual ~FMP4StaticDataReader() = default;
		virtual void SetParseData(TSharedPtrTS<FWaitableBuffer> InResponseBuffer)
		{
			ResponseBuffer = InResponseBuffer;
			DataSize = ResponseBuffer->Num();
			Data = (const uint8*)ResponseBuffer->GetLinearReadData();
			CurrentOffset = 0;
		}
		virtual bool HaveParseData() const
		{
			return ResponseBuffer.IsValid();
		}
	private:
		//----------------------------------------------------------------------
		// Methods from IGenericDataReader
		//
		int64 ReadData(void* IntoBuffer, int64 NumBytesToRead, int64 InFromOffset) override
		{
			if (ResponseBuffer.IsValid() && NumBytesToRead <= DataSize - CurrentOffset)
			{
				if (IntoBuffer)
				{
					FMemory::Memcpy(IntoBuffer, Data+CurrentOffset, NumBytesToRead);
				}
				CurrentOffset += NumBytesToRead;
				return NumBytesToRead;
			}
			return -1;
		}
		bool HasReachedEOF() const override
		{
			return ResponseBuffer.IsValid() ? ResponseBuffer->GetEOD() && CurrentOffset >= DataSize : true;
		}
		bool HasReadBeenAborted() const override
		{
			return ResponseBuffer.IsValid() ? ResponseBuffer->WasAborted() : true;
		}
		int64 GetCurrentOffset() const override
		{
			return CurrentOffset;
		}
		int64 GetTotalSize() const override
		{
			check(!"this should not be called");
			return -1;
		}

		TSharedPtrTS<FWaitableBuffer> ResponseBuffer;
		const uint8* Data = nullptr;
		int64 DataSize = 0;
		int64 CurrentOffset = 0;
	};



} // namespace Electra
