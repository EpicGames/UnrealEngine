// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParserMKV.h"

#include "Misc/Build.h"
#include "BufferedDataReader.h"

#include "StreamTypes.h"
#include "Utilities/ISO639-Map.h"
#include "Utilities/BCP47-Helpers.h"
#include "InfoLog.h"
#include "Player/PlayerSessionServices.h"
#include "Player/PlayerStreamFilter.h"

#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"

// Needed to extract information from headers of AVC, HEVC and AAC files.
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"

DECLARE_LOG_CATEGORY_EXTERN(LogElectraMKVParser, Log, All);
DEFINE_LOG_CATEGORY(LogElectraMKVParser);


namespace Electra
{
	enum EIntMKVError
	{
		ERRCODE_MKV_INTERNAL_BAD_FORMAT = 1,
		ERRCODE_MKV_INTERNAL_BAD_CUE_DISTANCE = 2,
		ERRCODE_MKV_INTERNAL_CLIENT_FAILED_TO_DO_WHAT_HE_WAS_TOLD = 3,
		ERRCODE_MKV_INTERNAL_UNSUPPORTED_FEATURE = 4,
		ERRCODE_MKV_INTERNAL_UNHANDLED_CASE = 5,
	};

	#define DEFAULT_MATROSKA_TIMESCALE 1000000


	class IMKVFetcher
	{
	public:
		virtual ~IMKVFetcher()  {}

		virtual bool FetchAtEOS() = 0;
		virtual int64 FetchCurrentOffset() = 0;
		virtual bool FetchElementID(uint32& OutID) = 0;
		virtual bool FetchElementLength(int64& OutVal) = 0;
		virtual bool FetchSkipOver(int64 InNumToSkip) = 0;
		virtual bool FetchUInt(uint64& OutVal, int64 InNumBytes, uint64 InDefaultValue) = 0;
		virtual bool FetchFloat(double& OutVal, int64 InNumBytes, double InDefaultValue) = 0;
		virtual bool FetchByteArray(TArray<uint8>& OutVal, int64 InNumBytes) = 0;
		virtual bool FetchSeekTo(int64 InAbsolutePosition) = 0;
	};



	class IMKVElementReader
	{
	public:
		virtual FErrorDetail LastError() const = 0;
		virtual int64 CurrentOffset() const = 0;
		virtual bool Prefetch(int64 NumBytes) = 0;
		virtual bool Read(uint8& OutValue) = 0;
		virtual bool Read(uint16& OutValue) = 0;
		virtual bool Read(uint32& OutValue) = 0;
		virtual bool Read(uint64& OutValue) = 0;
		virtual bool Read(TArray<uint8>& OutValue, int64 NumBytes) = 0;
		virtual bool Skip(int64 NumBytes) = 0;
	};

	class FMKVEBMLReader
	{
	public:
		FMKVEBMLReader(IMKVElementReader* InReader) : Reader(InReader) {}

		int32 ReadSVINT(int64& OutVINT)
		{
			uint64 vint;
			int32 nbits = ReadVINT(vint);
			if (nbits <= 0)
			{
				OutVINT = 0;
				return nbits;
			}
			int32 nb = (nbits + 7) / 8;
			OutVINT = vint - ((1LL << (7 * ((nbits + 7) / 8) - 1)) - 1);
			return nbits;
		}
		int32 ReadVINT(uint64& OutVINT)
		{
			uint64 vint = 0;
			uint8 nb;
			if (!Reader->Read(nb))
			{
				LastError = Reader->LastError();
				OutVINT = 0;
				return -1;
			}
			// For the purpose of reading a length used in Matroska, the first byte must not be zero.
			if (nb == 0)
			{
				LastError.SetError(UEMEDIA_ERROR_FORMAT_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(FString::Printf(TEXT("Invalid VINT value starting with 0 at offset %lld"), (long long int)Reader->CurrentOffset()));
				OutVINT = 0;
				return -1;
			}
			int32 lz = kLeadingZeroTable8bit[nb];
			int32 numBits = 7-lz + 8*lz;
			vint = nb & (0x7f >> lz);
			for(; lz; --lz)
			{
				if (!Reader->Read(nb))
				{
					LastError = Reader->LastError();
					OutVINT = 0;
					return -1;
				}
				vint = (vint << 8) | nb;
			}
			OutVINT = vint;
			return numBits;
		}
		bool ReadElementLength(int64& OutLength)
		{
			uint64 vint;
			int32 nb = ReadVINT(vint);
			if (nb > 0)
			{
				// If all bits in the length are set this indicates "unknown data size".
				if (vint == (1ULL << nb)-1)
				{
					OutLength = TNumericLimits<int64>::Max();
					//LastError.SetError(UEMEDIA_ERROR_FORMAT_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Elements of unknown data size are not supported!"));
					//return false;
					return true;
				}
				else
				{
					OutLength = (int64)vint;
				}
				return true;
			}
			OutLength = -1;
			return false;
		}
		bool ReadElementID(uint32& OutElementID)
		{
			uint64 vint;
			int32 nb = ReadVINT(vint);
			if (nb > 0)
			{
				OutElementID = (uint32)vint;
				OutElementID |= 1U << nb;
				return true;
			}
			OutElementID = 0;
			return false;
		}
		bool ReadEBMLuint(uint64& OutValue, int64 InElementLen, uint64 InDefaultValue)
		{
			if (InElementLen == 0)
			{
				OutValue = InDefaultValue;
				return true;
			}
			OutValue = 0;
			Reader->Prefetch(InElementLen);
			for(int64 i=0; i<InElementLen; ++i)
			{
				uint8 b;
				if (!Reader->Read(b))
				{
					LastError = Reader->LastError();
					return false;
				}
				OutValue = (OutValue << 8) | b;
			}
			return true;
		}
		bool ReadEBMLint(int64& OutValue, int64 InElementLen, int64 InDefaultValue)
		{
			if (InElementLen == 0)
			{
				OutValue = InDefaultValue;
				return true;
			}
			OutValue = 0;
			Reader->Prefetch(InElementLen);
			for(int64 i=0; i<InElementLen; ++i)
			{
				uint8 b;
				if (!Reader->Read(b))
				{
					LastError = Reader->LastError();
					return false;
				}
				OutValue = (OutValue << 8) | b;
			}
			int32 shift = 64 - InElementLen * 8;
			OutValue = (OutValue << shift) >> shift;
			return true;
		}
		bool ReadEBMLfloat(double& OutValue, int64 InElementLen, double InDefaultValue)
		{
			if (InElementLen == 0)
			{
				OutValue = InDefaultValue;
				return true;
			}
			else if (InElementLen == 4)
			{
				union UF32
				{
					float f;
					uint32 u;
				} uf32;
				if (!Reader->Read(uf32.u))
				{
					LastError = Reader->LastError();
					return false;
				}
				OutValue = uf32.f;
				return true;
			}
			else if (InElementLen == 8)
			{
				union UF64
				{
					double d;
					uint64 u;
				} uf64;
				if (!Reader->Read(uf64.u))
				{
					LastError = Reader->LastError();
					return false;
				}
				OutValue = uf64.d;
				return true;
			}
			LastError.SetError(UEMEDIA_ERROR_FORMAT_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(FString::Printf(TEXT("Unsupported size of floating point value at offset %lld"), (long long int)Reader->CurrentOffset()));
			return false;
		}
		bool ReadEBMLbyteArray(TArray<uint8>& OutValue, int64 InElementLen)
		{
			if (InElementLen == 0)
			{
				return true;
			}
			return Reader->Read(OutValue, InElementLen);
		}
		FErrorDetail GetError() const
		{ return LastError; }
	private:
		FErrorDetail LastError;
		IMKVElementReader* Reader = nullptr;
		static inline const uint8 kLeadingZeroTable8bit[256] =
		{
			8,7,6,6,5,5,5,5,4,4,4,4,4,4,4,4,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,
			2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,
			1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,
			1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,
			0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,
			0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,
			0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,
			0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0
		};
	};


	class FBufferedMKVReader : private IMKVElementReader, public FBufferedDataReader, public FMKVEBMLReader, public IMKVFetcher
	{
	public:
		FBufferedMKVReader(IDataProvider* InDataProvider) : FBufferedDataReader(InDataProvider), FMKVEBMLReader(this) { }
		virtual ~FBufferedMKVReader() {}

		bool FetchAtEOS() override
		{ return IsAtEOS(); }
		int64 FetchCurrentOffset() override
		{ return CurrentOffset(); }
		bool FetchElementID(uint32& OutID) override
		{ return ReadElementID(OutID); }
		bool FetchElementLength(int64& OutVal) override
		{ return ReadElementLength(OutVal); }
		bool FetchSkipOver(int64 InNumToSkip) override
		{ return SkipOver(InNumToSkip); }
		bool FetchUInt(uint64& OutVal, int64 InNumBytes, uint64 InDefaultValue) override
		{ return ReadEBMLuint(OutVal, InNumBytes, InDefaultValue); }
		bool FetchFloat(double& OutVal, int64 InNumBytes, double InDefaultValue) override
		{ return ReadEBMLfloat(OutVal, InNumBytes, InDefaultValue); }
		bool FetchByteArray(TArray<uint8>& OutVal, int64 InNumBytes) override
		{ return ReadEBMLbyteArray(OutVal, InNumBytes); }
		bool FetchSeekTo(int64 InAbsolutePosition) override
		{ return SeekTo(InAbsolutePosition); }

	private:
		FErrorDetail LastError() const override { return FBufferedDataReader::GetLastError(); }
		int64 CurrentOffset() const override { return FBufferedDataReader::GetCurrentOffset(); }
		bool Prefetch(int64 NumBytes) override { return FBufferedDataReader::PrepareToRead(NumBytes); }
		bool Read(uint8& OutValue) override { return FBufferedDataReader::ReadU8(OutValue); }
		bool Read(uint16& OutValue) override { return FBufferedDataReader::ReadU16BE(OutValue); }
		bool Read(uint32& OutValue) override { return FBufferedDataReader::ReadU32BE(OutValue); }
		bool Read(uint64& OutValue) override { return FBufferedDataReader::ReadU64BE(OutValue); }
		bool Read(TArray<uint8>& OutValue, int64 NumBytes) override { return FBufferedDataReader::ReadByteArray(OutValue, NumBytes); }
		bool Skip(int64 NumBytes) override { return FBufferedDataReader::SkipOver(NumBytes); }
	};


	enum class EEBMLDataType
	{
		EBMLuint,
		EBMLint,
		EBMLfloat,
		EBMLansi,
		EBMLutf8,
		EBMLbinary,
		EBMLstruct,
		EBMLvoid,
		EBMLcrc32,
		EBMLstop
	};

	struct FEBMLElement
	{
		uint32 ID;
		EEBMLDataType Type;
		// This is a struct and not a union because a union named initializer requires C++20
		struct FDefaultValue
		{
			uint64 u64;
			int64 i64;
			double f64;
			const TCHAR* utf8;
		} Default;
	};
	#define DEF_ZERO() { 0, 0, 0.0, nullptr }
	#define DEF_UINT(u) { u, 0, 0.0, nullptr }

	static const FEBMLElement EBML_Header { 0x1A45DFA3, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement EBML_Version { 0x4286, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement EBML_ReadVersion { 0x42F7, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement EBML_MaxIDLength { 0x42F2, EEBMLDataType::EBMLuint, DEF_UINT(4) };
	static const FEBMLElement EBML_MaxSizeLength { 0x42F3, EEBMLDataType::EBMLuint, DEF_UINT(8) };
	static const FEBMLElement EBML_DocType {0x4282, EEBMLDataType::EBMLansi, DEF_ZERO() };
	static const FEBMLElement EBML_DocTypeVersion { 0x4287, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement EBML_DocTypeReadVersion { 0x4285, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement EBML_Void {0xEC, EEBMLDataType::EBMLvoid, DEF_ZERO() };
	static const FEBMLElement EBML_CRC32 {0xBF, EEBMLDataType::EBMLcrc32, DEF_ZERO() };

	static const FEBMLElement MKV_Segment { 0x18538067, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_SeekHead { 0x114D9B74, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Info { 0x1549A966, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Cluster { 0x1F43B675, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Tracks { 0x1654AE6B, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Cues { 0x1C53BB6B, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Attachments { 0x1941A469, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Chapters { 0x1043A770, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Tags { 0x1254C367, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Cluster_StopParsing { 0x1F43B675, EEBMLDataType::EBMLstop, DEF_ZERO() };

	static const FEBMLElement MKV_Seek { 0x4DBB, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_SeekID { 0x53AB, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_SeekPosition { 0x53AC, EEBMLDataType::EBMLuint, DEF_ZERO() };

	static const FEBMLElement MKV_TimestampScale { 0x2AD7B1, EEBMLDataType::EBMLuint, DEF_UINT(DEFAULT_MATROSKA_TIMESCALE) };
	static const FEBMLElement MKV_Duration { 0x4489, EEBMLDataType::EBMLfloat, DEF_ZERO() };

	static const FEBMLElement MKV_TrackEntry { 0xAE, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_TrackNumber { 0xD7, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_TrackUID { 0x73C5, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_TrackType { 0x83, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_FlagEnabled { 0xB9, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement MKV_FlagDefault { 0x88, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement MKV_FlagForced { 0x55AA, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_FlagHearingImpaired { 0x55AB, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_FlagVisualImpaired { 0x55AC, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_FlagTextDescriptions { 0x55AD, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_FlagOriginal { 0x55AE, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_FlagCommentary { 0x55AF, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_FlagLacing { 0x9C, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement MKV_DefaultDuration { 0x23E383, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_DefaultDecodedFieldDuration { 0x234E7A, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_MaxBlockAdditionID { 0x55EE, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_BlockAdditionMapping { 0x41E4, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_BlockAddIDValue { 0x41F0, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_BlockAddIDType { 0x41E7, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_BlockAddIDExtraData { 0x41ED, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_Name { 0x536E, EEBMLDataType::EBMLutf8, DEF_ZERO() };
	static const FEBMLElement MKV_Language { 0x22B59C, EEBMLDataType::EBMLansi, DEF_ZERO() };
	static const FEBMLElement MKV_LanguageBCP47 { 0x22B59D, EEBMLDataType::EBMLansi, DEF_ZERO() };
	static const FEBMLElement MKV_CodecID { 0x86, EEBMLDataType::EBMLansi, DEF_ZERO() };
	static const FEBMLElement MKV_CodecPrivate { 0x63A2, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_CodecName { 0x258688, EEBMLDataType::EBMLutf8, DEF_ZERO() };
	static const FEBMLElement MKV_CodecDelay { 0x56AA, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_SeekPreRoll { 0x56BB, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_Video { 0xE0, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_FlagInterlaced { 0x9A, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_FieldOrder { 0x9D, EEBMLDataType::EBMLuint, DEF_UINT(2) };
	static const FEBMLElement MKV_StereoMode { 0x53B8, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_AlphaMode { 0x53C0, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_PixelWidth { 0xB0, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_PixelHeight { 0xBA, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_PixelCropBottom { 0x54AA, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_PixelCropTop { 0x54BB, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_PixelCropLeft { 0x54CC, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_PixelCropRight { 0x54DD, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_DisplayWidth { 0x54B0, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_DisplayHeight { 0x54BA, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_DisplayUnit { 0x54B2, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_UncompressedFourCC { 0x2EB524, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_FrameRate { 0x2383E3, EEBMLDataType::EBMLfloat, DEF_ZERO() };					// DEPRECATED
	static const FEBMLElement MKV_Colour { 0x55B0, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_MatrixCoefficients { 0x55B1, EEBMLDataType::EBMLuint, DEF_UINT(2) };
	static const FEBMLElement MKV_BitsPerChannel { 0x55B2, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_ChromaSubsamplingHorz { 0x55B3, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_ChromaSubsamplingVert { 0x55B4, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_CbSubsamplingHorz { 0x55B5, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_CbSubsamplingVert { 0x55B6, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_ChromaSitingHorz { 0x55B7, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_ChromaSitingVert { 0x55B8, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_Range { 0x55B9, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_TransferCharacteristics { 0x55BA, EEBMLDataType::EBMLuint, DEF_UINT(2) };
	static const FEBMLElement MKV_Primaries { 0x55BB, EEBMLDataType::EBMLuint, DEF_UINT(2) };
	static const FEBMLElement MKV_MaxCLL { 0x55BC, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_MaxFALL { 0x55BD, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_MasteringMetadata { 0x55D0, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_PrimaryRChromaticityX { 0x55D1, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_PrimaryRChromaticityY { 0x55D2, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_PrimaryGChromaticityX { 0x55D3, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_PrimaryGChromaticityY { 0x55D4, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_PrimaryBChromaticityX { 0x55D5, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_PrimaryBChromaticityY { 0x55D6, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_WhitePointChromaticityX { 0x55D7, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_WhitePointChromaticityY { 0x55D8, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_LuminanceMax { 0x55D9, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_LuminanceMin { 0x55DA, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_Audio { 0xE1, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_SamplingFrequency { 0xB5, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_OutputSamplingFrequency { 0x78B5, EEBMLDataType::EBMLfloat, DEF_ZERO() };
	static const FEBMLElement MKV_Channels { 0x9F, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement MKV_BitDepth { 0x6264, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_Emphasis { 0x52F1, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_ContentEncodings { 0x6D80, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_ContentEncoding { 0x6240, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_ContentEncodingOrder { 0x5031, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_ContentEncodingScope { 0x5032, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement MKV_ContentEncodingType { 0x5033, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_ContentCompression { 0x5034, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_ContentCompAlgo { 0x4254, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_ContentCompSettings { 0x4255, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_ContentEncryption { 0x5035, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_ContentEncAlgo { 0x47E1, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_ContentEncKeyID { 0x47E2, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_ContentEncAESSettings { 0x47E7, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_AESSettingsCipherMode { 0x47E8, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_TrackTimestampScale { 0x23314F, EEBMLDataType::EBMLfloat, DEF_ZERO() };		// DEPRECATED

	static const FEBMLElement MKV_AttachedFile { 0x61A7, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_FileDescription { 0x467E, EEBMLDataType::EBMLutf8, DEF_ZERO() };
	static const FEBMLElement MKV_FileName { 0x466E, EEBMLDataType::EBMLutf8, DEF_ZERO() };
	static const FEBMLElement MKV_FileMediaType { 0x4660, EEBMLDataType::EBMLansi, DEF_ZERO() };
	static const FEBMLElement MKV_FileData { 0x465C, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_FileUID { 0x46AE, EEBMLDataType::EBMLuint, DEF_ZERO() };

	static const FEBMLElement MKV_Tag { 0x7373, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Targets { 0x63C0, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_TargetTypeValue { 0x68CA, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_TargetType { 0x63CA, EEBMLDataType::EBMLansi, DEF_ZERO() };
	static const FEBMLElement MKV_TagTrackUID { 0x63C5, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_TagEditionUID { 0x63C9, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_TagChapterUID { 0x63C4, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_TagAttachmentUID { 0x63C6, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_SimpleTag { 0x67C8, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_TagName { 0x45A3, EEBMLDataType::EBMLutf8, DEF_ZERO() };
	static const FEBMLElement MKV_TagLanguage { 0x447A, EEBMLDataType::EBMLansi, DEF_ZERO() };
	static const FEBMLElement MKV_TagLanguageBCP47 { 0x447B, EEBMLDataType::EBMLansi, DEF_ZERO() };
	static const FEBMLElement MKV_TagDefault { 0x4484, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement MKV_TagString { 0x4487, EEBMLDataType::EBMLutf8, DEF_ZERO() };
	static const FEBMLElement MKV_TagBinary { 0x4485, EEBMLDataType::EBMLbinary, DEF_ZERO() };

	static const FEBMLElement MKV_CuePoint { 0xBB, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_CueTime { 0xB3, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_CueTrackPositions { 0xB7, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_CueTrack { 0xF7, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_CueClusterPosition { 0xF1, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_CueRelativePosition { 0xF0, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_CueDuration { 0xB2, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_CueBlockNumber { 0x5378, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_CueCodecState { 0xEA, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_CueReference { 0xDB, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_CueRefTime { 0x96, EEBMLDataType::EBMLuint, DEF_ZERO() };

	static const FEBMLElement MKV_Timestamp { 0xE7, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_Position { 0xA7, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_PrevSize { 0xAB, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_SimpleBlock { 0xA3, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_BlockGroup { 0xA0, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_Block { 0xA1, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_BlockAdditions { 0x75A1, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_BlockMore { 0xA6, EEBMLDataType::EBMLstruct, DEF_ZERO() };
	static const FEBMLElement MKV_BlockAdditional { 0xA5, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_BlockAddID { 0xEE, EEBMLDataType::EBMLuint, DEF_UINT(1) };
	static const FEBMLElement MKV_BlockDuration { 0x9B, EEBMLDataType::EBMLuint, DEF_ZERO() };
	static const FEBMLElement MKV_ReferencePriority { 0xFA, EEBMLDataType::EBMLuint, DEF_UINT(0) };
	static const FEBMLElement MKV_ReferenceBlock { 0xFB, EEBMLDataType::EBMLint, DEF_ZERO() };
	static const FEBMLElement MKV_CodecState { 0xA4, EEBMLDataType::EBMLbinary, DEF_ZERO() };
	static const FEBMLElement MKV_DiscardPadding { 0x75A2, EEBMLDataType::EBMLint, DEF_ZERO() };




	class FParserMKV : public TSharedFromThis<FParserMKV, ESPMode::ThreadSafe>, public IParserMKV, public FBufferedDataReader::IDataProvider
	{
	public:
		// Internal element pointer types
		template <typename T>
		using TMKVElementPtr = TSharedPtr<T, ESPMode::ThreadSafe>;
		template<typename T, typename... ArgTypes>
		static TSharedRef<T, ESPMode::ThreadSafe> MakeMKVElementPtr(ArgTypes&&... Args)
		{
			return MakeShared<T, ESPMode::ThreadSafe>(Forward<ArgTypes>(Args)...);
		}

		template <typename T>
		using TMKVUniquePtr = TUniquePtr<T>;
		#define MakeMKVUniquePtr MakeUnique


		FParserMKV(IPlayerSessionServices* InPlayerSession);

		FErrorDetail ParseHeader(IGenericDataReader* DataReader, EParserFlags ParseFlags) override;
		FErrorDetail PrepareTracks() override;
		FTimeValue GetDuration() const override;
		int32 GetNumberOfTracks() const override;
		const ITrack* GetTrackByIndex(int32 Index) const override;
		const ITrack* GetTrackByTrackID(uint64 TrackID) const override;
		TSharedPtrTS<IClusterParser> CreateClusterParser(IGenericDataReader* DataReader, const TArray<uint64>& TrackIDsToParse, EClusterParseFlags ParseFlags) const override;
		void AddCue(int64 InCueTimestamp, uint64 InTrackID, int64 InCueRelativePosition, uint64 InCueBlockNumber, int64 InClusterPosition) override;

		int64 OnReadAssetData(void* Destination, int64 NumBytes, int64 FromOffset, int64* OutTotalSize) override;
	private:
		enum EMKVTrackType
		{
			Video = 1,
			Audio = 2,
			Complex = 3,
			Logo = 16,
			Subtitle = 17,
			Buttons = 18,
			Control = 32,
			Metadata = 33
		};

		enum class EMKVVideoInterlaceType
		{
			Undetermined,
			Interlaced,
			Progressive
		};

		enum class EParseResult
		{
			Ok,
			Error,
			StopParsing
		};

		using SyntaxElementHandler = TFunction<EParseResult(IMKVFetcher*,uint32,int64,int64)>;
		struct FSyntaxElement
		{
			const FEBMLElement* Element;
			void* DataValue;
			SyntaxElementHandler NewStructParseFN;
		};
		#define SYNTAXELEMENT_FN(Class, List) [this](IMKVFetcher* InReader, uint32 InID, int64 InByteOffset, int64 InByteSize){return CreateAndParseElement<Class>(InReader, InID, InByteOffset, InByteSize, List); }
		#define ELEMENT_ENTRY(ElemName) { &MKV_##ElemName, &ElemName }


		class FMKVEBMLElement
		{
		public:
			FMKVEBMLElement(uint32 InID, int64 InByteOffset, int64 InByteSize) : ID(InID), ByteOffset(InByteOffset), ByteSize(InByteSize) { }
			virtual ~FMKVEBMLElement() = default;
			virtual uint32 GetID() const
			{ return ID; }
			virtual int64 GetByteOffset() const
			{ return ByteOffset; }
			virtual int64 GetByteSize() const
			{ return ByteSize; }
			virtual int64 GetElementOffset() const
			{ return ByteOffset + ElementOffset; }

			virtual EParseResult ParseElement(IMKVFetcher* InReader) = 0;

			virtual EParseResult ParseNextElementFromList(uint32 InElementID, const FSyntaxElement* InListOfElements, IMKVFetcher* InReader)
			{
				for(const FSyntaxElement* Elem=InListOfElements; Elem->Element; ++Elem)
				{
					if (InElementID == Elem->Element->ID)
					{
						int64 StartOffset = InReader->FetchCurrentOffset();
						uint32 ElementID;
						int64 ElementLen;
						if (InReader->FetchAtEOS())
						{
							return EParseResult::StopParsing;
						}
						else if (!InReader->FetchElementID(ElementID) || !InReader->FetchElementLength(ElementLen))
						{
							return EParseResult::Error;
						}
						// Is this the type we expected?
						if (ElementID != InElementID)
						{
							return EParseResult::Error;
						}
						return ParseOneElement(Elem, StartOffset, ElementLen, InReader);
					}
				}
				return EParseResult::Error;
			}

			virtual EParseResult ParseElementList(const FSyntaxElement* InListOfElements, IMKVFetcher* InReader)
			{
				for(int64 BytesToGo=ByteSize; BytesToGo>0; )
				{
					int64 StartOffset = InReader->FetchCurrentOffset();
					uint32 ElementID;
					int64 ElementLen;
					if (InReader->FetchAtEOS())
					{
						return EParseResult::StopParsing;
					}
					else if (!InReader->FetchElementID(ElementID) || !InReader->FetchElementLength(ElementLen))
					{
						return EParseResult::Error;
					}
					//
					bool bIgnore = ElementID == EBML_Void.ID || ElementID == EBML_CRC32.ID;
					if (!bIgnore)
					{
						const FSyntaxElement* Elem=InListOfElements;
						for(; Elem->Element; ++Elem)
						{
							if (ElementID == Elem->Element->ID)
							{
								EParseResult Result = ParseOneElement(Elem, StartOffset, ElementLen, InReader);
								if (Result != EParseResult::Ok)
								{
									return Result;
								}
								break;
							}
						}
						if (Elem->Element == nullptr)
						{
							bIgnore = true;
						}
					}
					if (bIgnore)
					{
						if (!InReader->FetchSkipOver(ElementLen))
						{
							return EParseResult::Error;
						}
					}
					BytesToGo -= InReader->FetchCurrentOffset() - StartOffset;
				}
				return EParseResult::Ok;
			}

		protected:
			template<typename T>
			EParseResult CreateAndParseElement(IMKVFetcher* InReader, uint32 InID, int64 InByteOffset, int64 InByteSize, TArray<TMKVElementPtr<T>>& ElementList)
			{
				TMKVElementPtr<T> NewElement = MakeMKVElementPtr<T>(InID, InByteOffset, InByteSize);
				NewElement->ElementOffset = InReader->FetchCurrentOffset() - InByteOffset;
				EParseResult Result = NewElement->ParseElement(InReader);
				if (Result == EParseResult::Ok)
				{
					ElementList.Emplace(MoveTemp(NewElement));
				}
				return Result;
			}

			template<typename T>
			EParseResult CreateAndParseElement(IMKVFetcher* InReader, uint32 InID, int64 InByteOffset, int64 InByteSize, TArray<TMKVUniquePtr<T>>& ElementList)
			{
				TMKVUniquePtr<T> NewElement = MakeMKVUniquePtr<T>(InID, InByteOffset, InByteSize);
				NewElement->ElementOffset = InReader->FetchCurrentOffset() - InByteOffset;
				EParseResult Result = NewElement->ParseElement(InReader);
				if (Result == EParseResult::Ok)
				{
					ElementList.Emplace(MoveTemp(NewElement));
				}
				return Result;
			}

			EParseResult ParseOneElement(const FSyntaxElement* Elem, int64 StartOffset, int64 ElementLen, IMKVFetcher* InReader)
			{
				switch(Elem->Element->Type)
				{
					case EEBMLDataType::EBMLuint:
					{
						if (!InReader->FetchUInt(*reinterpret_cast<uint64*>(Elem->DataValue), ElementLen, Elem->Element->Default.u64))
						{
							return EParseResult::Error;
						}
						break;
					}
					case EEBMLDataType::EBMLfloat:
					{
						if (!InReader->FetchFloat(*reinterpret_cast<double*>(Elem->DataValue), ElementLen, Elem->Element->Default.f64))
						{
							return EParseResult::Error;
						}
						break;
					}
					case EEBMLDataType::EBMLansi:
					{
						TArray<uint8> Chars;
						if (!InReader->FetchByteArray(Chars, ElementLen))
						{
							return EParseResult::Error;
						}
						Chars.Add(0);
						auto Cnv = StringCast<TCHAR>((const ANSICHAR*)Chars.GetData());
						FString UTF8Text = FString::ConstructFromPtrSize(Cnv.Get(), Cnv.Length());
						*reinterpret_cast<FString*>(Elem->DataValue) = MoveTemp(UTF8Text);
						break;
					}
					case EEBMLDataType::EBMLutf8:
					{
						TArray<uint8> Chars;
						if (!InReader->FetchByteArray(Chars, ElementLen))
						{
							return EParseResult::Error;
						}
						Chars.Add(0);
						auto Cnv = StringCast<TCHAR>((const UTF8CHAR*)Chars.GetData());
						FString UTF8Text = FString::ConstructFromPtrSize(Cnv.Get(), Cnv.Length());
						*reinterpret_cast<FString*>(Elem->DataValue) = MoveTemp(UTF8Text);
						break;
						}
					case EEBMLDataType::EBMLbinary:
					{
						if (!InReader->FetchByteArray(*reinterpret_cast<TArray<uint8>*>(Elem->DataValue), ElementLen))
						{
							return EParseResult::Error;
						}
						break;
					}
					case EEBMLDataType::EBMLstruct:
					{
						check(Elem->NewStructParseFN);
						EParseResult Result = Elem->NewStructParseFN ? Elem->NewStructParseFN(InReader, Elem->Element->ID, StartOffset, ElementLen) : EParseResult::Error;
						if (Result != EParseResult::Ok)
						{
							return Result;
						}
						break;
					}
					case EEBMLDataType::EBMLstop:
					{
						return EParseResult::StopParsing;
					}
					default:
					{
						check(!"Not handled yet");
						return EParseResult::Error;
					}
				}
				return EParseResult::Ok;
			}

			uint32 ID = 0;
			int64 ByteOffset = 0;
			int64 ByteSize = 0;
			int32 ElementOffset = 0;
		};

		/**
		 * EBML Header.
		 * This is not Matroska specific.
		 */
		class FMKVEBMLHeader : public FMKVEBMLElement
		{
		public:
			FMKVEBMLHeader(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVEBMLHeader() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetVersion() const { return Version; }
			uint64 GetReadVersion() const { return ReadVersion; }
			uint64 GetMaxIDLength() const { return MaxIDLength; }
			uint64 GetMaxSizeLength() const { return MaxSizeLength; }
			const FString& GetDocType() const { return DocType; }
			uint64 GetDocTypeVersion() const { return DocTypeVersion; }
			uint64 GetDocTypeReadVersion() const { return DocTypeReadVersion; }
		private:
			FString DocType;
			uint64 Version = 0;
			uint64 ReadVersion = 0;
			uint64 MaxIDLength = 0;
			uint64 MaxSizeLength = 0;
			uint64 DocTypeVersion = 0;
			uint64 DocTypeReadVersion = 0;
			const FSyntaxElement Elements[8] =
			{
				{ &EBML_Version, &Version },
				{ &EBML_ReadVersion, &ReadVersion },
				{ &EBML_MaxIDLength, &MaxIDLength },
				{ &EBML_MaxSizeLength, &MaxSizeLength },
				{ &EBML_DocType, &DocType },
				{ &EBML_DocTypeVersion, &DocTypeVersion },
				{ &EBML_DocTypeReadVersion, &DocTypeReadVersion },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Seek
		 */
		class FMKVSeek : public FMKVEBMLElement
		{
		public:
			FMKVSeek(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVSeek() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetSeekPosition() const
			{ return SeekPosition; }

			uint32 GetSeekID() const
			{
				uint32 id = 0;
				if (SeekID.Num() <= 4)
				{
					for(int32 i=0; i<4; ++i)
					{
						id = (id << 8) | SeekID[i];
					}
				}
				return id;
			}

		private:
			TArray<uint8> SeekID;
			uint64 SeekPosition = 0;
			const FSyntaxElement Elements[3] =
			{
				ELEMENT_ENTRY(SeekID),
				ELEMENT_ENTRY(SeekPosition),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska SeekHead
		 */
		class FMKVSeekHead : public FMKVEBMLElement
		{
		public:
			FMKVSeekHead(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVSeekHead() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader)
			{
				EParseResult Result = ParseElementList(Elements, InReader);
				// We sort the Seek elements by their file position. This may avoid too many random file accesses
				// when loading the elements after a cluster.
				Seeks.Sort([](const TMKVElementPtr<FMKVSeek>& a, const TMKVElementPtr<FMKVSeek>& b){return a->GetSeekPosition() < b->GetSeekPosition();});
				return Result;
			}

			bool HaveInfo() const
			{ return HaveID(MKV_Info.ID); }
			bool HaveTracks() const
			{ return HaveID(MKV_Tracks.ID); }
			bool HaveCues() const
			{ return HaveID(MKV_Cues.ID); }
			bool HaveAttachments() const
			{ return HaveID(MKV_Attachments.ID); }
			bool HaveChapters() const
			{ return HaveID(MKV_Chapters.ID); }
			bool HaveTags() const
			{ return HaveID(MKV_Tags.ID); }

			uint64 GetInfoPosition() const
			{ return GetPositionOfID(MKV_Info.ID); }
			uint64 GetTracksPosition() const
			{ return GetPositionOfID(MKV_Tracks.ID); }
			uint64 GetCuesPosition() const
			{ return GetPositionOfID(MKV_Cues.ID); }
			uint64 GetAttachmentsPosition() const
			{ return GetPositionOfID(MKV_Attachments.ID); }
			uint64 GetChaptersPosition() const
			{ return GetPositionOfID(MKV_Chapters.ID); }
			uint64 GetTagsPosition() const
			{ return GetPositionOfID(MKV_Tags.ID); }
			uint64 GetPositionOfID(uint32 InID) const
			{
				auto Seek = Seeks.FindByPredicate([InID](const TMKVElementPtr<const FMKVSeek>& InElem) { return InElem->GetSeekID() == InID; });
				return Seek ? (*Seek)->GetSeekPosition() : 0;
			}
			const TArray<TMKVElementPtr<FMKVSeek>>& GetSeeks() const
			{ return Seeks; }
		private:
			bool HaveID(uint32 InID) const
			{ return !!Seeks.FindByPredicate([InID](const TMKVElementPtr<const FMKVSeek>& InElem) { return InElem->GetSeekID() == InID; }); }

			TArray<TMKVElementPtr<FMKVSeek>> Seeks;
			const FSyntaxElement Elements[2] =
			{
				{ &MKV_Seek, 0, SYNTAXELEMENT_FN(FMKVSeek, Seeks) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Info
		 */
		class FMKVInfo : public FMKVEBMLElement
		{
		public:
			FMKVInfo(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVInfo() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetTimestampScale() const
			{ return TimestampScale; }
			double GetDuration() const
			{ return Duration; }

		private:
			double Duration = 0.0;
			uint64 TimestampScale = DEFAULT_MATROSKA_TIMESCALE;
			const FSyntaxElement Elements[3] =
			{
				ELEMENT_ENTRY(TimestampScale),
				ELEMENT_ENTRY(Duration),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Block Addition Mapping
		 */
		class FMKVBlockAdditionMapping : public FMKVEBMLElement
		{
		public:
			FMKVBlockAdditionMapping(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVBlockAdditionMapping() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

		private:
			TArray<uint8> BlockAddIDExtraData;
			uint64 BlockAddIDValue = 0;
			uint64 BlockAddIDType = 0;
			const FSyntaxElement Elements[4] =
			{
				ELEMENT_ENTRY(BlockAddIDValue),
				ELEMENT_ENTRY(BlockAddIDType),
				ELEMENT_ENTRY(BlockAddIDExtraData),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Mastering Metadata
		 */
		class FMKVMasteringMetadata : public FMKVEBMLElement
		{
		public:
			FMKVMasteringMetadata(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVMasteringMetadata() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			double GetPrimaryRChromaticityX() const { return PrimaryRChromaticityX; }
			double GetPrimaryRChromaticityY() const { return PrimaryRChromaticityY; }
			double GetPrimaryGChromaticityX() const { return PrimaryGChromaticityX; }
			double GetPrimaryGChromaticityY() const { return PrimaryGChromaticityY; }
			double GetPrimaryBChromaticityX() const { return PrimaryBChromaticityX; }
			double GetPrimaryBChromaticityY() const { return PrimaryBChromaticityY; }
			double GetWhitePointChromaticityX() const { return WhitePointChromaticityX; }
			double GetWhitePointChromaticityY() const { return WhitePointChromaticityY; }
			double GetLuminanceMax() const { return LuminanceMax; }
			double GetLuminanceMin() const { return LuminanceMin; }
		private:
			double PrimaryRChromaticityX = 0.0;
			double PrimaryRChromaticityY = 0.0;
			double PrimaryGChromaticityX = 0.0;
			double PrimaryGChromaticityY = 0.0;
			double PrimaryBChromaticityX = 0.0;
			double PrimaryBChromaticityY = 0.0;
			double WhitePointChromaticityX = 0.0;
			double WhitePointChromaticityY = 0.0;
			double LuminanceMax = 0.0;
			double LuminanceMin = 0.0;
			const FSyntaxElement Elements[11] =
			{
				ELEMENT_ENTRY(PrimaryRChromaticityX),
				ELEMENT_ENTRY(PrimaryRChromaticityY),
				ELEMENT_ENTRY(PrimaryGChromaticityX),
				ELEMENT_ENTRY(PrimaryGChromaticityY),
				ELEMENT_ENTRY(PrimaryBChromaticityX),
				ELEMENT_ENTRY(PrimaryBChromaticityY),
				ELEMENT_ENTRY(WhitePointChromaticityX),
				ELEMENT_ENTRY(WhitePointChromaticityY),
				ELEMENT_ENTRY(LuminanceMax),
				ELEMENT_ENTRY(LuminanceMin),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Colour
		 */
		class FMKVColour : public FMKVEBMLElement
		{
		public:
			FMKVColour(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVColour() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			int32 GetMatrixCoefficients() const
			{ return (int32)MatrixCoefficients; }
			int32 GetBitsPerChannel() const
			{ return (int32)BitsPerChannel; }
			int32 GetChromaSubsamplingHorz() const
			{ return (int32)ChromaSubsamplingHorz; }
			int32 GetChromaSubsamplingVert() const
			{ return (int32)ChromaSubsamplingVert; }
			int32 GetCbSubsamplingHorz() const
			{ return (int32)CbSubsamplingHorz; }
			int32 GetCbSubsamplingVert() const
			{ return (int32)CbSubsamplingVert; }
			int32 GetChromaSitingHorz() const
			{ return (int32)ChromaSitingHorz; }
			int32 GetChromaSitingVert() const
			{ return (int32)ChromaSitingVert; }
			int32 GetRange() const
			{ return (int32)Range; }
			int32 GetTransferCharacteristics() const
			{ return (int32)TransferCharacteristics; }
			int32 GetPrimaries() const
			{ return (int32)Primaries; }
			uint16 GetMaxCLL() const
			{ return (uint16)MaxCLL; }
			uint16 GetMaxFALL() const
			{ return (uint16)MaxFALL; }
			TMKVElementPtr<FMKVMasteringMetadata> GetMasteringMetadata() const
			{ return MasteringMetadatas.Num() ? MasteringMetadatas[0] : nullptr; }
		private:
			TArray<TMKVElementPtr<FMKVMasteringMetadata>> MasteringMetadatas;
			uint64 MatrixCoefficients = 2;
			uint64 BitsPerChannel = 0;
			uint64 ChromaSubsamplingHorz = 0;
			uint64 ChromaSubsamplingVert = 0;
			uint64 CbSubsamplingHorz = 0;
			uint64 CbSubsamplingVert = 0;
			uint64 ChromaSitingHorz = 0;
			uint64 ChromaSitingVert = 0;
			uint64 Range = 0;
			uint64 TransferCharacteristics = 2;
			uint64 Primaries = 2;
			uint64 MaxCLL = 0;
			uint64 MaxFALL = 0;
			const FSyntaxElement Elements[15] =
			{
				ELEMENT_ENTRY(MatrixCoefficients),
				ELEMENT_ENTRY(BitsPerChannel),
				ELEMENT_ENTRY(ChromaSubsamplingHorz),
				ELEMENT_ENTRY(ChromaSubsamplingVert),
				ELEMENT_ENTRY(CbSubsamplingHorz),
				ELEMENT_ENTRY(CbSubsamplingVert),
				ELEMENT_ENTRY(ChromaSitingHorz),
				ELEMENT_ENTRY(ChromaSitingVert),
				ELEMENT_ENTRY(Range),
				ELEMENT_ENTRY(TransferCharacteristics),
				ELEMENT_ENTRY(Primaries),
				ELEMENT_ENTRY(MaxCLL),
				ELEMENT_ENTRY(MaxFALL),
				{ &MKV_MasteringMetadata, 0, SYNTAXELEMENT_FN(FMKVMasteringMetadata, MasteringMetadatas) },
				{ nullptr, nullptr }
			};
		};

		/**
		 * Matroska Video
		 */
		class FMKVVideo : public FMKVEBMLElement
		{
		public:
			FMKVVideo(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVVideo() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			EMKVVideoInterlaceType GetInterlaced() const
			{ return FlagInterlaced == 2 ? EMKVVideoInterlaceType::Progressive : FlagInterlaced == 1 ? EMKVVideoInterlaceType::Interlaced : EMKVVideoInterlaceType::Undetermined; }
			int32 GetDisplayWidth() const
			{ return (int32)(PixelWidth - PixelCropLeft - PixelCropRight); }
			int32 GetDisplayHeight() const
			{ return (int32)(PixelHeight - PixelCropTop - PixelCropBottom); }
			int32 GetCropLeft() const
			{ return (int32)PixelCropLeft; }
			int32 GetCropRight() const
			{ return (int32)PixelCropRight; }
			int32 GetCropTop() const
			{ return (int32)PixelCropTop; }
			int32 GetCropBottom() const
			{ return (int32)PixelCropBottom; }
			double GetFrameRate() const
			{ return FrameRate; }
			TMKVElementPtr<FMKVColour> GetColours() const
			{ return Colours.Num() ? Colours[0] : nullptr; }
			int32 GetAlphaMode() const
			{ return (int32)AlphaMode; }
		private:
			TArray<TMKVElementPtr<FMKVColour>> Colours;
			TArray<uint8> UncompressedFourCC;
			double FrameRate = 0.0;
			uint64 FlagInterlaced = 0;
			uint64 FieldOrder = 2;
			uint64 StereoMode = 0;
			uint64 AlphaMode = 0;
			uint64 PixelWidth = 0;
			uint64 PixelHeight = 0;
			uint64 PixelCropBottom = 0;
			uint64 PixelCropTop = 0;
			uint64 PixelCropLeft = 0;
			uint64 PixelCropRight = 0;
			uint64 DisplayWidth = 0;
			uint64 DisplayHeight = 0;
			uint64 DisplayUnit = 0;
			const FSyntaxElement Elements[17] =
			{
				ELEMENT_ENTRY(FlagInterlaced),
				ELEMENT_ENTRY(FieldOrder),
				ELEMENT_ENTRY(StereoMode),
				ELEMENT_ENTRY(AlphaMode),
				ELEMENT_ENTRY(PixelWidth),
				ELEMENT_ENTRY(PixelHeight),
				ELEMENT_ENTRY(PixelCropBottom),
				ELEMENT_ENTRY(PixelCropTop),
				ELEMENT_ENTRY(PixelCropLeft),
				ELEMENT_ENTRY(PixelCropRight),
				ELEMENT_ENTRY(DisplayWidth),
				ELEMENT_ENTRY(DisplayHeight),
				ELEMENT_ENTRY(DisplayUnit),
				ELEMENT_ENTRY(UncompressedFourCC),
				ELEMENT_ENTRY(FrameRate),
				{ &MKV_Colour, 0, SYNTAXELEMENT_FN(FMKVColour, Colours) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Audio
		 */
		class FMKVAudio : public FMKVEBMLElement
		{
		public:
			FMKVAudio(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVAudio() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader)
			{
				EParseResult Result = ParseElementList(Elements, InReader);
				if (OutputSamplingFrequency == 0.0)
				{
					OutputSamplingFrequency = SamplingFrequency;
				}
				return Result;
			}

			int32 GetOutputSampleRate() const
			{ return (int32) OutputSamplingFrequency; }
			int32 GetNumberOfChannels() const
			{ return (int32) Channels; }

		private:
			double SamplingFrequency = 0.0;
			double OutputSamplingFrequency = 0.0;
			uint64 Channels = 1;
			uint64 BitDepth = 0;
			uint64 Emphasis = 0;
			const FSyntaxElement Elements[6] =
			{
				ELEMENT_ENTRY(SamplingFrequency),
				ELEMENT_ENTRY(OutputSamplingFrequency),
				ELEMENT_ENTRY(Channels),
				ELEMENT_ENTRY(BitDepth),
				ELEMENT_ENTRY(Emphasis),
				{ nullptr, nullptr }
			};
		};


		/**
		* Matroska ContentEncAESSettings
		*/
		class FMKVContentEncAESSettings : public FMKVEBMLElement
		{
		public:
			FMKVContentEncAESSettings(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVContentEncAESSettings() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetAESSettingsCipherMode() const
			{ return AESSettingsCipherMode; }
		private:
			uint64 AESSettingsCipherMode = 0;
			const FSyntaxElement Elements[2] =
			{
				ELEMENT_ENTRY(AESSettingsCipherMode),
				{ nullptr, nullptr }
			};
		};


		/**
		* Matroska Content Encryption
		*/
		class FMKVContentEncryption : public FMKVEBMLElement
		{
		public:
			FMKVContentEncryption(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVContentEncryption() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetContentEncAlgo() const
			{ return ContentEncAlgo; }
			const TArray<uint8>& GetContentEncKeyID() const
			{ return ContentEncKeyID; }
			TMKVElementPtr<FMKVContentEncAESSettings> GetContentEncAESSettings() const
			{ return ContentEncAESSettings.Num() ? ContentEncAESSettings[0] : nullptr; }
		private:
			TArray<TMKVElementPtr<FMKVContentEncAESSettings>> ContentEncAESSettings;
			TArray<uint8> ContentEncKeyID;
			uint64 ContentEncAlgo = 0;
			const FSyntaxElement Elements[4] =
			{
				ELEMENT_ENTRY(ContentEncAlgo),
				ELEMENT_ENTRY(ContentEncKeyID),
				{ &MKV_ContentEncAESSettings, 0, SYNTAXELEMENT_FN(FMKVContentEncAESSettings, ContentEncAESSettings) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Content Compression
		 */
		class FMKVContentCompression : public FMKVEBMLElement
		{
		public:
			FMKVContentCompression(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVContentCompression() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetContentCompAlgo() const
			{ return ContentCompAlgo; }
			const TArray<uint8>& GetContentCompSettings() const
			{ return ContentCompSettings; }
		private:
			TArray<uint8> ContentCompSettings;
			uint64 ContentCompAlgo = 0;
			const FSyntaxElement Elements[3] =
			{
				ELEMENT_ENTRY(ContentCompAlgo),
				ELEMENT_ENTRY(ContentCompSettings),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Content Encoding
		 */
		class FMKVContentEncoding : public FMKVEBMLElement
		{
		public:
			FMKVContentEncoding(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVContentEncoding() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetContentEncodingOrder() const
			{ return ContentEncodingOrder; }
			uint64 GetContentEncodingScope() const
			{ return ContentEncodingScope; }
			uint64 GetContentEncodingType() const
			{ return ContentEncodingType; }
			TMKVElementPtr<FMKVContentCompression> GetContentCompression() const
			{ return ContentCompression.Num() ? ContentCompression[0] : nullptr; }
			TMKVElementPtr<FMKVContentEncryption> GetContentEncryption() const
			{ return ContentEncryption.Num() ? ContentEncryption[0] : nullptr; }
		private:
			TArray<TMKVElementPtr<FMKVContentCompression>> ContentCompression;
			TArray<TMKVElementPtr<FMKVContentEncryption>> ContentEncryption;
			uint64 ContentEncodingOrder = 0;
			uint64 ContentEncodingScope = 1;
			uint64 ContentEncodingType = 0;
			const FSyntaxElement Elements[6] =
			{
				ELEMENT_ENTRY(ContentEncodingOrder),
				ELEMENT_ENTRY(ContentEncodingScope),
				ELEMENT_ENTRY(ContentEncodingType),
				{ &MKV_ContentCompression, 0, SYNTAXELEMENT_FN(FMKVContentCompression, ContentCompression) },
				{ &MKV_ContentEncryption, 0, SYNTAXELEMENT_FN(FMKVContentEncryption, ContentEncryption) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Content Encodings
		 */
		class FMKVContentEncodings : public FMKVEBMLElement
		{
		public:
			FMKVContentEncodings(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVContentEncodings() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader)
			{
				EParseResult Result = ParseElementList(Elements, InReader);
				// The content encodings must be processed in descending `ContentEncodingOrder`, but it is not mandated
				// that they are sorted such in the file. We sort them now.
				ContentEncoding.StableSort([](const TMKVElementPtr<FMKVContentEncoding>& a, const TMKVElementPtr<FMKVContentEncoding>& b){return a->GetContentEncodingOrder() > b->GetContentEncodingOrder();});
				return Result;
			}
			const TArray<TMKVElementPtr<FMKVContentEncoding>>& GetContentEncoding() const
			{ return ContentEncoding; }

		private:
			TArray<TMKVElementPtr<FMKVContentEncoding>> ContentEncoding;
			const FSyntaxElement Elements[2] =
			{
				{ &MKV_ContentEncoding, 0, SYNTAXELEMENT_FN(FMKVContentEncoding, ContentEncoding) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Track Entry
		 */
		class FMKVTrackEntry : public FMKVEBMLElement
		{
		public:
			FMKVTrackEntry(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVTrackEntry() = default;
			EParseResult ParseElement(IMKVFetcher* InReader)  override
			{
				EParseResult Result = ParseElementList(Elements, InReader);
				BCP47::ParseRFC5646Tag(LanguageTag, GetLanguage());
				return Result;
			}

			uint64 GetTrackNumber() const
			{ return TrackNumber; }
			uint64 GetTrackType() const
			{ return TrackType; }
			bool IsEnabled() const
			{ return !!FlagEnabled; }
			bool IsDefault() const
			{ return !!FlagDefault; }
			bool IsForced() const
			{ return !!FlagForced; }
			bool IsHearingImpaired() const
			{ return !!FlagHearingImpaired; }
			bool IsVisualImpaired() const
			{ return !!FlagVisualImpaired; }
			bool IsTextDescription() const
			{ return !!FlagTextDescriptions; }
			bool IsOriginal() const
			{ return !!FlagOriginal; }
			bool IsCommentary() const
			{ return !!FlagCommentary; }
			uint64 GetDefaultDurationNanos() const
			{ return DefaultDuration; }
			void SetDefaultDurationNanos(uint64 InDefaultDuration)
			{ DefaultDuration = InDefaultDuration; }
			uint64 GetCodecDelayNanos() const
			{ return CodecDelay; }
			const FString& GetName() const
			{ return Name; }
			const FString& GetLanguage() const
			{ return LanguageBCP47.Len() ? LanguageBCP47 : Language; }
			const BCP47::FLanguageTag& GetLanguageTag() const
			{ return LanguageTag; }
			const FString& GetCodecID() const
			{ return CodecID; }
			const TArray<uint8>& GetCodecPrivate() const
			{ return CodecPrivate; }
			TMKVElementPtr<FMKVVideo> GetVideo() const
			{ return Videos.Num() ? Videos[0] : nullptr; }
			TMKVElementPtr<FMKVAudio> GetAudio() const
			{ return Audios.Num() ? Audios[0] : nullptr; }
			TMKVElementPtr<FMKVContentEncodings> GetContentEncodings() const
			{ return ContentEncodings.Num() ? ContentEncodings[0] : nullptr; }
			double GetTrackTimestampScale() const
			{ return TrackTimestampScale; }
		private:
			TArray<TMKVElementPtr<FMKVBlockAdditionMapping>> BlockAdditionMappings;
			TArray<TMKVElementPtr<FMKVVideo>> Videos;
			TArray<TMKVElementPtr<FMKVAudio>> Audios;
			TArray<TMKVElementPtr<FMKVContentEncodings>> ContentEncodings;
			TArray<uint8> CodecPrivate;
			FString Name;
			FString Language {TEXT("eng")};
			FString LanguageBCP47;
			BCP47::FLanguageTag LanguageTag;
			FString CodecID;
			FString CodecName;
			double TrackTimestampScale = 1.0;
			uint64 TrackNumber = 0;
			uint64 TrackUID = 0;
			uint64 TrackType = 0;
			uint64 FlagEnabled = 1;
			uint64 FlagDefault = 1;
			uint64 FlagForced = 0;
			uint64 FlagHearingImpaired = 0;
			uint64 FlagVisualImpaired = 0;
			uint64 FlagTextDescriptions = 0;
			uint64 FlagOriginal = 0;
			uint64 FlagCommentary = 0;
			uint64 FlagLacing = 1;
			uint64 DefaultDuration = 0;
			uint64 DefaultDecodedFieldDuration = 0;
			uint64 MaxBlockAdditionID = 0;
			uint64 CodecDelay = 0;
			uint64 SeekPreRoll = 0;

			const FSyntaxElement Elements[29] =
			{
				ELEMENT_ENTRY(TrackNumber),
				ELEMENT_ENTRY(TrackUID),
				ELEMENT_ENTRY(TrackType),
				ELEMENT_ENTRY(FlagEnabled),
				ELEMENT_ENTRY(FlagDefault),
				ELEMENT_ENTRY(FlagForced),
				ELEMENT_ENTRY(FlagHearingImpaired),
				ELEMENT_ENTRY(FlagVisualImpaired),
				ELEMENT_ENTRY(FlagTextDescriptions),
				ELEMENT_ENTRY(FlagOriginal),
				ELEMENT_ENTRY(FlagCommentary),
				ELEMENT_ENTRY(FlagLacing),
				ELEMENT_ENTRY(DefaultDuration),
				ELEMENT_ENTRY(DefaultDecodedFieldDuration),
				ELEMENT_ENTRY(MaxBlockAdditionID),
				{ &MKV_BlockAdditionMapping, 0, SYNTAXELEMENT_FN(FMKVBlockAdditionMapping, BlockAdditionMappings) },
				ELEMENT_ENTRY(Name),
				ELEMENT_ENTRY(Language),
				ELEMENT_ENTRY(LanguageBCP47),
				ELEMENT_ENTRY(CodecID),
				ELEMENT_ENTRY(CodecPrivate),
				ELEMENT_ENTRY(CodecName),
				ELEMENT_ENTRY(CodecDelay),
				ELEMENT_ENTRY(SeekPreRoll),
				{ &MKV_Video, 0, SYNTAXELEMENT_FN(FMKVVideo, Videos) },
				{ &MKV_Audio, 0, SYNTAXELEMENT_FN(FMKVAudio, Audios) },
				{ &MKV_ContentEncodings, 0, SYNTAXELEMENT_FN(FMKVContentEncodings, ContentEncodings) },
				ELEMENT_ENTRY(TrackTimestampScale),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Tracks
		 */
		class FMKVTracks : public FMKVEBMLElement
		{
		public:
			FMKVTracks(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVTracks() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			int32 GetNumberOfTracks() const
			{ return Tracks.Num(); }
			TMKVElementPtr<FMKVTrackEntry> GetTrackByIndex(int32 InIndex) const
			{ return InIndex >= 0 && InIndex < GetNumberOfTracks() ? Tracks[InIndex] : nullptr; }
		private:
			TArray<TMKVElementPtr<FMKVTrackEntry>> Tracks;
			const FSyntaxElement Elements[2] =
			{
				{ &MKV_TrackEntry, 0, SYNTAXELEMENT_FN(FMKVTrackEntry, Tracks) },
				{ nullptr, nullptr }
			};
		};

		/**
		 * Matroska Attached File
		 */
		class FMKVAttachedFile : public FMKVEBMLElement
		{
		public:
			FMKVAttachedFile(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVAttachedFile() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

		private:
			FString FileDescription;
			FString FileName;
			FString FileMediaType;
			TArray<uint8> FileData;
			uint64 FileUID = 0;
			const FSyntaxElement Elements[6] =
			{
				ELEMENT_ENTRY(FileDescription),
				ELEMENT_ENTRY(FileName),
				ELEMENT_ENTRY(FileMediaType),
				ELEMENT_ENTRY(FileData),
				ELEMENT_ENTRY(FileUID),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Attachments
		 */
		class FMKVAttachments : public FMKVEBMLElement
		{
		public:
			FMKVAttachments(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVAttachments() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

		private:
			TArray<TMKVElementPtr<FMKVAttachedFile>> Files;
			const FSyntaxElement Elements[2] =
			{
				{ &MKV_AttachedFile, 0, SYNTAXELEMENT_FN(FMKVAttachedFile, Files) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Chapters
		 */
		class FMKVChapters : public FMKVEBMLElement
		{
		public:
			FMKVChapters(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVChapters() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader)
			{
				// Not using chapters. Skip over them.
				return InReader->FetchSkipOver(ByteSize) ? EParseResult::Ok : EParseResult::Error;
			}

		private:
		};


		/**
		 * Matroska SimpleTag
		 */
		class FMKVSimpleTag : public FMKVEBMLElement
		{
		public:
			FMKVSimpleTag(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVSimpleTag() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

		private:
			TArray<uint8> TagBinary;
			FString TagString;
			FString TagName;
			FString TagLanguage;
			FString TagLanguageBCP47;
			uint64 TagDefault = 1;
			const FSyntaxElement Elements[7] =
			{
				ELEMENT_ENTRY(TagName),
				ELEMENT_ENTRY(TagLanguage),
				ELEMENT_ENTRY(TagLanguageBCP47),
				ELEMENT_ENTRY(TagDefault),
				ELEMENT_ENTRY(TagString),
				ELEMENT_ENTRY(TagBinary),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Targets
		 */
		class FMKVTargets : public FMKVEBMLElement
		{
		public:
			FMKVTargets(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVTargets() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

		private:
			FString TargetType;
			uint64 TargetTypeValue = 0;
			uint64 TagTrackUID = 0;
			uint64 TagEditionUID = 0;
			uint64 TagChapterUID = 0;
			uint64 TagAttachmentUID = 0;
			const FSyntaxElement Elements[7] =
			{
				ELEMENT_ENTRY(TargetTypeValue),
				ELEMENT_ENTRY(TargetType),
				ELEMENT_ENTRY(TagTrackUID),
				ELEMENT_ENTRY(TagEditionUID),
				ELEMENT_ENTRY(TagChapterUID),
				ELEMENT_ENTRY(TagAttachmentUID),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Tag
		 */
		class FMKVTag : public FMKVEBMLElement
		{
		public:
			FMKVTag(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVTag() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

		private:
			TArray<TMKVElementPtr<FMKVTargets>> Targets;
			TArray<TMKVElementPtr<FMKVSimpleTag>> SimpleTags;
			const FSyntaxElement Elements[3] =
			{
				{ &MKV_Targets, 0, SYNTAXELEMENT_FN(FMKVTargets, Targets) },
				{ &MKV_SimpleTag, 0, SYNTAXELEMENT_FN(FMKVSimpleTag, SimpleTags) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Tags
		 */
		class FMKVTags : public FMKVEBMLElement
		{
		public:
			FMKVTags(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVTags() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

		private:
			TArray<TMKVElementPtr<FMKVTag>> Tags;
			const FSyntaxElement Elements[2] =
			{
				{ &MKV_Tag, 0, SYNTAXELEMENT_FN(FMKVTag, Tags) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska CueReference
		 */
		class FMKVCueReference : public FMKVEBMLElement
		{
		public:
			FMKVCueReference(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVCueReference() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

		private:
			uint64 CueRefTime = 0;
			const FSyntaxElement Elements[2] =
			{
				ELEMENT_ENTRY(CueRefTime),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska CueTrackPositions
		 */
		class FMKVCueTrackPositions : public FMKVEBMLElement
		{
		public:
			FMKVCueTrackPositions(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVCueTrackPositions() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetCueTrack() const
			{ return CueTrack; }
			uint64 GetCueClusterPosition() const
			{ return CueClusterPosition; }
			void SetCueTrack(uint64 InCueTrack)
			{ CueTrack = InCueTrack; }
			void SetClusterPosition(uint64 InCueClusterPosition)
			{ CueClusterPosition = InCueClusterPosition; }
		private:
			TArray<TMKVUniquePtr<FMKVCueReference>> CueReferences;
			uint64 CueTrack = 0;
			uint64 CueClusterPosition = 0;
			uint64 CueRelativePosition = 0;
			uint64 CueDuration = 0;
			uint64 CueBlockNumber = 0;
			uint64 CueCodecState = 0;
			const FSyntaxElement Elements[8] =
			{
				ELEMENT_ENTRY(CueTrack),
				ELEMENT_ENTRY(CueClusterPosition),
				ELEMENT_ENTRY(CueRelativePosition),
				ELEMENT_ENTRY(CueDuration),
				ELEMENT_ENTRY(CueBlockNumber),
				ELEMENT_ENTRY(CueCodecState),
				{ &MKV_CueReference, 0, SYNTAXELEMENT_FN(FMKVCueReference, CueReferences) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska CuePoint
		 */
		class FMKVCuePoint : public FMKVEBMLElement
		{
		public:
			FMKVCuePoint(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVCuePoint() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetCueTime() const
			{ return CueTime; }
			void SetCueTime(uint64 InCueTime)
			{ CueTime = InCueTime; }
			const TArray<TMKVUniquePtr<FMKVCueTrackPositions>>& GetCueTrackPositions() const
			{ return CueTrackPositions; }
			bool IsEmpty() const
			{ return CueTrackPositions.IsEmpty(); }
			uint32 GetUniqueID() const
			{ return UniqueID; }
			void SetUniqueID(uint32 InID)
			{ UniqueID = InID; }
			void MoveTrackPositionsTo(FMKVCuePoint* DestinationCuePoint)
			{
				TArray<TMKVUniquePtr<FMKVCueTrackPositions>>& Destination = DestinationCuePoint->CueTrackPositions;
				for(int32 i=0; i<CueTrackPositions.Num(); ++i)
				{
					Destination.Emplace(TMKVUniquePtr<FMKVCueTrackPositions>(CueTrackPositions[i].Release()));
				}
				CueTrackPositions.Empty();
			}
			void AddCuePosition(uint64 InTrackID, int64 InCueRelativePosition, uint64 InCueBlockNumber, int64 InClusterPosition)
			{
				// Does the entry already exist?
				for(int32 i=0; i<CueTrackPositions.Num(); ++i)
				{
					if (CueTrackPositions[i]->GetCueTrack() == InTrackID && CueTrackPositions[i]->GetCueClusterPosition() == InClusterPosition)
					{
						// As we do not care about relative position or block number we are done here.
						return;
					}
				}
				TMKVUniquePtr<FMKVCueTrackPositions> ctp = MakeMKVUniquePtr<FMKVCueTrackPositions>(MKV_CueTrackPositions.ID, 0, 0);
				ctp->SetCueTrack(InTrackID);
				ctp->SetClusterPosition((uint64)InClusterPosition);
				CueTrackPositions.Emplace(MoveTemp(ctp));
			}

		private:
			TArray<TMKVUniquePtr<FMKVCueTrackPositions>> CueTrackPositions;
			uint64 CueTime = 0;
			uint32 UniqueID = 0;
			const FSyntaxElement Elements[3] =
			{
				ELEMENT_ENTRY(CueTime),
				{ &MKV_CueTrackPositions, 0, SYNTAXELEMENT_FN(FMKVCueTrackPositions, CueTrackPositions) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska Cues
		 */
		class FMKVCues : public FMKVEBMLElement
		{
		public:
			FMKVCues(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVCues() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader)
			{
				EParseResult Result = ParseElementList(Elements, InReader);
				if (Result == EParseResult::Ok)
				{
					CombineCueTimes();
				}
				return Result;
			}

			const TArray<TMKVUniquePtr<FMKVCuePoint>>& GetCuePoints() const
			{ return CuePoints; }
			void AddCue(int64 InCueTimestamp, uint64 InTrackID, int64 InCueRelativePosition, uint64 InCueBlockNumber, int64 InClusterPosition, uint32& InOutNextCueUniqueID)
			{
				// Try to find an existing cue point.
				FMKVCuePoint* cp = nullptr;
				int32 Index = 0;
				for(int32 iMax=CuePoints.Num(); Index<iMax; ++Index)
				{
					const int64 ct = (int64) CuePoints[Index]->GetCueTime();
					if (ct < InCueTimestamp)
					{
						continue;
					}
					else if (ct == InCueTimestamp)
					{
						cp = CuePoints[Index].Get();
						break;
					}
					else
					{
						break;
					}
				}
				if (!cp)
				{
					TMKVUniquePtr<FMKVCuePoint> NewCuePoint = MakeMKVUniquePtr<FMKVCuePoint>(MKV_CuePoint.ID, 0, 0);
					cp = NewCuePoint.Get();
					cp->SetCueTime((uint64)InCueTimestamp);
					cp->SetUniqueID(++InOutNextCueUniqueID);
					CuePoints.Insert(MoveTemp(NewCuePoint), Index);
				}
				cp->AddCuePosition(InTrackID, InCueRelativePosition, InCueBlockNumber, InClusterPosition);
			}

			void DebugRemoveCuesByIndexRange(const TRangeSet<int32>& InRanges)
			{
				for(int32 i=0; i<CuePoints.Num(); ++i)
				{
					if (InRanges.Contains(i))
					{
						CuePoints[i].Reset();
					}
				}
				Compact();
			}

		private:
			void SortCuePoints()
			{
				CuePoints.StableSort([](const TMKVUniquePtr<FMKVCuePoint>& a, const TMKVUniquePtr<FMKVCuePoint>& b){return a->GetCueTime() < b->GetCueTime();});
			}
			void Compact()
			{
				for(int32 i=0; i<CuePoints.Num(); ++i)
				{
					if (!CuePoints[i].IsValid() || CuePoints[i]->IsEmpty())
					{
						CuePoints.RemoveAt(i);
						--i;
					}
				}
			}
			// Combine cues that have the same time.
			void CombineCueTimes()
			{
				// Sort by time first. Matroska does not mandate, only recommends, cues to be sorted.
				SortCuePoints();
				for(int32 i=0; i<CuePoints.Num()-1; )
				{
					int32 j = i + 1;
					for(; j<CuePoints.Num(); ++j)
					{
						if (CuePoints[i]->GetCueTime() != CuePoints[j]->GetCueTime())
						{
							break;
						}
						// Merge the cue track positions from index 'j' into 'i'.
						CuePoints[j]->MoveTrackPositionsTo(CuePoints[i].Get());
					}
					i = j;
				}
				// If we merged any elements we need to remove the now empty ones.
				// We do this unconditionally in case there have been cues with no track positions.
				Compact();
			}
			TArray<TMKVUniquePtr<FMKVCuePoint>> CuePoints;
			const FSyntaxElement Elements[2] =
			{
				{ &MKV_CuePoint, 0, SYNTAXELEMENT_FN(FMKVCuePoint, CuePoints) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska BlockMore
		 */
		class FMKVBlockMore : public FMKVEBMLElement
		{
		public:
			FMKVBlockMore(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVBlockMore() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			uint64 GetBlockAddID() const
			{ return BlockAddID; }
			const TArray<uint8>& GetBlockAdditional() const
			{ return BlockAdditional; }
		private:
			TArray<uint8> BlockAdditional;
			uint64 BlockAddID = 1;
			const FSyntaxElement Elements[3] =
			{
				ELEMENT_ENTRY(BlockAdditional),
				ELEMENT_ENTRY(BlockAddID),
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska BlockAdditions
		 */
		class FMKVBlockAdditions : public FMKVEBMLElement
		{
		public:
			FMKVBlockAdditions(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVBlockAdditions() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader) { return ParseElementList(Elements, InReader); }

			const TArray<TMKVUniquePtr<FMKVBlockMore>>& GetBlockMore() const
			{ return BlockMore; }
		private:
			TArray<TMKVUniquePtr<FMKVBlockMore>> BlockMore;
			const FSyntaxElement Elements[2] =
			{
				{ &MKV_BlockMore, 0, SYNTAXELEMENT_FN(FMKVBlockMore, BlockMore) },
				{ nullptr, nullptr }
			};
		};


		/**
		 * Matroska segment.
		 */
		class FMKVSegment : public FMKVEBMLElement
		{
		public:
			FMKVSegment(uint32 InID, int64 InByteOffset, int64 InByteSize) : FMKVEBMLElement(InID, InByteOffset, InByteSize) { }
			virtual ~FMKVSegment() = default;
			virtual EParseResult ParseElement(IMKVFetcher* InReader)
			{
				ElementOffset = InReader->FetchCurrentOffset() - ByteOffset;
				return ParseElementList(Elements, InReader);
			}

			bool HaveSeekHead() const
			{ return !!SeekHeads.Num(); }
			bool HaveInfo() const
			{ return !!Infos.Num(); }
			bool HaveTracks() const
			{ return !!Tracks.Num(); }
			bool HaveCues() const
			{ return !!Cues.Num(); }
			bool HaveTags() const
			{ return !!Tags.Num(); }
			bool HaveAttachments() const
			{ return !!Attachments.Num(); }
			bool HaveChapters() const
			{ return !!Chapters.Num(); }

			TMKVElementPtr<const FMKVSeekHead> GetSeekHead() const
			{ return HaveSeekHead() ? SeekHeads[0] : nullptr; }
			TMKVElementPtr<const FMKVInfo> GetInfo() const
			{ return HaveInfo() ? Infos[0] : nullptr; }
			TMKVElementPtr<const FMKVTracks> GetTracks() const
			{ return HaveTracks() ? Tracks[0] : nullptr; }
			TMKVElementPtr<const FMKVAttachments> GetAttachments() const
			{ return HaveAttachments() ? Attachments[0] : nullptr; }
			TMKVElementPtr<const FMKVTags> GetTags() const
			{ return HaveTags() ? Tags[0] : nullptr; }
			TMKVElementPtr<const FMKVCues> GetCues() const
			{ return HaveCues() ? Cues[0] : nullptr; }
			TMKVElementPtr<FMKVCues> GetCues()
			{ return HaveCues() ? Cues[0] : nullptr; }

			EParseResult LoadMissingElements(IMKVFetcher* InReader, EParserFlags InFlags);
			TMKVElementPtr<FMKVCues> GetOrCreateCues()
			{
				if (Cues.IsEmpty())
				{
					Cues.Emplace(MakeMKVElementPtr<FMKVCues>(MKV_Cues.ID, 0, 0));
				}
				return Cues[0];
			}
		private:
			EParseResult LoadAndParseElement(uint32 InElementID, IMKVFetcher* InReader);
			bool HaveTopLevelElementByID(uint32 InID) const
			{
				if (InID == MKV_Info.ID) return HaveInfo();
				else if (InID == MKV_Tracks.ID) return HaveTracks();
				else if (InID == MKV_Attachments.ID) return HaveAttachments();
				else if (InID == MKV_Tags.ID) return HaveTags();
				else if (InID == MKV_Cues.ID) return HaveCues();
				else if (InID == MKV_Chapters.ID) return HaveChapters();
				else return false;
			}
			TArray<TMKVElementPtr<FMKVSeekHead>> SeekHeads;
			TArray<TMKVElementPtr<FMKVInfo>> Infos;
			TArray<TMKVElementPtr<FMKVTracks>> Tracks;
			TArray<TMKVElementPtr<FMKVAttachments>> Attachments;
			TArray<TMKVElementPtr<FMKVChapters>> Chapters;
			TArray<TMKVElementPtr<FMKVTags>> Tags;
			TArray<TMKVElementPtr<FMKVCues>> Cues;
			const FSyntaxElement Elements[9] =
			{
				{ &MKV_SeekHead, 0, SYNTAXELEMENT_FN(FMKVSeekHead, SeekHeads) },
				{ &MKV_Info, 0, SYNTAXELEMENT_FN(FMKVInfo, Infos) },
				{ &MKV_Tracks, 0, SYNTAXELEMENT_FN(FMKVTracks, Tracks) },
				{ &MKV_Cluster_StopParsing, 0 },
				{ &MKV_Attachments, 0, SYNTAXELEMENT_FN(FMKVAttachments, Attachments) },
				{ &MKV_Chapters, 0, SYNTAXELEMENT_FN(FMKVChapters, Chapters) },
				{ &MKV_Tags, 0, SYNTAXELEMENT_FN(FMKVTags, Tags) },
				{ &MKV_Cues, 0, SYNTAXELEMENT_FN(FMKVCues, Cues) },
				{ nullptr, nullptr }
			};
		};

	public:
		class FMKVTrack;

		class FMKVCueIterator : public IParserMKV::ICueIterator
		{
		public:
			virtual ~FMKVCueIterator() {}
			FMKVCueIterator(const FMKVTrack* InParentTrack, int64 InTotalFilesize) : ParentTrack(InParentTrack), TotalFilesize(InTotalFilesize)
			{
				if (InParentTrack)
				{
					TrackID = InParentTrack->QuickInfo.AlternateCueTrackID ? InParentTrack->QuickInfo.AlternateCueTrackID : InParentTrack->GetID();
				}
			}
			FMKVCueIterator(const FMKVCueIterator& rhs)
			{
				AssignFrom(rhs);
			}

			UEMediaError StartAtTime(const FTimeValue& AtTime, ESearchMode SearchMode) override;
			UEMediaError StartAtFirst() override;
			UEMediaError StartAtUniqueID(uint32 CueUniqueID) override;
			UEMediaError Next() override;

			bool IsAtEOS() const override;
			const ITrack* GetTrack() const override;
			FTimeValue GetTimestamp() const override;
			int64 GetClusterFileOffset() const override;
			int64 GetClusterFileSize() const override;
			FTimeValue GetClusterDuration() const override;
			bool IsLastCluster() const override;
			uint32 GetUniqueID() const override
			{ return ThisCueUniqueID; }
			uint32 GetNextUniqueID() const override
			{ return NextCueUniqueID; }

			bool IsValid() const
			{
				return ThisCueUniqueID != ~0U;
			}
			void AssignFrom(const FMKVCueIterator& rhs)
			{
				ParentTrack = rhs.ParentTrack;
				TrackID = rhs.TrackID;
				TimestampScale = rhs.TimestampScale;
				ThisCueUniqueID = rhs.ThisCueUniqueID;
				NextCueUniqueID = rhs.NextCueUniqueID;
				CueTime = rhs.CueTime;
				ClusterDuration = rhs.ClusterDuration;
				ClusterFileOffset = rhs.ClusterFileOffset;
				ClusterFileSize = rhs.ClusterFileSize;
				TotalFilesize = rhs.TotalFilesize;
			}
			UEMediaError InternalStartAtTime(const FTimeValue& AtTime, ESearchMode SearchMode);
			UEMediaError InternalStartAtFirst();
			UEMediaError InternalStartAtUniqueID(uint32 InUniqueID);
			UEMediaError InternalNext();
			void InternalSetupWithNextCluster();

			FTimeValue CueTime;
			FTimeValue ClusterDuration;
			const FMKVTrack* ParentTrack = nullptr;
			uint64 TrackID = 0;
			int64 ClusterFileOffset = 0;
			int64 ClusterFileSize = 0;
			int64 TotalFilesize = 0;
			uint32 ThisCueUniqueID= ~0U;
			uint32 NextCueUniqueID = ~0U;
			int32 TimestampScale = 0;
		};


		class FMKVTrack : public IParserMKV::ITrack
		{
		public:
			struct FQuickInfo
			{
				FStreamCodecInformation CodecInfo;
				ElectraCDM::FMediaCDMSampleInfo DecryptionInfo;
				TArray<uint8> CSD;
				TArray<uint8> HeaderStrippedBytes;
				uint64 DefaultDurationNanos = 0;
				uint64 CodecDelayNanos = 0;
				bool bIsEncrypted = false;
				uint64 AlternateCueTrackID = 0;
			};

			virtual ~FMKVTrack() {}
			uint64 GetID() const override
			{ return Track->GetTrackNumber(); }
			FString GetName() const override
			{ return Track->GetName(); }
			const TArray<uint8>& GetCodecSpecificData() const override
			{ return QuickInfo.CSD; }
			const FStreamCodecInformation& GetCodecInformation() const override
			{ return QuickInfo.CodecInfo; }
			const FString GetLanguage() const override
			{ return Track->GetLanguage(); }
			const BCP47::FLanguageTag GetLanguageTag() const override
			{ return Track->GetLanguageTag(); }
			IParserMKV::ICueIterator* CreateCueIterator() const override;
			const FQuickInfo GetQuickInfo() const
			{ return QuickInfo;}
			int64 GetDefaultDurationNanos() const
			{ return (int64)Track->GetDefaultDurationNanos(); }

			FQuickInfo QuickInfo;
			TMKVElementPtr<FMKVTrackEntry> Track;
			const FParserMKV* ParentMKV = nullptr;
		};


		class FMKVClusterParser : public IParserMKV::IClusterParser
		{
		public:
			class FCombinedAction
			{
			public:
				uint64 GetTrackID() const { return TrackID; }
				FTimeValue GetPTS() const { return PTS; }
				FTimeValue GetDTS() const { return DTS; }
				FTimeValue GetDuration() const { return SampleDuration; }
				bool IsKeyFrame() const { return bIsKeyFrame; }
				int64 GetTimestamp() const { return Timestamp; }
				int64 GetSegmentRelativePosition() const { return 0; }
				int64 GetClusterPosition() const { return ClusterPosition; }
				int64 GetNumBytesToSkip() const { return NumBytesToSkip; }
				const TArray<uint8>& GetPrependData() const { return TrackInfo.HeaderStrippedBytes; }
				int64 GetNumBytesToRead() const { return NumBytesToRead; }
				void GetDecryptionInfo(ElectraCDM::FMediaCDMSampleInfo& OutSampleDecryptionInfo) const { OutSampleDecryptionInfo = TrackInfo.DecryptionInfo; }
				const TMap<uint64, TArray<uint8>>& GetBlockAdditionalData() const { return BlockAdditionalData; }

				void Clear()
				{
					TrackID = 0;
					BlockAdditionalData.Empty();
					DTS.SetToInvalid();
					PTS.SetToInvalid();
					SampleDuration.SetToInvalid();
					bIsKeyFrame = false;
					Timestamp = -1;
					NumBytesToSkip = 0;
					NumBytesToRead = 0;
				}
				FMKVTrack::FQuickInfo TrackInfo;
				TMap<uint64, TArray<uint8>> BlockAdditionalData;
				FTimeValue DTS;
				FTimeValue PTS;
				FTimeValue SampleDuration;
				uint64 TrackID = 0;
				int64 ClusterPosition = 0;
				int64 CurrentOffset = 0;
				int64 Timestamp = -1;
				int64 NumBytesToSkip = 0;
				int64 NumBytesToRead = 0;
				bool bIsKeyFrame = false;
			};

			#define OVERRIDE_BASE_ACTIONS	\
				uint64 GetTrackID() const override { return ActionParams.GetTrackID(); } \
				FTimeValue GetPTS() const override { return ActionParams.GetPTS(); } \
				FTimeValue GetDTS() const override { return ActionParams.GetDTS(); } \
				FTimeValue GetDuration() const override { return ActionParams.GetDuration(); } \
				bool IsKeyFrame() const override { return ActionParams.IsKeyFrame(); } \
				int64 GetTimestamp() const override { return ActionParams.GetTimestamp(); } \
				int64 GetSegmentRelativePosition() const override { return ActionParams.GetSegmentRelativePosition(); } \
				int64 GetClusterPosition() const override { return ActionParams.GetClusterPosition(); }

			class FActionSkipOver : public IActionSkipOver
			{
				const FCombinedAction& ActionParams;
			public:
				OVERRIDE_BASE_ACTIONS
				FActionSkipOver(const FCombinedAction& InParams) : ActionParams(InParams) {}
				int64 GetNumBytesToSkip() const override { return ActionParams.GetNumBytesToSkip(); }
			};

			class FActionPrependData : public IActionPrependData
			{
				const FCombinedAction& ActionParams;
			public:
				OVERRIDE_BASE_ACTIONS
				FActionPrependData(const FCombinedAction& InParams) : ActionParams(InParams) {}
				const TArray<uint8>& GetPrependData() const override { return ActionParams.GetPrependData(); }
			};

			class FActionReadFrameData : public IActionReadFrameData
			{
				const FCombinedAction& ActionParams;
			public:
				OVERRIDE_BASE_ACTIONS
				FActionReadFrameData(const FCombinedAction& InParams) : ActionParams(InParams) {}
				int64 GetNumBytesToRead() const override { return ActionParams.GetNumBytesToRead(); }
			};

			class FActionFrameDone : public IActionFrameDone
			{
				const FCombinedAction& ActionParams;
			public:
				OVERRIDE_BASE_ACTIONS
				FActionFrameDone(const FCombinedAction& InParams) : ActionParams(InParams) {}
				const TMap<uint64, TArray<uint8>>& GetBlockAdditionalData() const override { return ActionParams.GetBlockAdditionalData(); }
			};

			class FActionDecryptData : public IActionDecryptData
			{
				const FCombinedAction& ActionParams;
			public:
				OVERRIDE_BASE_ACTIONS
				FActionDecryptData(const FCombinedAction& InParams) : ActionParams(InParams) {}
				void GetDecryptionInfo(ElectraCDM::FMediaCDMSampleInfo& OutSampleDecryptionInfo) const override { ActionParams.GetDecryptionInfo(OutSampleDecryptionInfo); }
			};
			#undef OVERRIDE_BASE_ACTIONS

			FMKVClusterParser(TSharedPtrTS<const FParserMKV> InParentMKV, IGenericDataReader* InDataReader, const TArray<uint64>& InTrackIDsToParse, EClusterParseFlags InParseFlags);
			virtual ~FMKVClusterParser() {}
			EParseAction NextParseAction() override;
			FErrorDetail GetLastError() const override;
			const IAction* GetAction() const override;
			int64 GetClusterPosition() const override;
			int64 GetClusterBlockPosition() const override;
		private:
			class FClusterDataReader : public IMKVElementReader, public FMKVEBMLReader, public IMKVFetcher
			{
			public:
				FClusterDataReader(IGenericDataReader* InReader) : FMKVEBMLReader(this), Reader(InReader), StartOffset(InReader->GetCurrentOffset()), Offset(InReader->GetCurrentOffset()) {}
				virtual ~FClusterDataReader() {}
				int64 GetStartOffset() const { return StartOffset; }
				int64 GetCurrentOffset() const { return CurrentOffset(); }
				int64 GetTotalSize() const { return Reader->GetTotalSize(); }
				int64 GetEndOffset() const { return StartOffset + GetTotalSize(); }
				bool HasReadBeenAborted() const { return bWasAborted || Reader->HasReadBeenAborted(); }
				bool ReachedEOS() const { return bReachedEOS; }
				bool SkipOver(int64 NumBytes) { return Skip(NumBytes); }

				bool Read(uint8& OutValue) override
				{ return Validate(Reader->ReadData(&OutValue, sizeof(OutValue), Offset), sizeof(OutValue)); }
				bool Read(uint16& OutValue) override
				{ return ReadValue(OutValue); }
				bool Read(uint32& OutValue) override
				{ return ReadValue(OutValue); }
				bool Read(uint64& OutValue) override
				{ return ReadValue(OutValue); }
				bool Read(TArray<uint8>& OutValue, int64 NumBytes) override
				{ OutValue.AddUninitialized((int32)NumBytes); return Validate(Reader->ReadData(OutValue.GetData(), NumBytes, Offset), NumBytes); }
				bool Skip(int64 NumBytes) override
				{ return Validate(Reader->ReadData(nullptr, NumBytes, Offset), NumBytes); }

				void SetOffset(int64 InOffset)
				{ Offset = InOffset; }


				bool FetchAtEOS() override
				{ return ReachedEOS(); }
				int64 FetchCurrentOffset() override
				{ return GetCurrentOffset(); }
				bool FetchElementID(uint32& OutID) override
				{ return ReadElementID(OutID);  }
				bool FetchElementLength(int64& OutVal) override
				{ return ReadElementLength(OutVal);  }
				bool FetchSkipOver(int64 InNumToSkip) override
				{ return SkipOver(InNumToSkip); }
				bool FetchUInt(uint64& OutVal, int64 InNumBytes, uint64 InDefaultValue) override
				{ return ReadEBMLuint(OutVal, InNumBytes, InDefaultValue); }
				bool FetchFloat(double& OutVal, int64 InNumBytes, double InDefaultValue) override
				{ return ReadEBMLfloat(OutVal, InNumBytes, InDefaultValue); }
				bool FetchByteArray(TArray<uint8>& OutVal, int64 InNumBytes) override
				{ return ReadEBMLbyteArray(OutVal, InNumBytes); }
				bool FetchSeekTo(int64 InAbsolutePosition) override
				{ check(!"not implemented"); return false;}
			private:
				FErrorDetail LastError() const override { return Error; }
				int64 CurrentOffset() const override { return Reader->GetCurrentOffset(); }
				bool Prefetch(int64 NumBytes) override { return true; }

				template<typename T>
				bool ReadValue(T& OutValue)
				{
					if (Validate(Reader->ReadData(&OutValue, sizeof(T), Offset), sizeof(T)))
					{
					#if PLATFORM_LITTLE_ENDIAN
						OutValue = Utils::EndianSwap(OutValue);
					#endif
						return true;
					}
					return false;
				}

				bool Validate(int64 InNumRead, int64 InExpected)
				{
					if (InNumRead == InExpected)
					{
						Offset += InNumRead;
						return true;
					}
					else if (InNumRead == 0)
					{
						bReachedEOS = true;
					}
					bWasAborted = Reader->HasReadBeenAborted();
					return false;
				}

				IGenericDataReader* Reader = nullptr;
				int64 StartOffset = 0;
				int64 Offset = 0;
				FErrorDetail Error;
				bool bReachedEOS = false;
				bool bWasAborted = false;
			};


			IParserMKV::IClusterParser::EParseAction SetReadError()
			{
				if (Reader->ReachedEOS())
				{
					return IParserMKV::IClusterParser::EParseAction::EndOfData;
				}
				else
				{
					ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(FString::Printf(TEXT("Error reading at offset %lld."), (long long int)(Reader->GetCurrentOffset())));
					return IParserMKV::IClusterParser::EParseAction::Failure;
				}
			}
			IParserMKV::IClusterParser::EParseAction SetErrorAndReturn()
			{
				NextAction = nullptr;
				PreviousActionType = IParserMKV::IClusterParser::EParseAction::Failure;
				return PreviousActionType;
			}

			EParseAction ParseUntilStartOfBlock(bool bFirstInCluster);
			EParseAction StartSimpleBlock();
			EParseAction ContinueSimpleBlock();
			EParseAction StartBlockGroup();
			EParseAction ContinueBlockGroup();
			bool HandleBlockElement(uint32 InElementID, int64 InElementSize);

			FErrorDetail ErrorDetail;
			FCombinedAction Params;
			FActionSkipOver ActionSkip { Params };
			FActionPrependData ActionPrepend { Params };
			FActionReadFrameData ActionRead { Params };
			FActionDecryptData ActionDecrypt { Params };
			FActionFrameDone ActionFrameDone { Params };
			TWeakPtrTS<const FParserMKV> ParentMKV;
			TUniquePtr<FClusterDataReader> Reader;
			const IAction* NextAction = nullptr;
			int32 TimestampScale = 1000000;

			// Work vars
			struct FTrackWorkVars
			{
				FTrackWorkVars()
				{
					Restart();
				}
				void Restart()
				{
					BlockAdditions.Reset();
					DTS = 0;
					PTS = 0;
					SampleDuration = 0;
					DurationSumNanos = 0;
					FrameNum = 0;
				}

				FMKVTrack::FQuickInfo TrackInfo;
				TUniquePtr<FMKVBlockAdditions> BlockAdditions;
				int64 DTS;
				int64 PTS;
				int64 SampleDuration;
				int64 DurationSumNanos;
				int32 FrameNum;
			};
			TMap<uint64, FTrackWorkVars> TrackWorkVars;
			TArray<int64> CurrentBlockFrameSizes;
			FTrackWorkVars* CurrentTrackVars = nullptr;
			EParseAction PreviousActionType = IParserMKV::IClusterParser::EParseAction::EndOfData;
			EClusterParseFlags ParseFlags = EClusterParseFlags::ClusterParseFlag_Default;
			uint64 ClusterTimestamp = 0;
			int64 SegmentBaseOffset = 0;
			int64 CurrentClusterStartOffset = 0;
			int64 CurrentClusterSize = 0;
			int64 CurrentClusterEndOffset = 0;
			int64 CurrentBlockStartOffset = 0;
			int64 CurrentBlockDataOffset = 0;
			int64 CurrentBlockEndOffset = 0;
			int64 CurrentBlockGroupBlockEndOffset = 0;
			int64 CurrentBlockSize = 0;
			int64 CurrentBlockDuration = 0;
			uint32 CurrentBlockTypeID = 0;
			int32 CurrentBlockReferenceCount = 0;
			int32 CurrentBlockFrameSizeIndex = 0;
			uint8 CurrentBlockFlags = 0;
			bool bCheckForClusterStart = true;
			bool bIgnoreNonCluster = false;
		};

	public:
		TSharedPtrTS<const FMKVSegment> GetSegment() const
		{ return Segment; }
		int32 GetTimestampScale() const
		{ return TimestampScale; }
		FTimeValue GetContentDuration() const
		{ return ContentDuration; }
		int64 GetTotalFileSize() const
		{ return TotalFilesize; }
		const TArray<TMKVElementPtr<FMKVTrack>>& GetTracks() const
		{ return Tracks; }
		TMKVElementPtr<FMKVTrack> GetTrackByID(uint64 InID) const
		{
			auto Trk = Tracks.FindByPredicate([InID](const TMKVElementPtr<FMKVTrack>& InElem) { return InElem->GetID() == InID; });
			return Trk ? *Trk : nullptr;
		}
	private:
		bool SetupCodecInfo(FStreamCodecInformation& OutCodecInformation, TMKVElementPtr<FMKVTrackEntry> InFromTrack);

		IGenericDataReader* DataReader = nullptr;
		IPlayerSessionServices* PlayerSessionServices = nullptr;

		const TMap<FString,FString> CodecMapping
		{
			{ TEXT("V_MPEG4/ISO/AVC"), TEXT("avc") },
			{ TEXT("V_MPEGH/ISO/HEVC"), TEXT("hevc") },
			{ TEXT("V_VP8"), TEXT("vp8") },
			{ TEXT("V_VP9"), TEXT("vp9") },
			{ TEXT("A_AAC"), TEXT("aac") },
			{ TEXT("A_AAC/MPEG4/LC"), TEXT("aac") },
			{ TEXT("A_AAC/MPEG4/LC/SBR"), TEXT("aac") },
			{ TEXT("A_OPUS"), TEXT("Opus") },
			{ TEXT("A_FLAC"), TEXT("FLAC") },
			{ TEXT("A_MPEG/L3"), TEXT("MPEG") },
			{ TEXT("A_MPEG/L2"), TEXT("MPEG") },
			{ TEXT("A_MPEG/L1"), TEXT("MPEG") },
			//{ TEXT("S_TEXT/UTF8"), TEXT("") },
			//{ TEXT("S_TEXT/WEBVTT"), TEXT("") },
			//{ TEXT(""), TEXT("") },
		};

		FErrorDetail LastError;
		FTimeValue ContentDuration;
		TArray<TMKVElementPtr<FMKVTrack>> Tracks;
		TArray<uint64> AlternateCueTrackNumbers;
		int32 NumVideoTracks = 0;
		TUniquePtr<FBufferedMKVReader> Reader;
		TSharedPtrTS<FMKVSegment> Segment;
		int64 TotalFilesize = -1;
		uint32 NextCueUniqueID = 0;
		int32 TimestampScale = DEFAULT_MATROSKA_TIMESCALE;
	};




	FParserMKV::EParseResult FParserMKV::FMKVSegment::LoadAndParseElement(uint32 InElementID, IMKVFetcher* InReader)
	{
		// Get the file offset from the SeekHead.
		TMKVElementPtr<const FMKVSeekHead> SeekHead = GetSeekHead();
		uint64 FileOffset = SeekHead.IsValid() ? SeekHead->GetPositionOfID(InElementID) : 0;
		if (FileOffset)
		{
			FileOffset += GetElementOffset();
			if (!InReader->FetchSeekTo((int64)FileOffset))
			{
				return FParserMKV::EParseResult::Error;
			}
			return ParseNextElementFromList(InElementID, Elements, InReader);
		}
		return FParserMKV::EParseResult::Ok;
	}

	FParserMKV::EParseResult FParserMKV::FMKVSegment::LoadMissingElements(IMKVFetcher* InReader, EParserFlags InFlags)
	{
		TMKVElementPtr<const FMKVSeekHead> SeekHead = GetSeekHead();
		if (SeekHead.IsValid())
		{
			TArray<uint32> Level1IDs;
			if ((InFlags & EParserFlags::ParseFlag_OnlyTracks) != 0)
			{
				Level1IDs.Add(MKV_Info.ID);
				Level1IDs.Add(MKV_Tracks.ID);
			}
			else if ((InFlags & EParserFlags::ParseFlag_OnlyEssentialLevel1) != 0)
			{
				Level1IDs.Add(MKV_Info.ID);
				Level1IDs.Add(MKV_Tracks.ID);
				Level1IDs.Add(MKV_Cues.ID);
			}
			else
			{
				Level1IDs.Add(MKV_Info.ID);
				Level1IDs.Add(MKV_Tracks.ID);
				Level1IDs.Add(MKV_Cues.ID);
				Level1IDs.Add(MKV_Attachments.ID);
				Level1IDs.Add(MKV_Chapters.ID);
				Level1IDs.Add(MKV_Tags.ID);
			}

			const TArray<TMKVElementPtr<FMKVSeek>>& Seeks = SeekHead->GetSeeks();
			for(int32 i=0; i<Seeks.Num(); ++i)
			{
				uint32 SeekID = Seeks[i]->GetSeekID();
				if (SeekID && SeekID != MKV_Cluster.ID && !HaveTopLevelElementByID(SeekID))
				{
					if (!Level1IDs.Contains(SeekID))
					{
						continue;
					}
					EParseResult Result = LoadAndParseElement(SeekID, InReader);
					if (Result != EParseResult::Ok && Result != EParseResult::StopParsing)
					{
						return Result;
					}
				}
			}
		}
		return EParseResult::Ok;
	}

	int64 FParserMKV::OnReadAssetData(void* Destination, int64 NumBytes, int64 FromOffset, int64* OutTotalSize)
	{
		if (DataReader)
		{
			int64 NumRead = DataReader->ReadData(Destination, NumBytes, FromOffset);
			if (OutTotalSize)
			{
				*OutTotalSize = DataReader->GetTotalSize();
			}
			if (DataReader->HasReadBeenAborted())
			{
				return static_cast<int64>(FBufferedDataReader::IDataProvider::EError::Aborted);
			}
			else if (NumRead >= 0)
			{
				return NumRead;
			}
		}
		return static_cast<int64>(FBufferedDataReader::IDataProvider::EError::Failed);
	}


	FParserMKV::FParserMKV(IPlayerSessionServices* InPlayerSession)
	{
		PlayerSessionServices = InPlayerSession;
	}

	FErrorDetail FParserMKV::ParseHeader(IGenericDataReader* InDataReader, EParserFlags ParseFlags)
	{
		DataReader = InDataReader;

		Reader = MakeUnique<FBufferedMKVReader>(this);

		// Basic check if the file starts with an EBML Header
		uint32 Value32 = 0;
		if (!Reader->PeekU32BE(Value32) || Value32 != EBML_Header.ID)
		{
			return LastError.SetError(UEMEDIA_ERROR_FORMAT_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Invalid EBML header"));
		}

		// Read and check the EBML header
		int64 ElementStartOffset = Reader->GetCurrentOffset();
		int64 ElementLen;
		uint32 ElementID;
		if (!Reader->ReadElementID(ElementID) || !Reader->ReadElementLength(ElementLen))
		{
			return LastError = Reader->GetError();
		}
		FMKVEBMLHeader Hdr(ElementID, ElementStartOffset, ElementLen);
		if (Hdr.ParseElement(Reader.Get()) != EParseResult::Ok)
		{
			return LastError.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Could not parse EBML header"));
		}
		// Conformance checks
		if (Hdr.GetVersion() > 1 || Hdr.GetReadVersion() > 1 || Hdr.GetMaxIDLength() > 4 || Hdr.GetMaxSizeLength() > 8 || Hdr.GetDocTypeReadVersion() > 4)
		{
			return LastError.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT)
							.SetMessage(FString::Printf(TEXT("Unsupported EBML header version %lld, read version %lld, ID length %lld, size length %lld, doc type read version %lld"),
								 (long long int)Hdr.GetVersion(), (long long int)Hdr.GetReadVersion(), (long long int)Hdr.GetMaxIDLength(), (long long int)Hdr.GetMaxSizeLength(), (long long int)Hdr.GetDocTypeReadVersion()));
		}
		// Must be either "matroska" or "webm"
		if (!Hdr.GetDocType().Equals(TEXT("matroska")) && !Hdr.GetDocType().Equals(TEXT("webm")))
		{
			return LastError.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(FString::Printf(TEXT("Unsupported doc type \"%s\""), *Hdr.GetDocType()));
		}

		// The EBML Header must be followed by a Matroska Segment.
		if (!Reader->PeekU32BE(Value32) || Value32 != MKV_Segment.ID)
		{
			return LastError.SetError(UEMEDIA_ERROR_FORMAT_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Not a Matroska Segment following the EBML header"));
		}
		ElementStartOffset = Reader->GetCurrentOffset();
		if (!Reader->ReadElementID(ElementID) || !Reader->ReadElementLength(ElementLen))
		{
			return LastError = Reader->GetError();
		}
		TSharedPtrTS<FMKVSegment> Seg = MakeSharedTS<FMKVSegment>(ElementID, ElementStartOffset, ElementLen);
		EParseResult Result = Seg->ParseElement(Reader.Get());
		if (Result != EParseResult::Ok && Result != EParseResult::StopParsing)
		{
			return LastError.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Could not parse segment"));
		}

		// Load the elements we need that we did not get before encountering the first cluster.
		Result = Seg->LoadMissingElements(Reader.Get(), ParseFlags);
		if (Result != EParseResult::Ok && Result != EParseResult::StopParsing)
		{
			return LastError.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Could not load top level elements"));
		}

		// Get the often used values.
		TMKVElementPtr<const FMKVInfo> Info = Seg->GetInfo();
		if (Info.IsValid())
		{
			uint64 tss = Info->GetTimestampScale();
			check(TimestampScale < 0x7fffffffU);
			TimestampScale = (int32) tss;
			double Duration = Info->GetDuration();
			int64 Nanos = (int64)( Duration * (int32)TimestampScale );
			ContentDuration.SetFromND(Nanos, 1000000000U);
		}

		// Assign a unique ID to each Cue.
		// We need this to correctly identify a cue when iterating when we insert new cues during parsing.
		bool bIsSeekable = false;
		if (Seg->HaveCues())
		{
			const TArray<TMKVUniquePtr<FMKVCuePoint>>& CuePoints = Seg->GetCues()->GetCuePoints();
			for(int32 i=0; i<CuePoints.Num(); ++i)
			{
				CuePoints[i]->SetUniqueID(++NextCueUniqueID);
			}

			bIsSeekable = true;
			if ((ParseFlags & EParserFlags::ParseFlag_SuppressCueWarning) == 0)
			{
				// Check the distance between cues and put them into different buckets of 5s duration each,
				// with at worst a 20s bucket.
				uint32 CueDeltas[10] = {0};
				int64 PrevCueTime = 0;

				for(int32 i=0; i<CuePoints.Num(); ++i)
				{
					int64 ct = (int64)CuePoints[i]->GetCueTime();
					int64 deltaSecs = ((ct - PrevCueTime) * TimestampScale) / 1000000000L;
					int32 bucketIdx = deltaSecs / 5;
					++CueDeltas[bucketIdx < 0 ? 0 : bucketIdx <= 4 ? bucketIdx : 4];
					PrevCueTime = ct;
				}
				// And finally the duration from the last cue to the end
				int64 ds = (ContentDuration.GetAsHNS() * 100 - PrevCueTime * TimestampScale) / 1000000000L;
				int32 bucketIdx = ds / 5;
				++CueDeltas[bucketIdx < 0 ? 0 : bucketIdx <= 4 ? bucketIdx : 4];

				// Examine the buckets
				if (CueDeltas[4])
				{
					return LastError.SetError(UEMEDIA_ERROR_NOT_SUPPORTED).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_CUE_DISTANCE).SetMessage(TEXT("Distance between Cues too great for seeking."));
				}
				else if (CueDeltas[3])
				{
					UE_LOGF(LogElectraMKVParser, Warning, "Distance between neighboring Cues exceeds 15 seconds. Seek performance will be unacceptable!");
				}
				else if (CueDeltas[2])
				{
					UE_LOGF(LogElectraMKVParser, Log, "Distance between neighboring Cues exceeds 10 seconds. Seek performance will be poor!");
				}
				else if (CueDeltas[1])
				{
					UE_LOGF(LogElectraMKVParser, Log, "Distance between neighboring Cues exceeds 5 seconds. Seek performance will be impaired!");
				}
			}
			#if 0
				// Remove all but the first cue for testing
				TRangeSet<int32> DbgDelete;
				DbgDelete.Add(TRange<int32>::Inclusive(1, CuePoints.Num()));
				Seg->GetCues()->DebugRemoveCuesByIndexRange(DbgDelete);
			#endif
		}
		else if ((ParseFlags & EParserFlags::ParseFlag_OnlyTracks) == 0)
		{
			return LastError.SetError(UEMEDIA_ERROR_NOT_SUPPORTED).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_CUE_DISTANCE).SetMessage(TEXT("No Cues present. Seeking and locating the start is impossible."));
		}

		TotalFilesize = Reader->GetTotalDataSize();

		// Everything we needed to read must have been read by now.
		// The reader we were passed will no longer be valid once we return.
		DataReader = nullptr;

		// All fine so far
		Segment = MoveTemp(Seg);
		return LastError;
	}


	bool FParserMKV::SetupCodecInfo(FStreamCodecInformation& OutCodecInformation, TMKVElementPtr<FMKVTrackEntry> InFromTrack)
	{
		const FString* Codec4CCPtr = CodecMapping.Find(InFromTrack->GetCodecID());
		if (!Codec4CCPtr)
		{
			// For some codecs a look at the codec private data might identify the codec.
			return false;
		}
		FString Codec4CC(*Codec4CCPtr);

		struct FCodecVideoColorInfo
		{
			TOptional<int32> ColourPrimaries;
			TOptional<int32> TransferCharacteristics;
			TOptional<int32> MatrixCoefficients;
			TOptional<int32> VideoFullRangeFlag;
			TOptional<int32> VideoFormat;
			TOptional<int32> BitDepthLuma;
			TOptional<int32> ChromaSubsampling;
		};

		auto ParseColorInformation = [](FStreamCodecInformation& InOutCodecInformation, FCodecVideoColorInfo& InOutCodecVideoColorInfo, const TMKVElementPtr<FMKVColour>& InColours) -> void
		{
			if (InColours.IsValid())
			{
				InOutCodecVideoColorInfo.ColourPrimaries = InColours->GetPrimaries();
				InOutCodecVideoColorInfo.TransferCharacteristics = InColours->GetTransferCharacteristics();
				InOutCodecVideoColorInfo.MatrixCoefficients = InColours->GetMatrixCoefficients();
				if ((!InOutCodecVideoColorInfo.BitDepthLuma.IsSet() || InOutCodecVideoColorInfo.BitDepthLuma.GetValue() == 0) && InColours->GetBitsPerChannel())
				{
					InOutCodecVideoColorInfo.BitDepthLuma = InColours->GetBitsPerChannel();
				}
				ElectraDecodersUtil::MPEG::FCommonColorimetry colr;
				colr.video_format = 5;	// unspecified
				colr.colour_primaries = InColours->GetPrimaries();
				colr.transfer_characteristics = InColours->GetTransferCharacteristics();
				colr.matrix_coeffs = InColours->GetMatrixCoefficients();
				if (InColours->GetRange() == 2)
				{
					InOutCodecVideoColorInfo.VideoFullRangeFlag = 1;
					colr.video_full_range_flag = 1;
				}
				InOutCodecInformation.GetExtras().Set(TEXT("colr_box"), FVariantValue(ElectraDecodersUtil::MPEG::BuildCOLRBox(colr)));

				// Color light level values present?
				if (InColours->GetMaxCLL() && InColours->GetMaxFALL())
				{
					ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info clli;
					clli.max_content_light_level = InColours->GetMaxCLL();
					clli.max_pic_average_light_level = InColours->GetMaxFALL();
					InOutCodecInformation.GetExtras().Set(TEXT("clli_box"), FVariantValue(ElectraDecodersUtil::MPEG::BuildCLLIBox(clli)));
				}

				// Mastering display color volume present?
				TMKVElementPtr<FMKVMasteringMetadata> MasteringMetadata = InColours->GetMasteringMetadata();
				if (MasteringMetadata.IsValid())
				{
					const double ST2086_Chroma_Scale = 50000.0f;
					const double ST2086_Luma_Scale = 10000.0f;
					ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume mdcv;
					mdcv.display_primaries_x[0] = (uint16)(FMath::RoundToInt32(MasteringMetadata->GetPrimaryRChromaticityX() * ST2086_Chroma_Scale));
					mdcv.display_primaries_y[0] = (uint16)(FMath::RoundToInt32(MasteringMetadata->GetPrimaryRChromaticityY() * ST2086_Chroma_Scale));
					mdcv.display_primaries_x[1] = (uint16)(FMath::RoundToInt32(MasteringMetadata->GetPrimaryGChromaticityX() * ST2086_Chroma_Scale));
					mdcv.display_primaries_y[1] = (uint16)(FMath::RoundToInt32(MasteringMetadata->GetPrimaryGChromaticityY() * ST2086_Chroma_Scale));
					mdcv.display_primaries_x[2] = (uint16)(FMath::RoundToInt32(MasteringMetadata->GetPrimaryBChromaticityX() * ST2086_Chroma_Scale));
					mdcv.display_primaries_y[2] = (uint16)(FMath::RoundToInt32(MasteringMetadata->GetPrimaryBChromaticityY() * ST2086_Chroma_Scale));
					mdcv.white_point_x = (uint16)(FMath::RoundToInt32(MasteringMetadata->GetWhitePointChromaticityX() * ST2086_Chroma_Scale));
					mdcv.white_point_y = (uint16)(FMath::RoundToInt32(MasteringMetadata->GetWhitePointChromaticityY() * ST2086_Chroma_Scale));
					mdcv.max_display_mastering_luminance = (uint32)(FMath::RoundToInt32(MasteringMetadata->GetLuminanceMax() * ST2086_Luma_Scale));
					mdcv.min_display_mastering_luminance = (uint32)(FMath::RoundToInt32(MasteringMetadata->GetLuminanceMin() * ST2086_Luma_Scale));
					InOutCodecInformation.GetExtras().Set(TEXT("mdcv_box"), FVariantValue(ElectraDecodersUtil::MPEG::BuildMDCVBox(mdcv)));
				}
			}
		};






		// AVC / H.264 ?
		if (InFromTrack->GetTrackType() == EMKVTrackType::Video && Codec4CC.Equals(TEXT("avc")))
		{
			// The codec private data is an AVCDecoderConfigurationRecord as per ISO/IEC-14496-15
			ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord CodecSpecificDataAVC;
			if (!CodecSpecificDataAVC.Parse(InFromTrack->GetCodecPrivate()))
			{
				return false;
			}

			TMKVElementPtr<FMKVVideo> Video = InFromTrack->GetVideo();

			OutCodecInformation.GetExtras().Set(StreamCodecInformationOptions::DecoderConfigurationRecord, FVariantValue(InFromTrack->GetCodecPrivate()));
			OutCodecInformation.SetStreamType(EStreamType::Video);
			OutCodecInformation.SetMimeType(TEXT("video/mp4"));
			OutCodecInformation.SetCodec(FStreamCodecInformation::ECodec::H264);
			OutCodecInformation.SetCodecSpecificData(CodecSpecificDataAVC.GetCodecSpecificData());
			OutCodecInformation.SetDecoderConfigRecord(InFromTrack->GetCodecPrivate());
			OutCodecInformation.SetStreamLanguageTag(InFromTrack->GetLanguageTag());
			if (CodecSpecificDataAVC.GetSequenceParameterSets().Num())
			{
				ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet sps;
				if (!ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(sps, CodecSpecificDataAVC.GetSequenceParameterSets()[0]))
				{
					return false;
				}
				int32 CropL, CropR, CropT, CropB;
				sps.GetCrop(CropL, CropR, CropT, CropB);
				OutCodecInformation.SetResolution(FStreamCodecInformation::FResolution(sps.GetWidth() - CropL - CropR, sps.GetHeight() - CropT - CropB));
				OutCodecInformation.SetCrop(FStreamCodecInformation::FCrop(CropL, CropT, CropR, CropB));
				FStreamCodecInformation::FAspectRatio ar;
				sps.GetAspect(ar.Width, ar.Height);
				OutCodecInformation.SetAspectRatio(ar);
				if (sps.GetTiming().Denom)
				{
					OutCodecInformation.SetFrameRate(FTimeFraction(sps.GetTiming().Num, sps.GetTiming().Denom));
				}
				else if (InFromTrack->GetDefaultDurationNanos())
				{
					OutCodecInformation.SetFrameRate(FTimeFraction(1000000000, InFromTrack->GetDefaultDurationNanos()));
				}
				OutCodecInformation.SetProfile(sps.profile_idc);
				OutCodecInformation.SetProfileLevel(sps.level_idc);
				uint8 Constraints = (sps.constraint_set0_flag << 7) | (sps.constraint_set1_flag << 6) | (sps.constraint_set2_flag << 5) | (sps.constraint_set3_flag << 4) | (sps.constraint_set4_flag << 3) | (sps.constraint_set5_flag << 2);
				OutCodecInformation.SetProfileConstraints(Constraints);
				OutCodecInformation.SetCodecSpecifierRFC6381(FString::Printf(TEXT("avc1.%02x%02x%02x"), sps.profile_idc, Constraints, sps.level_idc));
				// If there is no default duration set we try to calculate it from the frame rate
				if (InFromTrack->GetDefaultDurationNanos() == 0 && OutCodecInformation.GetFrameRate().IsValid())
				{
					FTimeFraction ooFPS(OutCodecInformation.GetFrameRate().GetDenominator(), OutCodecInformation.GetFrameRate().GetNumerator());
					int64 nanos = ooFPS.GetAsTimebase(1000000000);
					if (nanos >= 0)
					{
						InFromTrack->SetDefaultDurationNanos((uint64)nanos);
					}
				}
			}
			else
			{
				if (!Video.IsValid())
				{
					return false;
				}

				OutCodecInformation.SetProfile(CodecSpecificDataAVC.GetAVCProfileIndication());
				OutCodecInformation.SetProfileLevel(CodecSpecificDataAVC.GetAVCLevelIndication());
				OutCodecInformation.SetProfileConstraints(CodecSpecificDataAVC.GetProfileCompatibility());
				OutCodecInformation.SetCodecSpecifierRFC6381(FString::Printf(TEXT("avc3.%02x%02x%02x"), CodecSpecificDataAVC.GetAVCProfileIndication(), CodecSpecificDataAVC.GetProfileCompatibility(), CodecSpecificDataAVC.GetAVCLevelIndication()));
				OutCodecInformation.SetResolution(FStreamCodecInformation::FResolution(Video->GetDisplayWidth(), Video->GetDisplayHeight()));
				OutCodecInformation.SetCrop(FStreamCodecInformation::FCrop(Video->GetCropLeft(), Video->GetCropTop(), Video->GetCropRight(), Video->GetCropBottom()));
				OutCodecInformation.SetAspectRatio(FStreamCodecInformation::FAspectRatio(1, 1));
			}
			// If there is no default duration at this point we try to set it from the frame rate.
			// That is a deprecated element however, so the chances that it is set are fairly slim.
			if (InFromTrack->GetDefaultDurationNanos() == 0 && Video.IsValid() && Video->GetFrameRate() > 0.0)
			{
				int64 nanos = (int64)(1000000000.0 / Video->GetFrameRate());
				if (nanos >= 0)
				{
					InFromTrack->SetDefaultDurationNanos((uint64)nanos);
				}
			}
			return true;
		}
		// HEVC / H.265 ?
		else if (InFromTrack->GetTrackType() == EMKVTrackType::Video && Codec4CC.Equals(TEXT("hevc")))
		{
			// The codec private data is an HEVCDecoderConfigurationRecord as per ISO/IEC-14496-15
			ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord CodecSpecificDataHEVC;
			if (!CodecSpecificDataHEVC.Parse(InFromTrack->GetCodecPrivate()))
			{
				return false;
			}

			TMKVElementPtr<FMKVVideo> Video = InFromTrack->GetVideo();

			OutCodecInformation.GetExtras().Set(StreamCodecInformationOptions::DecoderConfigurationRecord, FVariantValue(InFromTrack->GetCodecPrivate()));
			OutCodecInformation.SetStreamType(EStreamType::Video);
			OutCodecInformation.SetMimeType(TEXT("video/mp4"));
			OutCodecInformation.SetCodec(FStreamCodecInformation::ECodec::H265);
			OutCodecInformation.SetCodecSpecificData(CodecSpecificDataHEVC.GetCodecSpecificData());
			OutCodecInformation.SetDecoderConfigRecord(InFromTrack->GetCodecPrivate());
			OutCodecInformation.SetStreamLanguageTag(InFromTrack->GetLanguageTag());
			if (CodecSpecificDataHEVC.GetParsedSequenceParameterSets().IsEmpty())
			{
				return false;
			}
			const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet& sps = CodecSpecificDataHEVC.GetParsedSequenceParameterSets()[0];
			int32 CropL, CropR, CropT, CropB;
			sps.GetCrop(CropL, CropR, CropT, CropB);
			OutCodecInformation.SetResolution(FStreamCodecInformation::FResolution(sps.GetWidth() - CropL - CropR, sps.GetHeight() - CropT - CropB));
			OutCodecInformation.SetCrop(FStreamCodecInformation::FCrop(CropL, CropT, CropR, CropB));
			FStreamCodecInformation::FAspectRatio ar;
			sps.GetAspect(ar.Width, ar.Height);
			OutCodecInformation.SetAspectRatio(ar);
			OutCodecInformation.SetFrameRate(sps.GetTiming().Denom ? FTimeFraction(sps.GetTiming().Num, sps.GetTiming().Denom) : FTimeFraction());
			OutCodecInformation.SetProfileSpace(sps.profile_tier_level.general_profile_space);
			OutCodecInformation.SetProfileTier(sps.profile_tier_level.general_tier_flag);
			OutCodecInformation.SetProfile(sps.profile_tier_level.general_profile_idc);
			OutCodecInformation.SetProfileLevel(sps.profile_tier_level.general_level_idc);
			OutCodecInformation.SetProfileConstraints(sps.GetConstraintFlags());
			OutCodecInformation.SetProfileCompatibilityFlags(sps.profile_tier_level.general_profile_compatibility_flags);
			OutCodecInformation.SetCodecSpecifierRFC6381(sps.GetRFC6381(TEXT("hvc1")));
			// If there is no default duration set we try to calculate it from the VUI timing of the SPS if present.
			if (InFromTrack->GetDefaultDurationNanos() == 0)
			{
				ElectraDecodersUtil::FFractionalValue fv = sps.GetTiming();
				FTimeFraction fr(fv.Denom, fv.Num);
				if (fr.IsValid())
				{
					int64 nanos = fr.GetAsTimebase(1000000000);
					check(nanos >= 0);
					if (nanos >= 0)
					{
						InFromTrack->SetDefaultDurationNanos((uint64)nanos);
					}
				}
			}
			// If there is no default duration at this point we try to set it from the frame rate.
			// That is a deprecated element however, so the chances that it is set are fairly slim.
			if (InFromTrack->GetDefaultDurationNanos() == 0 && Video.IsValid() && Video->GetFrameRate() > 0.0)
			{
				int64 nanos = (int64)(1000000000.0 / Video->GetFrameRate());
				if (nanos >= 0)
				{
					InFromTrack->SetDefaultDurationNanos((uint64)nanos);
				}
			}
			return true;
		}
		// VP8 ?
		else if (InFromTrack->GetTrackType() == EMKVTrackType::Video && Codec4CC.Equals(TEXT("vp8")))
		{
			TMKVElementPtr<FMKVVideo> Video = InFromTrack->GetVideo();
			if (!Video.IsValid())
			{
				return false;
			}

			// Parse color information
			FCodecVideoColorInfo CodecVideoColorInfo;
			ParseColorInformation(OutCodecInformation, CodecVideoColorInfo, Video->GetColours());
			OutCodecInformation.GetExtras().Set(TEXT("alphamode"), FVariantValue((int64) Video->GetAlphaMode()));

			OutCodecInformation.SetStreamType(EStreamType::Video);
			OutCodecInformation.SetMimeType(TEXT("video/VP8"));
			OutCodecInformation.SetStreamLanguageTag(InFromTrack->GetLanguageTag());
			OutCodecInformation.SetCodec(FStreamCodecInformation::ECodec::Video4CC);
			OutCodecInformation.SetCodec4CC(Utils::Make4CC('v','p','0','8'));
			FString crfc = FString::Printf(TEXT("vp08.%02d.%02d.%02d.%02d.%02d.%02d.%02d.%02d"),
				OutCodecInformation.GetProfile(), OutCodecInformation.GetProfileLevel(),
				CodecVideoColorInfo.BitDepthLuma.Get(8),
				CodecVideoColorInfo.ChromaSubsampling.Get(0),
				CodecVideoColorInfo.ColourPrimaries.Get(2),
				CodecVideoColorInfo.TransferCharacteristics.Get(2),
				CodecVideoColorInfo.MatrixCoefficients.Get(2),
				CodecVideoColorInfo.VideoFullRangeFlag.Get(0));
			OutCodecInformation.SetCodecSpecifierRFC6381(crfc);

			// Construct a VPCodecConfigurationBox
			TArray<uint8> vpcC { 1,0,0,0, 0,0,0,0, 0,0,0,0 };
			vpcC[4] = (uint8)OutCodecInformation.GetProfile();
			vpcC[5] = (uint8)OutCodecInformation.GetProfileLevel();
			vpcC[6] = (uint8)((CodecVideoColorInfo.BitDepthLuma.Get(8) << 4) +
							  (CodecVideoColorInfo.ChromaSubsampling.Get(0) << 1) +
							   CodecVideoColorInfo.VideoFullRangeFlag.Get(0));
			vpcC[7] = (uint8)CodecVideoColorInfo.ColourPrimaries.Get(2);
			vpcC[8] = (uint8)CodecVideoColorInfo.TransferCharacteristics.Get(2);
			vpcC[9] = (uint8)CodecVideoColorInfo.MatrixCoefficients.Get(2);
			OutCodecInformation.GetExtras().Set(StreamCodecInformationOptions::VPccBox, FVariantValue(vpcC));

			int32 CropL = Video->GetCropLeft();
			int32 CropR = Video->GetCropRight();
			int32 CropT = Video->GetCropTop();
			int32 CropB = Video->GetCropBottom();
			OutCodecInformation.SetResolution(FStreamCodecInformation::FResolution(Video->GetDisplayWidth(), Video->GetDisplayHeight()));
			OutCodecInformation.SetCrop(FStreamCodecInformation::FCrop(CropL, CropT, CropR, CropB));
			OutCodecInformation.SetAspectRatio(FStreamCodecInformation::FAspectRatio(1, 1));

			// If there is no default duration at this point we try to set it from the frame rate.
			// That is a deprecated element however, so the chances that it is set are fairly slim.
			if (InFromTrack->GetDefaultDurationNanos() == 0 && Video.IsValid() && Video->GetFrameRate() > 0.0)
			{
				int64 nanos = (int64)(1000000000.0 / Video->GetFrameRate());
				if (nanos >= 0)
				{
					InFromTrack->SetDefaultDurationNanos((uint64)nanos);
				}
			}
			// Calculate the frame rate from the default duration.
			if (InFromTrack->GetDefaultDurationNanos() > 0)
			{
				FTimeFraction fr(1000000000, InFromTrack->GetDefaultDurationNanos());
				OutCodecInformation.SetFrameRate(fr);
			}
			return true;
		}
		// VP9?
		else if (InFromTrack->GetTrackType() == EMKVTrackType::Video && Codec4CC.Equals(TEXT("vp9")))
		{
			TMKVElementPtr<FMKVVideo> Video = InFromTrack->GetVideo();
			if (!Video.IsValid())
			{
				return false;
			}

			FCodecVideoColorInfo CodecVideoColorInfo;

			// The codec private data, if present, should be this: https://www.webmproject.org/docs/container/#vp9-codec-feature-metadata-codecprivate
			if (InFromTrack->GetCodecPrivate().Num())
			{
				const uint8* cp = InFromTrack->GetCodecPrivate().GetData();

				// Check for potentially malformed private data. It may be possible for the muxer to have
				// put a VPCodecConfigurationRecord in here instead of the format specified above.
				if (InFromTrack->GetCodecPrivate().Num() == 8 && cp[6] == 0 && cp[7] == 0 && (((cp[2] >> 4) == 8) || ((cp[2] >> 4) == 10) || ((cp[2] >> 4) == 12)))
				{
					OutCodecInformation.SetProfile(cp[0]);
					OutCodecInformation.SetProfileLevel(cp[1]);
					CodecVideoColorInfo.BitDepthLuma = cp[2] >> 4;
					CodecVideoColorInfo.ChromaSubsampling = (cp[2] >> 3) & 7;
					CodecVideoColorInfo.VideoFullRangeFlag = cp[2] & 1;
					CodecVideoColorInfo.ColourPrimaries = cp[3];
					CodecVideoColorInfo.TransferCharacteristics = cp[4];
					CodecVideoColorInfo.MatrixCoefficients = cp[5];
				}
				else
				{
					const uint8* cpe = cp + InFromTrack->GetCodecPrivate().Num();
					while(cp < cpe)
					{
						switch(*cp++)
						{
							// Profile
							case 1: if (*cp++ == 1) OutCodecInformation.SetProfile(*cp++); else return false; break;
							// Level
							case 2: if (*cp++ == 1) OutCodecInformation.SetProfileLevel(*cp++); else return false; break;
							// Bit depth
							case 3: if (*cp++ == 1) CodecVideoColorInfo.BitDepthLuma = *cp++; else return false; break;
							// Chroma subsampling
							case 4: if (*cp++ == 1) CodecVideoColorInfo.ChromaSubsampling = *cp++; else return false; break;
							default:
							{
								uint8 len = *cp++;
								cp += len;
								break;
							}
						}
					}
				}
			}

			// Parse color information
			ParseColorInformation(OutCodecInformation, CodecVideoColorInfo, Video->GetColours());
			OutCodecInformation.GetExtras().Set(TEXT("alphamode"), FVariantValue((int64) Video->GetAlphaMode()));

			OutCodecInformation.SetStreamType(EStreamType::Video);
			OutCodecInformation.SetMimeType(TEXT("video/VP9"));
			OutCodecInformation.SetStreamLanguageTag(InFromTrack->GetLanguageTag());
			OutCodecInformation.SetCodec(FStreamCodecInformation::ECodec::Video4CC);
			OutCodecInformation.SetCodec4CC(Utils::Make4CC('v','p','0','9'));
			FString crfc = FString::Printf(TEXT("vp09.%02d.%02d.%02d.%02d.%02d.%02d.%02d.%02d"),
				OutCodecInformation.GetProfile(), OutCodecInformation.GetProfileLevel(),
				CodecVideoColorInfo.BitDepthLuma.Get(8),
				CodecVideoColorInfo.ChromaSubsampling.Get(0),
				CodecVideoColorInfo.ColourPrimaries.Get(2),
				CodecVideoColorInfo.TransferCharacteristics.Get(2),
				CodecVideoColorInfo.MatrixCoefficients.Get(2),
				CodecVideoColorInfo.VideoFullRangeFlag.Get(0));
			OutCodecInformation.SetCodecSpecifierRFC6381(crfc);

			// Construct a VPCodecConfigurationBox
			TArray<uint8> vpcC { 1,0,0,0, 0,0,0,0, 0,0,0,0 };
			vpcC[4] = (uint8)OutCodecInformation.GetProfile();
			vpcC[5] = (uint8)OutCodecInformation.GetProfileLevel();
			vpcC[6] = (uint8)((CodecVideoColorInfo.BitDepthLuma.Get(8) << 4) +
							  (CodecVideoColorInfo.ChromaSubsampling.Get(0) << 1) +
							   CodecVideoColorInfo.VideoFullRangeFlag.Get(0));
			vpcC[7] = (uint8)CodecVideoColorInfo.ColourPrimaries.Get(2);
			vpcC[8] = (uint8)CodecVideoColorInfo.TransferCharacteristics.Get(2);
			vpcC[9] = (uint8)CodecVideoColorInfo.MatrixCoefficients.Get(2);
			OutCodecInformation.GetExtras().Set(StreamCodecInformationOptions::VPccBox, FVariantValue(vpcC));

			int32 CropL = Video->GetCropLeft();
			int32 CropR = Video->GetCropRight();
			int32 CropT = Video->GetCropTop();
			int32 CropB = Video->GetCropBottom();
			OutCodecInformation.SetResolution(FStreamCodecInformation::FResolution(Video->GetDisplayWidth(), Video->GetDisplayHeight()));
			OutCodecInformation.SetCrop(FStreamCodecInformation::FCrop(CropL, CropT, CropR, CropB));
			OutCodecInformation.SetAspectRatio(FStreamCodecInformation::FAspectRatio(1, 1));

			// If there is no default duration at this point we try to set it from the frame rate.
			// That is a deprecated element however, so the chances that it is set are fairly slim.
			if (InFromTrack->GetDefaultDurationNanos() == 0 && Video.IsValid() && Video->GetFrameRate() > 0.0)
			{
				int64 nanos = (int64)(1000000000.0 / Video->GetFrameRate());
				if (nanos >= 0)
				{
					InFromTrack->SetDefaultDurationNanos((uint64)nanos);
				}
			}
			// Calculate the frame rate from the default duration.
			if (InFromTrack->GetDefaultDurationNanos() > 0)
			{
				FTimeFraction fr(1000000000, InFromTrack->GetDefaultDurationNanos());
				OutCodecInformation.SetFrameRate(fr);
			}
			return true;
		}
		// AAC audio?
		else if (InFromTrack->GetTrackType() == EMKVTrackType::Audio && Codec4CC.Equals(TEXT("aac")))
		{
			// The codec private data is an AudioSpecificConfig as per ISO/IEC-14496-3
			ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord AudioSpecificConfig;
			if (!AudioSpecificConfig.ParseFrom(InFromTrack->GetCodecPrivate().GetData(), InFromTrack->GetCodecPrivate().Num()))
			{
				return false;
			}

			OutCodecInformation.GetExtras().Set(StreamCodecInformationOptions::DecoderConfigurationRecord, FVariantValue(InFromTrack->GetCodecPrivate()));
			OutCodecInformation.SetStreamType(EStreamType::Audio);
			OutCodecInformation.SetMimeType(TEXT("audio/mp4"));
			OutCodecInformation.SetCodec(FStreamCodecInformation::ECodec::AAC);
			OutCodecInformation.SetCodecSpecificData(AudioSpecificConfig.GetCodecSpecificData());
			OutCodecInformation.SetStreamLanguageTag(InFromTrack->GetLanguageTag());
			OutCodecInformation.SetCodecSpecifierRFC6381(FString::Printf(TEXT("mp4a.40.%d"), AudioSpecificConfig.ExtAOT ? AudioSpecificConfig.ExtAOT : AudioSpecificConfig.AOT));
			OutCodecInformation.SetSamplingRate(AudioSpecificConfig.ExtSamplingFrequency ? AudioSpecificConfig.ExtSamplingFrequency : AudioSpecificConfig.SamplingRate);
			OutCodecInformation.SetChannelConfiguration(AudioSpecificConfig.ChannelConfiguration);
			OutCodecInformation.SetNumberOfChannels(ElectraDecodersUtil::MPEG::AACUtils::GetNumberOfChannelsFromChannelConfiguration(AudioSpecificConfig.ChannelConfiguration));
			// We assume that all platforms can decode PS (parametric stereo). As such we change the channel count from mono to stereo
			// to convey the _decoded_ format, not the source format.
			if (AudioSpecificConfig.ChannelConfiguration == 1 && AudioSpecificConfig.PSSignal > 0)
			{
				OutCodecInformation.SetNumberOfChannels(2);
			}
			const int32 NumDecodedSamplesPerBlock = AudioSpecificConfig.SBRSignal > 0 ? 2048 : 1024;
			OutCodecInformation.GetExtras().Set(StreamCodecInformationOptions::SamplesPerBlock, FVariantValue((int64)NumDecodedSamplesPerBlock));
			// If there is no default duration set we try to calculate it from the sample rate.
			if (InFromTrack->GetDefaultDurationNanos() == 0)
			{
				FTimeFraction fr(NumDecodedSamplesPerBlock, (uint32)OutCodecInformation.GetSamplingRate());
				if (fr.IsValid())
				{
					int64 nanos = fr.GetAsTimebase(1000000000);
					check(nanos >= 0);
					if (nanos >= 0)
					{
						InFromTrack->SetDefaultDurationNanos((uint64)nanos);
					}
				}
			}
			return true;
		}
		// Opus audio?
		else if (InFromTrack->GetTrackType() == EMKVTrackType::Audio && Codec4CC.Equals(TEXT("Opus")))
		{
			// The codec private data is (presumably) an `OpusHead` structure as described here
			//   https://datatracker.ietf.org/doc/html/rfc7845#section-5.1
			// according to the Matroska Opus *DRAFT* described here: https://wiki.xiph.org/MatroskaOpus

			TArray<uint8> OpusHead = InFromTrack->GetCodecPrivate();
			const TArray<uint8> MagicOpusHeader {'O','p','u','s','H','e','a','d'};
			if (OpusHead.Num() < 8 || FMemory::Memcmp(OpusHead.GetData(), MagicOpusHeader.GetData(), MagicOpusHeader.Num()))
			{
				return false;
			}
			// Check minimum header size, version must be 1 and mapping family either 0, 1 or 255
			if (OpusHead.Num() < 19 || OpusHead[8]>15 || (OpusHead[18]!=0 && OpusHead[18]!=1 && OpusHead[18]!=255))
			{
				return false;
			}
			// A Matroska file stored codec specific data for AVC and HEVC video just like an ISO/IEC 14496-12 file does,
			// so for simplicities sake we convert this structure into the same as the `dOps` box in an mp4 would look like.
			// Remove the magic header.
			OpusHead.RemoveAt(0,8);
			// This header and the `dOps` box are identical in layout, except for byte ordering of int16/int32 values.
			// Change the version number from 1 to 0.
			OpusHead[0] = 0;
			int16* PreSkip = (int16*)(OpusHead.GetData() + 2);
			uint32* InputSampleRate = (uint32*)(OpusHead.GetData() + 4);
			int16* OutputGain = (int16*)(OpusHead.GetData() + 8);
			*PreSkip = Utils::EndianSwap(*PreSkip);
			*InputSampleRate = Utils::EndianSwap(*InputSampleRate);
			*OutputGain = Utils::EndianSwap(*OutputGain);

			TMKVElementPtr<FMKVAudio> Audio = InFromTrack->GetAudio();
			if (!Audio.IsValid())
			{
				return false;
			}

			OutCodecInformation.GetExtras().Set(StreamCodecInformationOptions::DOpsBox, FVariantValue(OpusHead));
			OutCodecInformation.SetStreamType(EStreamType::Audio);
			OutCodecInformation.SetMimeType(TEXT("audio/mp4"));
			OutCodecInformation.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
			OutCodecInformation.SetCodec4CC(Utils::Make4CC('O','p','u','s'));
			OutCodecInformation.SetDecoderConfigRecord(OpusHead);
			OutCodecInformation.SetCodecSpecificData(OpusHead);
			OutCodecInformation.SetStreamLanguageTag(InFromTrack->GetLanguageTag());
			OutCodecInformation.SetCodecSpecifierRFC6381(TEXT("Opus"));
			OutCodecInformation.SetSamplingRate(Audio->GetOutputSampleRate());
			OutCodecInformation.SetNumberOfChannels(Audio->GetNumberOfChannels());
			// If there is no default duration set we assume the encoded frame size was 20ms
			if (InFromTrack->GetDefaultDurationNanos() == 0)
			{
				FTimeFraction fr(20, 1000);
				if (fr.IsValid())
				{
					int64 nanos = fr.GetAsTimebase(1000000000);
					check(nanos >= 0);
					if (nanos >= 0)
					{
						InFromTrack->SetDefaultDurationNanos((uint64)nanos);
					}
				}
			}
			return true;
		}
		// FLAC audio?
		else if (InFromTrack->GetTrackType() == EMKVTrackType::Audio && Codec4CC.Equals(TEXT("FLAC")))
		{
			// The Private Data contains all the header/metadata packets before the first data packet.
			// These include the first header packet containing only the word fLaC as well as all metadata packets.
			//   https://datatracker.ietf.org/doc/draft-ietf-cellar-codec/

			TArray<uint8> FlacHead = InFromTrack->GetCodecPrivate();
			const TArray<uint8> MagicFlacHeader {'f','L','a','C'};
			if (FlacHead.Num() < 4 || FMemory::Memcmp(FlacHead.GetData(), MagicFlacHeader.GetData(), MagicFlacHeader.Num()))
			{
				return false;
			}
			// Replace the magic header with zeros, which is what the 'dfLa' box of an mp4 starts with
			// (the version number 0 and flags of 0)
			// Otherwise this is identical to the `dfLa` box.
			FlacHead[0] = FlacHead[1] = FlacHead[2] = FlacHead[3] = 0;

			TMKVElementPtr<FMKVAudio> Audio = InFromTrack->GetAudio();
			if (!Audio.IsValid())
			{
				return false;
			}

			OutCodecInformation.GetExtras().Set(TEXT("dfLa_box"), FVariantValue(FlacHead));
			OutCodecInformation.SetStreamType(EStreamType::Audio);
			OutCodecInformation.SetMimeType(TEXT("audio/mp4"));
			OutCodecInformation.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
			OutCodecInformation.SetCodec4CC(Utils::Make4CC('f','L','a','C'));
			OutCodecInformation.SetStreamLanguageTag(InFromTrack->GetLanguageTag());
			OutCodecInformation.SetCodecSpecifierRFC6381(TEXT("flac"));
			OutCodecInformation.SetSamplingRate(Audio->GetOutputSampleRate());
			OutCodecInformation.SetNumberOfChannels(Audio->GetNumberOfChannels());
			// Set the header as decoder config record and codec specific data, which requires removal of the full box bytes.
			FlacHead.RemoveAt(0, 4);
			OutCodecInformation.SetDecoderConfigRecord(FlacHead);
			OutCodecInformation.SetCodecSpecificData(FlacHead);
			// If there is no default duration set we try to calculate it from the sample rate.
			if (InFromTrack->GetDefaultDurationNanos() == 0)
			{
				UE_LOGF(LogElectraMKVParser, Error, "A_FLAC requires a sample default duration specified in the track!");
				return false;
			}
			return true;
		}
		// MPEG audio?
		else if (InFromTrack->GetTrackType() == EMKVTrackType::Audio && Codec4CC.Equals(TEXT("MPEG")))
		{
			TMKVElementPtr<FMKVAudio> Audio = InFromTrack->GetAudio();
			if (!Audio.IsValid())
			{
				return false;
			}

			OutCodecInformation.SetStreamType(EStreamType::Audio);
			OutCodecInformation.SetMimeType(TEXT("audio/mpeg"));
			OutCodecInformation.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
			OutCodecInformation.SetCodec4CC(Utils::Make4CC('m','p','g','a'));
			OutCodecInformation.SetProfile(1);
			FString Layer = InFromTrack->GetCodecID().Mid(8);
			int32 LayerValue = 3;
			if (Layer.Len())
			{
				LexFromString(LayerValue, *Layer);
			}
			OutCodecInformation.SetProfileLevel(LayerValue);
			OutCodecInformation.SetStreamLanguageTag(InFromTrack->GetLanguageTag());
			OutCodecInformation.SetCodecSpecifierRFC6381(TEXT("mp4a.6b"));
			OutCodecInformation.SetSamplingRate(Audio->GetOutputSampleRate());
			OutCodecInformation.SetNumberOfChannels(Audio->GetNumberOfChannels());
			// If there is no default duration set we try to calculate it from the sample rate.
			if (InFromTrack->GetDefaultDurationNanos() == 0)
			{
				UE_LOGF(LogElectraMKVParser, Error, "A_MPEG requires a sample default duration specified in the track!");
				return false;
			}
			return true;
		}
		return false;
	}


	FErrorDetail FParserMKV::PrepareTracks()
	{
		// Was a segment loaded successfully?
		if (!Segment.IsValid())
		{
			return LastError.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("No valid stream loaded and parsed, cannot prepare tracks."));
		}
		// Information is present?
		if (!Segment->HaveInfo())
		{
			return LastError.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("No segment info found, cannot prepare tracks."));
		}
		// Are there tracks?
		TMKVElementPtr<const FMKVTracks> MkvTracks = Segment->GetTracks();
		if (!MkvTracks.IsValid() || MkvTracks->GetNumberOfTracks() == 0)
		{
			return LastError.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("No tracks found in stream."));
		}
		IPlayerStreamFilter* StreamFilter = PlayerSessionServices ? PlayerSessionServices->GetStreamFilter() : nullptr;
		AlternateCueTrackNumbers.Empty();
		NumVideoTracks = 0;
		for(int32 nT=0, nTmax=MkvTracks->GetNumberOfTracks(); nT<nTmax; ++nT)
		{
			TMKVElementPtr<FMKVTrackEntry> Track = MkvTracks->GetTrackByIndex(nT);
			if (!Track.IsValid())
			{
				continue;
			}

			FStreamCodecInformation CodecInformation;

			NumVideoTracks += Track->GetTrackType() == EMKVTrackType::Video ? 1 : 0;
			if (!SetupCodecInfo(CodecInformation, Track))
			{
				if (Track->GetTrackType() == EMKVTrackType::Video)
				{
					AlternateCueTrackNumbers.Emplace(Track->GetTrackNumber());
				}
				continue;
			}
			if (StreamFilter && !StreamFilter->CanDecodeStream(CodecInformation))
			{
				if (Track->GetTrackType() == EMKVTrackType::Video)
				{
					AlternateCueTrackNumbers.Emplace(Track->GetTrackNumber());
				}
				continue;
			}

			if (Track->GetTrackTimestampScale() != 1.0)
			{
				UE_LOGF(LogElectraMKVParser, Warning, "Track timestamp scale present and not 1.0. Playback will probably not be correct!");
			}

			TMKVElementPtr<FMKVTrack> NewTrack = MakeMKVElementPtr<FMKVTrack>();
			NewTrack->ParentMKV = this;
			NewTrack->Track = Track;
			NewTrack->QuickInfo.CodecInfo = CodecInformation;
			NewTrack->QuickInfo.CSD = CodecInformation.GetCodecSpecificData();
			NewTrack->QuickInfo.DefaultDurationNanos = Track->GetDefaultDurationNanos();
			NewTrack->QuickInfo.CodecDelayNanos = Track->GetCodecDelayNanos();
			// Check supported content encodings
			TMKVElementPtr<FMKVContentEncodings> ContentEncoding = Track->GetContentEncodings();
			if (ContentEncoding.IsValid())
			{
				const TArray<TMKVElementPtr<FMKVContentEncoding>>& ContentEncodings = ContentEncoding->GetContentEncoding();
				for(int32 i=0; i<ContentEncodings.Num(); ++i)
				{
					const FMKVContentEncoding* ce = ContentEncodings[i].Get();
					if (ce->GetContentEncodingScope() != 1)
					{
						return LastError.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Unsupported content encoding scope."));
					}
					if (ce->GetContentEncodingType() > 1)
					{
						return LastError.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Unsupported content encoding type."));
					}
					// Compression?
					if (ce->GetContentEncodingType() == 0)
					{
						TMKVElementPtr<FMKVContentCompression> ContentCompression = ce->GetContentCompression();
						if (ContentCompression.IsValid() && ContentCompression->GetContentCompAlgo() != 3)
						{
							return LastError.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Unsupported content compression algorithm."));
						}
						NewTrack->QuickInfo.HeaderStrippedBytes = ContentCompression->GetContentCompSettings();
					}
					// Encryption
					else
					{
						TMKVElementPtr<FMKVContentEncryption> ContentEncryption = ce->GetContentEncryption();
						if (ContentEncryption.IsValid() && ContentEncryption->GetContentEncAlgo() != 0 && ContentEncryption->GetContentEncAlgo() != 5)
						{
							return LastError.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Unsupported content encryption algorithm."));
						}
						if (ContentEncryption->GetContentEncAlgo() == 5)
						{
							TMKVElementPtr<FMKVContentEncAESSettings> AESSettings = ContentEncryption->GetContentEncAESSettings();
							if (AESSettings.IsValid() && AESSettings->GetAESSettingsCipherMode() != 1 && AESSettings->GetAESSettingsCipherMode() != 2)
							{
								return LastError.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Unsupported content encryption AES cipher."));
							}
							NewTrack->QuickInfo.bIsEncrypted = true;
							NewTrack->QuickInfo.DecryptionInfo.DefaultKID = ContentEncryption->GetContentEncKeyID();
							NewTrack->QuickInfo.DecryptionInfo.Scheme4CC = AESSettings->GetAESSettingsCipherMode() == 1 ? Utils::Make4CC('c','e','n','c') : Utils::Make4CC('c','b','c','s');
						}
					}
				}
			}
			Tracks.Emplace(MoveTemp(NewTrack));
		}
		// If all video tracks cannot be used we need to assign one of their track IDs to the other tracks for seeking
		// because in the presence of video there will be no Cues generated for audio.
		if (NumVideoTracks && NumVideoTracks == AlternateCueTrackNumbers.Num())
		{
			for(auto &Trk : Tracks)
			{
				Trk->QuickInfo.AlternateCueTrackID = AlternateCueTrackNumbers[0];
			}
		}

		return LastError;
	}

	FTimeValue FParserMKV::GetDuration() const
	{
		return ContentDuration;
	}

	int32 FParserMKV::GetNumberOfTracks() const
	{
		return Tracks.Num();
	}

	const IParserMKV::ITrack* FParserMKV::GetTrackByIndex(int32 Index) const
	{
		return Index >= 0 && Index < Tracks.Num() ? Tracks[Index].Get() : nullptr;
	}

	const IParserMKV::ITrack* FParserMKV::GetTrackByTrackID(uint64 TrackID) const
	{
		for(int32 i=0; i<Tracks.Num(); ++i)
		{
			if (Tracks[i]->GetID() == TrackID)
			{
				return Tracks[i].Get();
			}
		}
		return nullptr;
	}

	IParserMKV::ICueIterator* FParserMKV::FMKVTrack::CreateCueIterator() const
	{
		return new FMKVCueIterator(this, ParentMKV->GetTotalFileSize());
	}


	UEMediaError FParserMKV::FMKVCueIterator::InternalStartAtFirst()
	{
		ThisCueUniqueID= ~0U;
		NextCueUniqueID = ~0U;
		TimestampScale = 0;
		ClusterFileOffset = 0;
		CueTime.SetToInvalid();
		if (!ParentTrack || !ParentTrack->ParentMKV)
		{
			return UEMEDIA_ERROR_INTERNAL;
		}
		TSharedPtrTS<const FMKVSegment> ParentSegment = ParentTrack->ParentMKV->GetSegment();
		if (!ParentSegment.IsValid() || !ParentSegment->HaveCues())
		{
			return UEMEDIA_ERROR_END_OF_STREAM;
		}
		TimestampScale = ParentTrack->ParentMKV->GetTimestampScale();
		// Get the cues
		const TArray<TMKVUniquePtr<FMKVCuePoint>>& CuePoints = ParentSegment->GetCues()->GetCuePoints();
		// Find the first cue that applies to the track we are iterating
		for(int32 CueIndex=0,LastCueIndex=CuePoints.Num(); CueIndex<LastCueIndex; ++CueIndex)
		{
			const TArray<TMKVUniquePtr<FMKVCueTrackPositions>>& CueTrackPositions = CuePoints[CueIndex]->GetCueTrackPositions();
			for(int32 j=0,jMax=CueTrackPositions.Num(); j<jMax; ++j)
			{
				if (CueTrackPositions[j]->GetCueTrack() == TrackID)
				{
					if (TimestampScale == DEFAULT_MATROSKA_TIMESCALE)
					{
						CueTime.SetFromND((int64)CuePoints[CueIndex]->GetCueTime()*10000, 10000000U);
					}
					else
					{
						CueTime.SetFromND((int64)CuePoints[CueIndex]->GetCueTime()*TimestampScale, 1000000000U);
					}
					ThisCueUniqueID = CuePoints[CueIndex]->GetUniqueID();
					ClusterFileOffset = ParentSegment->GetElementOffset() + (int64)CueTrackPositions[j]->GetCueClusterPosition();
					return UEMEDIA_ERROR_OK;
				}
  			}
		}
		return UEMEDIA_ERROR_END_OF_STREAM;
	}

	UEMediaError FParserMKV::FMKVCueIterator::InternalStartAtUniqueID(uint32 InUniqueID)
	{
		ThisCueUniqueID= ~0U;
		NextCueUniqueID = ~0U;
		TimestampScale = 0;
		ClusterFileOffset = 0;
		CueTime.SetToInvalid();
		if (!ParentTrack || !ParentTrack->ParentMKV)
		{
			return UEMEDIA_ERROR_INTERNAL;
		}
		TSharedPtrTS<const FMKVSegment> ParentSegment = ParentTrack->ParentMKV->GetSegment();
		if (!ParentSegment.IsValid() || !ParentSegment->HaveCues())
		{
			return UEMEDIA_ERROR_END_OF_STREAM;
		}
		TimestampScale = ParentTrack->ParentMKV->GetTimestampScale();
		// Get the cues
		const TArray<TMKVUniquePtr<FMKVCuePoint>>& CuePoints = ParentSegment->GetCues()->GetCuePoints();
		// Find the first cue that applies to the track we are iterating
		for(int32 CueIndex=0,LastCueIndex=CuePoints.Num(); CueIndex<LastCueIndex; ++CueIndex)
		{
			if (CuePoints[CueIndex]->GetUniqueID() != InUniqueID)
			{
				continue;
			}
			const TArray<TMKVUniquePtr<FMKVCueTrackPositions>>& CueTrackPositions = CuePoints[CueIndex]->GetCueTrackPositions();
			for(int32 j=0,jMax=CueTrackPositions.Num(); j<jMax; ++j)
			{
				if (CueTrackPositions[j]->GetCueTrack() == TrackID)
				{
					if (TimestampScale == DEFAULT_MATROSKA_TIMESCALE)
					{
						CueTime.SetFromND((int64)CuePoints[CueIndex]->GetCueTime()*10000, 10000000U);
					}
					else
					{
						CueTime.SetFromND((int64)CuePoints[CueIndex]->GetCueTime()*TimestampScale, 1000000000U);
					}
					ThisCueUniqueID = CuePoints[CueIndex]->GetUniqueID();
					ClusterFileOffset = ParentSegment->GetElementOffset() + (int64)CueTrackPositions[j]->GetCueClusterPosition();
					return UEMEDIA_ERROR_OK;
				}
			}
		}
		return UEMEDIA_ERROR_END_OF_STREAM;
	}


	UEMediaError FParserMKV::FMKVCueIterator::InternalNext()
	{
		if (IsAtEOS())
		{
			return UEMEDIA_ERROR_END_OF_STREAM;
		}

		if (!ParentTrack || !ParentTrack->ParentMKV)
		{
			return UEMEDIA_ERROR_INTERNAL;
		}
		TSharedPtrTS<const FMKVSegment> ParentSegment = ParentTrack->ParentMKV->GetSegment();
		if (!ParentSegment.IsValid() || !ParentSegment->HaveCues())
		{
			return UEMEDIA_ERROR_END_OF_STREAM;
		}

		const TArray<TMKVUniquePtr<FMKVCuePoint>>& CuePoints = ParentSegment->GetCues()->GetCuePoints();
		bool bFoundCurrent = false;
		for(int32 CueIndex=0,LastCueIndex=CuePoints.Num(); CueIndex<LastCueIndex; ++CueIndex)
		{
			if (ThisCueUniqueID == CuePoints[CueIndex]->GetUniqueID())
			{
				bFoundCurrent = true;
				continue;
			}
			if (!bFoundCurrent)
			{
				continue;
			}
			const TArray<TMKVUniquePtr<FMKVCueTrackPositions>>& CueTrackPositions = CuePoints[CueIndex]->GetCueTrackPositions();
			for(int32 j=0,jMax=CueTrackPositions.Num(); j<jMax; ++j)
			{
				if (CueTrackPositions[j]->GetCueTrack() == TrackID)
				{
					if (TimestampScale == DEFAULT_MATROSKA_TIMESCALE)
					{
						CueTime.SetFromND((int64)CuePoints[CueIndex]->GetCueTime()*10000, 10000000U);
					}
					else
					{
						CueTime.SetFromND((int64)CuePoints[CueIndex]->GetCueTime()*TimestampScale, 1000000000U);
					}
					ThisCueUniqueID = CuePoints[CueIndex]->GetUniqueID();
					ClusterFileOffset = ParentSegment->GetElementOffset() + (int64)CueTrackPositions[j]->GetCueClusterPosition();
					return UEMEDIA_ERROR_OK;
				}
			}
		}
		return UEMEDIA_ERROR_END_OF_STREAM;
	}

	UEMediaError FParserMKV::FMKVCueIterator::InternalStartAtTime(const FTimeValue& AtTime, ESearchMode SearchMode)
	{
		UEMediaError Error = InternalStartAtFirst();
		if (Error != UEMEDIA_ERROR_OK)
		{
			return Error;
		}

		FMKVCueIterator Best(*this);
		int64 localTime = AtTime.GetAsHNS();
		for(; Error == UEMEDIA_ERROR_OK; Error = InternalNext())
		{
			// As long as the time is before the local timestamp we're looking for we're taking it.
			int64 thisTime = GetTimestamp().GetAsHNS();
			if (thisTime < localTime)
			{
				Best.AssignFrom(*this);
			}
			else
			{
				// Otherwise, if we hit the time dead on or search for a time that's larger we're done.
				if (thisTime == localTime || SearchMode == IParserMKV::ICueIterator::ESearchMode::After)
				{
					return UEMEDIA_ERROR_OK;
				}
				else if (SearchMode == IParserMKV::ICueIterator::ESearchMode::Before)
				{
					// Is the best time we found so far less than or equal to what we're looking for?
					if (Best.GetTimestamp().GetAsHNS() <= localTime)
					{
						// Yes, get the best values back into this instance and return.
						AssignFrom(Best);
						return UEMEDIA_ERROR_OK;
					}
					// Didn't find what we're looking for.
					return UEMEDIA_ERROR_INSUFFICIENT_DATA;
				}
				else
				{
					// Should pick the closest one.
					if (localTime - Best.GetTimestamp().GetAsHNS() < thisTime - localTime)
					{
						AssignFrom(Best);
					}
					return UEMEDIA_ERROR_OK;
				}
			}
		}
		if (Best.IsValid())
		{
			if (Error == UEMEDIA_ERROR_END_OF_STREAM && SearchMode == IParserMKV::ICueIterator::ESearchMode::After)
			{
				return Error;
			}
			else if (SearchMode == IParserMKV::ICueIterator::ESearchMode::Before && Best.GetTimestamp().GetAsHNS() > localTime)
			{
				return UEMEDIA_ERROR_INSUFFICIENT_DATA;
			}
			AssignFrom(Best);
			return UEMEDIA_ERROR_OK;
		}
		return Error;
	}

	void FParserMKV::FMKVCueIterator::InternalSetupWithNextCluster()
	{
		FMKVCueIterator This(*this);
		if (This.InternalNext() == UEMEDIA_ERROR_OK)
		{
			ClusterDuration = This.CueTime - CueTime;
			ClusterFileSize = This.ClusterFileOffset - ClusterFileOffset;
			NextCueUniqueID = This.ThisCueUniqueID;
		}
		else
		{
			ClusterDuration = ParentTrack->ParentMKV->GetDuration() - CueTime;
			ClusterFileSize = TotalFilesize - ClusterFileOffset;
			NextCueUniqueID = ~0U;
		}
	}


	UEMediaError FParserMKV::FMKVCueIterator::StartAtTime(const FTimeValue& AtTime, ESearchMode SearchMode)
	{
		UEMediaError Error = InternalStartAtTime(AtTime, SearchMode);
		if (Error == UEMEDIA_ERROR_OK)
		{
			InternalSetupWithNextCluster();
		}
		return Error;
	}
	UEMediaError FParserMKV::FMKVCueIterator::StartAtFirst()
	{
		UEMediaError Error = InternalStartAtFirst();
		if (Error == UEMEDIA_ERROR_OK)
		{
			InternalSetupWithNextCluster();
		}
		return Error;
	}

	UEMediaError FParserMKV::FMKVCueIterator::StartAtUniqueID(uint32 CueUniqueID)
	{
		UEMediaError Error = InternalStartAtUniqueID(CueUniqueID);
		if (Error == UEMEDIA_ERROR_OK)
		{
			InternalSetupWithNextCluster();
		}
		return Error;
	}

	UEMediaError FParserMKV::FMKVCueIterator::Next()
	{
		UEMediaError Error = InternalNext();
		if (Error == UEMEDIA_ERROR_OK)
		{
			InternalSetupWithNextCluster();
		}
		return Error;
	}

	bool FParserMKV::FMKVCueIterator::IsAtEOS() const
	{
		return ThisCueUniqueID == ~0U;
	}
	const IParserMKV::ITrack* FParserMKV::FMKVCueIterator::GetTrack() const
	{
		return ParentTrack;
	}
	FTimeValue FParserMKV::FMKVCueIterator::GetTimestamp() const
	{
		return CueTime;
	}
	int64 FParserMKV::FMKVCueIterator::GetClusterFileOffset() const
	{
		return ClusterFileOffset;
	}

	int64 FParserMKV::FMKVCueIterator::GetClusterFileSize() const
	{
		return ClusterFileSize;
	}
	FTimeValue FParserMKV::FMKVCueIterator::GetClusterDuration() const
	{
		return ClusterDuration;
	}
	bool FParserMKV::FMKVCueIterator::IsLastCluster() const
	{
		return NextCueUniqueID == ~0U;
	}


	/*********************************************************************************************************************/



	TSharedPtrTS<IParserMKV::IClusterParser> FParserMKV::CreateClusterParser(IGenericDataReader* InDataReader, const TArray<uint64>& InTrackIDsToParse, EClusterParseFlags InParseFlags) const
	{
		FMKVClusterParser* Parser = new FMKVClusterParser(AsShared(), InDataReader, InTrackIDsToParse, InParseFlags);
		return TSharedPtrTS<IParserMKV::IClusterParser>(Parser);
	}

	void FParserMKV::AddCue(int64 InCueTimestamp, uint64 InTrackID, int64 InCueRelativePosition, uint64 InCueBlockNumber, int64 InClusterPosition)
	{
		if (InCueTimestamp < 0 || !Segment.IsValid())
		{
			return;
		}
		Segment->GetOrCreateCues()->AddCue(InCueTimestamp / TimestampScale, InTrackID, InCueRelativePosition, InCueBlockNumber, InClusterPosition, NextCueUniqueID);
	}

	FParserMKV::FMKVClusterParser::FMKVClusterParser(TSharedPtrTS<const FParserMKV> InParentMKV, IGenericDataReader* InDataReader, const TArray<uint64>& InTrackIDsToParse, EClusterParseFlags InParseFlags)
		: ParentMKV(InParentMKV), Reader(new FClusterDataReader(InDataReader))
	{
		SegmentBaseOffset = InParentMKV->GetSegment()->GetElementOffset();
		ParseFlags = InParseFlags;
		bCheckForClusterStart = true;
		CurrentClusterSize = 0;
		CurrentBlockTypeID = 0;
		CurrentBlockSize = 0;
		ClusterTimestamp = 0;
		bIgnoreNonCluster = (ParseFlags & EClusterParseFlags::ClusterParseFlag_AllowFullDocument) != 0;

		// Get the quick track info for each of the tracks we're interested in.
		for(auto &TrkId : InTrackIDsToParse)
		{
			TMKVElementPtr<FMKVTrack> Trk = InParentMKV->GetTrackByID(TrkId);
			if (Trk.IsValid())
			{
				FTrackWorkVars tv;
				tv.TrackInfo = Trk->GetQuickInfo();
				TrackWorkVars.Emplace(TrkId, MoveTemp(tv));
			}
		}
		TimestampScale = InParentMKV->GetTimestampScale();
	}


	IParserMKV::IClusterParser::EParseAction FParserMKV::FMKVClusterParser::ParseUntilStartOfBlock(bool bFirstInCluster)
	{
		CurrentBlockTypeID = 0;
		CurrentBlockSize = 0;
		bool bGotTimestamp = false;
		if (bFirstInCluster)
		{
			ClusterTimestamp = 0;
		}
		else
		{
			bGotTimestamp = true;
		}
		while(!Reader->HasReadBeenAborted() && Reader->GetCurrentOffset() < Reader->GetEndOffset())
		{
			uint32 ElementID;
			int64 ElementLen;
			int64 CurrentOffset = Reader->GetCurrentOffset();
			if (!Reader->ReadElementID(ElementID) || !Reader->ReadElementLength(ElementLen))
			{
				return SetReadError();
			}
			// We are only interested in the timestamp element
			if (ElementID == MKV_Timestamp.ID)
			{
				if (!Reader->ReadEBMLuint(ClusterTimestamp, ElementLen, MKV_Timestamp.Default.u64))
				{
					return SetReadError();
				}
				bGotTimestamp = true;
			}
			else if (ElementID == MKV_SimpleBlock.ID || ElementID == MKV_BlockGroup.ID)
			{
				if (!bGotTimestamp)
				{
					// Timestamp must come before the block!
					ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Cluster timestamp must come before the block."));
					return IClusterParser::EParseAction::Failure;
				}
				CurrentBlockStartOffset = CurrentOffset;
				CurrentBlockDataOffset = Reader->GetCurrentOffset();
				CurrentBlockTypeID = ElementID;
				CurrentBlockSize = ElementLen;
				CurrentBlockEndOffset = CurrentBlockDataOffset + ElementLen;
				return IClusterParser::EParseAction::ReadFrameData;
			}
			else
			{
				if (!Reader->SkipOver(ElementLen))
				{
					return SetReadError();
				}
			}
		}
		return Reader->HasReadBeenAborted() ? IClusterParser::EParseAction::Failure : IClusterParser::EParseAction::EndOfData;
	}


	IParserMKV::IClusterParser::EParseAction FParserMKV::FMKVClusterParser::StartSimpleBlock()
	{
		// The file reader is positioned on the start of a `SimpleBlock`. Start processing it.
		uint64 BlkTrackId = 0;
		uint16 Value16 = 0;
		CurrentBlockFlags = 0;
		if (Reader->ReadVINT(BlkTrackId)<=0 || !Reader->Read(Value16) || !Reader->Read(CurrentBlockFlags))
		{
			return SetReadError();
		}
		int64 CurrentOffset = Reader->GetCurrentOffset();
		int64 BlockDataSize = CurrentBlockSize - (CurrentOffset - CurrentBlockDataOffset);
		CurrentBlockDataOffset = CurrentOffset;
		CurrentBlockFrameSizes.Empty();
		CurrentBlockFrameSizeIndex = 0;

		CurrentTrackVars = TrackWorkVars.Find(BlkTrackId);
		if (CurrentTrackVars)
		{
			CurrentTrackVars->BlockAdditions.Reset();

			int16 BlkRelativeTimestamp = (int16)Value16;
			int32 LacingType = (CurrentBlockFlags >> 1) & 3;
			switch(LacingType)
			{
				default:
				{
					CurrentBlockFrameSizes.Emplace(BlockDataSize);
					break;
				}
				case 1:
				{
					uint8 nf = 0;
					if (!Reader->Read(nf))
					{
						return SetReadError();
					}
					int64 SizeSum = 0;
					for(int32 i=0; i<nf; ++i)
					{
						int32 SampleSize = 0;
						uint8 b;
						do
						{
							if (!Reader->Read(b))
							{
								return SetReadError();
							}
							SampleSize += b;
						} while(b == 255);
						CurrentBlockFrameSizes.Emplace(SampleSize);
						SizeSum += SampleSize;
					}
					CurrentOffset = Reader->GetCurrentOffset();
					CurrentBlockFrameSizes.Emplace(CurrentBlockEndOffset - CurrentOffset - SizeSum);
					break;
				}
				case 3:
				{
					uint8 nf = 0;
					if (!Reader->Read(nf))
					{
						return SetReadError();
					}
					int64 SizeSum = 0;
					for(int32 i=0; i<nf; ++i)
					{
						if (i == 0)
						{
							uint64 vint = 0;
							if (Reader->ReadVINT(vint) <= 0)
							{
								return SetReadError();
							}
							CurrentBlockFrameSizes.Emplace((int64)vint);
						}
						else
						{
							int64 vint = 0;
							if (Reader->ReadSVINT(vint) <= 0)
							{
								return SetReadError();
							}
							CurrentBlockFrameSizes.Emplace(CurrentBlockFrameSizes.Last() + vint);
						}
						SizeSum += CurrentBlockFrameSizes.Last();
					}
					CurrentOffset = Reader->GetCurrentOffset();
					CurrentBlockFrameSizes.Emplace(CurrentBlockEndOffset - CurrentOffset - SizeSum);
					break;
				}
				case 2:
				{
					uint8 nf = 0;
					if (!Reader->Read(nf))
					{
						return SetReadError();
					}
					--BlockDataSize;
					++CurrentOffset;
					int32 NumFrames = nf + 1;
					if ((BlockDataSize % NumFrames) != 0)
					{
						ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(TEXT("Remaining data size is not a multiple of the fixed size lacing."));
						return IParserMKV::IClusterParser::EParseAction::Failure;
					}
					int64 FrameSize = BlockDataSize / NumFrames;
					while(NumFrames--)
					{
						CurrentBlockFrameSizes.Add(FrameSize);
					}
					break;
				}
			}

			CurrentBlockDataOffset = CurrentOffset;

			// Setup the action parameters
			Params.ClusterPosition = CurrentClusterStartOffset - SegmentBaseOffset;
			Params.TrackInfo = CurrentTrackVars->TrackInfo;
			Params.TrackID = BlkTrackId;
			CurrentTrackVars->SampleDuration = CurrentTrackVars->TrackInfo.DefaultDurationNanos;
			CurrentTrackVars->DTS = (int64)ClusterTimestamp * TimestampScale + CurrentTrackVars->DurationSumNanos  - CurrentTrackVars->TrackInfo.CodecDelayNanos;
			Params.DTS.SetFromHNS(CurrentTrackVars->DTS / 100);
			CurrentTrackVars->PTS = ((int64)ClusterTimestamp + BlkRelativeTimestamp) * TimestampScale - CurrentTrackVars->TrackInfo.CodecDelayNanos;
			Params.PTS.SetFromHNS(CurrentTrackVars->PTS / 100);
			Params.Timestamp = ((int64)ClusterTimestamp + BlkRelativeTimestamp) * TimestampScale - CurrentTrackVars->TrackInfo.CodecDelayNanos;
			Params.SampleDuration.SetFromHNS(CurrentTrackVars->SampleDuration / 100);
			if (CurrentTrackVars->TrackInfo.DefaultDurationNanos > 0)
			{
				CurrentTrackVars->DurationSumNanos += CurrentTrackVars->TrackInfo.DefaultDurationNanos;
			}
			Params.bIsKeyFrame = !!(CurrentBlockFlags & 0x80);
			Params.NumBytesToSkip = 0;
			Params.NumBytesToRead = CurrentBlockFrameSizes[CurrentBlockFrameSizeIndex];
			// Are there stripped header bytes that need prepending?
			if (Params.TrackInfo.HeaderStrippedBytes.Num())
			{
				NextAction = &ActionPrepend;
				PreviousActionType = IParserMKV::IClusterParser::EParseAction::PrependData;
				return PreviousActionType;
			}
			NextAction = &ActionRead;
			PreviousActionType = IParserMKV::IClusterParser::EParseAction::ReadFrameData;
			return PreviousActionType;
		}
		else
		{
			// Skip over
			Params.Clear();
			Params.NumBytesToSkip = CurrentBlockEndOffset - Reader->GetCurrentOffset();
			NextAction = &ActionSkip;
			PreviousActionType = IParserMKV::IClusterParser::EParseAction::SkipOver;
			return PreviousActionType;
		}
	}

	IParserMKV::IClusterParser::EParseAction FParserMKV::FMKVClusterParser::ContinueSimpleBlock()
	{
		auto CheckEndOfFrameOrCluster = [this]() -> IParserMKV::IClusterParser::EParseAction
		{
			// End of the lace. Did we finish the cluster as well?
			int64 CurrentOffset = Reader->GetCurrentOffset();
			check(CurrentOffset == CurrentBlockEndOffset);
			if (CurrentOffset >= CurrentClusterEndOffset)
			{
				NextAction = nullptr;
				PreviousActionType = IParserMKV::IClusterParser::EParseAction::EndOfData;
				return PreviousActionType;
			}
			else
			{
				IParserMKV::IClusterParser::EParseAction Result = ParseUntilStartOfBlock(false);
				if (Result == IClusterParser::EParseAction::EndOfData)
				{
					NextAction = nullptr;
					PreviousActionType = IParserMKV::IClusterParser::EParseAction::EndOfData;
					return PreviousActionType;
				}
				else if (Result != IClusterParser::EParseAction::ReadFrameData)
				{
					ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(FString::Printf(TEXT("Failed to locate beginning of block around offset %lld."), (long long int)(Reader->GetCurrentOffset())));
					return SetErrorAndReturn();
				}
				return CurrentBlockTypeID == MKV_SimpleBlock.ID ? StartSimpleBlock() : StartBlockGroup();
			}
		};
		// What was the previous action?
		switch(PreviousActionType)
		{
			case IParserMKV::IClusterParser::EParseAction::PrependData:
			{
				// Prepending data has also already set up everything to read the following frame data.
				NextAction = &ActionRead;
				PreviousActionType = IParserMKV::IClusterParser::EParseAction::ReadFrameData;
				return PreviousActionType;
			}
			case IParserMKV::IClusterParser::EParseAction::ReadFrameData:
			{
				// Did the user actually read the data like he was supposed to?
				int64 CurrentOffset = Reader->GetCurrentOffset();
				check(CurrentOffset == CurrentBlockDataOffset + Params.NumBytesToRead);
				if (CurrentOffset != CurrentBlockDataOffset + Params.NumBytesToRead)
				{
					ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_CLIENT_FAILED_TO_DO_WHAT_HE_WAS_TOLD).SetMessage(TEXT("Client code did not correctly read the requested amount of data."));
					return SetErrorAndReturn();
				}
				Reader->SetOffset(CurrentBlockDataOffset = CurrentOffset);

				// Is the data encrypted?
				if (CurrentTrackVars->TrackInfo.bIsEncrypted)
				{
					ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_UNSUPPORTED_FEATURE).SetMessage(TEXT("Encryption is not currently supported."));
					return SetErrorAndReturn();
				/*
					check(!"not implemented yet");
					NextAction = &ActionDecrypt;
					return IParserMKV::IClusterParser::EParseAction::DecryptData;
				*/
				}
				// Otherwise we are done with this frame.
				NextAction = &ActionFrameDone;
				PreviousActionType = IParserMKV::IClusterParser::EParseAction::FrameDone;
				return PreviousActionType;
			}
			case IParserMKV::IClusterParser::EParseAction::SkipOver:
			{
				// Did the user actually skip the data like he was supposed to?
				int64 CurrentOffset = Reader->GetCurrentOffset();
				check(CurrentOffset == CurrentBlockDataOffset + Params.NumBytesToSkip);
				if (CurrentOffset != CurrentBlockDataOffset + Params.NumBytesToSkip)
				{
					ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_CLIENT_FAILED_TO_DO_WHAT_HE_WAS_TOLD).SetMessage(TEXT("Client code did not correctly skip over the data."));
					return SetErrorAndReturn();
				}
				Reader->SetOffset(CurrentBlockDataOffset = CurrentOffset);
				return CheckEndOfFrameOrCluster();
			}
			case IParserMKV::IClusterParser::EParseAction::DecryptData:
			{
				// Done.
				NextAction = &ActionFrameDone;
				PreviousActionType = IParserMKV::IClusterParser::EParseAction::FrameDone;
				return PreviousActionType;
			}
			case IParserMKV::IClusterParser::EParseAction::FrameDone:
			{
				// Continue with the next frame in the lace
				++CurrentTrackVars->FrameNum;
				if (++CurrentBlockFrameSizeIndex >= CurrentBlockFrameSizes.Num())
				{
					return CheckEndOfFrameOrCluster();
				}
				else
				{
					// Laces need to have a known duration since we need to advance the DTS and PTS.
					int64 durns = CurrentTrackVars->SampleDuration;
					if (durns <= 0)
					{
						ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_UNSUPPORTED_FEATURE).SetMessage(TEXT("Lacing requires a default duration to advance the timestamp by."));
						return SetErrorAndReturn();
					}
					CurrentTrackVars->DurationSumNanos += durns;
					CurrentTrackVars->DTS += durns;
					CurrentTrackVars->PTS += durns;
					Params.DTS.SetFromHNS(CurrentTrackVars->DTS / 100);
					Params.PTS.SetFromHNS(CurrentTrackVars->PTS / 100);
					Params.bIsKeyFrame = !!(CurrentBlockFlags & 0x80);
					Params.Timestamp = -1;
					Params.NumBytesToSkip = 0;
					Params.NumBytesToRead = CurrentBlockFrameSizes[CurrentBlockFrameSizeIndex];
					if (Params.TrackInfo.HeaderStrippedBytes.Num())
					{
						NextAction = &ActionPrepend;
						PreviousActionType = IParserMKV::IClusterParser::EParseAction::PrependData;
						return PreviousActionType;
					}
					NextAction = &ActionRead;
					PreviousActionType = IParserMKV::IClusterParser::EParseAction::ReadFrameData;
					return PreviousActionType;
				}
			}
			default:
			case IParserMKV::IClusterParser::EParseAction::Failure:
			{
				break;
			}
		}

		ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_UNHANDLED_CASE).SetMessage(TEXT("Internal error parsing during cluster parsing."));
		return SetErrorAndReturn();
	}


	IParserMKV::IClusterParser::EParseAction FParserMKV::FMKVClusterParser::StartBlockGroup()
	{
		// The file reader is positioned on the start of a `BlockGroup`. Start processing it.
		while(!Reader->HasReadBeenAborted() && Reader->GetCurrentOffset() < Reader->GetEndOffset())
		{
			uint32 ElementID;
			int64 ElementLen;
			int64 CurrentOffset = Reader->GetCurrentOffset();
			if (!Reader->ReadElementID(ElementID) || !Reader->ReadElementLength(ElementLen))
			{
				return SetReadError();
			}

			// Is this the block now?
			if (ElementID == MKV_Block.ID)
			{
				// We will now read the block of this block group as if it were a SimpleBlock (which it is).
				// For that we need to adjust the block parsing values to the internal block.
				CurrentBlockTypeID = ElementID;
				CurrentBlockGroupBlockEndOffset = CurrentBlockEndOffset;
				CurrentBlockSize = ElementLen;
				CurrentBlockDataOffset = Reader->GetCurrentOffset();
				CurrentBlockEndOffset = CurrentBlockDataOffset + ElementLen;
				CurrentBlockDuration = 0;
				CurrentBlockReferenceCount = 0;
				return StartSimpleBlock();
			}
			else if (!HandleBlockElement(ElementID, ElementLen))
			{
				// The error detail should have been set in HandleBlockElement() already
				return SetErrorAndReturn();
			}
		}
		ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_UNHANDLED_CASE).SetMessage(TEXT("Internal error parsing during cluster parsing."));
		return SetErrorAndReturn();
	}

	bool FParserMKV::FMKVClusterParser::HandleBlockElement(uint32 InElementID, int64 InElementSize)
	{
		if (InElementID == MKV_BlockDuration.ID)
		{
			uint64 BlkDur = 0;
			if (!Reader->ReadEBMLuint(BlkDur, InElementSize, 0U))
			{
				SetReadError();
				return false;
			}
			CurrentBlockDuration = (int64)BlkDur * TimestampScale;
			return true;
		}
		else if (InElementID == MKV_ReferenceBlock.ID)
		{
			int64 BlkRefTimeOffset = 0;
			if (!Reader->ReadEBMLint(BlkRefTimeOffset, InElementSize, 0U))
			{
				SetReadError();
				return false;
			}
			++CurrentBlockReferenceCount;
			return true;
		}
		else if (InElementID == MKV_BlockAdditions.ID)
		{
			// Block additions
			if (!CurrentTrackVars->BlockAdditions.IsValid())
			{
				CurrentTrackVars->BlockAdditions = MakeUnique<FMKVBlockAdditions>(InElementID, CurrentBlockStartOffset, InElementSize);
			}
			EParseResult BlkParseResult = CurrentTrackVars->BlockAdditions->ParseElement(Reader.Get());
			if (BlkParseResult == EParseResult::Error)
			{
				SetReadError();
				return false;
			}
		}
		else
		{
			if (!Reader->SkipOver(InElementSize))
			{
				SetReadError();
				return false;
			}
		}
		return true;
	}

	IParserMKV::IClusterParser::EParseAction FParserMKV::FMKVClusterParser::ContinueBlockGroup()
	{
		IClusterParser::EParseAction Result = ContinueSimpleBlock();
		switch(Result)
		{
			case IParserMKV::IClusterParser::EParseAction::FrameDone:
			{
				// We need to check if there are additional laced frames.
				// If so there is a problem if there is block additional data after the lace
				// since we have not read it yet and can therefor not finish this frame.
				// We treat laces in a block as an error.
				// If there is no block additional data though we could proceed.
				if (CurrentBlockFrameSizes.Num() > 1)
				{
					ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_UNSUPPORTED_FEATURE).SetMessage(TEXT("Lacing in BlockGroup is not supported."));
					return SetErrorAndReturn();
				}

				// We now finish the block by reading the remaining data, which should be
				// block additions or other level 3 block elements.
				while(!Reader->HasReadBeenAborted() && Reader->GetCurrentOffset() < CurrentBlockGroupBlockEndOffset)
				{
					uint32 ElementID;
					int64 ElementLen;
					int64 CurrentOffset = Reader->GetCurrentOffset();
					if (!Reader->ReadElementID(ElementID) || !Reader->ReadElementLength(ElementLen))
					{
						return SetReadError();
					}
					else if (!HandleBlockElement(ElementID, ElementLen))
					{
						// The error detail should have been set in HandleBlockElement() already
						return SetErrorAndReturn();
					}
				}
				// Set the end of the block back to what the end of the group was now that we have
				// finished parsing the group. That way we can continue regularly.
				CurrentBlockEndOffset = CurrentBlockGroupBlockEndOffset;

				// If this block had a duration set we need to apply it to the DTS of the next sample.
				if (CurrentBlockDuration > 0)
				{
					CurrentTrackVars->DurationSumNanos += CurrentBlockDuration;
					CurrentTrackVars->SampleDuration = CurrentBlockDuration;
					CurrentBlockDuration = 0;
				}
				// Set the duration of the sample, even though this is *NOT* the duration of *this* sample.
				// The duration of the sample is still the difference between two consecutive samples in display order.
				// We still set this in case this is the last frame (in display order) of the cluster.
				Params.SampleDuration.SetFromHNS(CurrentTrackVars->SampleDuration / 100);
				// Per definition a block that references no other block is a keyframe.
				// Note: this *SHOULD* be the case, but we have encountered files where this is NOT the case
				//       and an encoded VP9 frame that was not key also had no references.
				//       In this case the frame is tagged as key when it really is not.
				//       We chalk this up to a bug in the Matroska muxer that was used.
				Params.bIsKeyFrame = CurrentBlockReferenceCount == 0;

				// Set up the block additional data, if any.
				if (CurrentTrackVars->BlockAdditions.IsValid())
				{
					const TArray<TMKVUniquePtr<FMKVBlockMore>>& BlocksMore = CurrentTrackVars->BlockAdditions->GetBlockMore();
					for(int32 i=0; i<BlocksMore.Num(); ++i)
					{
						Params.BlockAdditionalData.Emplace(BlocksMore[i]->GetBlockAddID(), BlocksMore[i]->GetBlockAdditional());
					}
					CurrentTrackVars->BlockAdditions.Reset();
				}
				break;
			}
			default:
			{
				break;
			}
		}
		return Result;
	}


	IParserMKV::IClusterParser::EParseAction FParserMKV::FMKVClusterParser::NextParseAction()
	{
		// Start on a new cluster?
		if (bCheckForClusterStart)
		{
			while(1)
			{
				CurrentClusterStartOffset = Reader->GetCurrentOffset();
				// The next 4 bytes have to be a cluster ID
				uint32 ElementID = 0;
				if (!Reader->ReadElementID(ElementID))
				{
					return SetReadError();
				}
				if (ElementID == MKV_Cluster.ID)
				{
					break;
				}
				else
				{
					if ((ParseFlags & IParserMKV::EClusterParseFlags::ClusterParseFlag_AllowFullDocument) != 0)
					{
						if (ElementID == MKV_Segment.ID)
						{
							int64 SegmentLen = 0;
							if (!Reader->ReadElementLength(SegmentLen))
							{
								return SetReadError();
							}
							continue;
						}
						else
						{
							int64 BytesToSkip = 0;
							if (!Reader->ReadElementLength(BytesToSkip) || !Reader->Skip(BytesToSkip))
							{
								return SetReadError();
							}
						}
					}
					else
					{
						if (!bIgnoreNonCluster)
						{
							ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(FString::Printf(TEXT("Not a Matroska cluster at offset %lld."), (long long int)(Reader->GetCurrentOffset()-4)));
							return SetErrorAndReturn();
						}
						// If this is a file with multiple headers/segments we stop at this point.
						else if (ElementID == EBML_Header.ID || ElementID == MKV_Segment.ID)
						{
							return IParserMKV::IClusterParser::EParseAction::EndOfData;
						}
						else
						{
							// Skip over this and retry.
							// This is usually near the end of the file where Cues or Tags or similar follow the last cluster,
							// but it could also be that there is some element *between* clusters. That should not happen but
							// it is also not explicitly forbidden.
							int64 BytesToSkip = 0;
							if (!Reader->ReadElementLength(BytesToSkip) || !Reader->Skip(BytesToSkip))
							{
								return SetReadError();
							}
						}
					}
				}
			}

			// Restart the per-track work vars
			for(auto& tv : TrackWorkVars)
			{
				tv.Value.Restart();
			}

			CurrentClusterSize = 0;
			CurrentBlockTypeID = 0;
			CurrentBlockSize = 0;
			ClusterTimestamp = 0;
			if (!Reader->ReadElementLength(CurrentClusterSize))
			{
				return SetReadError();
			}
			CurrentClusterEndOffset = Reader->GetCurrentOffset() + CurrentClusterSize;
			if (ParseUntilStartOfBlock(true) != IClusterParser::EParseAction::ReadFrameData)
			{
				return SetReadError();
			}

			bCheckForClusterStart = false;
			IParserMKV::IClusterParser::EParseAction Result = CurrentBlockTypeID == MKV_SimpleBlock.ID ? StartSimpleBlock() : StartBlockGroup();
			if (Result == IParserMKV::IClusterParser::EParseAction::Failure)
			{
				ErrorDetail.SetError(UEMEDIA_ERROR_DETAIL).SetFacility(Facility::EFacility::MKVParser).SetCode(ERRCODE_MKV_INTERNAL_BAD_FORMAT).SetMessage(FString::Printf(TEXT("Failed to parse cluster at offset %lld."), (long long int)CurrentClusterStartOffset));
				return SetErrorAndReturn();
			}
			return Result;
		}
		else
		{
			// Continue in the current cluster.
			IParserMKV::IClusterParser::EParseAction Result = CurrentBlockTypeID == MKV_SimpleBlock.ID ? ContinueSimpleBlock() : ContinueBlockGroup();
			if (Result == IParserMKV::IClusterParser::EParseAction::EndOfData)
			{
				// Are we at the overall end of what we could read or just the end of one cluster?
				int64 CurrentOffset = Reader->GetCurrentOffset();
				if (CurrentOffset < Reader->GetEndOffset())
				{
					bCheckForClusterStart = true;
					bIgnoreNonCluster = true;
					return NextParseAction();
				}
			}
			return Result;
		}
	}
	FErrorDetail FParserMKV::FMKVClusterParser::GetLastError() const
	{
		return ErrorDetail;
	}
	const IParserMKV::IClusterParser::IAction* FParserMKV::FMKVClusterParser::GetAction() const
	{
		return NextAction;
	}

	int64 FParserMKV::FMKVClusterParser::GetClusterPosition() const
	{
		return CurrentClusterStartOffset;
	}

	int64 FParserMKV::FMKVClusterParser::GetClusterBlockPosition() const
	{
		return CurrentBlockStartOffset;
	}



	/*********************************************************************************************************************/

	TSharedPtrTS<IParserMKV> IParserMKV::CreateParser(IPlayerSessionServices* InPlayerSession)
	{
		return MakeSharedTS<FParserMKV>(InPlayerSession);
	}

} // namespace Electra
