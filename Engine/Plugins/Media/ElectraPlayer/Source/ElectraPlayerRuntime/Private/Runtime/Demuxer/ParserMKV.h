// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "StreamTypes.h"
#include "ParameterDictionary.h"
#include "ErrorDetail.h"
#include "ElectraEncryptedSampleInfo.h"
#include "BufferedDataReader.h"

namespace Electra
{
	//
	// Forward declarations
	//
	class IPlayerSessionServices;


	/**
	 * Interface for parsing a Matroska or WebM file.
	 */
	class IParserMKV
	{
	public:
		virtual ~IParserMKV() = default;

		static TSharedPtrTS<IParserMKV> CreateParser(IPlayerSessionServices* PlayerSession);

		enum EParserFlags
		{
			ParseFlag_Default = 0,
			ParseFlag_OnlyTracks = 1 << 0,
			ParseFlag_OnlyEssentialLevel1 = 1 << 1,
			ParseFlag_SuppressCueWarning = 1 << 2,
		};

		/**
		 * Parses the header boxes.
		 */
		virtual FErrorDetail ParseHeader(IGenericDataReader* DataReader, EParserFlags ParseFlags) = 0;


		/*******************************************************************************************************************/
		class ITrack;

		virtual FErrorDetail PrepareTracks() = 0;

		virtual FTimeValue GetDuration() const = 0;

		virtual int32 GetNumberOfTracks() const = 0;

		class ICueIterator
		{
		public:
			virtual ~ICueIterator() = default;

			enum class ESearchMode
			{
				Before,
				After,
				Closest
			};

			virtual UEMediaError StartAtTime(const FTimeValue& AtTime, ESearchMode SearchMode) = 0;
			virtual UEMediaError StartAtFirst() = 0;
			virtual UEMediaError StartAtUniqueID(uint32 CueUniqueID) = 0;
			virtual UEMediaError Next() = 0;
			virtual bool IsAtEOS() const = 0;

			virtual const ITrack* GetTrack() const = 0;
			virtual FTimeValue GetTimestamp() const = 0;
			virtual int64 GetClusterFileOffset() const = 0;
			virtual int64 GetClusterFileSize() const = 0;
			virtual FTimeValue GetClusterDuration() const = 0;
			virtual bool IsLastCluster() const = 0;

			virtual uint32 GetUniqueID() const = 0;
			virtual uint32 GetNextUniqueID() const = 0;
		};

		class IClusterParser
		{
		public:
			virtual ~IClusterParser() = default;

			enum class EParseAction
			{
				SkipOver,			// Skip over the next n bytes. GetAction() returns an IActionSkipOver action.
				PrependData,		// Prepend the frame data with constant data. GetAction() returns an IActionPrependData action.
				ReadFrameData,		// Read the next n bytes as frame data. GetAction() returns an IActionReadFrameData action.
				DecryptData,		// Decrypt the frame data. GetAction() returns an IActionDecryptData action.
				FrameDone,			// Finished this frame data, continue with the next frame. GetAction() returns an IActionFrameDone action. All values (ie timestamps) are valid only now.
				EndOfData,			// All cluster input consumed or skipped over. GetAction() returns nullptr.
				Failure				// An error occurred. GetAction() returns nullptr.
			};

			class IAction
			{
			public:
				virtual ~IAction() = default;
				virtual uint64 GetTrackID() const = 0;
				virtual FTimeValue GetPTS() const = 0;
				virtual FTimeValue GetDTS() const = 0;
				virtual FTimeValue GetDuration() const = 0;
				virtual bool IsKeyFrame() const = 0;
				virtual int64 GetTimestamp() const = 0;
				virtual int64 GetSegmentRelativePosition() const = 0;
				virtual int64 GetClusterPosition() const = 0;
			};

			class IActionSkipOver : public IAction
			{
			public:
				virtual int64 GetNumBytesToSkip() const = 0;
			};

			class IActionPrependData : public IAction
			{
			public:
				virtual const TArray<uint8>& GetPrependData() const = 0;
			};

			class IActionReadFrameData : public IAction
			{
			public:
				virtual int64 GetNumBytesToRead() const = 0;
			};

			class IActionDecryptData : public IAction
			{
			public:
				virtual void GetDecryptionInfo(ElectraCDM::FMediaCDMSampleInfo& OutSampleDecryptionInfo) const = 0;
			};

			class IActionFrameDone : public IAction
			{
			public:
				virtual const TMap<uint64, TArray<uint8>>& GetBlockAdditionalData() const = 0;
			};

			/**
			 * Performs parsing the cluster content or current frame, returning the next action to take.
			 *
			 * Returns the action to perform next.
			 */
			virtual EParseAction NextParseAction() = 0;

			/**
			 * Returns the error which resulted in returning the next action `Failure`
			 */
			virtual FErrorDetail GetLastError() const = 0;

			/**
			 * Returns the base class pointer of the next action to perform.
			 * Must be cast to the action indicated by the next action.
			 *
			 * The action is owned by the parser and must not be destroyed!
			 */
			virtual const IAction* GetAction() const = 0;

			/**
			 * Returns the start offset of the current cluster.
			 * This is needed for retries.
			 */
			virtual int64 GetClusterPosition() const = 0;

			/**
			 * Returns the offset of the current block (simple or group) in the cluster.
			 * This is needed for retries.
			 */
			virtual int64 GetClusterBlockPosition() const = 0;
		};


		class ITrack
		{
		public:
			virtual ~ITrack() = default;

			virtual uint64 GetID() const = 0;
			virtual FString GetName() const = 0;
			virtual const TArray<uint8>& GetCodecSpecificData() const = 0;
			virtual const FStreamCodecInformation& GetCodecInformation() const = 0;
			virtual const FString GetLanguage() const = 0;
			virtual const BCP47::FLanguageTag GetLanguageTag() const = 0;

			virtual ICueIterator* CreateCueIterator() const = 0;
		};


		virtual const ITrack* GetTrackByIndex(int32 Index) const = 0;
		virtual const ITrack* GetTrackByTrackID(uint64 TrackID) const = 0;

		enum EClusterParseFlags
		{
			ClusterParseFlag_Default = 0,
			// Allow the data to be a full EBML document instead of just a cluster.
			ClusterParseFlag_AllowFullDocument = 1 << 0,
		};

		/**
		 * Create a cluster parser.
		 * The data reader MUST start reading on a Matroska cluster.
		 */
		virtual TSharedPtrTS<IClusterParser> CreateClusterParser(IGenericDataReader* DataReader, const TArray<uint64>& TrackIDsToParse, EClusterParseFlags ParseFlags) const = 0;

		// Adds a cue of it does not exist yet. This may be called during cluster parsing for sync samples since not all
		// sync samples may have been added as cues in the multiplexing process.
		// NOTE: This should only be called for video samples.
		virtual void AddCue(int64 InCueTimestamp, uint64 InTrackID, int64 InCueRelativePosition, uint64 InCueBlockNumber, int64 InClusterPosition) = 0;
	};

} // namespace Electra
