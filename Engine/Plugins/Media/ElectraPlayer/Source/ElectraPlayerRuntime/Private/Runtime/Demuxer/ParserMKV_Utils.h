// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PlayerCore.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserMKV.h"
#include "Utilities/Utilities.h"

namespace Electra
{

	class FMKVStaticDataReader : public IGenericDataReader
	{
	public:
		FMKVStaticDataReader() = default;
		virtual ~FMKVStaticDataReader() = default;
		virtual void SetParseData(TSharedPtrTS<FWaitableBuffer> InResponseBuffer)
		{
			ResponseBuffer = InResponseBuffer;
			DataSize = ResponseBuffer->Num();
			Data = (const uint8*)ResponseBuffer->GetLinearReadData();
			CurrentOffset = 0;
		}
	private:
		int64 ReadData(void* InDestinationBuffer, int64 InNumBytesToRead, int64 InFromOffset) override
		{
			if (InFromOffset >= DataSize)
			{
				return 0;
			}
			InNumBytesToRead = Utils::Min(DataSize - InFromOffset, InNumBytesToRead);
			if (InNumBytesToRead <= 0)
			{
				return 0;
			}
			CurrentOffset = InFromOffset;
			if (InDestinationBuffer)
			{
				FMemory::Memcpy(InDestinationBuffer, Data+CurrentOffset, InNumBytesToRead);
			}
			return InNumBytesToRead;
		}
		int64 GetCurrentOffset() const override
		{ return CurrentOffset; }
		int64 GetTotalSize() const override
		{ return DataSize; }
		bool HasReadBeenAborted() const override
		{ return false; }
		bool HasReachedEOF() const override
		{ check(!"this should not be called"); return false; }

		TSharedPtrTS<FWaitableBuffer> ResponseBuffer;
		const uint8* Data = nullptr;
		int64 DataSize = 0;
		int64 CurrentOffset = 0;
	};



} // namespace Electra
