// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerFacility.h"
#include "Utilities/StringHelpers.h"

namespace Electra
{

class FErrorDetail
{
public:
	FErrorDetail()
	: Error(UEMEDIA_ERROR_OK), Component(Facility::EFacility::Unknown), Code(0)
	{
	}

	void Clear()
	{
		DetailMessage.Empty();
		PlatformMessage.Empty();
		Error     = UEMEDIA_ERROR_OK;
		Component = Facility::EFacility::Unknown;
		Code	  = 0;
	}

	bool IsTryAgain() const
	{
		return Error == UEMEDIA_ERROR_TRY_AGAIN;
	}

	bool IsSet() const
	{
		return Error != UEMEDIA_ERROR_OK || Code != 0 || Component != Facility::EFacility::Unknown || DetailMessage.Len() || PlatformMessage.Len();
	}

	bool IsOK() const
	{
		// OK means all was fine and the code can continue. This includes treating an abort as *not* being OK since the code must not continue.
		return !IsSet();
	}

	bool WasAborted() const
	{
		return GetError() == UEMEDIA_ERROR_ABORTED;
	}

	bool IsError() const
	{
		return IsSet() && !WasAborted();
	}

	FErrorDetail& SetTryAgain()
	{
		Error = UEMEDIA_ERROR_TRY_AGAIN;
		return *this;
	}

	FErrorDetail& SetError(UEMediaError InError)
	{
		Error = InError;
		return *this;
	}

	UEMediaError GetError() const
	{
		return Error;
	}

	FErrorDetail& SetFacility(Facility::EFacility InFacility)
	{
		Component = InFacility;
		return *this;
	}

	Facility::EFacility GetFacility() const
	{
		return Component;
	}

	FErrorDetail& SetCode(uint16 InCode)
	{
		Code = InCode;
		return *this;
	}

	uint16 GetCode() const
	{
		return Code;
	}

	FErrorDetail& SetMessage(const FString& InDetailMessage)
	{
		DetailMessage = InDetailMessage;
		return *this;
	}

	const FString& GetMessage() const
	{
		return DetailMessage;
	}

	FErrorDetail& SetPlatformMessage(const FString& InPlatformMessage)
	{
		PlatformMessage = InPlatformMessage;
		return *this;
	}

	const FString& GetPlatformMessage() const
	{
		return PlatformMessage;
	}

	FString GetPrintable() const
	{
		if (PlatformMessage.Len())
		{
			return FString::Printf(TEXT("error=%d in %s: code %u, \"%s\"; platform message \"%s\""), (int32)Error, Facility::GetName(Component), Code, *DetailMessage, *PlatformMessage);
		}
		else
		{
			return FString::Printf(TEXT("error=%d in %s: code %u, \"%s\""), (int32)Error, Facility::GetName(Component), Code, *DetailMessage);
		}
	}

private:
	FString				DetailMessage;
	FString				PlatformMessage;
	UEMediaError		Error;
	Facility::EFacility	Component;
	uint16				Code;
};

#define ELECTRA_IMPL_DEFAULT_ERROR_METHODS(ErrorFacility)																			\
static inline FErrorDetail CreateError(const FString& InMessage, uint16 InCode, UEMediaError InError=UEMEDIA_ERROR_FORMAT_ERROR)	\
{																																	\
	FErrorDetail Error;																												\
	Error.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);													\
	Error.SetFacility(Facility::EFacility::ErrorFacility);																			\
	Error.SetCode(InCode);																											\
	Error.SetMessage(InMessage);																									\
	return Error;																													\
}																																	\
static inline FErrorDetail CreateErrorAndLog(IPlayerSessionServices* InPlayerSessionServices, const FString& InMessage, uint16 InCode, UEMediaError InError=UEMEDIA_ERROR_FORMAT_ERROR)	\
{																																	\
	FErrorDetail Error = CreateError(InMessage, InCode, InError);																	\
	InPlayerSessionServices->PostLog(Facility::EFacility::ErrorFacility, IInfoLog::ELevel::Error, Error.GetPrintable());			\
	return Error;																													\
}																																	\
static inline void LogMessage(IPlayerSessionServices* InPlayerSessionServices, IInfoLog::ELevel Level, const FString& Message)		\
{																																	\
	InPlayerSessionServices->PostLog(Facility::EFacility::ErrorFacility, Level, Message);											\
}																																	\
																																	\
static inline FErrorDetail PostError(IPlayerSessionServices* InPlayerSessionServices, const FErrorDetail& Error)					\
{																																	\
	InPlayerSessionServices->PostLog(Facility::EFacility::ErrorFacility, IInfoLog::ELevel::Error, Error.GetPrintable());			\
	InPlayerSessionServices->PostError(Error);																						\
	return Error;																													\
}																																	\
static inline FErrorDetail PostError(IPlayerSessionServices* InPlayerSessionServices, const FString& Message, uint16 InCode)		\
{																																	\
	return PostError(InPlayerSessionServices, CreateError(Message, InCode));														\
}


#define ELECTRA_CLASS_DEFAULT_ERROR_METHODS(SessionService, ErrorFacility)															\
inline FErrorDetail CreateError(const FString& InMessage, uint16 InCode)															\
{																																	\
	FErrorDetail Error;																												\
	Error.SetError(UEMEDIA_ERROR_DETAIL);																							\
	Error.SetFacility(Facility::EFacility::ErrorFacility);																			\
	Error.SetCode(InCode);																											\
	Error.SetMessage(InMessage);																									\
	return Error;																													\
}																																	\
inline FErrorDetail CreateErrorAndLog(const FString& InMessage, uint16 InCode)														\
{																																	\
	FErrorDetail Error = CreateError(InMessage, InCode);																			\
	SessionService->PostLog(Facility::EFacility::ErrorFacility, IInfoLog::ELevel::Error, Error.GetPrintable());						\
	return Error;																													\
}																																	\
inline void LogMessage(IInfoLog::ELevel InLevel, const FString& InMessage)															\
{																																	\
	SessionService->PostLog(Facility::EFacility::ErrorFacility, InLevel, InMessage);												\
}																																	\
inline FErrorDetail PostError(const FErrorDetail& InError)																			\
{																																	\
	SessionService->PostError(InError);																								\
	return InError;																													\
}																																	\
inline FErrorDetail PostError(const FString& InMessage, uint16 InCode)																\
{																																	\
	return PostError(CreateError(InMessage, InCode));																				\
}


} // namespace Electra


