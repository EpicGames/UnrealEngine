// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTTP/HTTPManager.h"
#include "HTTP/HTTPResponseCache.h"
#include "Utilities/StringHelpers.h"
#include "Utilities/URLParser.h"
#include "PlayerTime.h"
#include "Player/PlayerSessionServices.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMisc.h"

// For base64 encoding/decoding
#include "Misc/Base64.h"

// Asynchronous termination
#include "Async/Async.h"

// For file:// scheme archive deserialization
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Serialization/Archive.h"
#include "Serialization/ArrayReader.h"

#include "ElectraPlayerPrivate.h"
#include "ElectraHTTPStream.h"


DECLARE_CYCLE_STAT(TEXT("FElectraHttpManager::WorkerThread"), STAT_ElectraPlayer_FElectraHttpManager_Worker, STATGROUP_ElectraPlayer);

DEFINE_LOG_CATEGORY(LogElectraHTTPManager);

namespace Electra
{

	class FElectraHttpManager : public IElectraHttpManager, public TSharedFromThis<FElectraHttpManager, ESPMode::ThreadSafe>
	{
	public:
		static TSharedPtrTS<FElectraHttpManager> Create();

		virtual ~FElectraHttpManager();
		void Initialize();

		virtual void AddRequest(TSharedPtrTS<FRequest> Request, bool bAutoRemoveWhenComplete) override;
		virtual void RemoveRequest(TSharedPtrTS<FRequest> Request, bool bDoNotWaitForRemoval) override;

	private:
		class TDeleter
		{
		public:
			void operator()(FElectraHttpManager* InInstanceToDelete)
			{
				TFunction<void()> DeleteTask = [InInstanceToDelete]()
				{
					delete InInstanceToDelete;
				};
				FMediaRunnable::EnqueueAsyncTask(MoveTemp(DeleteTask));
			}
		};

		struct FLocalByteStream
		{
			virtual ~FLocalByteStream() = default;
			virtual void SetConnected(TSharedPtrTS<FRequest> Request) = 0;
			virtual int32 Read(TSharedPtrTS<FWaitableBuffer> RcvBuffer, TSharedPtrTS<FRequest> Request) = 0;

			bool										bIsConnected = false;
			int64										FileStartOffset = 0;		//!< The base offset into the file data is requested at.
			int64										FileSize = 0;				//!< The size of the file requested
			int64										FileSizeToGo = 0;			//!< Number of bytes still to read from the file.
		};

		struct FFileStream : public FLocalByteStream
		{
			virtual ~FFileStream() = default;
			void SetConnected(TSharedPtrTS<FRequest> Request) override;
			int32 Read(TSharedPtrTS<FWaitableBuffer> RcvBuffer, TSharedPtrTS<FRequest> Request) override;
			TSharedPtr<FArchive, ESPMode::ThreadSafe>	Archive;
			FString										Filename;
		};

		struct FDataUrl : public FLocalByteStream
		{
			virtual ~FDataUrl() = default;
			bool SetData(const FString& InUrl);
			void SetConnected(TSharedPtrTS<FRequest> Request) override;
			int32 Read(TSharedPtrTS<FWaitableBuffer> RcvBuffer, TSharedPtrTS<FRequest> Request) override;
			TArray<uint8>								Data;
			FString										MimeType;
		};

		class FHTTPCallbackWrapper
		{
		public:
			FHTTPCallbackWrapper(TSharedPtrTS<FElectraHttpManager> InOwner) : Owner(InOwner) {}
			~FHTTPCallbackWrapper()
			{
				Unbind();
			}

			void ReportRequestNotification(IElectraHTTPStreamRequestPtr InRequest, EElectraHTTPStreamNotificationReason InReason, int64 InParam)
			{
				FScopeLock lock(&Lock);
				TSharedPtrTS<FElectraHttpManager> That = Owner.Pin();
				if (That.IsValid())
				{
					That->OnRequestNotification(InRequest, InReason, InParam);
				}
			}

			void Unbind()
			{
				FScopeLock lock(&Lock);
				Owner.Reset();
			}
		private:
			FCriticalSection Lock;
			TWeakPtrTS<FElectraHttpManager> Owner;
		};

		struct FHandle
		{
			~FHandle()
			{
				Cleanup();
			}

			void Cleanup()
			{
				if (HandleType == EHandleType::LocalHandle)
				{
					LocalByteStream.Reset();
				}
				else if (HandleType == EHandleType::ExternalHandle)
				{
					LocalByteStream.Reset();
				}
				else if (HandleType == EHandleType::HTTPHandle)
				{
					if (HttpsRequestCallbackWrapper.IsValid())
					{
						HttpsRequestCallbackWrapper->Unbind();
					}
					HttpsRequestCallbackWrapper.Reset();

					if (HttpRequest.IsValid() && !ActiveResponse.bHitCache && HttpRequest->GetResponse()->GetStatus() != IElectraHTTPStreamResponse::EStatus::Completed)
					{
						HttpRequest->Cancel();
					}
					HttpRequest.Reset();
				}
			}

			bool NeedsFetchOfUncachedParts()
			{
				check(HandleType == EHandleType::HTTPHandle);

				// Check if this request can be satisfied from the response cache.
				if (HttpResponseCache.IsValid())
				{
					TSharedPtrTS<IHTTPResponseCache::FCacheItem> CacheItem;
					IHTTPResponseCache::EScatterResult Result;
					Result = HttpResponseCache->GetScatteredCacheEntity(CacheItem, ActiveResponse.URL, ActiveResponse.Range, ActiveResponse.Quality);
					if (Result == IHTTPResponseCache::EScatterResult::FullHit)
					{
						check(CacheItem.IsValid());
						if (CacheItem.IsValid())
						{
							ActiveResponse.CacheResponse = MoveTemp(CacheItem);
							// Have all data, no need to fetch anything.
							return false;
						}
					}
					else if (Result == IHTTPResponseCache::EScatterResult::PartialHit)
					{
						check(CacheItem.IsValid());
						if (CacheItem.IsValid())
						{
							// Partial cached responses require the missing partial data to be fetched via sub range requests.
							ActiveResponse.bIsSubRangeRequest = true;

							// If there is no cached response it means that the first bytes are not in the cache but later
							// data is. The first part must be requested. Otherwise we can just return the cached response.
							if (CacheItem->Response.IsValid())
							{
								ActiveResponse.CacheResponse = MoveTemp(CacheItem);
								ActiveResponse.ReceivedContentRange = ActiveResponse.CacheResponse->Range;
								// Don't need to fetch the first part.
								return false;
							}
							else
							{
								// The missing range we need to fetch has been set up by the cache so we just need to use
								// that range in the request.
								ActiveResponse.Range = CacheItem->Range;
								HttpRequest->SetRange(ActiveResponse.Range.GetString());
							}
						}
					}
				}
				// Fetch needed.
				return true;
			}

			struct FRequestResponse
			{
				// URL
				FString URL;
				// Current range requested from server.
				ElectraHTTPStream::FHttpRange Range;
				// Current response received from server
				IElectraHTTPStreamResponsePtr Response;
				// Number of bytes passed along already.
				int64 NumBytesPassedOut = 0;
				// Original range requested for easier access and comparison.
				ElectraHTTPStream::FHttpRange OriginalRange;
				ElectraHTTPStream::FHttpRange ReceivedContentRange;
				//
				IHTTPResponseCache::FQualityInfo Quality;
				//
				bool bIsSubRangeRequest = false;
				int32 NumSubRangeRequest = 0;
				//
				TSharedPtrTS<IHTTPResponseCache::FCacheItem> CacheResponse;
				bool bHitCache = false;
				bool bWasAddedToCache = false;

				//
				int64 SizeRemaining() const
				{
					// Did we break this into sub range requests?
					if (bIsSubRangeRequest)
					{
						// The expected end position is either the end of the original requested range or the end of the document.
						int64 ExpectedEndPos = OriginalRange.GetEndIncluding() >=0 ? OriginalRange.GetEndIncluding()+1 : ReceivedContentRange.GetDocumentSize();
						// It should not be negative. The total document size should be available by now.
						// If it is not we may be faced with a chunked transfer of a document with unknown/infinite size, which is bad.
						check(ExpectedEndPos >= 0);
						if (ExpectedEndPos >= 0)
						{
							check(ReceivedContentRange.IsSet());
							int64 SizeToGo = ExpectedEndPos - (ReceivedContentRange.GetEndIncluding() + 1);
							check(SizeToGo >= 0);
							return SizeToGo;
						}
						else
						{
							UE_LOGF(LogElectraHTTPManager, Error, "Unknown document size in sub ranged download");
						}
					}
					return 0;
				}

				void ClearForNextSubRange()
				{
					CacheResponse.Reset();
					bHitCache = false;
					bWasAddedToCache = false;
				}

			};

			enum class EHandleType
			{
				Undefined,
				LocalHandle,
				ExternalHandle,
				HTTPHandle
			};

			FElectraHttpManager* 	Owner = nullptr;

			EHandleType				HandleType = EHandleType::Undefined;

			// Local file handle (for file:// and data:)
			TSharedPtrTS<FLocalByteStream> LocalByteStream;

			// HTTP handle
			IElectraHTTPStreamRequestPtr							HttpRequest;
			TSharedPtr<FHTTPCallbackWrapper, ESPMode::ThreadSafe>	HttpsRequestCallbackWrapper;
			bool													bHttpRequestFirstEvent = true;
			TSharedPtrTS<IHTTPResponseCache>						HttpResponseCache;

			FTimeValue				LastTimeDataReceived;
			FTimeValue				TimeAtNextProgressCallback;
			FTimeValue				TimeAtConnectionTimeoutCheck;

			FRequestResponse		ActiveResponse;
			bool					bResponseReceived = false;

			int64					BytesReadSoFar = 0;

			// Internal work variables mirroring HTTP::FConnectionInfo values that may change with sub range requests and
			// should not change for the original request.
			FTimeValue				RequestStartTime;
			bool					bIsConnected = false;
			bool					bHaveResponseHeaders = false;


			void ClearForNextSubRange()
			{
				LastTimeDataReceived.SetToInvalid();
				TimeAtConnectionTimeoutCheck.SetToInvalid();
				RequestStartTime.SetToInvalid();
				bHttpRequestFirstEvent = true;
				bResponseReceived = false;
				bIsConnected = false;
				bHaveResponseHeaders = false;
				ActiveResponse.ClearForNextSubRange();
			}
		};

		struct FTransportError
		{
			FTransportError()
			{
				Clear();
			}
			void Clear()
			{
				Message.Empty();
				ErrorCode = HTTP::EStatusErrorCode::ERRCODE_OK;
			}
			void Set(int32 InCode, const char* InMessage)
			{
				ErrorCode = InCode;
				Message = FString(InMessage);
			}
			void Set(int32 InCode, const FString& InMessage)
			{
				ErrorCode = InCode;
				Message = InMessage;
			}

			FString			Message;
			int32			ErrorCode;
		};

		FHandle* CreateLocalFileHandle(const FTimeValue& Now, FTransportError& OutError, const TSharedPtrTS<FRequest>& Request);
		FHandle* CreateHTTPHandle(const FTimeValue& Now, FTransportError& OutError, const TSharedPtrTS<FRequest>& Request);
		bool PrepareHTTPHandle(const FTimeValue& Now, FHandle* InHandle, const TSharedPtrTS<FRequest>& Request, bool bIsFirstSetup);
		void AddPendingRequests(const FTimeValue& Now);
		void RemovePendingRequests(const FTimeValue& Now);
		void HandleCompletedRequests();
		void HandlePeriodicCallbacks(const FTimeValue& Now);
		void HandleTimeouts(const FTimeValue& Now);
		void HandleLocalFileRequests();
		void HandleExternalDataRequests();
		void HandleCachedHTTPRequests(const FTimeValue& Now);
		void HandleHTTPRequests(const FTimeValue& Now);
		void HandleHTTPResponses(const FTimeValue& Now);
		void RemoveAllRequests();

		bool StartHTTPManager();
		void StopHTTPManager();
		void ProcessHTTPManager();

	public:
		void OnRequestNotification(IElectraHTTPStreamRequestPtr InRequest, EElectraHTTPStreamNotificationReason InReason, int64 InParam);

	private:
		struct FRemoveRequest
		{
			void SignalDone()
			{
				if (WaitingEvent.IsValid())
				{
					WaitingEvent->Signal();
					WaitingEvent.Reset();
				}
			}
			TSharedPtrTS<FRequest> Request;
			TSharedPtrTS<FMediaEvent> WaitingEvent;
		};

		FCriticalSection												Lock;

		FMediaEvent														RequestChangesEvent;
		TQueue<TSharedPtrTS<FRequest>, EQueueMode::Mpsc>				RequestsToAdd;
		TQueue<FRemoveRequest, EQueueMode::Mpsc>						RequestsToRemove;
		TQueue<TSharedPtrTS<FRequest>, EQueueMode::Mpsc>				RequestsCompleted;

		TSharedPtr<IElectraHTTPStream, ESPMode::ThreadSafe>				HTTPStreamHandler;
		FCriticalSection												HTTPStreamEventLock;
		TArray<IElectraHTTPStreamRequestPtr>							NotifiedRequests;

		TMap<FHandle*, TSharedPtrTS<FRequest>>							ActiveRequests;
		FTimeValue														ProgressInterval;

		bool															bManagerStarted = false;
		bool															bTerminate = false;

		static TWeakPtrTS<FElectraHttpManager>							SingletonSelf;
		static FCriticalSection											SingletonLock;
	};

	TWeakPtrTS<FElectraHttpManager>		FElectraHttpManager::SingletonSelf;
	FCriticalSection					FElectraHttpManager::SingletonLock;

	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/

	FString IElectraHttpManager::GetDefaultUserAgent()
	{
		return FString(TEXT("X-UnrealEngine-Agent"));
	}


	TSharedPtrTS<IElectraHttpManager> IElectraHttpManager::Create()
	{
		return FElectraHttpManager::Create();
	}

	TSharedPtrTS<FElectraHttpManager> FElectraHttpManager::Create()
	{
		FScopeLock lock(&SingletonLock);
		TSharedPtrTS<FElectraHttpManager> Self = SingletonSelf.Pin();
		if (!Self.IsValid())
		{
			FElectraHttpManager* Manager = new FElectraHttpManager;
			if (Manager)
			{
				// A custom deleter is required to ensure that the destruction will not happen within the enclosing worker thread
				// due to the shared pointer to ourselves passed as a callback delegate in StartHTTPManager() being released.
				// If that happens we encounter a deadlock in the worker thread trying to delete itself.
				// To avoid this the custom deleter moves the final destruction out as a task to the thread pool.
				Self = MakeShareable(Manager, TDeleter());
				Manager->Initialize();
			}
			SingletonSelf = Self;
		}
		return Self;
	}


	FElectraHttpManager::~FElectraHttpManager()
	{
		bTerminate = true;
		if (bManagerStarted)
		{
			StopHTTPManager();
		}
	}

	void FElectraHttpManager::Initialize()
	{
		ProgressInterval.SetFromMilliseconds(100);
		bManagerStarted = StartHTTPManager();
	}

	void FElectraHttpManager::AddRequest(TSharedPtrTS<FRequest> Request, bool bAutoRemoveWhenComplete)
	{
		FScopeLock lock(&Lock);
		if (!HTTPStreamHandler.IsValid())
		{
			Request->ConnectionInfo.EffectiveURL = Request->Parameters.URL;
			Request->ConnectionInfo.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTPMODULE_FAILURE;
			Request->ConnectionInfo.StatusInfo.ErrorDetail.
				SetError(UEMEDIA_ERROR_NOT_SUPPORTED).
				SetFacility(Facility::EFacility::HTTPReader).
				SetCode(HTTP::EStatusErrorCode::ERRCODE_HTTPMODULE_FAILURE).
				SetMessage(FString::Printf(TEXT("FElectraHttpManager did not initialize")));
			Request->ConnectionInfo.RequestEndTime = MEDIAutcTime::Current();
			Request->ConnectionInfo.bHasFinished = true;
			TSharedPtrTS<FWaitableBuffer> ReceiveBuffer = Request->ReceiveBuffer.Pin();
			if (ReceiveBuffer.IsValid())
			{
				ReceiveBuffer->SetEOD();
			}
			TSharedPtrTS<FProgressListener> ProgressListener = Request->ProgressListener.Pin();
			if (ProgressListener.IsValid())
			{
				if (ProgressListener->CompletionDelegate.IsBound())
				{
					ProgressListener->CallCompletionDelegate(Request.Get());
				}
			}
			return;
		}
		if (!bTerminate)
		{
			// Not currently supported. Reserved for future use.
			check(bAutoRemoveWhenComplete == false);
			//Request->bAutoRemoveWhenComplete = bAutoRemoveWhenComplete;
			RequestsToAdd.Enqueue(Request);
			RequestChangesEvent.Signal();
			HTTPStreamHandler->TriggerWorkSignal();
		}
	}

	void FElectraHttpManager::RemoveRequest(TSharedPtrTS<FRequest> Request, bool bDoNotWaitForRemoval)
	{
		if (!HTTPStreamHandler.IsValid())
		{
			return;
		}
		if (bDoNotWaitForRemoval)
		{
			Request->ReceiveBuffer.Reset();
			Request->ProgressListener.Reset();
			FRemoveRequest Remove;
			Remove.Request = Request;
			Lock.Lock();
			RequestsToRemove.Enqueue(MoveTemp(Remove));
			RequestChangesEvent.Signal();
			Lock.Unlock();
			HTTPStreamHandler->TriggerWorkSignal();
		}
		else
		{
			TSharedPtrTS<FMediaEvent> WaitingEvent = MakeSharedTS<FMediaEvent>();
			FRemoveRequest Remove;
			Remove.Request = Request;
			Remove.WaitingEvent = WaitingEvent;
			Lock.Lock();
			RequestsToRemove.Enqueue(MoveTemp(Remove));
			RequestChangesEvent.Signal();
			Lock.Unlock();
			HTTPStreamHandler->TriggerWorkSignal();
			WaitingEvent->Wait();
		}
	}

	FElectraHttpManager::FHandle* FElectraHttpManager::CreateLocalFileHandle(const FTimeValue& Now, FTransportError& OutError, const TSharedPtrTS<IElectraHttpManager::FRequest>& Request)
	{
		TUniquePtr<FHandle> Handle(new FHandle);
		Handle->Owner = this;
		Handle->HandleType = FHandle::EHandleType::LocalHandle;
		if (Request->Parameters.URL.Left(5).Equals(TEXT("data:")))
		{
			FDataUrl* DataUrl = new FDataUrl;
			Handle->LocalByteStream = MakeShareable<FLocalByteStream>(DataUrl);
			if (!DataUrl->SetData(Request->Parameters.URL))
			{
				OutError.Set(HTTP::EStatusErrorCode::ERRCODE_HTTP_FILE_COULDNT_READ_FILE, FString::Printf(TEXT("Failed to use data URL \"%s\""), *Request->Parameters.URL));
				return nullptr;
			}
		}
		else
		{
			FString Filename;
			// Unescape percent encoded characters in the URI, such as %20 representing a space.
			if (FURL_RFC3986::UrlDecode(Filename, Request->Parameters.URL))
			{
				FFileStream* FileStream = new FFileStream;
				Handle->LocalByteStream = MakeShareable<FLocalByteStream>(FileStream);
				FileStream->Filename = Filename.Mid(7); /* file:// */
				FileStream->Archive = MakeShareable(IFileManager::Get().CreateFileReader(*FileStream->Filename));
			}
			else
			{
				OutError.Set(HTTP::EStatusErrorCode::ERRCODE_HTTP_FILE_COULDNT_READ_FILE, FString::Printf(TEXT("Failed to parse file name \"%s\""), *Request->Parameters.URL));
				return nullptr;
			}
		}
		return Handle.Release();
	}

	FElectraHttpManager::FHandle* FElectraHttpManager::CreateHTTPHandle(const FTimeValue& Now, FTransportError& OutError, const TSharedPtrTS<IElectraHttpManager::FRequest>& Request)
	{
		TUniquePtr<FHandle> Handle(new FHandle);
		Handle->Owner = this;
		Handle->HandleType = FHandle::EHandleType::HTTPHandle;
		Handle->bHttpRequestFirstEvent = true;
		Handle->ActiveResponse.NumSubRangeRequest = 0;
		Handle->ActiveResponse.OriginalRange = Request->Parameters.Range;
		Handle->ActiveResponse.Quality.StreamType = Request->Parameters.StreamType;
		Handle->ActiveResponse.Quality.QualityIndex = Request->Parameters.QualityIndex;
		Handle->ActiveResponse.Quality.MaxQualityIndex = Request->Parameters.MaxQualityIndex;

		if (PrepareHTTPHandle(Now, Handle.Get(), Request, true))
		{
			return Handle.Release();
		}
		else
		{
			return nullptr;
		}
	}


	bool FElectraHttpManager::PrepareHTTPHandle(const FTimeValue& Now, FHandle* Handle, const TSharedPtrTS<FRequest>& Request, bool bIsFirstSetup)
	{
		Handle->HttpsRequestCallbackWrapper = MakeShared<FHTTPCallbackWrapper, ESPMode::ThreadSafe>(AsShared());
		Handle->ActiveResponse.URL = Request->Parameters.URL;
		Handle->RequestStartTime = Now;
		Handle->TimeAtConnectionTimeoutCheck.SetToPositiveInfinity();
		if (Request->Parameters.ConnectTimeout.IsValid())
		{
			Handle->TimeAtConnectionTimeoutCheck = Now + Request->Parameters.ConnectTimeout;
		}

		// Is this the first sub range request or a continuation?
		if (bIsFirstSetup)
		{
			Request->ConnectionInfo.RequestStartTime = Now;
			Handle->TimeAtNextProgressCallback = Now + ProgressInterval;
			Handle->ActiveResponse.Range = Handle->ActiveResponse.OriginalRange;
			Handle->ActiveResponse.NumSubRangeRequest = 0;
			Handle->BytesReadSoFar = 0;
		}
		else
		{
			// Set up the range to follow the previous.
			Handle->ActiveResponse.Range.SetStart(Handle->ActiveResponse.ReceivedContentRange.GetEndIncluding() + 1);
			if (Handle->ActiveResponse.OriginalRange.GetEndIncluding() >= 0)
			{
				Handle->ActiveResponse.Range.SetEndIncluding(Handle->ActiveResponse.OriginalRange.GetEndIncluding());
			}
			else if (Handle->ActiveResponse.ReceivedContentRange.GetDocumentSize() > 0)
			{
				Handle->ActiveResponse.Range.SetEndIncluding(Handle->ActiveResponse.ReceivedContentRange.GetDocumentSize() - 1);
			}
			else
			{
				Handle->ActiveResponse.Range.SetEndIncluding(-1);
			}
			++Handle->ActiveResponse.NumSubRangeRequest;
			Handle->BytesReadSoFar += Handle->ActiveResponse.NumBytesPassedOut;
			Handle->ClearForNextSubRange();
		}
		Handle->ActiveResponse.NumBytesPassedOut = 0;
		Handle->ActiveResponse.ReceivedContentRange.Reset();

		Handle->ActiveResponse.Response.Reset();

		// Response not yet received.
		Handle->bResponseReceived = false;

		// This could be for the next sub-range request that we also want to add to the cache!
		Handle->ActiveResponse.bWasAddedToCache = false;

		if (HTTPStreamHandler.IsValid())
		{
			Handle->HttpRequest = HTTPStreamHandler->CreateRequest();
			if (!Handle->HttpRequest.IsValid())
			{
				return false;
			}
			Handle->HttpRequest->SetVerb(TEXT("GET"));
			Handle->HttpResponseCache = Request->ResponseCache;

			if (!Request->Parameters.Verb.IsEmpty())
			{
				Handle->HttpRequest->SetVerb(Request->Parameters.Verb);
				// Disable the cache for anything but GET
				if (!Request->Parameters.Verb.Equals(TEXT("GET")))
				{
					Handle->HttpResponseCache.Reset();
				}

				// Add POST data
				if (Request->Parameters.Verb.Equals(TEXT("POST")))
				{
					IElectraHTTPStreamBuffer& pdb = Handle->HttpRequest->POSTDataBuffer();
					pdb.AddData(MoveTemp(Request->Parameters.PostData));
					pdb.SetEOS();
				}
			}

			if (Request->Parameters.bCollectTimingTraces)
			{
				Handle->HttpRequest->EnableTimingTraces();
			}
			Handle->HttpRequest->SetURL(Request->Parameters.URL);
			if (Request->Parameters.UserAgent.IsSet())
			{
				Handle->HttpRequest->SetUserAgent(Request->Parameters.UserAgent.Value());
			}
			else
			{
				Handle->HttpRequest->SetUserAgent(GetDefaultUserAgent());
			}
			Handle->HttpRequest->AllowCompression(!Request->Parameters.AcceptEncoding.GetWithDefault(TEXT("")).Equals(TEXT("identity")));
			if (Handle->ActiveResponse.Range.IsSet())
			{
				Handle->HttpRequest->SetRange(Handle->ActiveResponse.Range.GetString());
			}

			for(int32 i=0; i<Request->Parameters.RequestHeaders.Num(); ++i)
			{
				Handle->HttpRequest->AddHeader(Request->Parameters.RequestHeaders[i].Header, Request->Parameters.RequestHeaders[i].Value, false);
			}

			Handle->HttpRequest->NotificationDelegate().BindThreadSafeSP(Handle->HttpsRequestCallbackWrapper.ToSharedRef(), &FHTTPCallbackWrapper::ReportRequestNotification);
			return true;
		}
		else
		{
			UE_LOGF(LogElectraHTTPManager, Error, "FElectraHttpManager is not available, cannot create request.");
			return false;
		}
	}


	void FElectraHttpManager::AddPendingRequests(const FTimeValue& Now)
	{
		while(!RequestsToAdd.IsEmpty())
		{
			TSharedPtrTS<FRequest> Request;
			RequestsToAdd.Dequeue(Request);

			FTransportError HttpError;
			Request->ConnectionInfo.EffectiveURL = Request->Parameters.URL;

			// Is this a local file or a data url?
			if ((Request->Parameters.URL.Len() > 7 && Request->Parameters.URL.Left(7) == TEXT("file://")) ||
				(Request->Parameters.URL.Len() > 5 && Request->Parameters.URL.Left(5) == TEXT("data:")))
			{
				FHandle* Handle = CreateLocalFileHandle(Now, HttpError, Request);
				if (Handle)
				{
					ActiveRequests.Add(Handle, Request);

					Request->ConnectionInfo.RequestStartTime = Now;
					Handle->RequestStartTime = Now;
					Handle->TimeAtNextProgressCallback = Now + ProgressInterval;
					Handle->TimeAtConnectionTimeoutCheck.SetToPositiveInfinity();
				}
				else
				{
					Request->ConnectionInfo.StatusInfo.ErrorDetail.SetError(UEMEDIA_ERROR_INTERNAL).SetFacility(Facility::EFacility::HTTPReader).SetMessage(HttpError.Message).SetCode((uint16)HttpError.ErrorCode);
					RequestsCompleted.Enqueue(Request);
				}
			}
			// Finally it has to be a http request.
			else
			{
				FHandle* Handle = CreateHTTPHandle(Now, HttpError, Request);
				if (Handle)
				{
					ActiveRequests.Add(Handle, Request);
					if (Handle->NeedsFetchOfUncachedParts())
					{
						if (HTTPStreamHandler.IsValid())
						{
							HTTPStreamHandler->AddRequest(Handle->HttpRequest);
						}
						else
						{
							Request->ConnectionInfo.StatusInfo.ErrorDetail.SetError(UEMEDIA_ERROR_INTERNAL).SetFacility(Facility::EFacility::HTTPReader).SetMessage(TEXT("HTTP request failed on ProcessRequest()"));
							RequestsCompleted.Enqueue(Request);
						}
					}
				}
				else
				{
					Request->ConnectionInfo.StatusInfo.ErrorDetail.SetError(UEMEDIA_ERROR_INTERNAL).SetFacility(Facility::EFacility::HTTPReader).SetMessage(HttpError.Message).SetCode((uint16)HttpError.ErrorCode);
					RequestsCompleted.Enqueue(Request);
				}
			}
		}
	}

	void FElectraHttpManager::RemovePendingRequests(const FTimeValue& Now)
	{
		// Remove pending requests
		while(!RequestsToRemove.IsEmpty())
		{
			FRemoveRequest Next;
			if (RequestsToRemove.Dequeue(Next))
			{
				TSharedPtrTS<FRequest>	Request = Next.Request;

				// Is this an active request?
				for(TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
				{
					if (It.Value() == Request)
					{
						FHandle* Handle = It.Key();
						It.RemoveCurrent();
						delete Handle;
						break;
					}
				}

				// Removing an unfinished transfer that has not errored means it was aborted.
				if (!Request->ConnectionInfo.bHasFinished && !Request->ConnectionInfo.StatusInfo.ErrorDetail.IsError())
				{
					Request->ConnectionInfo.bWasAborted = true;
				}
				if (!Request->ConnectionInfo.RequestEndTime.IsValid())
				{
					Request->ConnectionInfo.RequestEndTime = Now;
				}
				Next.SignalDone();
			}
		}
	}

	void FElectraHttpManager::HandleCompletedRequests()
	{
		// Remove pending requests
		while(!RequestsCompleted.IsEmpty())
		{
			TSharedPtrTS<FRequest> Request;
			RequestsCompleted.Dequeue(Request);
			// Remove from active requests. It may not be in there if it had an error upon creating.
			for(TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
			{
				if (It.Value() == Request)
				{
					FHandle* Handle = It.Key();
					It.RemoveCurrent();
					delete Handle;
					break;
				}
			}

			Request->ConnectionInfo.bHasFinished = true;
			TSharedPtrTS<FWaitableBuffer>	ReceiveBuffer = Request->ReceiveBuffer.Pin();
			if (ReceiveBuffer.IsValid())
			{
				if (Request->ConnectionInfo.StatusInfo.ErrorCode)
				{
					ReceiveBuffer->SetHasErrored();
				}
				ReceiveBuffer->SetEOD();
			}

			// Call completion delegate.
			TSharedPtrTS<FProgressListener> ProgressListener = Request->ProgressListener.Pin();
			if (ProgressListener.IsValid())
			{
				if (ProgressListener->CompletionDelegate.IsBound())
				{
					ProgressListener->CallCompletionDelegate(Request.Get());
				}
			}
		}
	}


	void FElectraHttpManager::HandlePeriodicCallbacks(const FTimeValue& Now)
	{
		for(TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
		{
			FHandle* Handle = It.Key();
			// Fire periodic progress callback?
			if (Now >= Handle->TimeAtNextProgressCallback)
			{
				Handle->TimeAtNextProgressCallback += ProgressInterval - (Now - Handle->TimeAtNextProgressCallback);
				TSharedPtrTS<FProgressListener> ProgressListener = It.Value()->ProgressListener.Pin();
				if (ProgressListener.IsValid())
				{
					if (ProgressListener->ProgressDelegate.IsBound())
					{
						int32 Result = ProgressListener->CallProgressDelegate(It.Value().Get());
						// Did the progress callback ask to abort the download?
						if (Result)
						{
							It.Value()->ConnectionInfo.bWasAborted = true;
							It.Value()->ConnectionInfo.RequestEndTime = MEDIAutcTime::Current();
							if (Handle->HttpsRequestCallbackWrapper.IsValid())
							{
								Handle->HttpsRequestCallbackWrapper->Unbind();
							}
							RequestsCompleted.Enqueue(It.Value());
						}
					}
				}
			}
		}
	}


	void FElectraHttpManager::HandleTimeouts(const FTimeValue& Now)
	{
		for(TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
		{
			FHandle* Handle = It.Key();

			// Timeout handling for file handles is not desired. Skip to the next handle.
			if (Handle->HandleType == FHandle::EHandleType::LocalHandle)
			{
				continue;
			}
			// HTTP transfers that are cached or not processing also do not need to be checked.
			else if (Handle->HandleType == FHandle::EHandleType::HTTPHandle && (Handle->ActiveResponse.CacheResponse.IsValid() || (Handle->HttpRequest.IsValid() && Handle->HttpRequest->GetResponse()->GetState() == IElectraHTTPStreamResponse::EState::Finished)))
			{
				continue;
			}

			TSharedPtrTS<FRequest> Request = It.Value();

			// Time to check for a connection timeout?
			if (Now >= Handle->TimeAtConnectionTimeoutCheck)
			{
				// For our purposes we timeout when we are not connected / have not received any response header.
				if (!Handle->bIsConnected || !Handle->bHaveResponseHeaders)
				{
					Request->ConnectionInfo.StatusInfo.ConnectionTimeoutAfterMilliseconds = (Now - Handle->RequestStartTime).GetAsMilliseconds();
					Request->ConnectionInfo.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTP_CONNECTION_TIMEOUT;
					Request->ConnectionInfo.StatusInfo.ErrorDetail.
						SetError(UEMEDIA_ERROR_READ_ERROR).
						SetFacility(Facility::EFacility::HTTPReader).
						SetCode(HTTP::EStatusErrorCode::ERRCODE_HTTP_CONNECTION_TIMEOUT).
						SetMessage(FString::Printf(TEXT("Connection timeout after %d milliseconds, limit was %d"), Request->ConnectionInfo.StatusInfo.ConnectionTimeoutAfterMilliseconds, (int32)Request->Parameters.ConnectTimeout.GetAsMilliseconds()));
					Request->ConnectionInfo.RequestEndTime = Now;
					if (Handle->HttpsRequestCallbackWrapper.IsValid())
					{
						Handle->HttpsRequestCallbackWrapper->Unbind();
					}
					RequestsCompleted.Enqueue(Request);
				}
				Handle->TimeAtConnectionTimeoutCheck.SetToPositiveInfinity();
			}

			// Data timeout? This requires to be connected to the server and to have received at least one response header.
			if (Handle->LastTimeDataReceived.IsValid() && Request->Parameters.NoDataTimeout.IsValid())
			{
				FTimeValue DeltaTime = Now - Handle->LastTimeDataReceived;
				if (DeltaTime >= Request->Parameters.NoDataTimeout)
				{
					Request->ConnectionInfo.StatusInfo.NoDataTimeoutAfterMilliseconds = DeltaTime.GetAsMilliseconds();
					Request->ConnectionInfo.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTP_CONNECTION_TIMEOUT;
					Request->ConnectionInfo.StatusInfo.ErrorDetail.
						SetError(UEMEDIA_ERROR_READ_ERROR).
						SetFacility(Facility::EFacility::HTTPReader).
						SetCode(HTTP::EStatusErrorCode::ERRCODE_HTTP_CONNECTION_TIMEOUT).
						SetMessage(FString::Printf(TEXT("No data timeout after %d milliseconds, limit was %d. Received %lld of %lld bytes"), Request->ConnectionInfo.StatusInfo.NoDataTimeoutAfterMilliseconds, (int32)Request->Parameters.NoDataTimeout.GetAsMilliseconds()
							, (long long int)Request->ConnectionInfo.BytesReadSoFar, (long long int)Request->ConnectionInfo.ContentLength));
					Request->ConnectionInfo.RequestEndTime = Now;
					if (Handle->HttpsRequestCallbackWrapper.IsValid())
					{
						Handle->HttpsRequestCallbackWrapper->Unbind();
					}
					RequestsCompleted.Enqueue(Request);
				}
			}
		}
	}


	void FElectraHttpManager::RemoveAllRequests()
	{
		RequestsToAdd.Empty();
		while(!RequestsToRemove.IsEmpty())
		{
			FRemoveRequest Next;
			if (RequestsToRemove.Dequeue(Next))
			{
				Next.SignalDone();
			}
		}
		RequestsCompleted.Empty();
		while(ActiveRequests.Num() != 0)
		{
			TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator();
			FHandle* Handle = It.Key();
			It.RemoveCurrent();
			delete Handle;
		}
	}


	bool FElectraHttpManager::StartHTTPManager()
	{
		LLM_SCOPE(ELLMTag::ElectraPlayer);

		FParamDict HttpOptions;
		HTTPStreamHandler = IElectraHTTPStream::Create(HttpOptions);
		if (HTTPStreamHandler.IsValid())
		{
			HTTPStreamHandler->AddThreadHandlerDelegate(IElectraHTTPStream::FElectraHTTPStreamThreadHandlerDelegate::CreateThreadSafeSP(AsShared(), &FElectraHttpManager::ProcessHTTPManager));
			return true;
		}
		else
		{
			UE_LOGF(LogElectraHTTPManager, Error, "Failed to create an FElectraHttpManager instance.");
			return false;
		}
	}

	void FElectraHttpManager::StopHTTPManager()
	{
		LLM_SCOPE(ELLMTag::ElectraPlayer);

		if (HTTPStreamHandler.IsValid())
		{
			HTTPStreamHandler->RemoveThreadHandlerDelegate();
		}

		RemoveAllRequests();

		if (HTTPStreamHandler.IsValid())
		{
			HTTPStreamHandler->Close();
			HTTPStreamHandler.Reset();
		}
	}

	void FElectraHttpManager::ProcessHTTPManager()
	{
		LLM_SCOPE(ELLMTag::ElectraPlayer);

		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_FElectraHttpManager_Worker);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, ElectraHttpManager_Worker);

		FTimeValue Now = MEDIAutcTime::Current();
		// Add and remove pending requests
		Lock.Lock();
		AddPendingRequests(Now);
		RemovePendingRequests(Now);
		Lock.Unlock();

		// Handle local file requests.
		HandleLocalFileRequests();
		// Handle external requests.
		HandleExternalDataRequests();
		// Handle requests that have a cached response.
		HandleCachedHTTPRequests(Now);

		// Handle HTTP requests
		HandleHTTPRequests(Now);
		HandleHTTPResponses(Now);

		// Handle periodic progress callbacks. Do this before handling the completed requests in case a callback asks to abort.
		HandlePeriodicCallbacks(Now);
		// Handle timeouts after the progress callbacks.
		HandleTimeouts(Now);
		// Handle all finished requests.
		HandleCompletedRequests();
	}



	void FElectraHttpManager::HandleLocalFileRequests()
	{
		for(TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
		{
			FHandle* Handle = It.Key();
			if (Handle->HandleType == FHandle::EHandleType::LocalHandle && Handle->LocalByteStream.IsValid())
			{
				TSharedPtrTS<FRequest> Request = It.Value();
				// Establish the file handle as "connected"
				Handle->LocalByteStream->SetConnected(Request);
				// Read from local file
				TSharedPtrTS<FWaitableBuffer> ReceiveBuffer = Request->ReceiveBuffer.Pin();
				if (ReceiveBuffer.IsValid())
				{
					int32 NumBytesRead = Handle->LocalByteStream->Read(ReceiveBuffer, Request);
					if (NumBytesRead > 0)
					{
						Handle->LastTimeDataReceived = MEDIAutcTime::Current();
					}
					// Reading done?
					if (Handle->LocalByteStream->FileSizeToGo <= 0)
					{
						Request->ConnectionInfo.RequestEndTime = Request->ConnectionInfo.StatusInfo.OccurredAtUTC = MEDIAutcTime::Current();
						RequestsCompleted.Enqueue(Request);
					}
				}
				else
				{
					// With the receive buffer having been released we can abort the transfer.
					Request->ConnectionInfo.bWasAborted = true;
					Request->ConnectionInfo.RequestEndTime = Request->ConnectionInfo.StatusInfo.OccurredAtUTC = MEDIAutcTime::Current();
					RequestsCompleted.Enqueue(Request);
				}
			}
		}
	}

	void FElectraHttpManager::HandleExternalDataRequests()
	{
		for (TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
		{
			FHandle* Handle = It.Key();
			if (Handle->HandleType == FHandle::EHandleType::ExternalHandle)
			{
				TSharedPtrTS<FRequest> Request = It.Value();
				// Establish the file handle as "connected"
				Handle->LocalByteStream->SetConnected(Request);
				// Read data
				TSharedPtrTS<FWaitableBuffer> ReceiveBuffer = Request->ReceiveBuffer.Pin();
				if (ReceiveBuffer.IsValid())
				{
					int32 NumBytesRead = Handle->LocalByteStream->Read(ReceiveBuffer, Request);
					if (NumBytesRead > 0)
					{
						Handle->LastTimeDataReceived = MEDIAutcTime::Current();
					}
					// Reading done?
					if (Handle->LocalByteStream->FileSizeToGo <= 0)
					{
						Request->ConnectionInfo.RequestEndTime = Request->ConnectionInfo.StatusInfo.OccurredAtUTC = MEDIAutcTime::Current();
						RequestsCompleted.Enqueue(Request);
					}
				}
				else
				{
					// With the receive buffer having been released we can abort the transfer.
					Request->ConnectionInfo.bWasAborted = true;
					Request->ConnectionInfo.RequestEndTime = Request->ConnectionInfo.StatusInfo.OccurredAtUTC = MEDIAutcTime::Current();
					RequestsCompleted.Enqueue(Request);
				}
			}
		}
	}

	void FElectraHttpManager::HandleCachedHTTPRequests(const FTimeValue& Now)
	{
		for(TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
		{
			FHandle* Handle = It.Key();
			if (Handle->HandleType == FHandle::EHandleType::HTTPHandle)
			{
				TSharedPtrTS<FRequest> Request = It.Value();
				if (Handle->ActiveResponse.CacheResponse.IsValid() && !Handle->ActiveResponse.bHitCache)
				{
					Handle->ActiveResponse.bHitCache = true;
					FScopeLock lock(&HTTPStreamEventLock);
					NotifiedRequests.AddUnique(Handle->HttpRequest);
				}
			}
		}
	}


	void FElectraHttpManager::HandleHTTPRequests(const FTimeValue& Now)
	{
		// Get the events that have fired so far into a local map and clear out the original.
		HTTPStreamEventLock.Lock();
		TArray<IElectraHTTPStreamRequestPtr> Notifieds = MoveTemp(NotifiedRequests);
		HTTPStreamEventLock.Unlock();
		for(auto &Notified : Notifieds)
		{
			// Find the request.
			for(TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
			{
				FHandle* Handle = It.Key();
				if (Handle->HandleType == FHandle::EHandleType::HTTPHandle && Handle->HttpRequest == Notified)
				{
					TSharedPtrTS<FRequest> Request = It.Value();
					HTTP::FConnectionInfo& ci = Request->ConnectionInfo;

					IElectraHTTPStreamResponsePtr Response = Handle->HttpRequest->GetResponse();

					if (Handle->ActiveResponse.CacheResponse.IsValid() && Handle->ActiveResponse.bHitCache)
					{
						Response = Handle->ActiveResponse.CacheResponse->Response;
						Handle->ActiveResponse.CacheResponse.Reset();
					}

					if (Response->GetState() > IElectraHTTPStreamResponse::EState::Connecting && Handle->bHttpRequestFirstEvent)
					{
						if (Handle->ActiveResponse.NumSubRangeRequest == 0)
						{
							ci.bIsConnected = true;
							ci.TimeForDNSResolve = Response->GetTimeUntilNameResolved();
							ci.TimeUntilConnected = Response->GetTimeUntilConnected();
						}
						Handle->bIsConnected = true;
						Handle->bHttpRequestFirstEvent = false;
					}

					// Get the response headers.
					if (Response->GetState() >= IElectraHTTPStreamResponse::EState::ReceivedResponseHeaders && !ci.bHaveResponseHeaders)
					{
						Handle->LastTimeDataReceived = Now;

						TArray<FElectraHTTPStreamHeader> Headers;
						Response->GetAllHeaders(Headers);

						for(auto &Header : Headers)
						{
							HTTP::FHTTPHeader h;
							h.Header = Header.Header;
							h.Value = Header.Value;
							ci.ResponseHeaders.Add(h);
						}
						ci.ContentLengthHeader = Response->GetContentLengthHeader();
						ci.ContentRangeHeader = Response->GetContentRangeHeader();
						ci.ContentType = Response->GetContentTypeHeader();
						ci.bIsChunked = Response->GetTransferEncodingHeader().Find(TEXT("chunked"), ESearchCase::IgnoreCase) != INDEX_NONE;
						ci.HTTPVersionReceived = Response->GetHTTPStatusLine().Find(TEXT("HTTP/1.1"), ESearchCase::IgnoreCase) != INDEX_NONE ? 11 :
													Response->GetHTTPStatusLine().Find(TEXT("HTTP/2"), ESearchCase::IgnoreCase) != INDEX_NONE ? 20 :
													Response->GetHTTPStatusLine().Find(TEXT("HTTP/1.0"), ESearchCase::IgnoreCase) != INDEX_NONE ? 10 : 11;

						ci.EffectiveURL = Response->GetEffectiveURL();
						ci.StatusInfo.HTTPStatus = Response->GetHTTPResponseCode();
						ci.NumberOfRedirections = 0;
						bool bContentRangeOk = ci.ContentRangeHeader.Len() ? Handle->ActiveResponse.ReceivedContentRange.ParseFromContentRangeResponse(ci.ContentRangeHeader) : true;

						// Content length needs a bit of special handling.
						if (Handle->ActiveResponse.NumSubRangeRequest == 0)
						{
							bool bHaveSize = false;
							// Is there a document size from a Content-Range header?
							if (!ci.ContentRangeHeader.IsEmpty())
							{
								ElectraHTTPStream::FHttpRange ContentRange;
								if (ContentRange.ParseFromContentRangeResponse(ci.ContentRangeHeader))
								{
									int64 ds = ContentRange.GetDocumentSize();
									// Was the request for a range or the entire document?
									if (Handle->ActiveResponse.OriginalRange.IsEverything())
									{
										ci.ContentLength = ds;
									}
									else
									{
										// A range was requested. Was it an open ended range?
										if (Handle->ActiveResponse.OriginalRange.IsOpenEnded())
										{
											// Content size is the document size minus the start.
											ci.ContentLength = ds >= 0 ? ds - Handle->ActiveResponse.OriginalRange.GetStart() : -1;
										}
										else
										{
											// Request was for an actual range.
											int64 end = Handle->ActiveResponse.OriginalRange.GetEndIncluding() + 1;
											if (ds >= 0 && end > ds)
											{
												end = ds;
											}
											ci.ContentLength = end - Handle->ActiveResponse.OriginalRange.GetStart();
										}
									}
									bHaveSize = true;
								}
							}
							if (!bHaveSize)
							{
								if (!ci.ContentLengthHeader.IsEmpty())
								{
									// Parse from "Content-Length: " header
									LexFromString(ci.ContentLength, *ci.ContentLengthHeader);
								}
							}
						}

						// If we requested a byte range we need to check if we got the correct range back.
						// For compatibilities sake a 200 response will also be accepted as long as the number of bytes match
						// the number requested. This then requires the Content-Length response header to be present.
						if (Request->Parameters.Range.IsSet() && !Request->Parameters.Range.IsEverything())
						{
							if (ci.StatusInfo.HTTPStatus == 206 && ci.ContentRangeHeader.Len() && bContentRangeOk)
							{
								// We assume that the returned range is what we have requested. It's possible to check for it
								// but not trivial since the range could be open ended on either side in the request and values
								// provided in the response.
							}
							else if (ci.StatusInfo.HTTPStatus == 200 && ci.ContentLength == Request->Parameters.Range.GetNumberOfBytes())
							{
								// Allow a 200 response if the number of bytes received matches the number of bytes requested.
							}
							else
							{
								// Not good.
								ci.bResponseNotRanged = true;
							}
						}

						ci.bHaveResponseHeaders = true;
						Handle->bHaveResponseHeaders = true;

						// Check for HTTP errors. Redirects are not really expected to reach us here. If they do there were too many.
						static const TArray<int32> GoodHTTPResponseCodes = { 200, 204, 206, 304 };
						bool bHTTPResponseOk = GoodHTTPResponseCodes.Contains(ci.StatusInfo.HTTPStatus);
						bool bFailed = false;
						if (!bHTTPResponseOk || ci.bResponseNotRanged)
						{
							ci.StatusInfo.ErrorDetail.SetError(UEMEDIA_ERROR_READ_ERROR).SetFacility(Facility::EFacility::HTTPReader).SetCode(HTTP::EStatusErrorCode::ERRCODE_HTTPMODULE_FAILURE);
							if (ci.StatusInfo.HTTPStatus >= 400)
							{
								ci.StatusInfo.ErrorDetail.SetMessage(FString::Printf(TEXT("HTTP returned status %d"), ci.StatusInfo.HTTPStatus));
								ci.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTP_RETURNED_ERROR;
								bFailed = true;
							}
							else if (ci.StatusInfo.HTTPStatus >= 300)
							{
								ci.StatusInfo.ErrorDetail.SetMessage(FString::Printf(TEXT("Reached HTTP redirection limit with returned status %d"), ci.StatusInfo.HTTPStatus));
								ci.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTP_RETURNED_ERROR;
								bFailed = true;
							}
							else if (ci.StatusInfo.HTTPStatus == 0)
							{
								ci.StatusInfo.ErrorDetail.SetMessage(FString::Printf(TEXT("Failed to get response from server")));
								ci.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTP_RETURNED_ERROR;
								bFailed = true;
							}
							else if (ci.bResponseNotRanged)
							{
								ci.StatusInfo.ErrorDetail.SetMessage(FString::Printf(TEXT("Did not receive HTTP 206 for range request")));
								ci.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTP_RANGE_ERROR;
								bFailed = true;
							}
						}

						// After having handled the headers we allow the response to get consumed.
						// Further error handling (which at this point can only be a closed connection or timeouts)
						// is handled in HandleHTTPResponses().
						if (!Handle->HttpRequest->HasFailed() && !bFailed)
						{
							Handle->ActiveResponse.Response = Response;
						}
						else
						{
							ci.bHasFinished = true;
							ci.RequestEndTime = ci.StatusInfo.OccurredAtUTC = Now;
							Handle->HttpRequest->Cancel();
							RequestsCompleted.Enqueue(Request);
						}
					}
				}
			}
		}
	}


	void FElectraHttpManager::HandleHTTPResponses(const FTimeValue& Now)
	{
		for(TMap<FHandle*, TSharedPtrTS<FRequest>>::TIterator It = ActiveRequests.CreateIterator(); It; ++It)
		{
			FHandle* Handle = It.Key();
			if (Handle->HandleType == FHandle::EHandleType::HTTPHandle)
			{
				TSharedPtrTS<FRequest> Request = It.Value();

				// Active response? This will not be set if the HTTP response code indicated an error already.
				// For our use cases we do not need to read the response body which will be an error message only anyway.
				IElectraHTTPStreamResponsePtr Response = Handle->ActiveResponse.Response;
				if (Response.IsValid())
				{
					HTTP::FConnectionInfo& ci = Request->ConnectionInfo;
					ci.bIsCachedResponse = Handle->ActiveResponse.bHitCache;

					// Set the effective URL after possible redirections.
					ci.EffectiveURL = Response->GetEffectiveURL();
					// Copy all new timing traces across.
					Response->GetTimingTraces(&ci.TimingTraces, TNumericLimits<int32>::Max());

					/*
						Removed because it is possible for a response to have been gzip encoded with the _compressed_
						payload being larger than the _decompressed_.
						The Content-Length header conveys the size of the transmitted (compressed) data which will
						then be larger than the decompressed result for these cases.


					bool bCompletedWithInsufficientData = !Request->Parameters.Verb.Equals(TEXT("HEAD")) &&
														  Response->GetStatus() == IElectraHTTPStreamResponse::EStatus::Completed &&
														  Response->GetResponseData().GetLengthFromResponseHeader() > Response->GetResponseData().GetNumTotalBytesAdded();
					*/
					// Has it failed?
					if (Handle->HttpRequest->HasFailed() ) //|| bCompletedWithInsufficientData)
					{
						ci.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_WRITE_ERROR;
						ci.StatusInfo.bReadError = true;
						ci.StatusInfo.ErrorDetail.SetMessage(FString::Printf(TEXT("Read error after receiving %lld of %lld bytes"), (long long int)ci.BytesReadSoFar, (long long int)ci.ContentLength));
						ci.bHasFinished = true;
						ci.RequestEndTime = ci.StatusInfo.OccurredAtUTC = Now;
						Handle->ActiveResponse.Response.Reset();
						RequestsCompleted.Enqueue(Request);
						continue;
					}

					bool bHasFinished = Response->GetResponseData().HasAllDataBeenConsumed();
					int64 NumDataAvailable = Response->GetResponseData().GetNumBytesAvailableForRead();
					if (NumDataAvailable > 0)
					{
						if (ci.TimeUntilFirstByte <= 0.0)
						{
							ci.TimeUntilFirstByte = Response->GetTimeUntilFirstByte();
						}
						Handle->LastTimeDataReceived = Now;

						// Receive buffer still there?
						TSharedPtrTS<FWaitableBuffer> ReceiveBuffer = Request->ReceiveBuffer.Pin();
						if (ReceiveBuffer.IsValid())
						{
							int64 RequiredBufferSize = ci.ContentLength > 0 ? ci.ContentLength : 0;
							int64 BufferSizeAfterPush = ReceiveBuffer->Num() + NumDataAvailable;
							if (BufferSizeAfterPush > RequiredBufferSize)
							{
								RequiredBufferSize = BufferSizeAfterPush;
							}
							bool bBufferUsable = ReceiveBuffer->EnlargeTo(RequiredBufferSize);
							int64 BufferPushableSize = bBufferUsable ? NumDataAvailable : 0;
							if (bBufferUsable)
							{
								if (BufferPushableSize)
								{
									const uint8* NewDataPtr = nullptr;
									int64 NewDataSize = 0;
									Response->GetResponseData().LockBuffer(NewDataPtr, NewDataSize);
									int64 NumToCopy = BufferPushableSize < NewDataSize ? BufferPushableSize : NewDataSize;
									bBufferUsable = ReceiveBuffer->PushData(NewDataPtr, NumToCopy);
									Response->GetResponseData().UnlockBuffer(bBufferUsable ? NumToCopy : 0);
									if (bBufferUsable)
									{
										Handle->ActiveResponse.NumBytesPassedOut += NumToCopy;
										ci.BytesReadSoFar += NumToCopy;
									}
								}
							}
							if (!bBufferUsable)
							{
								ci.bWasAborted = true;
								bHasFinished = true;
							}
						}
						else
						{
							// With the receive buffer having been released we can abort the transfer.
							ci.bWasAborted = true;
							if (Handle->HttpsRequestCallbackWrapper.IsValid())
							{
								Handle->HttpsRequestCallbackWrapper->Unbind();
							}
							bHasFinished = true;
						}
					}

					if (bHasFinished)
					{
						if (!ci.bWasAborted && Response->GetResponseData().HasAllDataBeenConsumed())
						{
							// Add to response cache unless this was a cached response already.
							if (Request->ResponseCache.IsValid() && Response->GetResponseData().IsCachable() && !Handle->ActiveResponse.bHitCache && !Handle->ActiveResponse.bWasAddedToCache)
							{
								Handle->ActiveResponse.bWasAddedToCache = true;
								TSharedPtrTS<IHTTPResponseCache::FCacheItem> CacheItem = MakeSharedTS<IHTTPResponseCache::FCacheItem>();
								CacheItem->RequestedURL = Request->Parameters.URL;
								CacheItem->EffectiveURL = Response->GetEffectiveURL();
								CacheItem->Range = Handle->ActiveResponse.Range;
								// Make sure the range is always set, even for non-partial responses.
								CacheItem->Range.DocumentSize = ci.ContentLength > Handle->ActiveResponse.ReceivedContentRange.GetDocumentSize() ? ci.ContentLength : Handle->ActiveResponse.ReceivedContentRange.GetDocumentSize();
								if (CacheItem->Range.GetStart() < 0)
								{
									CacheItem->Range.SetStart(0);
								}
								if (CacheItem->Range.GetEndIncluding() < 0)
								{
									CacheItem->Range.SetEndIncluding(CacheItem->Range.DocumentSize - 1);
								}
								CacheItem->Response = Response;
								CacheItem->Quality.QualityIndex = Request->Parameters.QualityIndex;
								CacheItem->Quality.MaxQualityIndex = Request->Parameters.MaxQualityIndex;
								CacheItem->Quality.StreamType = Request->Parameters.StreamType;
								Request->ResponseCache->CacheEntity(CacheItem);
							}

							// Check if this was a sub ranged request and if there is still data to go for the original request.
							if (Handle->ActiveResponse.SizeRemaining() == 0)
							{
								// All done now.
								TSharedPtrTS<FWaitableBuffer> ReceiveBuffer = Request->ReceiveBuffer.Pin();
								if (ReceiveBuffer.IsValid())
								{
									ReceiveBuffer->SetEOD();
								}
							}
							else
							{
								// Still another sub range to go.
								if (PrepareHTTPHandle(Now, Handle, Request, false))
								{
									// We need to parse the headers from the new request so we have to clear the flag.
									ci.bHaveResponseHeaders = false;
									ci.ResponseHeaders.Empty();
									if (HTTPStreamHandler.IsValid())
									{
										if (Handle->NeedsFetchOfUncachedParts())
										{
											HTTPStreamHandler->AddRequest(Handle->HttpRequest);
										}
										bHasFinished = false;
									}
									else
									{
										ci.StatusInfo.ErrorDetail.SetError(UEMEDIA_ERROR_INTERNAL).SetFacility(Facility::EFacility::HTTPReader).SetMessage(TEXT("HTTP sub request failed on AddRequest()"));
									}
								}
								else
								{
									//
									ci.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_WRITE_ERROR;
									ci.StatusInfo.bReadError = true;
									ci.StatusInfo.ErrorDetail.SetMessage(FString::Printf(TEXT("Error setting up the next sub range request")));
								}
							}
						}

						if (bHasFinished)
						{
							ci.bHasFinished = true;
							ci.RequestEndTime = ci.StatusInfo.OccurredAtUTC = Now;

							Handle->ActiveResponse.Response.Reset();
							RequestsCompleted.Enqueue(Request);
						}
					}
				}
			}
		}
	}



	void FElectraHttpManager::OnRequestNotification(IElectraHTTPStreamRequestPtr InRequest, EElectraHTTPStreamNotificationReason InReason, int64 InParam)
	{
		// Notify only for headers and completion, not data transfer.
		if (InReason == EElectraHTTPStreamNotificationReason::ReceivedHeaders || InReason == EElectraHTTPStreamNotificationReason::Completed)
		{
			FScopeLock lock(&HTTPStreamEventLock);
			NotifiedRequests.AddUnique(MoveTemp(InRequest));
		}
	}





	void FElectraHttpManager::FFileStream::SetConnected(TSharedPtrTS<FRequest> Request)
	{
		if (!bIsConnected)
		{
			bIsConnected = true;
			// Go through the notions of this being a network request.
			Request->ConnectionInfo.bIsConnected = true;
			Request->ConnectionInfo.bHaveResponseHeaders = true;
			Request->ConnectionInfo.ContentType = TEXT("application/octet-stream");
			Request->ConnectionInfo.EffectiveURL = Request->Parameters.URL;
			Request->ConnectionInfo.HTTPVersionReceived = 11;
			Request->ConnectionInfo.bIsChunked = false;

			if (Archive.IsValid())
			{
				// Range request?
				if (!Request->Parameters.Range.IsSet())
				{
					Request->ConnectionInfo.ContentLength = Archive->TotalSize();
					Request->ConnectionInfo.StatusInfo.HTTPStatus = 200;
					FileStartOffset = 0;
					FileSize = Archive->TotalSize();
					FileSizeToGo = FileSize;
					Request->ConnectionInfo.ContentLengthHeader = FString::Printf(TEXT("Content-Length: %lld"), (long long int)FileSize);
				}
				else
				{
					int64 fs = Archive->TotalSize();
					int64 off = 0;
					int64 end = fs - 1;
					// For now we support partial data only from the beginning of the file, not the end (aka, seek_set and not seek_end)
					check(Request->Parameters.Range.Start >= 0);
					if (Request->Parameters.Range.Start >= 0)
					{
						off = Request->Parameters.Range.Start;
						if (off < fs)
						{
							end = Request->Parameters.Range.EndIncluding;
							if (end < 0 || end >= fs)
							{
								end = fs - 1;
							}
							int64 numBytes = end - off + 1;

							Request->ConnectionInfo.ContentLength = numBytes;
							Request->ConnectionInfo.StatusInfo.HTTPStatus = 206;
							FileStartOffset = off;
							FileSize = Archive->TotalSize();
							FileSizeToGo = numBytes;
							Request->ConnectionInfo.ContentLengthHeader = FString::Printf(TEXT("Content-Length: %lld"), (long long int)numBytes);
							Request->ConnectionInfo.ContentRangeHeader = FString::Printf(TEXT("Content-Range: bytes %lld-%lld/%lld"), (long long int)off, (long long int)end, (long long int)fs);
							Archive->Seek(off);
						}
						else
						{
							Request->ConnectionInfo.StatusInfo.HTTPStatus = 416;		// Range not satisfiable
							Request->ConnectionInfo.ContentRangeHeader = FString::Printf(TEXT("Content-Range: bytes */%lld"), (long long int)fs);
						}
					}
				}
			}
			else
			{
				Request->ConnectionInfo.StatusInfo.HTTPStatus = 404;	// File not found
				Request->ConnectionInfo.StatusInfo.ErrorDetail.SetMessage(FString::Printf(TEXT("HTTP returned status 404")));
				Request->ConnectionInfo.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTP_RETURNED_ERROR;
			}
		}
	}

	int32 FElectraHttpManager::FFileStream::Read(TSharedPtrTS<FWaitableBuffer> RcvBuffer, TSharedPtrTS<FRequest> Request)
	{
		int64 NumToRead = FileSizeToGo;
		if (NumToRead)
		{
			void* Dst = (void*) RcvBuffer->GetLinearWriteData(NumToRead);
			if (Dst)
			{
				Archive->Serialize(Dst, NumToRead);
				RcvBuffer->AppendedNewData(NumToRead);
				Request->ConnectionInfo.BytesReadSoFar += NumToRead;
				FileSizeToGo -= NumToRead;
			}
			else
			{
				NumToRead = 0;
			}
		}
		return NumToRead;
	}


	bool FElectraHttpManager::FDataUrl::SetData(const FString& InUrl)
	{
		// See https://en.wikipedia.org/wiki/Data_URI_scheme
		FURL_RFC3986 du;
		TArray<FString> dp;
		if (du.Parse(InUrl) && du.GetPathComponents(dp) && dp.Num() > 0)
		{
			// The last component, even if empty, is the data.
			// The second to last may be `base64` to indicate the encoding
			// The first, unless it is the `base64`, tends to be the mime type
			// Anything in between we do not care about, not even the charset.
			int32 base64Pos = INDEX_NONE;
			if (dp.Find(TEXT("base64"), base64Pos))
			{
				if (!FBase64::Decode(dp.Last(), Data))
				{
					return false;
				}
			}
			else
			{
				FString unesc;
				if (!FURL_RFC3986::UrlDecode(unesc, dp.Last()))
				{
					return false;
				}
				StringHelpers::StringToArray(Data, unesc);
			}
			// If `base64` is not the first then we assume the first to be the mime type
			if (base64Pos > 0)
			{
				MimeType = dp[0];
			}
			else
			{
				MimeType = TEXT("text/plain;charset=US-ASCII");
			}
			return true;
		}
		return false;
	}

	void FElectraHttpManager::FDataUrl::SetConnected(TSharedPtrTS<FRequest> Request)
	{
		if (!bIsConnected)
		{
			bIsConnected = true;
			// Go through the notions of this being a network request.
			Request->ConnectionInfo.bIsConnected = true;
			Request->ConnectionInfo.bHaveResponseHeaders = true;
			Request->ConnectionInfo.ContentType = MimeType;
			Request->ConnectionInfo.EffectiveURL.Empty();	// There is no actual URL with a data url.
			Request->ConnectionInfo.HTTPVersionReceived = 11;
			Request->ConnectionInfo.bIsChunked = false;

			// Range request?
			if (!Request->Parameters.Range.IsSet())
			{
				Request->ConnectionInfo.ContentLength = Data.Num();
				Request->ConnectionInfo.StatusInfo.HTTPStatus = 200;
				FileStartOffset = 0;
				FileSize = Data.Num();
				FileSizeToGo = FileSize;
				Request->ConnectionInfo.ContentLengthHeader = FString::Printf(TEXT("Content-Length: %lld"), (long long int)FileSize);
			}
			else
			{
				int64 fs = Data.Num();
				int64 off = 0;
				int64 end = fs - 1;
				// For now we support partial data only from the beginning of the file, not the end (aka, seek_set and not seek_end)
				check(Request->Parameters.Range.Start >= 0);
				if (Request->Parameters.Range.Start >= 0)
				{
					off = Request->Parameters.Range.Start;
					if (off < fs)
					{
						end = Request->Parameters.Range.EndIncluding;
						if (end < 0 || end >= fs)
						{
							end = fs - 1;
						}
						int64 numBytes = end - off + 1;

						Request->ConnectionInfo.ContentLength = numBytes;
						Request->ConnectionInfo.StatusInfo.HTTPStatus = 206;
						FileStartOffset = off;
						FileSize = Data.Num();
						FileSizeToGo = numBytes;
						Request->ConnectionInfo.ContentLengthHeader = FString::Printf(TEXT("Content-Length: %lld"), (long long int)numBytes);
						Request->ConnectionInfo.ContentRangeHeader = FString::Printf(TEXT("Content-Range: bytes %lld-%lld/%lld"), (long long int)off, (long long int)end, (long long int)fs);
					}
					else
					{
						Request->ConnectionInfo.StatusInfo.HTTPStatus = 416;		// Range not satisfiable
						Request->ConnectionInfo.ContentRangeHeader = FString::Printf(TEXT("Content-Range: bytes */%lld"), (long long int)fs);
					}
				}
			}
		}
	}

	int32 FElectraHttpManager::FDataUrl::Read(TSharedPtrTS<FWaitableBuffer> RcvBuffer, TSharedPtrTS<FRequest> Request)
	{
		if (RcvBuffer->EnlargeTo(FileSizeToGo))
		{
			if (RcvBuffer->PushData(Data.GetData() + FileStartOffset, FileSizeToGo))
			{
				Request->ConnectionInfo.BytesReadSoFar += FileSizeToGo;
			}
		}
		int32 NumRead = (int32) FileSizeToGo;
		FileSizeToGo = 0;
		return NumRead;
	}





} // namespace Electra


