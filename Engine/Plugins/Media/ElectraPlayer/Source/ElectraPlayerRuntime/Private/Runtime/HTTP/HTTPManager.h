// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Delegates/Delegate.h"

#include "StreamDataBuffer.h"
#include "StreamTypes.h"
#include "OptionalValue.h"

#include "ParameterDictionary.h"
#include "ErrorDetail.h"
#include "ElectraHTTPStream.h"
#include "Utilities/HttpRangeHeader.h"


DECLARE_LOG_CATEGORY_EXTERN(LogElectraHTTPManager, Log, All);

namespace Electra
{
	class IHTTPResponseCache;

	namespace HTTP
	{
		enum EStatusErrorCode
		{
			ERRCODE_OK = 0,
			ERRCODE_HTTP_CONNECTION_TIMEOUT = 1,
			ERRCODE_HTTP_RETURNED_ERROR = 2,
			ERRCODE_WRITE_ERROR = 3,
			ERRCODE_HTTP_RANGE_ERROR = 4,
			ERRCODE_HTTP_FILE_COULDNT_READ_FILE = 5,
			ERRCODE_HTTPMODULE_FAILURE = 100
		};

		struct FStatusInfo
		{
			void Empty()
			{
				ErrorDetail.Clear();
				OccurredAtUTC.SetToInvalid();
				HTTPStatus = 0;
				ErrorCode = 0;
				ConnectionTimeoutAfterMilliseconds = 0;
				NoDataTimeoutAfterMilliseconds = 0;
				bRedirectionError = false;
				bReadError = false;
			}
			FErrorDetail	ErrorDetail;
			FTimeValue		OccurredAtUTC;
			int32			HTTPStatus = 0;
			int32			ErrorCode = 0;
			int32			ConnectionTimeoutAfterMilliseconds = 0;
			int32			NoDataTimeoutAfterMilliseconds = 0;
			bool			bRedirectionError = false;
			bool			bReadError = false;
		};

		struct FHTTPHeader : public FElectraHTTPStreamHeader
		{
			FHTTPHeader() = default;
			FHTTPHeader(const FString& InHeader, const FString& InValue)
			{
				Header = InHeader;
				Value = InValue;
			}
			void SetFromString(const FString& InString)
			{
				int32 ColonPos;
				if (InString.FindChar(TCHAR(':'), ColonPos))
				{
					Header = InString.Left(ColonPos);
					Value = InString.Mid(ColonPos + 2);
				}
			}
			static void ParseFromString(FHTTPHeader& OutHeader, const FString& InString)
			{
				OutHeader.SetFromString(InString);
			}
		};

		struct FConnectionInfo
		{
			TArray<FHTTPHeader>					ResponseHeaders;					//!< Response headers
			FString								EffectiveURL;						//!< Effective URL after all redirections
			FString								ContentType;						//!< Content-Type header
			FString  							ContentRangeHeader;
			FString  							ContentLengthHeader;
			FTimeValue							RequestStartTime;					//!< Time at which the request was started
			FTimeValue							RequestEndTime;						//!< Time at which the request ended
			double								TimeForDNSResolve = 0.0;			//!< Time it took to resolve DNS
			double								TimeUntilConnected = 0.0;			//!< Time it took until connected to the server
			double								TimeUntilFirstByte = 0.0;			//!< Time it took until received the first response data byte
			int64								ContentLength = -1;					//!< Content length, if known. Chunked transfers may have no length (set to -1). Compressed file will store compressed size here!
			int64								BytesReadSoFar = 0;					//!< Number of bytes read so far.
			int32								NumberOfRedirections = 0;			//!< Number of redirections performed
			int32								HTTPVersionReceived = 0;			//!< Version of HTTP header received (10 = 1.0, 11=1.1, 20=2.0)
			bool								bIsConnected = false;				//!< true once connected to the server
			bool								bHaveResponseHeaders = false;		//!< true when response headers have been received
			bool								bIsChunked = false;					//!< true if response is received in chunks
			bool								bWasAborted = false;				//!< true if transfer was aborted.
			bool								bHasFinished = false;				//!< true once the connection is closed regardless of state.
			bool								bResponseNotRanged = false;			//!< true if the response is not a range as was requested.
			bool								bIsCachedResponse = false;			//!< true if the response came from the cache.
			FStatusInfo							StatusInfo;

			mutable FCriticalSection			Lock;
			TArray<IElectraHTTPStreamResponse::FTimingTrace> TimingTraces;

			void GetTimingTraces(TArray<IElectraHTTPStreamResponse::FTimingTrace>& OutTimingTraces) const
			{
				FScopeLock lock(&Lock);
				OutTimingTraces = TimingTraces;
			}

			FConnectionInfo() = default;

			FConnectionInfo(const FConnectionInfo& rhs)
			{
				CopyFrom(rhs);
			}

			FConnectionInfo& operator = (const FConnectionInfo& rhs)
			{
				if (this != &rhs)
				{
					CopyFrom(rhs);
				}
				return *this;
			}

		private:
			FConnectionInfo& CopyFrom(const FConnectionInfo& rhs)
			{
				FScopeLock l1(&Lock);
				FScopeLock l2(&rhs.Lock);
				ResponseHeaders = rhs.ResponseHeaders;
				EffectiveURL = rhs.EffectiveURL;
				ContentType = rhs.ContentType;
				ContentRangeHeader = rhs.ContentRangeHeader;
				ContentLengthHeader = rhs.ContentLengthHeader;
				RequestStartTime = rhs.RequestStartTime;
				RequestEndTime = rhs.RequestEndTime;
				TimeForDNSResolve = rhs.TimeForDNSResolve;
				TimeUntilConnected = rhs.TimeUntilConnected;
				TimeUntilFirstByte = rhs.TimeUntilFirstByte;
				ContentLength = rhs.ContentLength;
				BytesReadSoFar = rhs.BytesReadSoFar;
				NumberOfRedirections = rhs.NumberOfRedirections;
				HTTPVersionReceived = rhs.HTTPVersionReceived;
				bIsConnected = rhs.bIsConnected;
				bHaveResponseHeaders = rhs.bHaveResponseHeaders;
				bIsChunked = rhs.bIsChunked;
				bWasAborted = rhs.bWasAborted;
				bHasFinished = rhs.bHasFinished;
				bResponseNotRanged = rhs.bResponseNotRanged;
				bIsCachedResponse = rhs.bIsCachedResponse;
				StatusInfo = rhs.StatusInfo;
				TimingTraces = rhs.TimingTraces;
				return *this;
			}
		};

	} // namespace HTTP


	class IElectraHttpManager
	{
	public:
		static TSharedPtrTS<IElectraHttpManager> Create();

		static FString GetDefaultUserAgent();

		virtual ~IElectraHttpManager() = default;

		struct FRequest;

		struct FProgressListener
		{
			DECLARE_DELEGATE_RetVal_OneParam(int32, FProgressDelegate, const FRequest*);
			DECLARE_DELEGATE_OneParam(FCompletionDelegate, const FRequest*);

			~FProgressListener()
			{
				Clear();
			}
			void Clear()
			{
				FScopeLock lock(&Lock);
				ProgressDelegate.Unbind();
				CompletionDelegate.Unbind();
			}
			int32 CallProgressDelegate(const FRequest* Request)
			{
				FScopeLock lock(&Lock);
				if (ProgressDelegate.IsBound())
				{
					return ProgressDelegate.Execute(Request);
				}
				return 0;
			}
			void CallCompletionDelegate(const FRequest* Request)
			{
				FScopeLock lock(&Lock);
				if (CompletionDelegate.IsBound())
				{
					CompletionDelegate.Execute(Request);
				}
			}
			FCriticalSection Lock;
			FProgressDelegate ProgressDelegate;
			FCompletionDelegate CompletionDelegate;
		};

		struct FParams
		{
			void AddFromHeaderList(const TArray<FString>& InHeaderList)
			{
				for(int32 i=0; i<InHeaderList.Num(); ++i)
				{
					HTTP::FHTTPHeader h;
					h.SetFromString(InHeaderList[i]);
					RequestHeaders.Emplace(MoveTemp(h));
				}
			}

			FString								URL;							//!< URL
			FString								Verb;							//!< GET (default if not set), HEAD, OPTIONS,....
			TMediaOptionalValue<FString>		UserAgent;						//!< Custom user agent string
			ElectraHTTPStream::FHttpRange		Range;							//!< Optional request range
			TArray<HTTP::FHTTPHeader>			RequestHeaders;					//!< Request headers
			TMediaOptionalValue<FString>		AcceptEncoding;					//!< Optional accepted encoding
			FTimeValue							ConnectTimeout;					//!< Optional timeout for connecting to the server
			FTimeValue							NoDataTimeout;					//!< Optional timeout when no data is being received
			TArray<uint8>						PostData;						//!< Data for POST
			bool								bCollectTimingTraces = false;	//!< Whether or not to collect download timing traces.

			EStreamType							StreamType = EStreamType::Unsupported;
			int32								QualityIndex = 0;
			int32								MaxQualityIndex = 0;
		};


		struct FRequest
		{
			FParams								Parameters;
			FParamDict							Options;
			HTTP::FConnectionInfo				ConnectionInfo;
			TWeakPtrTS<FWaitableBuffer>			ReceiveBuffer;
			TWeakPtrTS<FProgressListener>		ProgressListener;
			TSharedPtrTS<IHTTPResponseCache>	ResponseCache;
			bool								bAutoRemoveWhenComplete = false;
		};

		virtual void AddRequest(TSharedPtrTS<FRequest> Request, bool bAutoRemoveWhenComplete) = 0;
		virtual void RemoveRequest(TSharedPtrTS<FRequest> Request, bool bDoNotWaitForRemoval) = 0;

	};


} // namespace Electra


