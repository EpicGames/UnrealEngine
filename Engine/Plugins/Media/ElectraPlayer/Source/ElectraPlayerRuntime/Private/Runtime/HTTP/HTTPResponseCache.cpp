// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTTPResponseCache.h"
#include "ElectraHTTPStreamBuffer.h"


namespace Electra
{

class FHTTPResponseCache : public IHTTPResponseCache
{
public:
	FHTTPResponseCache(int64 InMaxByteCapacity, int32 InMaxEntries, TSharedPtr<IElectraPlayerDataCache, ESPMode::ThreadSafe> ExternalCache);
	virtual ~FHTTPResponseCache();

	void Disable() override;
	void HandleEntityExpiration() override;
	void CacheEntity(TSharedPtrTS<FCacheItem> EntityToAdd) override;
	EScatterResult GetScatteredCacheEntity(TSharedPtrTS<FCacheItem>& OutScatteredCachedEntity, const FString& URL, const ElectraHTTPStream::FHttpRange& Range, const FQualityInfo& ForQuality) override;

private:
	/**
	 * This class wraps IElectraHTTPStreamBuffer with the intention that the original buffer
	 * can be shared multiple times, each having a new set of internal read offsets.
	 * Otherwise it would not be possible to return the buffer to multiple cached responses.
	 * Also, since the original buffer has been read from before it was added to the cache
	 * it will have its read offsets already pointing to the end.
	 */
	class FWrappedHTTPStreamBuffer : public IElectraHTTPStreamBuffer
	{
	public:
		FWrappedHTTPStreamBuffer(IElectraHTTPStreamBuffer& InBufferToWrap)
			: WrappedBuffer(InBufferToWrap)
		{
			InBufferToWrap.GetBaseBuffer(WrappedBufferBaseAddress, WrappedBufferSize);
		}
		virtual ~FWrappedHTTPStreamBuffer() = default;

		void AddData(const TArray<uint8>& InNewData) override
		{ }
		void AddData(TArray<uint8>&& InNewData) override
		{ }
		void AddData(const TConstArrayView<const uint8>& InNewData) override
		{ }
		void AddData(const IElectraHTTPStreamBuffer& Other, int64 Offset, int64 NumBytes) override
		{ }
		int64 GetNumTotalBytesAdded() const override
		{ return WrappedBuffer.GetNumTotalBytesAdded(); }
		bool GetEOS() const override
		{ return true; }
		void SetEOS() override
		{ }
		void ClearEOS() override
		{ }
		bool IsCachable() const override
		{ return false; }
		void SetIsCachable(bool bInIsCachable) override
		{ }
		void SetLengthFromResponseHeader(int64 InLengthFromResponseHeader) override
		{ }
		int64 GetLengthFromResponseHeader() const override
		{ return WrappedBuffer.GetLengthFromResponseHeader(); }

		int64 GetNumTotalBytesHandedOut() const override
		{
			return NumBytesHandedOut;
		}
		int64 GetNumBytesAvailableForRead()	const override
		{
			return WrappedBufferSize - NextReadPosInBuffer;
		}

		void LockBuffer(const uint8*& OutNextReadAddress, int64& OutNumBytesAvailable) override
		{
			OutNextReadAddress = WrappedBufferBaseAddress + NextReadPosInBuffer;
			OutNumBytesAvailable = WrappedBufferSize - NextReadPosInBuffer;
		}
		void UnlockBuffer(int64 NumBytesConsumed) override
		{
			check(NumBytesConsumed >= 0 && NumBytesConsumed <= WrappedBufferSize - NextReadPosInBuffer);
			if (NumBytesConsumed > 0)
			{
				if (NumBytesConsumed > WrappedBufferSize - NextReadPosInBuffer)
				{
					NumBytesConsumed = WrappedBufferSize - NextReadPosInBuffer;
				}
				NextReadPosInBuffer += NumBytesConsumed;
				NumBytesHandedOut += NumBytesConsumed;
			}
		}
		bool RewindToBeginning() override
		{
			NextReadPosInBuffer = 0;
			NumBytesHandedOut = 0;
			return true;
		}
		bool HasAllDataBeenConsumed() const override
		{
			return WrappedBufferSize - NextReadPosInBuffer == 0;
		}

	private:
		virtual void GetBaseBuffer(const uint8*& OutBaseAddress, int64& OutBytesInBuffer) override
		{
			check(!"Why do you get here?");
		}
		IElectraHTTPStreamBuffer& WrappedBuffer;
		const uint8* WrappedBufferBaseAddress;
		int64 WrappedBufferSize;

		int64 NextReadPosInBuffer = 0;
		int64 NumBytesHandedOut = 0;
	};


	class FWrappedCacheResponse : public IElectraHTTPStreamResponse
	{
	public:
		FWrappedCacheResponse(IElectraHTTPStreamResponsePtr InResponseToWrap)
			: WrappedResponse(InResponseToWrap)
			, WrappedBuffer(InResponseToWrap->GetResponseData())
		{
			CopyHeadersExceptDateFrom(InResponseToWrap);
		}
		virtual ~FWrappedCacheResponse() = default;

		EStatus GetStatus() override
		{ return WrappedResponse->GetStatus(); }
		EState GetState() override
		{ return WrappedResponse->GetState(); }
		FString GetErrorMessage() override
		{ return WrappedResponse->GetErrorMessage(); }
		int32 GetHTTPResponseCode() override
		{ return WrappedResponse->GetHTTPResponseCode(); }
		int64 GetNumResponseBytesReceived() override
		{ return WrappedResponse->GetNumResponseBytesReceived(); }
		int64 GetNumRequestBytesSent() override
		{ return WrappedResponse->GetNumRequestBytesSent(); }
		FString GetEffectiveURL() override
		{ return WrappedResponse->GetEffectiveURL(); }
		FString GetHTTPStatusLine() override
		{ return WrappedResponse->GetHTTPStatusLine(); }
		FString GetContentLengthHeader() override
		{ return WrappedResponse->GetContentLengthHeader(); }
		FString GetContentRangeHeader() override
		{ return WrappedResponse->GetContentRangeHeader(); }
		FString GetAcceptRangesHeader() override
		{ return WrappedResponse->GetAcceptRangesHeader(); }
		FString GetTransferEncodingHeader() override
		{ return WrappedResponse->GetTransferEncodingHeader(); }
		FString GetContentTypeHeader() override
		{ return WrappedResponse->GetContentTypeHeader(); }
		double GetTimeElapsed() override
		{ return WrappedResponse->GetTimeElapsed(); }
		double GetTimeSinceLastDataArrived() override
		{ return WrappedResponse->GetTimeSinceLastDataArrived(); }
		double GetTimeUntilNameResolved() override
		{ return WrappedResponse->GetTimeUntilNameResolved(); }
		double GetTimeUntilConnected() override
		{ return WrappedResponse->GetTimeUntilConnected(); }
		double GetTimeUntilRequestSent() override
		{ return WrappedResponse->GetTimeUntilRequestSent(); }
		double GetTimeUntilHeadersAvailable() override
		{ return WrappedResponse->GetTimeUntilHeadersAvailable(); }
		double GetTimeUntilFirstByte() override
		{ return WrappedResponse->GetTimeUntilFirstByte(); }
		double GetTimeUntilFinished() override
		{ return WrappedResponse->GetTimeUntilFinished(); }

		int32 GetTimingTraces(TArray<FTimingTrace>* OutTraces, int32 InNumToRemove) override
		{ return /*WrappedResponse->GetTimeElapsed(OutTraces, 0);*/ 0; }
		void SetEnableTimingTraces() override
		{ WrappedResponse->SetEnableTimingTraces(); }

		void GetAllHeaders(TArray<FElectraHTTPStreamHeader>& OutHeaders) override
		{ OutHeaders = Headers; }

		IElectraHTTPStreamBuffer& GetResponseData() override
		{ return WrappedBuffer; }

	private:
		void CopyHeadersExceptDateFrom(IElectraHTTPStreamResponsePtr CachedResponse)
		{
			check(CachedResponse);
			if (CachedResponse.IsValid())
			{
				TArray<FElectraHTTPStreamHeader> CachedHeaders;
				CachedResponse->GetAllHeaders(CachedHeaders);
				// Copy all headers except for those containing the date
				for(auto &Header : CachedHeaders)
				{
					if (!Header.Header.Equals(TEXT("Date"), ESearchCase::IgnoreCase))
					{
						Headers.Emplace(Header);
					}
				}
			}
		}

		IElectraHTTPStreamResponsePtr WrappedResponse;
		TArray<FElectraHTTPStreamHeader> Headers;
		FWrappedHTTPStreamBuffer WrappedBuffer;
	};


	class FSynthesizedCacheResponse : public IElectraHTTPStreamResponse
	{
	public:
		virtual ~FSynthesizedCacheResponse() = default;
		FSynthesizedCacheResponse()
		{
			// The new response is itself not cachable.
			Buffer.SetIsCachable(false);
		}

		EStatus GetStatus() override
		{ return EStatus::Completed; }
		EState GetState() override
		{ return EState::Finished; }
		FString GetErrorMessage() override
		{ return FString(); }
		int32 GetHTTPResponseCode() override
		{ return HTTPResponseCode; }
		int64 GetNumResponseBytesReceived() override
		{ return Buffer.GetNumTotalBytesAdded(); }
		int64 GetNumRequestBytesSent() override
		{ return 0; }
		FString GetEffectiveURL() override
		{ return URL; }

		void GetAllHeaders(TArray<FElectraHTTPStreamHeader>& OutHeaders) override
		{ OutHeaders = Headers; }
		FString GetHTTPStatusLine() override
		{ return StatusLine; }
		FString GetContentLengthHeader() override
		{ return GetHeader(TEXT("Content-Length")); }
		FString GetContentRangeHeader() override
		{ return GetHeader(TEXT("Content-Range")); }
		FString GetAcceptRangesHeader() override
		{ return GetHeader(TEXT("Accept-Ranges")); }
		FString GetTransferEncodingHeader() override
		{ return GetHeader(TEXT("Transfer-Encoding")); }
		FString GetContentTypeHeader() override
		{ return GetHeader(TEXT("Content-Type")); }

		IElectraHTTPStreamBuffer& GetResponseData() override
		{ return Buffer; }

		double GetTimeElapsed() override
		{ return 0.0; }
		double GetTimeSinceLastDataArrived() override
		{ return 0.0; }
		double GetTimeUntilNameResolved() override
		{ return TimeUntilNameResolved; }
		double GetTimeUntilConnected() override
		{ return TimeUntilConnected; }
		double GetTimeUntilRequestSent() override
		{ return TimeUntilRequestSent; }
		double GetTimeUntilHeadersAvailable() override
		{ return TimeUntilHeadersAvailable; }
		double GetTimeUntilFirstByte() override
		{ return TimeUntilFirstByte; }
		double GetTimeUntilFinished() override
		{ return TimeUntilFinished; }
		int32 GetTimingTraces(TArray<FTimingTrace>* OutTraces, int32 InNumToRemove) override
		{ return 0; }
		void SetEnableTimingTraces() override
		{ }

		void SetTimeUntilNameResolved(double InTimeUntilNameResolved)
		{ TimeUntilNameResolved = InTimeUntilNameResolved; }
		void SetTimeUntilConnected(double InTimeUntilConnected)
		{ TimeUntilConnected = InTimeUntilConnected; }
		void SetTimeUntilRequestSent(double InTimeUntilRequestSent)
		{ TimeUntilRequestSent = InTimeUntilRequestSent; }
		void SetTimeUntilHeadersAvailable(double InTimeUntilHeadersAvailable)
		{ TimeUntilHeadersAvailable = InTimeUntilHeadersAvailable; }
		void SetTimeUntilFirstByte(double InTimeUntilFirstByte)
		{ TimeUntilFirstByte = InTimeUntilFirstByte; }
		void SetTimeUntilFinished(double InTimeUntilFinished)
		{ TimeUntilFinished = InTimeUntilFinished; }

		void CopyHeadersExceptContentSizesAndDateFrom(IElectraHTTPStreamResponsePtr CachedResponse)
		{
			check(CachedResponse);
			if (CachedResponse.IsValid())
			{
				TArray<FElectraHTTPStreamHeader> CachedHeaders;
				CachedResponse->GetAllHeaders(CachedHeaders);
				// Copy all headers except for those containing the content size, length or date.
				for(auto &Header : CachedHeaders)
				{
					if (!Header.Header.Equals(TEXT("Content-Length"), ESearchCase::IgnoreCase) &&
						!Header.Header.Equals(TEXT("Content-Range"), ESearchCase::IgnoreCase) &&
						!Header.Header.Equals(TEXT("Date"), ESearchCase::IgnoreCase))
					{
						Headers.Emplace(Header);
					}
				}
			}
		}
		void AddHeader(FString InHeader, FString InValue)
		{
			FElectraHTTPStreamHeader Hdr;
			Hdr.Header = MoveTemp(InHeader);
			Hdr.Value = MoveTemp(InValue);
			Headers.Emplace(MoveTemp(Hdr));
		}
		void SetURL(const FString& InUrl)
		{ URL = InUrl; }
		void SetHTTPStatusLine(const FString& InStatusLine)
		{ StatusLine = InStatusLine; }
		void SetHTTPResponseCode(int32 InResponseCode)
		{ HTTPResponseCode = InResponseCode; }
	private:
		FString GetHeader(const TCHAR* const InHeader)
		{
			for(auto& Hdr : Headers)
			{
				if (Hdr.Header.Equals(InHeader, ESearchCase::IgnoreCase))
				{
					return Hdr.Value;
				}
			}
			return FString();
		}
		FElectraHTTPStreamBuffer Buffer;
		TArray<FElectraHTTPStreamHeader> Headers;
		FString URL;
		FString StatusLine;
		int32 HTTPResponseCode = 200;

		double TimeUntilNameResolved = 0.0;
		double TimeUntilConnected = 0.0;
		double TimeUntilRequestSent = 0.0;
		double TimeUntilHeadersAvailable = 0.0;
		double TimeUntilFirstByte = 0.0;
		double TimeUntilFinished = 0.0;
	};

	void EvictToAddSize(int64 ResponseSize);

	TSharedPtr<IElectraPlayerDataCache, ESPMode::ThreadSafe> ExternalCache;
	int64 MaxElementSize = 0;
	int32 MaxNumElements = 0;

	int64 SizeInUse = 0;

	mutable FCriticalSection Lock;
	TArray<TSharedPtrTS<FCacheItem>> Cache;
};

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtrTS<IHTTPResponseCache> IHTTPResponseCache::Create(int64 InMaxByteCapacity, int32 InMaxEntries, TSharedPtr<IElectraPlayerDataCache, ESPMode::ThreadSafe> ExternalCache)
{
	return MakeSharedTS<FHTTPResponseCache>(InMaxByteCapacity, InMaxEntries, ExternalCache);
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

FHTTPResponseCache::FHTTPResponseCache(int64 InMaxByteCapacity, int32 InMaxEntries, TSharedPtr<IElectraPlayerDataCache, ESPMode::ThreadSafe> InExternalCache)
	: ExternalCache(InExternalCache)
{
	// Only configure our cache when there is no external one!
	if (!ExternalCache.IsValid())
	{
		MaxElementSize = InMaxByteCapacity >= 0 ? InMaxByteCapacity : 0;
		MaxNumElements = InMaxEntries >= 0 ? InMaxEntries : 0;
	}
}

FHTTPResponseCache::~FHTTPResponseCache()
{
	FScopeLock ScopeLock(&Lock);
	Cache.Empty();
}


void FHTTPResponseCache::EvictToAddSize(int64 ResponseSize)
{
	FScopeLock ScopeLock(&Lock);
	while(Cache.Num() && (MaxElementSize - SizeInUse < ResponseSize || Cache.Num() >= MaxNumElements))
	{
		TSharedPtrTS<FCacheItem> Item = Cache.Pop();
		SizeInUse -= Item->Response->GetResponseData().GetNumTotalBytesAdded();
	}
}


void FHTTPResponseCache::Disable()
{
	FScopeLock ScopeLock(&Lock);
	Cache.Empty();
	ExternalCache.Reset();
	MaxElementSize = 0;
	MaxNumElements = 0;
	SizeInUse = 0;
}


void FHTTPResponseCache::HandleEntityExpiration()
{
}


void FHTTPResponseCache::CacheEntity(TSharedPtrTS<FCacheItem> EntityToAdd)
{
	if (EntityToAdd.IsValid())
	{
		if (EntityToAdd->EffectiveURL.Len())
		{
			if (!ExternalCache.IsValid())
			{
				int64 SizeRequired = EntityToAdd->Response->GetResponseData().GetNumTotalBytesAdded();
				EvictToAddSize(SizeRequired);
				FScopeLock ScopeLock(&Lock);
				if (SizeInUse + SizeRequired <= MaxElementSize && Cache.Num() < MaxNumElements)
				{
					Cache.Insert(MoveTemp(EntityToAdd), 0);
					SizeInUse += SizeRequired;
				}
			}
			else
			{
				IElectraPlayerDataCache::FItemInfo Info;
				Info.URI = EntityToAdd->RequestedURL;
				switch(EntityToAdd->Quality.StreamType)
				{
					case EStreamType::Video:
						Info.StreamType = IElectraPlayerDataCache::FItemInfo::EStreamType::Video;
						break;
					case EStreamType::Audio:
						Info.StreamType = IElectraPlayerDataCache::FItemInfo::EStreamType::Audio;
						break;
					default:
						Info.StreamType = IElectraPlayerDataCache::FItemInfo::EStreamType::Other;
						break;
				}
				Info.QualityIndex = EntityToAdd->Quality.QualityIndex;
				Info.MaxQualityIndex = EntityToAdd->Quality.MaxQualityIndex;

				if (EntityToAdd->Range.GetStart() == 0 && EntityToAdd->Range.GetEndIncluding() + 1 == EntityToAdd->Range.GetDocumentSize())
				{
					Info.Range.TotalSize = EntityToAdd->Range.GetDocumentSize();
				}
				else
				{
					Info.Range.Start = EntityToAdd->Range.GetStart();
					Info.Range.EndIncluding = EntityToAdd->Range.GetEndIncluding();
					Info.Range.TotalSize = EntityToAdd->Range.GetDocumentSize();
				}
				const uint8* Data = nullptr;
				int64 DataSize = 0;
				EntityToAdd->Response->GetResponseData().GetBaseBuffer(Data, DataSize);
				TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> Buf = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();
				Buf->AddUninitialized(DataSize);
				FMemory::Memcpy(Buf->GetData(), Data, DataSize);
				ExternalCache->AddElementToCache(Info, Buf);
			}
		}
	}
}

FHTTPResponseCache::EScatterResult FHTTPResponseCache::GetScatteredCacheEntity(TSharedPtrTS<FCacheItem>& OutScatteredCachedEntity, const FString& URL, const ElectraHTTPStream::FHttpRange& InRange, const FQualityInfo& InForQuality)
{
	if (!ExternalCache.IsValid())
	{
		FScopeLock lock(&Lock);
		// Get a list of all cached blocks for this URL.
		TArray<TSharedPtrTS<FCacheItem>> CachedBlocks;
		for(auto &Entry : Cache)
		{
			if (Entry->RequestedURL.Equals(URL) || Entry->EffectiveURL.Equals(URL))
			{
				CachedBlocks.Add(Entry);
			}
		}

		ElectraHTTPStream::FHttpRange Range(InRange);

		// Set up the default output that encompasses the request.
		// This is what needs to be fetched for cache misses.
		OutScatteredCachedEntity = MakeSharedTS<IHTTPResponseCache::FCacheItem>();
		OutScatteredCachedEntity->RequestedURL = URL;
		OutScatteredCachedEntity->EffectiveURL = URL;
		OutScatteredCachedEntity->Range = Range;

		// Quick out if there are no cached blocks at all.
		if (CachedBlocks.Num() == 0)
		{
			return EScatterResult::Miss;
		}

		// Adjust the input range to valid values.
		Range.DocumentSize = CachedBlocks[0]->Range.GetDocumentSize();
		if (Range.GetStart() < 0)
		{
			Range.SetStart(0);
		}
		if (Range.GetEndIncluding() < 0)
		{
			Range.SetEndIncluding(Range.GetDocumentSize() - 1);
		}

		// Sort the blocks by ascending range.
		CachedBlocks.Sort([](const TSharedPtrTS<FCacheItem>& e1, const TSharedPtrTS<FCacheItem>& e2)
		{
			return e1->Range.GetStart() < e2->Range.GetStart();
		});

		// Find a cached block that overlaps the requested range.
		int64 rs = Range.GetStart();
		int64 re = Range.GetEndIncluding();
		int32 firstIndex;
		for(firstIndex=0; firstIndex<CachedBlocks.Num(); ++firstIndex)
		{
			// Any overlap?
			const int64 cbstart = CachedBlocks[firstIndex]->Range.GetStart();
			const int64 cbend = CachedBlocks[firstIndex]->Range.GetEndIncluding();
			if (re >= cbstart && rs <= cbend)
			{
				// Is the request start before the cached block, ie we need to get additional data at the beginning?
				if (rs < cbstart)
				{
					// The end of the requested range may also not be covered by the cache, but that is not relevant
					// here as we move forward sequentially, one overlap at a time.
					// Data up to the beginning of this cached block must be fetched first.
					OutScatteredCachedEntity->Range = Range;
					OutScatteredCachedEntity->Range.SetEndIncluding(cbstart - 1);
					OutScatteredCachedEntity->EffectiveURL = CachedBlocks[firstIndex]->EffectiveURL;
					return EScatterResult::PartialHit;
				}
				else
				{
					break;
				}
			}
		}
		// No overlap anywhere?
		if (firstIndex == CachedBlocks.Num())
		{
			return EScatterResult::Miss;
		}

		// Exact match?
		if (rs == CachedBlocks[firstIndex]->Range.GetStart() && re == CachedBlocks[firstIndex]->Range.GetEndIncluding())
		{
			OutScatteredCachedEntity->Response = MakeShared<FWrappedCacheResponse, ESPMode::ThreadSafe>(CachedBlocks[firstIndex]->Response);
			OutScatteredCachedEntity->EffectiveURL = CachedBlocks[firstIndex]->EffectiveURL;
			Cache.Remove(CachedBlocks[firstIndex]);
			Cache.Insert(CachedBlocks[firstIndex], 0);
			return EScatterResult::FullHit;
		}

		// Set up the partial cache response.
		TSharedPtr<FSynthesizedCacheResponse, ESPMode::ThreadSafe> Response = MakeShared<FSynthesizedCacheResponse, ESPMode::ThreadSafe>();
		Response->SetURL(CachedBlocks[firstIndex]->EffectiveURL);
		OutScatteredCachedEntity->EffectiveURL = CachedBlocks[firstIndex]->EffectiveURL;
		Response->CopyHeadersExceptContentSizesAndDateFrom(CachedBlocks[firstIndex]->Response);
		Response->SetHTTPStatusLine(CachedBlocks[firstIndex]->Response->GetHTTPStatusLine());

		// There is some overlap. Find how many consecutive bytes we can get from the cached blocks.
		TArray<TSharedPtrTS<FCacheItem>> TouchedBlocks;
		while(1)
		{
			int64 cbstart = CachedBlocks[firstIndex]->Range.GetStart();
			int64 cbend = CachedBlocks[firstIndex]->Range.GetEndIncluding();
			int64 last = re < cbend ? re : cbend;
			int64 offset = rs - cbstart;
			int64 numAvail = last + 1 - cbstart - offset;

			// Add the bytes from the cache to the new response.
			Response->GetResponseData().AddData(CachedBlocks[firstIndex]->Response->GetResponseData(), offset, numAvail);

			// Remember which blocks we just touched.
			TouchedBlocks.Add(CachedBlocks[firstIndex]);

			// Move up the start to the end of this cached block.
			rs = last + 1;

			// End fully included?
			if (re <= cbend)
			{
				break;
			}
			// Is there another cached block?
			if (++firstIndex >= CachedBlocks.Num())
			{
				break;
			}
			// Does its start coincide with what we need next?
			if (CachedBlocks[firstIndex]->Range.GetStart() != rs)
			{
				break;
			}
		}

		// Move the blocks we touched to the front of the cached blocks (LRU).
		for(auto &Touched : TouchedBlocks)
		{
			Cache.Remove(Touched);
			Cache.Insert(Touched, 0);
		}

		Range.SetEndIncluding(rs - 1);
		if (InRange.IsSet())
		{
			Response->AddHeader(TEXT("Content-Range"), FString::Printf(TEXT("bytes %lld-%lld/%lld"), (long long int)Range.GetStart(), (long long int)Range.GetEndIncluding(), (long long int)Range.GetDocumentSize()));
			Response->SetHTTPResponseCode(206);
			FString StatusLine = Response->GetHTTPStatusLine();
			StatusLine.ReplaceInline(TEXT("200"), TEXT("206"));
			Response->SetHTTPStatusLine(StatusLine);
		}
		Response->AddHeader(TEXT("Content-Length"), FString::Printf(TEXT("%lld"), (long long int)Response->GetResponseData().GetNumTotalBytesAdded()));
		Response->GetResponseData().SetLengthFromResponseHeader(Response->GetResponseData().GetNumTotalBytesAdded());
		Response->GetResponseData().SetEOS();

		/*
		Response->SetTimeUntilNameResolved(double InTimeUntilNameResolved);
		Response->SetTimeUntilConnected(double InTimeUntilConnected);
		Response->SetTimeUntilRequestSent(double InTimeUntilRequestSent);
		Response->SetTimeUntilHeadersAvailable(double InTimeUntilHeadersAvailable);
		Response->SetTimeUntilFirstByte(double InTimeUntilFirstByte);
		Response->SetTimeUntilFinished(double InTimeUntilFinished);
		*/

		OutScatteredCachedEntity->Response = Response;
		OutScatteredCachedEntity->Range = Range;

		return EScatterResult::PartialHit;
	}
	else
	{
		IElectraPlayerDataCache::FItemInfo Info;
		Info.URI = URL;
		switch(InForQuality.StreamType)
		{
			case EStreamType::Video:
				Info.StreamType = IElectraPlayerDataCache::FItemInfo::EStreamType::Video;
				break;
			case EStreamType::Audio:
				Info.StreamType = IElectraPlayerDataCache::FItemInfo::EStreamType::Audio;
				break;
			default:
				Info.StreamType = IElectraPlayerDataCache::FItemInfo::EStreamType::Other;
				break;
		}
		Info.QualityIndex = InForQuality.QualityIndex;
		Info.MaxQualityIndex = InForQuality.MaxQualityIndex;
		Info.Range.Start = InRange.GetStart();
		Info.Range.EndIncluding = InRange.GetEndIncluding();
		Info.Range.TotalSize = -1;

		struct FResult
		{
			IElectraPlayerDataCache::ECacheResult Result;
			IElectraPlayerDataCache::FItemInfo ItemInfoFromCache;
			IElectraPlayerDataCache::FCacheDataPtr DataFromCache;
			FMediaEvent Event;
		};
		TSharedPtrTS<FResult> CacheResult = MakeSharedTS<FResult>();

		IElectraPlayerDataCache::FCachedDataReadCompleted FinishedDelegate;
		FinishedDelegate.BindLambda([CacheResult](IElectraPlayerDataCache::ECacheResult InResult, IElectraPlayerDataCache::FItemInfo InItemInfoFromCache, IElectraPlayerDataCache::FCacheDataPtr InDataFromCache)
		{
			CacheResult->Result = InResult;
			CacheResult->ItemInfoFromCache = InItemInfoFromCache;
			CacheResult->DataFromCache = InDataFromCache;
			CacheResult->Event.Signal();
		});
		ExternalCache->GetElementFromCache(Info, FinishedDelegate);
		CacheResult->Event.Wait();

		OutScatteredCachedEntity = MakeSharedTS<IHTTPResponseCache::FCacheItem>();
		OutScatteredCachedEntity->RequestedURL = URL;
		OutScatteredCachedEntity->EffectiveURL = URL;
		OutScatteredCachedEntity->Range = InRange;

		if (CacheResult->Result == IElectraPlayerDataCache::ECacheResult::Miss)
		{
			// Set up the default output that encompasses the request.
			// This is what needs to be fetched for cache misses.
			return EScatterResult::Miss;
		}
		else
		{
			TSharedPtr<FSynthesizedCacheResponse, ESPMode::ThreadSafe> Response = MakeShared<FSynthesizedCacheResponse, ESPMode::ThreadSafe>();
			Response->SetURL(URL);

			// Add the bytes from the cache to the new response.
			if (CacheResult->DataFromCache.IsValid())
			{
				Response->GetResponseData().AddData(*CacheResult->DataFromCache.Get());
			}

			ElectraHTTPStream::FHttpRange Range(InRange);
			Range.SetStart(CacheResult->ItemInfoFromCache.Range.Start);
			Range.SetEndIncluding(CacheResult->ItemInfoFromCache.Range.EndIncluding);
			Range.DocumentSize = CacheResult->ItemInfoFromCache.Range.TotalSize;
			if (Range.IsSet())
			{
				Response->AddHeader(TEXT("Content-Range"), FString::Printf(TEXT("bytes %lld-%lld/%lld"), (long long int)Range.GetStart(), (long long int)Range.GetEndIncluding(), (long long int)Range.GetDocumentSize()));
				Response->SetHTTPResponseCode(206);
				Response->SetHTTPStatusLine(TEXT("HTTP/1.1 206 Partial Content"));
			}
			else
			{
				Response->SetHTTPResponseCode(200);
				Response->SetHTTPStatusLine(TEXT("HTTP/1.1 200 OK"));
			}
			Response->AddHeader(TEXT("Content-Length"), FString::Printf(TEXT("%lld"), (long long int)Response->GetResponseData().GetNumTotalBytesAdded()));
			Response->GetResponseData().SetLengthFromResponseHeader(Response->GetResponseData().GetNumTotalBytesAdded());
			Response->GetResponseData().SetEOS();

			OutScatteredCachedEntity->Response = Response;
			OutScatteredCachedEntity->Range = Range;

			return EScatterResult::FullHit;
		}
	}
}


} // namespace Electra

