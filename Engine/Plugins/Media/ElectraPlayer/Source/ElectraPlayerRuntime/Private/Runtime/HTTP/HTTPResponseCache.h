// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "ParameterDictionary.h"
#include "ElectraHTTPStream.h"
#include "Utilities/HttpRangeHeader.h"
#include "IElectraPlayerDataCache.h"
#include "StreamTypes.h"

namespace Electra
{

class IHTTPResponseCache
{
public:
	struct FQualityInfo
	{
		EStreamType StreamType = EStreamType::Unsupported;
		int32 QualityIndex = 0;
		int32 MaxQualityIndex = 0;
	};
	struct FCacheItem
	{
		FString RequestedURL;
		FString EffectiveURL;
		ElectraHTTPStream::FHttpRange Range;
		FTimeValue ExpiresAtUTC;
		IElectraHTTPStreamResponsePtr Response;
		FQualityInfo Quality;
	};

	static TSharedPtrTS<IHTTPResponseCache> Create(int64 InMaxByteCapacity, int32 InMaxEntries, TSharedPtr<IElectraPlayerDataCache, ESPMode::ThreadSafe> ExternalCache);

	virtual ~IHTTPResponseCache() = default;

	/**
	 * Call this to disable the cache. Caching makes no sense for Live video streams.
	 */
	virtual void Disable() = 0;

	/**
	 * Call this periodically to handle expiration times of cached entities.
	 */
	virtual void HandleEntityExpiration() = 0;

	/**
	 * Add an entity to the cache.
	 */
	virtual void CacheEntity(TSharedPtrTS<FCacheItem> EntityToAdd) = 0;


	enum EScatterResult
	{
		FullHit,			//!< The entire request is present in the cache
		PartialHit,			//!< Partial data is present in the cache
		Miss				//!< No part of the request is present in the cache
	};

	/**
	 * Returns the first cached portion of a request.
	 * This is used to create responses for which partially cached data already exist and break the request apart
	 * to fetch the next portion that is not cached yet.
	 * 
	 * Returns false to indicate that the range specified in the (otherwise empty) cached response must be requested from the origin.
	 * Returns true if a - possibly partial - cached response exists. The range that is cached is given in the cache item.
	 */
	virtual EScatterResult GetScatteredCacheEntity(TSharedPtrTS<FCacheItem>& OutScatteredCachedEntity, const FString& URL, const ElectraHTTPStream::FHttpRange& Range, const FQualityInfo& ForQuality) = 0;

protected:
	IHTTPResponseCache() = default;
	IHTTPResponseCache(const IHTTPResponseCache& other) = delete;
	IHTTPResponseCache& operator = (const IHTTPResponseCache& other) = delete;
};


} // namespace Electra

