// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "SynchronizedClock.h"


namespace Electra
{

	/**
	 *
	 */
	class IInfoLog
	{
	public:
		enum ELevel
		{
			// This is a bit mask to allow filtering on exact level (like: Info and Error (but not Warning))
			// A single event is only exactly one of those however!
			Verbose = 1 << 0,
			Info = 1 << 1,
			Warning = 1 << 2,
			Error = 1 << 3
		};
		static const TCHAR* GetLevelName(ELevel InLevel)
		{
			switch (InLevel)
			{
				case Verbose:
					return TEXT("Verbose");
				case Info:
					return TEXT("Info");
				case Warning:
					return TEXT("Warning");
				case Error:
					return TEXT("Error");
			}
			return TEXT("???");
		}

	};


} // namespace Electra


