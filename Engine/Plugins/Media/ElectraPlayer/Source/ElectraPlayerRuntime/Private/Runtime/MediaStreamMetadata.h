// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MP4Metadata.h"

namespace Electra
{

// We use the mp4 metadata parser as-is for now.
using FMetadataParser = MP4Utilities::FMP4MetadataParser;

} // namespace Electra
