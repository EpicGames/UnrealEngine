// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"


/**
 * "Optional" value. Includes a boolean to indicate whether or not it is set.
**/
template<typename T>
struct TMediaOptionalValue
{
	TMediaOptionalValue() : bValueIsSet(false) {}
	TMediaOptionalValue(const T& v) : OptionalValue(v), bValueIsSet(true) {}
	void Set(const T& v) { OptionalValue = v; bValueIsSet = true; }
	void SetIfNot(const T& v) { if (!IsSet()) Set(v); }
	bool IsSet() const { return bValueIsSet; }
	const T& Value() const { return OptionalValue; }
	T GetWithDefault(const T& Default) const { return bValueIsSet ? OptionalValue : Default; }
	void Reset() { bValueIsSet = false; }
private:
	// Hide assignment from public view
	TMediaOptionalValue& operator = (const T& v) { Set(v); return(*this); }
	T		OptionalValue;
	bool	bValueIsSet;
};
