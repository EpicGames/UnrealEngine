// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/PimplPtr.h"
#include "IMediaTimeSource.h"
#include "MediaSamples.h"

#include "ElectraTextureSample.h"
#include "IElectraAudioSample.h"

#include "PlayerClock.h"
#include "ParameterDictionary.h"

#include <atomic>

namespace Electra
{

	class IOutputHandlerBase
	{
	public:
		virtual ~IOutputHandlerBase() = default;

		virtual void SetMediaSamples(TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> InMediaSamples)
		{
			CurrentMediaSamples = MoveTemp(InMediaSamples);
		}

		virtual bool PreparePool(const FParamDict& InParameters) = 0;
		virtual void ClosePool() = 0;
		virtual const FParamDict& GetPoolProperties() = 0;

		virtual void SetClock(TSharedPtr<IMediaRenderClock, ESPMode::ThreadSafe> InClock) = 0;

		virtual void StartOutput() = 0;
		virtual void StopOutput() = 0;
		virtual void Flush() = 0;

		enum class EBufferResult
		{
			Ok,
			NoBuffer
		};

		struct FEnqueuedSampleInfo
		{
			FMediaTimeStamp Timestamp;
			FTimespan Duration;
			uint64 SampleID;
			bool bIsDummy;
		};

		virtual FTimespan GetEnqueuedSampleDuration() = 0;
		virtual void GetEnqueuedSampleInfo(TArray<FEnqueuedSampleInfo>& OutOptionalSampleInfos) = 0;

	protected:
		TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> CurrentMediaSamples;
	};


	class FOutputHandlerVideo : public IOutputHandlerBase, public TSharedFromThis<FOutputHandlerVideo, ESPMode::ThreadSafe>
	{
	public:
		FOutputHandlerVideo();
		virtual ~FOutputHandlerVideo();

		/** Sets the texture pool from where ObtainBuffer() will get an output sample. */
		void SetOutputTexturePool(TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> InOutputTexturePool);

		bool PreparePool(const FParamDict& InParameters) override;
		void ClosePool() override;
		const FParamDict& GetPoolProperties() override;

		void SetClock(TSharedPtr<IMediaRenderClock, ESPMode::ThreadSafe> InClock) override;
		void StartOutput() override;
		void StopOutput() override;
		void Flush() override;

		FTimespan GetEnqueuedSampleDuration() override;
		void GetEnqueuedSampleInfo(TArray<FEnqueuedSampleInfo>& OutOptionalSampleInfos) override;

		bool CanReceiveOutputSample();
		EBufferResult ObtainOutputSample(FElectraTextureSamplePtr& OutTextureSample);
		enum class EReturnSampleType
		{
			SendToQueue,
			DontSendToQueue,
			DummySample
		};

		void ReturnOutputSample(FElectraTextureSamplePtr InTextureSampleToReturn, EReturnSampleType InSendToOutputQueueType);

		/** Returns the total number of video samples added to the queue. */
		uint64 GetTotalAddedSamples();
		/** Returns the total number of samples that were dropped from the queue. */
		uint64 GetTotalNumDroppedSamples();
		/** Returns the number of dropped samples since the last time this methods was called. */
		uint64 GetNewlyNumDroppedSamples(bool bInClear);

	private:
		void ReleaseSampleToPool(IElectraTextureSampleBase* InTextureSampleToReturn, uint64 InSampleID);
		void SendAllPendingSamples();

		TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> OutputTexturePool;
		TSharedPtr<IMediaRenderClock, ESPMode::ThreadSafe> Clock;

		struct FPendingSample
		{
			FEnqueuedSampleInfo SampleInfo;
			FElectraTextureSamplePtr Sample;
		};
		FCriticalSection PendingOutputSamplesLock;
		TArray<FPendingSample> PendingOutputSamples;
		FParamDict PoolProperties;
		std::atomic<uint64> NumTotalSamplesAdded { 0 };
		std::atomic<uint64> NumTotalSamplesDropped { 0 };
		std::atomic<uint64> TotalSamplesDroppedAtLastCall { 0 };
		int32 NumOutputSamples = 0;
		bool bSendOutput = false;
		bool bDidSendFirst = false;

		FCriticalSection EnqueuedSampleInfosLock;
		TArray<FEnqueuedSampleInfo> EnqueuedSampleInfos;
		FTimespan EnqueuedSampleDuration;
		uint64 NextSampleID = 0;
	};


	class FOutputHandlerAudio : public IOutputHandlerBase, public TSharedFromThis<FOutputHandlerAudio, ESPMode::ThreadSafe>
	{
	public:
		FOutputHandlerAudio();
		virtual ~FOutputHandlerAudio();

		/** Sets the sample pool from where ObtainBuffer() will get an output sample. */
		void SetOutputAudioSamplePool(TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InOutputAudioSamplePool);

		bool PreparePool(const FParamDict& InParameters) override;
		void ClosePool() override;
		const FParamDict& GetPoolProperties() override;

		void SetClock(TSharedPtr<IMediaRenderClock, ESPMode::ThreadSafe> InClock) override;
		void StartOutput() override;
		void StopOutput() override;
		void Flush() override;

		FTimespan GetEnqueuedSampleDuration() override;
		void GetEnqueuedSampleInfo(TArray<FEnqueuedSampleInfo>& OutOptionalSampleInfos) override;

		bool CanReceiveOutputSample();
		EBufferResult ObtainOutputSample(FElectraAudioSamplePtr& OutTextureSample);
		enum class EReturnSampleType
		{
			SendToQueue,
			DontSendToQueue,
			DummySample
		};

		void ReturnOutputSample(FElectraAudioSamplePtr InTextureSampleToReturn, EReturnSampleType InSendToOutputQueueType);

		void SetPlaybackRate(double InCurrentPlaybackRate, double InIntendedPlaybackRate, bool bInCurrentlyPaused);
		FTimeRange GetSupportedRenderRateScale();
		void SetPlayRateScale(double InNewScale);
		double GetPlayRateScale();

	private:
		// Constants used for time stretching
		const uint32 NumInterpolationSamplesAt48kHz = 60;
		const double MinPlaybackSpeed = 0.8;
		const double MaxPlaybackSpeed = 1.5;
		const double MinResampleSpeed = 0.98;
		const double MaxResampleSpeed = 1.02;

		void ProcessSampleBlock(TArray<FElectraAudioSamplePtr>& OutProcessedSampleBlocks, FElectraAudioSamplePtr InSampleBlock, double InRate);
		void ReleaseSampleToPool(FElectraAudioSample* InTextureSampleToReturn, uint64 InSampleID);
		void SendPendingSamples();

		TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> OutputAudioSamplePool;
		TSharedPtr<IMediaRenderClock, ESPMode::ThreadSafe> Clock;

		struct FPendingSample
		{
			FEnqueuedSampleInfo SampleInfo;
			FElectraAudioSamplePtr Sample;
		};
		FCriticalSection PendingOutputSamplesLock;
		TArray<FPendingSample> PendingOutputSamples;
		FParamDict PoolProperties;
		int32 NumOutputSamples = 0;
		bool bSendOutput = false;

		double CurrentPlaybackRate = 0.0;
		double IntendedPlaybackRate = 0.0;

		FCriticalSection EnqueuedSampleInfosLock;
		TArray<FEnqueuedSampleInfo> EnqueuedSampleInfos;
		FTimespan EnqueuedSampleDuration;
		uint64 NextSampleID = 0;

		struct FAudioVars;
		TPimplPtr<FAudioVars> AudioVars;
	};


	namespace OutputHandlerOptionKeys
	{
		static const FName NumBuffers(TEXT("num_buffers"));
		static const FName PTS(TEXT("pts"));
		static const FName Duration(TEXT("duration"));
	}

} // namespace Electra
