// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputHandler.h"
#include "Utilities/Utilities.h"

/***************************************************************************************************************************************************/

#ifndef WITH_SOUNDTOUCHZ
#define WITH_SOUNDTOUCHZ 0
#endif
#if WITH_SOUNDTOUCHZ
#include "SoundTouchZ.h"
#endif

/***************************************************************************************************************************************************/
//#define ENABLE_PLAYRATE_OVERRIDE_CVAR

#ifdef ENABLE_PLAYRATE_OVERRIDE_CVAR
#include "HAL/IConsoleManager.h"
static TAutoConsoleVariable<float> CVarElectraPR(TEXT("Electra.PR"), 1.0f, TEXT("Playback rate"), ECVF_Default);
#endif

/***************************************************************************************************************************************************/

namespace Electra
{

	class ITimestretcher
	{
	public:
		ITimestretcher() = default;
		virtual ~ITimestretcher() = default;
		virtual void Initialize(FElectraAudioSamplePtr InReferenceSample) {}
		virtual void ProcessSample(TArray<FElectraAudioSamplePtr>& OutProcessedSamples, FElectraAudioSamplePtr InSample, TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InAudioSamplePool, double InRate) {}
		virtual void Finalize(TArray<FElectraAudioSamplePtr>& OutProcessedSamples, TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InAudioSamplePool) {}
#if WITH_SOUNDTOUCHZ
		static TUniquePtr<ITimestretcher> Create();
#else
		static TUniquePtr<ITimestretcher> Create()
		{ return nullptr; }
#endif
	};


	// Audio manipulation variables
	struct FOutputHandlerAudio::FAudioVars
	{
		struct FConfig
		{
			uint32 SampleRate = 0;
			uint32 NumChannels = 0;
			bool DiffersFromChannelCount(FElectraAudioSamplePtr InSample)
			{ return InSample->GetChannels() != NumChannels; }
			bool DiffersFromSampleRate(FElectraAudioSamplePtr InSample)
			{ return InSample->GetSampleRate() != SampleRate; }
			void Update(FElectraAudioSamplePtr InSample)
			{
				SampleRate = InSample->GetSampleRate();
				NumChannels = InSample->GetChannels();
			}
			void Reset()
			{
				SampleRate = 0;
				NumChannels = 0;
			}
		};

		enum class EState
		{
			Disengaged,
			Engaged
		};

		FConfig CurrentConfig;
		TUniquePtr<ITimestretcher> Timestretcher;
		EState CurrentState = EState::Disengaged;
		float LastSampleValuePerChannel[256];
		double RateScale = 1.0;

		FAudioVars()
		{
			Reset();
			CurrentConfig.Reset();
			Timestretcher = ITimestretcher::Create();
		}
		~FAudioVars()
		{
		}
		void Reset()
		{
			CurrentState = EState::Disengaged;
			FMemory::Memzero(LastSampleValuePerChannel);
		}
		void UpdateLastSampleValues(const TArray<FElectraAudioSamplePtr>& InProcessedSampleBlocks)
		{
			if (InProcessedSampleBlocks.Num())
			{
				FElectraAudioSamplePtr Sample = InProcessedSampleBlocks.Last();
				const uint32 NumSamples = Sample->GetFrames();
				if (NumSamples)
				{
					const uint32 NumChannels = Sample->GetChannels();
					const float* InSamples = (const float*)Sample->GetBuffer() + (NumSamples - 1) * NumChannels;
					check(NumChannels <= UE_ARRAY_COUNT(LastSampleValuePerChannel));
					FMemory::Memcpy(LastSampleValuePerChannel, InSamples, sizeof(float) * NumChannels);
				}
			}
		}
		void InterpolateStartFromLastSampleValue(FElectraAudioSamplePtr InSample, uint32 NumInterpolationSamples)
		{
			if (InSample)
			{
				// Number of samples over which to interpolate depends on sampling rate.
				const uint32 NumSamples = InSample->GetFrames();
				uint32 NumInter = Utils::Min((uint32)(NumInterpolationSamples * (InSample->GetSampleRate() / 48000.0)), NumSamples);
				if (NumInter)
				{
					const uint32 NumChannels = InSample->GetChannels();
					float* InSamples = (float *)InSample->GetWritableBuffer();
					float *LastInterpSample = InSamples + (NumInter-1) * NumChannels;
					const float Step = 1.0f / (NumInter-1);
					for(uint32 i=1; i<NumInter; ++i)
					{
						for(uint32 j=0; j<NumChannels; ++j)
						{
							InSamples[j] = LastSampleValuePerChannel[j] + ((LastInterpSample[j] - LastSampleValuePerChannel[j]) * (i*Step));
						}
						InSamples += NumChannels;
					}
				}
			}
		}

	};

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


	FOutputHandlerAudio::FOutputHandlerAudio()
	{
		AudioVars = MakePimpl<FAudioVars>();
	}

	FOutputHandlerAudio::~FOutputHandlerAudio()
	{
		AudioVars.Reset();
	}

	bool FOutputHandlerAudio::PreparePool(const FParamDict& InParameters)
	{
		// How many samples are asked for?
		NumOutputSamples = (int32) InParameters.GetValue(OutputHandlerOptionKeys::NumBuffers).SafeGetInt64(0);
		// Set this in this pool's configuration.
		PoolProperties.Set(OutputHandlerOptionKeys::NumBuffers, FVariantValue((int64)NumOutputSamples));
		// For now this is all we do. The pool itself is not being limited.
		return true;
	}

	void FOutputHandlerAudio::ClosePool()
	{
		ensure(PendingOutputSamples.IsEmpty());
		NumOutputSamples = 0;
		PoolProperties.Clear();
	}

	const FParamDict& FOutputHandlerAudio::GetPoolProperties()
	{
		return PoolProperties;
	}

	void FOutputHandlerAudio::SetClock(TSharedPtr<IMediaRenderClock, ESPMode::ThreadSafe> InClock)
	{
		Clock = MoveTemp(InClock);
	}

	void FOutputHandlerAudio::StartOutput()
	{
		bSendOutput = true;
		SendPendingSamples();
	}

	void FOutputHandlerAudio::StopOutput()
	{
		bSendOutput = false;
	}

	void FOutputHandlerAudio::Flush()
	{
		// Anything not sent yet we unbind our release notification callback and just drop here.
		PendingOutputSamplesLock.Lock();
		while(PendingOutputSamples.Num())
		{
			FPendingSample ps = PendingOutputSamples[0];
			PendingOutputSamples.RemoveAt(0);
			ps.Sample->GetReleaseDelegate().Unbind();
			PendingOutputSamplesLock.Unlock();
			ps.Sample.Reset();
			PendingOutputSamplesLock.Lock();
		}
		PendingOutputSamplesLock.Unlock();

		// Flush any samples in the media queue.
		auto MediaSamples = CurrentMediaSamples;
		if (MediaSamples.IsValid())
		{
			TRange<FTimespan> AllTime(FTimespan::MinValue(), FTimespan::MaxValue());
			TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> FlushSample;
			while(MediaSamples->FetchAudio(AllTime, FlushSample))
			{ }
		}

		// Anything that has not been returned by now we drop from the sample info stats.
		EnqueuedSampleInfosLock.Lock();
		EnqueuedSampleInfos.Empty();
		EnqueuedSampleDuration = FTimespan::Zero();
		EnqueuedSampleInfosLock.Unlock();
	}

	FTimespan FOutputHandlerAudio::GetEnqueuedSampleDuration()
	{
		FScopeLock lock(&EnqueuedSampleInfosLock);
		return EnqueuedSampleDuration;
	}

	void FOutputHandlerAudio::GetEnqueuedSampleInfo(TArray<FEnqueuedSampleInfo>& OutOptionalSampleInfos)
	{
		FScopeLock lock(&EnqueuedSampleInfosLock);
		OutOptionalSampleInfos = EnqueuedSampleInfos;
	}

	void FOutputHandlerAudio::SetOutputAudioSamplePool(TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InOutputAudioSamplePool)
	{
		OutputAudioSamplePool = MoveTemp(InOutputAudioSamplePool);
	}

	bool FOutputHandlerAudio::CanReceiveOutputSample()
	{
		auto MediaSamples = CurrentMediaSamples;
		if (!MediaSamples.IsValid())
		{
			return false;
		}
		bool bCanRcv = true;
		PendingOutputSamplesLock.Lock();
		int32 NumPendingOutputSamples = PendingOutputSamples.Num();
		PendingOutputSamplesLock.Unlock();
		if (1 + NumPendingOutputSamples > NumOutputSamples)
		{
			return false;
		}
		bCanRcv = MediaSamples->CanReceiveAudioSamples(1 + NumPendingOutputSamples);
		return bCanRcv;
	}

	IOutputHandlerBase::EBufferResult FOutputHandlerAudio::ObtainOutputSample(FElectraAudioSamplePtr& OutAudioSample)
	{
		if (OutputAudioSamplePool)
		{
			OutAudioSample = OutputAudioSamplePool->AcquireShared();
			return IOutputHandlerBase::EBufferResult::Ok;
		}
		return IOutputHandlerBase::EBufferResult::NoBuffer;
	}

	void FOutputHandlerAudio::ReturnOutputSample(FElectraAudioSamplePtr InAudioSampleToReturn, EReturnSampleType InSendToOutputQueueType)
	{
		auto MediaSamples = CurrentMediaSamples;
		if (InAudioSampleToReturn && InSendToOutputQueueType != EReturnSampleType::DontSendToQueue && MediaSamples.IsValid())
		{
			TArray<FElectraAudioSamplePtr> ProcessedSampleBlocks;
			double Rate = GetPlayRateScale();

		#ifdef ENABLE_PLAYRATE_OVERRIDE_CVAR
			Rate = Utils::Max(Utils::Min((double)CVarElectraPR.GetValueOnAnyThread(), MaxPlaybackSpeed), MinPlaybackSpeed);
		#endif

			ProcessSampleBlock(ProcessedSampleBlocks, InAudioSampleToReturn, Rate);
			while(!ProcessedSampleBlocks.IsEmpty())
			{
				InAudioSampleToReturn = ProcessedSampleBlocks[0];
				ProcessedSampleBlocks.RemoveAt(0);

				uint64 SampleID = ++NextSampleID;

				// Log the timestamp and duration of the sample
				FEnqueuedSampleInfo si;
				si.Timestamp = InAudioSampleToReturn->GetTime();
				si.Duration = InAudioSampleToReturn->GetDuration();
				si.SampleID = SampleID;
				si.bIsDummy = InSendToOutputQueueType == EReturnSampleType::DummySample;
				EnqueuedSampleInfosLock.Lock();
				EnqueuedSampleDuration += si.Duration;
				EnqueuedSampleInfos.Emplace(si);	// do not move, this is needed below.
				EnqueuedSampleInfosLock.Unlock();

				// Set our notification callback when the sample is being returned to the pool.
				InAudioSampleToReturn->GetReleaseDelegate().BindThreadSafeSP(AsShared(), &FOutputHandlerAudio::ReleaseSampleToPool, SampleID);

				FPendingSample ps;
				ps.SampleInfo = si;
				ps.Sample = MoveTemp(InAudioSampleToReturn);
				PendingOutputSamplesLock.Lock();
				PendingOutputSamples.Emplace(MoveTemp(ps));
				PendingOutputSamplesLock.Unlock();
			}

			SendPendingSamples();
		}
	}

	void FOutputHandlerAudio::SendPendingSamples()
	{
		auto MediaSamples = CurrentMediaSamples;
		if (!MediaSamples.IsValid())
		{
			return;
		}
		while(bSendOutput)
		{
			if (!MediaSamples->CanReceiveAudioSamples(1))
			{
				return;
			}
			else
			{
				PendingOutputSamplesLock.Lock();
				if (PendingOutputSamples.IsEmpty())
				{
					PendingOutputSamplesLock.Unlock();
					return;
				}
				FPendingSample ps = PendingOutputSamples[0];
				PendingOutputSamples.RemoveAt(0);
				PendingOutputSamplesLock.Unlock();
				if (!ps.SampleInfo.bIsDummy)
				{
					MediaSamples->AddAudio(ps.Sample.ToSharedRef());
				}
				ps.Sample.Reset();
			}
		}
	}

	void FOutputHandlerAudio::ReleaseSampleToPool(FElectraAudioSample* InAudioSampleToReturn, uint64 InSampleID)
	{
		FTimeValue RenderTime;

		// Remove the sample stats.
		FEnqueuedSampleInfo si;
		EnqueuedSampleInfosLock.Lock();
		for(int32 i=0, iMax=EnqueuedSampleInfos.Num(); i<iMax; ++i)
		{
			if (EnqueuedSampleInfos[i].SampleID == InSampleID)
			{
				RenderTime.SetFromTimespan(EnqueuedSampleInfos[i].Timestamp.GetTime(), EnqueuedSampleInfos[i].Timestamp.GetIndexValue());
				EnqueuedSampleDuration -= EnqueuedSampleInfos[i].Duration;
				EnqueuedSampleInfos.RemoveAt(i);
				break;
			}
		}
		EnqueuedSampleInfosLock.Unlock();

		if (Clock && RenderTime.IsValid())
		{
			Clock->SetCurrentTime(IMediaRenderClock::ERendererType::Audio, RenderTime);
		}

		// Send any new pending samples, "pumping" the output as much as possible to avoid
		// getting stuck with all pending samples due to `CanReceiveOutputSample()` returning
		// 'false' and thus no call to `ReturnOutputSample()` being made.
		SendPendingSamples();
	}



	void FOutputHandlerAudio::SetPlaybackRate(double InCurrentPlaybackRate, double InIntendedPlaybackRate, bool bInCurrentlyPaused)
	{
		CurrentPlaybackRate = bInCurrentlyPaused ? 0.0 : InCurrentPlaybackRate;
		IntendedPlaybackRate = bInCurrentlyPaused ? 0.0 : InIntendedPlaybackRate;
	}

	FTimeRange FOutputHandlerAudio::GetSupportedRenderRateScale()
	{
		FTimeRange Range;
		Range.Start.SetFromSeconds(MinPlaybackSpeed);
		Range.End.SetFromSeconds(MaxPlaybackSpeed);
		return Range;
	}

	void FOutputHandlerAudio::SetPlayRateScale(double InNewScale)
	{
		// Clamp to within permitted range just in case.
		InNewScale = Utils::Max(Utils::Min(InNewScale, MaxPlaybackSpeed), MinPlaybackSpeed);
		// Quantize to 0.005 multiples.
		AudioVars->RateScale = (double)((int32)(InNewScale * 200.0)) / 200.0;
	}

	double FOutputHandlerAudio::GetPlayRateScale()
	{
		return AudioVars->RateScale;
	}

	void FOutputHandlerAudio::ProcessSampleBlock(TArray<FElectraAudioSamplePtr>& OutProcessedSampleBlocks, FElectraAudioSamplePtr InSampleBlock, double InRate)
	{
		if (!OutputAudioSamplePool)
		{
			return;
		}

		auto DisengageTimestretcher = [&]() -> void
		{
			AudioVars->Timestretcher->Finalize(OutProcessedSampleBlocks, OutputAudioSamplePool);
			AudioVars->CurrentState = FAudioVars::EState::Disengaged;
			AudioVars->UpdateLastSampleValues(OutProcessedSampleBlocks);
			AudioVars->InterpolateStartFromLastSampleValue(InSampleBlock, NumInterpolationSamplesAt48kHz);
		};

		if (InRate == 1.0)
		{
			// Get any pending trime stretch samples first
			if (AudioVars->CurrentState == FAudioVars::EState::Engaged)
			{
				DisengageTimestretcher();
			}
			// Add the new block to the list.
			OutProcessedSampleBlocks.Emplace(MoveTemp(InSampleBlock));
		}
		else
		{
			// Small enough change to use resampler where pitch changes may not be that noticeable?
#if WITH_SOUNDTOUCHZ
			bool bOnlyResample = InRate >= MinResampleSpeed && InRate <= MaxResampleSpeed;
#else
			bool bOnlyResample = InRate >= MinPlaybackSpeed && InRate <= MaxPlaybackSpeed;
#endif
			if (bOnlyResample)
			{
				if (AudioVars->CurrentState == FAudioVars::EState::Engaged)
				{
					DisengageTimestretcher();
				}

				// We need a new sample block.
				FElectraAudioSamplePtr Resampled = OutputAudioSamplePool->AcquireShared();
				if (!ensure(Resampled))
				{
					return;
				}
				const uint32 NumChannels = InSampleBlock->GetChannels();
				uint32 NumFrames = InSampleBlock->GetFrames();
				uint32 NumOutputFrames = (uint32) FMath::RoundToZero(NumFrames / InRate);

				if (Resampled->AllocateFor(EMediaAudioSampleFormat::Float, NumChannels, NumOutputFrames))
				{
					Resampled->SetParameters(InSampleBlock->GetSampleRate(), InSampleBlock->GetTime(), InSampleBlock->GetDuration());
					float Offset = 0.0f;
					uint32 o = 0;
					const float Step = (float)NumFrames / (float)NumOutputFrames;
					const float* SourceSamples = (const float*)InSampleBlock->GetBuffer();
					float* TargetSamples = (float*)Resampled->GetWritableBuffer();
					while(o < NumOutputFrames)
					{
						uint32 I0 = (int32)Offset;
						if (I0+1 >= NumFrames)
						{
							break;
						}
						float F0 = Offset - I0;
						for(uint32 nC=0; nC<NumChannels; ++nC)
						{
							float S0 = SourceSamples[I0       * NumChannels + nC];
							float S1 = SourceSamples[(I0 + 1) * NumChannels + nC];
							float S = S0 + (S1-S0) * F0;
							TargetSamples[o * NumChannels + nC] = S;
						}
						++o;
						Offset += Step;
					}
					Resampled->SetNumFrames(o);
					OutProcessedSampleBlocks.Emplace(MoveTemp(Resampled));
				}
			}
			else if (AudioVars->Timestretcher)
			{
				if (AudioVars->CurrentState == FAudioVars::EState::Disengaged)
				{
					AudioVars->Timestretcher->Initialize(InSampleBlock);
					AudioVars->CurrentConfig.Update(InSampleBlock);
					AudioVars->CurrentState = FAudioVars::EState::Engaged;
				}
				// Check if there is a format change
				else if (AudioVars->CurrentConfig.DiffersFromChannelCount(InSampleBlock) || AudioVars->CurrentConfig.DiffersFromSampleRate(InSampleBlock))
				{
					AudioVars->Timestretcher->Finalize(OutProcessedSampleBlocks, OutputAudioSamplePool);
					AudioVars->CurrentState = FAudioVars::EState::Disengaged;
					// If the channel count is the same then we can interpolate, otherwise we skip that step.
					if (!AudioVars->CurrentConfig.DiffersFromChannelCount(InSampleBlock))
					{
						AudioVars->UpdateLastSampleValues(OutProcessedSampleBlocks);
						AudioVars->InterpolateStartFromLastSampleValue(InSampleBlock, NumInterpolationSamplesAt48kHz);
					}
					AudioVars->Timestretcher->Initialize(InSampleBlock);
					AudioVars->CurrentConfig.Update(InSampleBlock);
				}
				AudioVars->Timestretcher->ProcessSample(OutProcessedSampleBlocks, InSampleBlock, OutputAudioSamplePool, InRate);
			}
			else
			{
				// Should not actually ever get here, but if then use the input as-is.
				OutProcessedSampleBlocks.Emplace(MoveTemp(InSampleBlock));
			}
		}
		AudioVars->UpdateLastSampleValues(OutProcessedSampleBlocks);
	}


#if WITH_SOUNDTOUCHZ
	class FTimestretcher : public ITimestretcher
	{
	public:
		FTimestretcher() = default;
		virtual ~FTimestretcher();
		void Initialize(FElectraAudioSamplePtr InReferenceSample) override;
		void ProcessSample(TArray<FElectraAudioSamplePtr>& OutProcessedSamples, FElectraAudioSamplePtr InSample, TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InAudioSamplePool, double InRate) override;
		void Finalize(TArray<FElectraAudioSamplePtr>& OutProcessedSamples, TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InAudioSamplePool) override;

	private:
		uint32 GetNextBlockMarker()
		{
			if (NextBlockNumber >= 1000)
			{
				NextBlockNumber = 0;
			}
			NextBlockNumber += 10;
			return NextBlockNumber;
		}
		void PrepareInput(FElectraAudioSamplePtr InSample, uint32 InBlockMarker);
		void PullOutput(FElectraAudioSamplePtr OutSample, uint32 InNumSamples);
		void ProcessOutput(TArray<FElectraAudioSamplePtr>& OutProcessedSamples, TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InAudioSamplePool, bool bFinalBlock);

		struct FSampleBlockInfo
		{
			FMediaTimeStamp Timestamp;
			FTimespan Duration;
			uint32 NumFrames = 0;
			uint32 SampleRate = 0;
			uint32 NumChannels = 0;
			uint32 SequenceMarker = 0;
		};

		FSoundTouch ST;
		TArray<FSampleBlockInfo> SampleBlockInfos;

		uint32 SampleRate = 0;
		uint32 ChannelCount = 0;

		double CurrentRate = 1.0;

		uint32 NextBlockNumber = 0;

		float* InputBuffer = nullptr;
		uint32 InputBufferSize = 0;

		float* OutputBuffer = nullptr;
		uint32 MaxSamplesInOutputBuffer = 0;
		uint32 NumSamplesInOutput = 0;
	};

	TUniquePtr<ITimestretcher> ITimestretcher::Create()
	{
		return MakeUnique<FTimestretcher>();
	}

	FTimestretcher::~FTimestretcher()
	{
		FMemory::Free(InputBuffer);
		FMemory::Free(OutputBuffer);
	}

	void FTimestretcher::Initialize(FElectraAudioSamplePtr InReferenceSample)
	{
		if (ensure(InReferenceSample))
		{
			SampleBlockInfos.Empty();
			SampleRate = InReferenceSample->GetSampleRate();
			ChannelCount = InReferenceSample->GetChannels();
			CurrentRate = 1.0;
			NextBlockNumber = 0;
			NumSamplesInOutput = 0;
			ST.Clear();
			ST.SetSampleRate(SampleRate);
			ST.SetChannels(ChannelCount + 1);
			ST.SetTempo(CurrentRate);
			ST.SetSetting(FSoundTouch::ESetting::UseQuickseek, 1);
			ST.SetSetting(FSoundTouch::ESetting::UseAAFilter, 0);
		}
	}

	void FTimestretcher::PrepareInput(FElectraAudioSamplePtr InSample, uint32 InBlockMarker)
	{
		uint32 NumFrames = InSample->GetFrames();
		uint32 BufferSize = NumFrames * (ChannelCount + 1) * sizeof(float);
		if (BufferSize > InputBufferSize)
		{
			InputBuffer = (float*)FMemory::Realloc(InputBuffer, BufferSize);
			InputBufferSize = BufferSize;
		}

		float* Flt = InputBuffer;
		const float* Samples = (const float*)InSample->GetBuffer();
		const float FltMarker = InBlockMarker;
		for(uint32 i=0; i<NumFrames; ++i)
		{
			for(uint32 j=0; j<ChannelCount; ++j)
			{
				*Flt++ = *Samples++;
			}
			*Flt++ = FltMarker;
		}
	}

	void FTimestretcher::ProcessSample(TArray<FElectraAudioSamplePtr>& OutProcessedSamples, FElectraAudioSamplePtr InSample, TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InAudioSamplePool, double InRate)
	{
		if (InSample)
		{
			check(InSample->GetChannels() == ChannelCount && InSample->GetSampleRate() == SampleRate);

			uint32 BlockMarker = GetNextBlockMarker();

			FSampleBlockInfo si;
			si.Timestamp = InSample->GetTime();
			si.Duration = InSample->GetDuration();
			si.NumFrames = InSample->GetFrames();
			si.SampleRate = InSample->GetSampleRate();
			si.NumChannels = InSample->GetChannels();
			si.SequenceMarker = BlockMarker;

			PrepareInput(InSample, BlockMarker);
			if (InRate != CurrentRate)
			{
				ST.SetTempo(InRate);
				CurrentRate = InRate;
			}
			ST.PutSamples(InputBuffer, InSample->GetFrames());
			SampleBlockInfos.Emplace(MoveTemp(si));
			ProcessOutput(OutProcessedSamples, InAudioSamplePool, false);
		}
	}

	void FTimestretcher::PullOutput(FElectraAudioSamplePtr OutSample, uint32 InNumSamples)
	{
		float* FltTarget = (float*)OutSample->GetWritableBuffer();
		const float *FltSource = OutputBuffer;
		for(uint32 i=0; i<InNumSamples; ++i)
		{
			for(uint32 j=0; j<ChannelCount; ++j)
			{
				*FltTarget++ = *FltSource++;
			}
			++FltSource;
		}
		check(InNumSamples <= NumSamplesInOutput);
		uint32 NumRemaining = NumSamplesInOutput - InNumSamples;
		if (NumRemaining)
		{
			FMemory::Memmove(OutputBuffer, FltSource, sizeof(float) * NumRemaining * (ChannelCount + 1));
		}
		NumSamplesInOutput = NumRemaining;
	}

	void FTimestretcher::ProcessOutput(TArray<FElectraAudioSamplePtr>& OutProcessedSamples, TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InAudioSamplePool, bool bFinalBlock)
	{
		uint32 NumAvail = ST.NumSamples();
		if (NumAvail == 0)
		{
			return;
		}
		uint32 nf = NumSamplesInOutput + NumAvail;
		if (nf >= MaxSamplesInOutputBuffer)
		{
			OutputBuffer = (float*)FMemory::Realloc(OutputBuffer, sizeof(float) * (ChannelCount + 1) * nf);
			MaxSamplesInOutputBuffer = nf;
		}
		// Append the newly processed output to our output work buffer.
		uint32 NumGot = ST.ReceiveSamples(OutputBuffer + (NumSamplesInOutput * (ChannelCount + 1)), NumAvail);
		NumSamplesInOutput += NumGot;

		// Find the marker of the next block, which is an indication for the end of the current block.
		while(NumSamplesInOutput && SampleBlockInfos.Num() > 1)
		{
			const float* MarkerBufEnd = OutputBuffer + (ChannelCount + 1) * NumSamplesInOutput - 1;
			const float* MarkerBufStart = OutputBuffer + ChannelCount;
			float NextMarkerValue = SampleBlockInfos[1].SequenceMarker;
			float NextMarkerValueMin = NextMarkerValue - 5.0f;
			float NextMarkerValueMax = NextMarkerValue + 5.0f;
			bool bFoundNextMarker = false;
			bool bGotSample = false;
			while(MarkerBufEnd > MarkerBufStart)
			{
				if (*MarkerBufEnd >= NextMarkerValueMin && *MarkerBufEnd <= NextMarkerValueMax)
				{
					if (bFoundNextMarker)
					{
						uint32 NumCurrentFrames = 1 + (MarkerBufEnd - MarkerBufStart) / (ChannelCount + 1);

						FElectraAudioSamplePtr NewSample = InAudioSamplePool->AcquireShared();
						if (!ensure(NewSample))
						{
							return;
						}
						if (NewSample->AllocateFor(EMediaAudioSampleFormat::Float, ChannelCount, NumCurrentFrames))
						{
							FSampleBlockInfo si = SampleBlockInfos[0];
							SampleBlockInfos.RemoveAt(0);
							NewSample->SetParameters(si.SampleRate, si.Timestamp, si.Duration);
//UE_LOGF(LogTemp, Log, "%lld (%lld): %u -> %u", (long long int)si.Timestamp.GetTime().GetTicks(), (long long int)si.Duration.GetTicks(), si.NumFrames, NumCurrentFrames);
							PullOutput(NewSample, NumCurrentFrames);
							OutProcessedSamples.Emplace(MoveTemp(NewSample));
							bGotSample = true;
							break;
						}
					}
					else
					{
						// Found the next marker. Now continue looking backwards for the current marker.
						bFoundNextMarker = true;
						NextMarkerValue = SampleBlockInfos[0].SequenceMarker;
						NextMarkerValueMin = NextMarkerValue - 5.0f;
						NextMarkerValueMax = NextMarkerValue + 5.0f;
					}
				}
				MarkerBufEnd -= ChannelCount + 1;
			}
			if (!bGotSample)
			{
				break;
			}
		}
		if (bFinalBlock && SampleBlockInfos.Num())
		{
			const float* MarkerBufEnd = OutputBuffer + (ChannelCount + 1) * NumSamplesInOutput;
			const float* MarkerBufStart = OutputBuffer + ChannelCount;
			const float* CurrentMarkerPos = MarkerBufEnd - 1;
			float NextMarkerValue = SampleBlockInfos[0].SequenceMarker;
			float NextMarkerValueMin = NextMarkerValue - 5.0f;
			float NextMarkerValueMax = NextMarkerValue + 5.0f;
			while(CurrentMarkerPos > MarkerBufStart)
			{
				if (*CurrentMarkerPos >= NextMarkerValueMin && *CurrentMarkerPos <= NextMarkerValueMax)
				{
					uint32 NumCurrentFrames = (CurrentMarkerPos - MarkerBufStart) / (ChannelCount + 1);

					FElectraAudioSamplePtr NewSample = InAudioSamplePool->AcquireShared();
					if (!ensure(NewSample))
					{
						return;
					}
					if (NewSample->AllocateFor(EMediaAudioSampleFormat::Float, ChannelCount, NumCurrentFrames))
					{
						FSampleBlockInfo si = SampleBlockInfos[0];
						SampleBlockInfos.RemoveAt(0);
						NewSample->SetParameters(si.SampleRate, si.Timestamp, si.Duration);
//UE_LOGF(LogTemp, Log, "%lld (%lld): %u -> %u", (long long int)si.Timestamp.GetTime().GetTicks(), (long long int)si.Duration.GetTicks(), si.NumFrames, NumCurrentFrames);
						PullOutput(NewSample, NumCurrentFrames);
						OutProcessedSamples.Emplace(MoveTemp(NewSample));
						break;
					}
				}
				CurrentMarkerPos -= ChannelCount + 1;
			}
		}
	}

	void FTimestretcher::Finalize(TArray<FElectraAudioSamplePtr>& OutProcessedSamples, TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> InAudioSamplePool)
	{
		// Call flush to produce the remaining output.
		// This may create silent samples in the end, which also have zero in the channel containing
		// our marker block. Since we do not ever use zero for a marker we can identify the end
		// by looking for that zero marker.
		ST.Flush();
		ProcessOutput(OutProcessedSamples, InAudioSamplePool,true);
	}


#endif

#if 0
	void FOutputHandlerAudio::DebugModifySamples(FElectraAudioSamplePtr OutSampleBlock, FElectraAudioSamplePtr InSampleBlock)
	{
		uint32 Chans = Utils::Max(Utils::Min((uint32)CVarElectraChannels.GetValueOnAnyThread(), 8u), 1u);
		if (Chans != InSampleBlock->GetChannels())
		{
			OutSampleBlock->AllocateFor(EMediaAudioSampleFormat::Float, Chans, InSampleBlock->GetFrames());
			OutSampleBlock->SetParameters(InSampleBlock->GetSampleRate(), InSampleBlock->GetTime(), InSampleBlock->GetDuration());
			const float* SrcL = (const float*)InSampleBlock->GetBuffer();
			float* Tgt = (float*)OutSampleBlock->GetWritableBuffer();
			for(uint32 fr=0,nfr=InSampleBlock->GetFrames(); fr<nfr; ++fr)
			{
				for(uint32 ch=0; ch<Chans; ++ch)
				{
					*Tgt++ = *SrcL;
				}
				SrcL += InSampleBlock->GetChannels();
			}
			Swap(OutSampleBlock, InSampleBlock);
		}

		float Rate = Utils::Max(Utils::Min((float)CVarElectraResample.GetValueOnAnyThread(), 48000.0f), 8000.0f);
		const uint32 NumChannels = InSampleBlock->GetChannels();
		const uint32 SrcRate = InSampleBlock->GetSampleRate();
		uint32 NumFrames = InSampleBlock->GetFrames();
		uint32 NumOutputFrames = (uint32) FMath::RoundToZero(NumFrames * Rate / SrcRate);
		if (OutSampleBlock->AllocateFor(EMediaAudioSampleFormat::Float, NumChannels, NumOutputFrames))
		{
			OutSampleBlock->SetParameters((uint32)Rate, InSampleBlock->GetTime(), InSampleBlock->GetDuration());
			float Offset = 0.0f;
			uint32 o = 0;
			const float Step = (float)NumFrames / (float)NumOutputFrames;
			const float* SourceSamples = (const float*)InSampleBlock->GetBuffer();
			float* TargetSamples = (float*)OutSampleBlock->GetWritableBuffer();
			while(o < NumOutputFrames)
			{
				uint32 I0 = (int32)Offset;
				if (I0+1 >= NumFrames)
				{
					break;
				}
				float F0 = Offset - I0;
				for(uint32 nC=0; nC<NumChannels; ++nC)
				{
					float S0 = SourceSamples[I0       * NumChannels + nC];
					float S1 = SourceSamples[(I0 + 1) * NumChannels + nC];
					float S = S0 + (S1-S0) * F0;
					TargetSamples[o * NumChannels + nC] = S;
				}
				++o;
				Offset += Step;
			}
			OutSampleBlock->SetNumFrames(o);
		}
	}
#endif

} // namespace Electra
