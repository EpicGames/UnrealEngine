// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputHandler.h"

namespace Electra
{

	FOutputHandlerVideo::FOutputHandlerVideo()
	{
	}

	FOutputHandlerVideo::~FOutputHandlerVideo()
	{
	}

	bool FOutputHandlerVideo::PreparePool(const FParamDict& InParameters)
	{
		// How many samples are asked for?
		NumOutputSamples = (int32) InParameters.GetValue(OutputHandlerOptionKeys::NumBuffers).SafeGetInt64(0);
		// Set this in this pool's configuration.
		PoolProperties.Set(OutputHandlerOptionKeys::NumBuffers, FVariantValue((int64)NumOutputSamples));
		// For now this is all we do. The pool itself is not being limited.
		return true;
	}

	void FOutputHandlerVideo::ClosePool()
	{
		ensure(PendingOutputSamples.IsEmpty());
		NumOutputSamples = 0;
		PoolProperties.Clear();
	}

	const FParamDict& FOutputHandlerVideo::GetPoolProperties()
	{
		return PoolProperties;
	}

	void FOutputHandlerVideo::SetClock(TSharedPtr<IMediaRenderClock, ESPMode::ThreadSafe> InClock)
	{
		Clock = MoveTemp(InClock);
	}

	void FOutputHandlerVideo::StartOutput()
	{
		SendAllPendingSamples();
		bSendOutput = true;
	}

	void FOutputHandlerVideo::StopOutput()
	{
		bSendOutput = false;
	}

	void FOutputHandlerVideo::Flush()
	{
		// Anything not sent yet we unbind our release notification callback and just drop here.
		PendingOutputSamplesLock.Lock();
		while(PendingOutputSamples.Num())
		{
			FPendingSample ps = PendingOutputSamples[0];
			PendingOutputSamples.RemoveAt(0);
			ps.Sample->GetReleaseDelegate().Unbind();
			PendingOutputSamplesLock.Unlock();
			ps.Sample.Reset();
			PendingOutputSamplesLock.Lock();
		}
		PendingOutputSamplesLock.Unlock();

		// Flush any samples in the media queue.
		auto MediaSamples = CurrentMediaSamples;
		if (MediaSamples.IsValid())
		{
			TRange<FTimespan> AllTime(FTimespan::MinValue(), FTimespan::MaxValue());
			TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> FlushSample;
			while(MediaSamples->FetchVideo(AllTime, FlushSample))
			{ }
		}

		// Anything that has not been returned by now we drop from the sample info stats.
		EnqueuedSampleInfosLock.Lock();
		EnqueuedSampleInfos.Empty();
		EnqueuedSampleDuration = FTimespan::Zero();
		EnqueuedSampleInfosLock.Unlock();

		// Allow sending the next first frame right away again.
		bDidSendFirst = false;
	}


	uint64 FOutputHandlerVideo::GetTotalAddedSamples()
	{
		return NumTotalSamplesAdded;
	}

	uint64 FOutputHandlerVideo::GetTotalNumDroppedSamples()
	{
		auto MediaSamples = CurrentMediaSamples;
		if (MediaSamples.IsValid())
		{
			NumTotalSamplesDropped = MediaSamples->GetNumDroppedVideoSamples(false);
		}
		return NumTotalSamplesDropped;
	}

	uint64 FOutputHandlerVideo::GetNewlyNumDroppedSamples(bool bInClear)
	{
		uint64 nd = GetTotalNumDroppedSamples();
		uint64 Diff = (uint64)(nd - TotalSamplesDroppedAtLastCall);
		if (bInClear)
		{
			TotalSamplesDroppedAtLastCall = nd;
		}
		return Diff;
	}

	FTimespan FOutputHandlerVideo::GetEnqueuedSampleDuration()
	{
		FScopeLock lock(&EnqueuedSampleInfosLock);
		return EnqueuedSampleDuration;
	}

	void FOutputHandlerVideo::GetEnqueuedSampleInfo(TArray<FEnqueuedSampleInfo>& OutOptionalSampleInfos)
	{
		FScopeLock lock(&EnqueuedSampleInfosLock);
		OutOptionalSampleInfos = EnqueuedSampleInfos;
	}

	void FOutputHandlerVideo::SetOutputTexturePool(TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> InOutputTexturePool)
	{
		OutputTexturePool = MoveTemp(InOutputTexturePool);
	}

	bool FOutputHandlerVideo::CanReceiveOutputSample()
	{
		auto MediaSamples = CurrentMediaSamples;
		if (!MediaSamples.IsValid())
		{
			return false;
		}
		bool bCanRcv = true;
		PendingOutputSamplesLock.Lock();
		int32 NumPendingOutputSamples = PendingOutputSamples.Num();
		PendingOutputSamplesLock.Unlock();
		if (1 + NumPendingOutputSamples > NumOutputSamples)
		{
			return false;
		}
		bCanRcv = MediaSamples->CanReceiveVideoSamples(1 + NumPendingOutputSamples);
		return bCanRcv;
	}

	IOutputHandlerBase::EBufferResult FOutputHandlerVideo::ObtainOutputSample(FElectraTextureSamplePtr& OutTextureSample)
	{
		if (OutputTexturePool)
		{
			OutTextureSample = OutputTexturePool->AcquireShared();
			return IOutputHandlerBase::EBufferResult::Ok;
		}
		return IOutputHandlerBase::EBufferResult::NoBuffer;
	}

	void FOutputHandlerVideo::ReturnOutputSample(FElectraTextureSamplePtr InTextureSampleToReturn, EReturnSampleType InSendToOutputQueueType)
	{
		auto MediaSamples = CurrentMediaSamples;
		if (InTextureSampleToReturn && InSendToOutputQueueType != EReturnSampleType::DontSendToQueue && MediaSamples.IsValid())
		{
			uint64 SampleID = ++NextSampleID;

			// Log the timestamp and duration of the sample
			FEnqueuedSampleInfo si;
			si.Timestamp = InTextureSampleToReturn->GetTime();
			si.Duration = InTextureSampleToReturn->GetDuration();
			si.SampleID = SampleID;
			si.bIsDummy = InSendToOutputQueueType == EReturnSampleType::DummySample;
			EnqueuedSampleInfosLock.Lock();
			EnqueuedSampleDuration += si.Duration;
			EnqueuedSampleInfos.Emplace(si);	// do not move, this is needed below.
			EnqueuedSampleInfosLock.Unlock();

			// Set our notification callback when the sample is being returned to the pool.
			InTextureSampleToReturn->GetReleaseDelegate().BindThreadSafeSP(AsShared(), &FOutputHandlerVideo::ReleaseSampleToPool, SampleID);

			// When not sending output, send the first frame anyway to get displayed when scrubbing.
			bool bHoldBack = !bSendOutput && bDidSendFirst;
			if (!bHoldBack)
			{
				bDidSendFirst = true;
				// Do not pass up dummy samples. We only keep them tracked for their duration
				// to ensure the pipeline is not running dry.
				if (!si.bIsDummy)
				{
					MediaSamples->AddVideo(InTextureSampleToReturn.ToSharedRef());
					++NumTotalSamplesAdded;
				}
			}
			else
			{
				FPendingSample ps;
				ps.SampleInfo = si;
				ps.Sample = MoveTemp(InTextureSampleToReturn);
				PendingOutputSamplesLock.Lock();
				PendingOutputSamples.Emplace(MoveTemp(ps));
				PendingOutputSamplesLock.Unlock();
			}
		}
	}

	void FOutputHandlerVideo::SendAllPendingSamples()
	{
		auto MediaSamples = CurrentMediaSamples;
		PendingOutputSamplesLock.Lock();
		while(PendingOutputSamples.Num())
		{
			FPendingSample ps = PendingOutputSamples[0];
			PendingOutputSamples.RemoveAt(0);
			PendingOutputSamplesLock.Unlock();
			if (MediaSamples.IsValid() && !ps.SampleInfo.bIsDummy)
			{
				MediaSamples->AddVideo(ps.Sample.ToSharedRef());
				++NumTotalSamplesAdded;
			}
			ps.Sample.Reset();
			PendingOutputSamplesLock.Lock();
		}
		PendingOutputSamplesLock.Unlock();
	}

	void FOutputHandlerVideo::ReleaseSampleToPool(IElectraTextureSampleBase* InTextureSampleToReturn, uint64 InSampleID)
	{
		FTimeValue RenderTime;

		// Remove the sample stats.
		FEnqueuedSampleInfo si;
		EnqueuedSampleInfosLock.Lock();
		for(int32 i=0, iMax=EnqueuedSampleInfos.Num(); i<iMax; ++i)
		{
			if (EnqueuedSampleInfos[i].SampleID == InSampleID)
			{
				RenderTime.SetFromTimespan(EnqueuedSampleInfos[i].Timestamp.GetTime(), EnqueuedSampleInfos[i].Timestamp.GetIndexValue());
				EnqueuedSampleDuration -= EnqueuedSampleInfos[i].Duration;
				EnqueuedSampleInfos.RemoveAt(i);
				break;
			}
		}
		EnqueuedSampleInfosLock.Unlock();

		if (Clock && RenderTime.IsValid())
		{
			Clock->SetCurrentTime(IMediaRenderClock::ERendererType::Video, RenderTime);
		}
	}



} // namespace Electra
