// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynchronizedClock.h"
#include "PlayerPlatform.h"

namespace Electra
{

	namespace Platform
	{
		FString GetPlatformID()
		{
		#if PLATFORM_MAC
			return FString(TEXT("Mac"));
		#else
			return FString(TEXT("iOS"));
		#endif
		}
	}


	int64 MEDIAutcTime::CurrentMSec()
	{
		FTimespan localTime(FDateTime::UtcNow().GetTicks());
		return (int64)localTime.GetTotalMilliseconds();
	}

}
