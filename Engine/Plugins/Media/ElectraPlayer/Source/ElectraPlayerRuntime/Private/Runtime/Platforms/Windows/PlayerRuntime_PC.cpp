// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynchronizedClock.h"
#include "PlayerPlatform.h"

namespace Electra
{

	namespace Platform
	{
		FString GetPlatformID()
		{
			return FString(TEXT("Win"));
		}
	}

	int64 MEDIAutcTime::CurrentMSec()
	{
		FTimespan localTime(FDateTime::UtcNow().GetTicks());
		return (int64)localTime.GetTotalMilliseconds();
	}

}
