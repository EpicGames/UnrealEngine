// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/ABRRules/ABRFixedStream.h"
#include "Player/ABRRules/ABRStatisticTypes.h"
#include "Player/AdaptivePlayerOptionKeynames.h"

#include "Player/Manifest.h"
#include "Utilities/Utilities.h"


namespace Electra
{

class FABRFixedStream : public IABRFixedStream
{
public:
	FABRFixedStream(IABRInfoInterface* InIInfo);
	virtual ~FABRFixedStream() = default;

	FABRDownloadProgressDecision ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override;
	void ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override;
	void ReportBufferingStart(Metrics::EBufferingReason BufferingReason) override {}
	void ReportBufferingEnd(Metrics::EBufferingReason BufferingReason) override {}
	void ReportPlaybackPaused() override {}
	void ReportPlaybackResumed() override {}
	void ReportPlaybackEnded() override {}

	FTimeValue GetMinBufferTimeForPlayback(IAdaptiveStreamSelector::EMinBufferType InBufferingType, FTimeValue InDefaultMBT) override
	{ return FTimeValue(); }
	IAdaptiveStreamSelector::FRebufferAction GetRebufferAction() override;
	IAdaptiveStreamSelector::EHandlingAction PeriodicHandle() override { return IAdaptiveStreamSelector::EHandlingAction::None; }
	void DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...)) override { }

	void RepresentationsChanged(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPlayPeriod) override {}
	void SetBandwidth(int64 bitsPerSecond) override {}
	void SetForcedNextBandwidth(int64 bitsPerSecond, double minBufferTimeBeforePlayback) override {}
	int64 GetLastBandwidth() override;
	int64 GetAverageBandwidth() override;
	int64 GetAverageThroughput() override;
	double GetAverageLatency() override;

	void PrepareStreamCandidateList(TArray<TSharedPtrTS<FABRStreamInformation>>& OutCandidates, EStreamType StreamType, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction EvaluateForError(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction EvaluateForQuality(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction PerformSelection(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;

private:
	IABRInfoInterface* Info;
	FCriticalSection Lock;
	TSimpleMovingAverage<double> AverageBandwidth;
	TSimpleMovingAverage<double> AverageLatency;
	TSimpleMovingAverage<int64> AverageThroughput;
};



IABRFixedStream* IABRFixedStream::Create(IABRInfoInterface* InIInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType)
{
	return new FABRFixedStream(InIInfo);
}






FABRFixedStream::FABRFixedStream(IABRInfoInterface* InIInfo)
	: Info(InIInfo)
{
	const int32 HistorySize = 3;
	AverageBandwidth.Resize(HistorySize);
	AverageLatency.Resize(HistorySize);
	AverageThroughput.Resize(HistorySize);
}


FABRDownloadProgressDecision FABRFixedStream::ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
{
	FABRDownloadProgressDecision Decision;
	Decision.Flags = FABRDownloadProgressDecision::eABR_EmitPartialData;
	return Decision;
}

void FABRFixedStream::ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
{
	if (SegmentDownloadStats.SegmentType == Metrics::ESegmentType::Media && SegmentDownloadStats.StreamType == EStreamType::Video &&
		(SegmentDownloadStats.bWasSuccessful || SegmentDownloadStats.NumBytesDownloaded))	// any number of bytes received counts!
	{
		if (!SegmentDownloadStats.bIsCachedResponse)
		{
			double v = -1.0;
			if (SegmentDownloadStats.NumBytesDownloaded && SegmentDownloadStats.TimeToDownload > 0.0)
			{
				v = SegmentDownloadStats.NumBytesDownloaded * 8 / SegmentDownloadStats.TimeToDownload;
			}
			if (v > 0.0)
			{
				FScopeLock lock(&Lock);
				AverageBandwidth.AddValue(v);
				AverageLatency.AddValue(SegmentDownloadStats.TimeToFirstByte);
				AverageThroughput.AddValue((int64)v);
			}
		}
	}
}


int64 FABRFixedStream::GetLastBandwidth()
{
	FScopeLock lock(&Lock);
	return (int64)AverageBandwidth.GetLastSample();
}

int64 FABRFixedStream::GetAverageBandwidth()
{
	FScopeLock lock(&Lock);
	return (int64)AverageBandwidth.GetSMA();;
}

int64 FABRFixedStream::GetAverageThroughput()
{
	FScopeLock lock(&Lock);
	return AverageThroughput.GetSMA();
}

double FABRFixedStream::GetAverageLatency()
{
	FScopeLock lock(&Lock);
	return AverageLatency.GetSMA();
}


IAdaptiveStreamSelector::FRebufferAction FABRFixedStream::GetRebufferAction()
{
	IAdaptiveStreamSelector::FRebufferAction Action;
	if (Info->GetOptionValue(OptionThrowErrorWhenRebuffering).SafeGetBool(false))
	{
		Action.Action = IAdaptiveStreamSelector::FRebufferAction::EAction::ThrowError;
	}
	else if (Info->GetOptionValue(OptionRebufferingContinuesLoading).SafeGetBool(false))
	{
		Action.Action = IAdaptiveStreamSelector::FRebufferAction::EAction::ContinueLoading;
	}
	else
	{
		Action.Action = IAdaptiveStreamSelector::FRebufferAction::EAction::Restart;
	}
	return Action;
}


void FABRFixedStream::PrepareStreamCandidateList(TArray<TSharedPtrTS<FABRStreamInformation>>& OutCandidates, EStreamType StreamType, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const FTimeValue& TimeNow)
{
	// Get the list of streams for this type.
	OutCandidates = Info->GetStreamInformations(StreamType);
}

IAdaptiveStreamSelector::ESegmentAction FABRFixedStream::EvaluateForError(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	if (CurrentSegment.IsValid())
	{
		Metrics::FSegmentDownloadStats Stats;
		CurrentSegment->GetDownloadStats(Stats);
		// Try to get the stream information for the current downloaded segment. We may not find it on a period transition where the
		// segment is the last of the previous period.
		TSharedPtrTS<FABRStreamInformation> CurrentStreamInfo = Info->GetStreamInformation(Stats);
		if (CurrentStreamInfo.IsValid())
		{
			if (!Stats.bWasSuccessful)
			{
				// Too many failures already?
				if (Stats.RetryNumber >= 2)
				{
					Info->LogMessage(IInfoLog::ELevel::Error, FString::Printf(TEXT("Exceeded permissable number of retries (%d). Failing now."), Stats.RetryNumber));
					return IAdaptiveStreamSelector::ESegmentAction::Fail;
				}
				return Stats.bParseFailure ? IAdaptiveStreamSelector::ESegmentAction::Fail : IAdaptiveStreamSelector::ESegmentAction::Retry;
			}
			// Update stats
			CurrentStreamInfo->Health.LastDownloadStats = Stats;
		}
	}
	return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
}


IAdaptiveStreamSelector::ESegmentAction FABRFixedStream::EvaluateForQuality(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
}


IAdaptiveStreamSelector::ESegmentAction FABRFixedStream::PerformSelection(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
}

} // namespace Electra
