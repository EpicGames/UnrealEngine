// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Player/ABRRules/ABRInternal.h"

namespace Electra
{

class IABRFixedStream : public IABRRule
{
public:
	static IABRFixedStream* Create(IABRInfoInterface* InIInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType);
	virtual ~IABRFixedStream() = default;
};

} // namespace Electra

