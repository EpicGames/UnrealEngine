// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Player/AdaptiveStreamingPlayerABR.h"

namespace Electra
{
	enum class EABRPresentationType
	{
		OnDemand,
		Live,
		LowLatency
	};

	struct FABRStreamInformation
	{
		struct FStreamHealth
		{
			FStreamHealth()
			{
				Reset();
			}
			void Reset()
			{
				BecomesAvailableAgainAtUTC.SetToZero();
				LastDownloadStats = {};
			}
			FTimeValue						BecomesAvailableAgainAtUTC;
			Metrics::FSegmentDownloadStats	LastDownloadStats;
		};

		FString											AdaptationSetUniqueID;
		FString											RepresentationUniqueID;

		FStreamHealth									Health;
		// Set for convenience.
		FStreamCodecInformation::FResolution			Resolution;
		int32											Bitrate = 0;
		int32											QualityIndex = 0;
		bool											bLowLatencyEnabled = false;
	};

	class IABRInfoInterface : public IPlayerABRLiveControl
	{
	public:
		virtual ~IABRInfoInterface() = default;
		virtual bool HaveOptionValue(const FName& InOption) = 0;
		virtual const FVariantValue GetOptionValue(const FName& InOption) = 0;

		virtual TSharedPtrTS<FABRStreamInformation> GetStreamInformation(const Metrics::FSegmentDownloadStats& FromDownloadStats) = 0;
		virtual const TArray<TSharedPtrTS<FABRStreamInformation>>& GetStreamInformations(EStreamType InForStreamType) = 0;
		virtual FStreamCodecInformation::FResolution GetMaxStreamResolution() = 0;
		virtual int32 GetBandwidthCeiling() = 0;

		virtual void LogMessage(IInfoLog::ELevel Level, const FString& Message) = 0;
	};


	class IABRRule
	{
	public:
		virtual ~IABRRule() = default;

		// Same as the ones from IAdaptiveStreamingPlayerMetrics
		virtual FABRDownloadProgressDecision ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) = 0;
		virtual void ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) = 0;
		virtual void ReportBufferingStart(Metrics::EBufferingReason BufferingReason) = 0;
		virtual void ReportBufferingEnd(Metrics::EBufferingReason BufferingReason) = 0;
		virtual void ReportPlaybackPaused() = 0;
		virtual void ReportPlaybackResumed() = 0;
		virtual void ReportPlaybackEnded() = 0;

		virtual FTimeValue GetMinBufferTimeForPlayback(IAdaptiveStreamSelector::EMinBufferType InBufferingType, FTimeValue InDefaultMBT) = 0;
		virtual IAdaptiveStreamSelector::FRebufferAction GetRebufferAction() = 0;
		virtual IAdaptiveStreamSelector::EHandlingAction PeriodicHandle() = 0;
		virtual void DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...)) = 0;

		virtual void RepresentationsChanged(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPlayPeriod) = 0;
		virtual void SetBandwidth(int64 bitsPerSecond) = 0;
		virtual void SetForcedNextBandwidth(int64 bitsPerSecond, double minBufferTimeBeforePlayback) = 0;
		virtual int64 GetLastBandwidth() = 0;
		virtual int64 GetAverageBandwidth() = 0;
		virtual int64 GetAverageThroughput() = 0;
		virtual double GetAverageLatency() = 0;

		virtual void PrepareStreamCandidateList(TArray<TSharedPtrTS<FABRStreamInformation>>& OutCandidates, EStreamType StreamType, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const FTimeValue& TimeNow) = 0;
		virtual IAdaptiveStreamSelector::ESegmentAction EvaluateForError(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) = 0;
		virtual IAdaptiveStreamSelector::ESegmentAction EvaluateForQuality(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) = 0;
		virtual IAdaptiveStreamSelector::ESegmentAction PerformSelection(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) = 0;
	};

} // namespace Electra

