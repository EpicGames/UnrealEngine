// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/ABRRules/ABRLiveStream.h"
#include "Player/ABRRules/ABRStatisticTypes.h"
#include "Player/ABRRules/ABROptionKeynames.h"
#include "Player/AdaptivePlayerOptionKeynames.h"

#include "Player/Manifest.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Utilities/Utilities.h"
#include "Utilities/URLParser.h"

/***************************************************************************************************************************************************/
//#define ENABLE_LATENCY_OVERRIDE_CVAR

#ifdef ENABLE_LATENCY_OVERRIDE_CVAR
#include "HAL/IConsoleManager.h"
#include "Async/Async.h"
static TAutoConsoleVariable<float> CVarElectraTL(TEXT("Electra.TL"), 3.0f, TEXT("Target latency"), ECVF_Default);
#endif
/***************************************************************************************************************************************************/


namespace Electra
{


class FABRLiveStream : public IABRLiveStream
{
public:
	FABRLiveStream(IABRInfoInterface* InInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType);
	virtual ~FABRLiveStream();

	FABRDownloadProgressDecision ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override;
	void ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override;
	void ReportBufferingStart(Metrics::EBufferingReason BufferingReason) override;
	void ReportBufferingEnd(Metrics::EBufferingReason BufferingReason) override;
	void ReportPlaybackPaused() override;
	void ReportPlaybackResumed() override;
	void ReportPlaybackEnded() override;

	FTimeValue GetMinBufferTimeForPlayback(IAdaptiveStreamSelector::EMinBufferType InBufferingType, FTimeValue InDefaultMBT) override;
	IAdaptiveStreamSelector::FRebufferAction GetRebufferAction() override;
	IAdaptiveStreamSelector::EHandlingAction PeriodicHandle() override;
	void DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...)) override;

	void RepresentationsChanged(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPlayPeriod) override;
	void SetBandwidth(int64 bitsPerSecond) override;
	void SetForcedNextBandwidth(int64 bitsPerSecond, double minBufferTimeBeforePlayback) override;
	int64 GetLastBandwidth() override;
	int64 GetAverageBandwidth() override;
	int64 GetAverageThroughput() override;
	double GetAverageLatency() override;

	void PrepareStreamCandidateList(TArray<TSharedPtrTS<FABRStreamInformation>>& OutCandidates, EStreamType StreamType, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction EvaluateForError(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction EvaluateForQuality(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction PerformSelection(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;

private:
	struct FLatencyConfiguration
	{
		FLatencyConfiguration()
		{ Reset(); }
		void Reset()
		{
			// Defaults to use when no values are provided by the playlist.
			TargetLatency = 6.0;
			MinLatency = 2.0;
			MaxLatency = 12.0;
			MinPlayRate = 0.9;
			MaxPlayRate = 1.2;
		}
		double TargetLatency;
		double MinLatency;
		double MaxLatency;
		double MinPlayRate;
		double MaxPlayRate;

		// Threshold above which we will attempt to meet the target latency, expressed as a scale of the target latency. If below we do nothing.
		double ActivationThreshold = 0.02;	// +/- 2%
		// Difference in calculated vs. current rate above which to apply the new rate.
		double MinRateChangeUseThreshold = 0.02;
		// Amount of content in the buffer below which playback slows down to avoid running empty.
		double LowBufferContentBackoff = 0.5;
		// Do not fall below this many seconds away from the target latency while trying to recover the buffer.
		double LowBufferMaxTargetLatencyDistance = 1.0;
		// Min playrate to drop to when slowing down for recovering the buffer
		double LowBufferMinMinPlayRate = 0.8;
	};


	struct FDecisionAttributes
	{
		int32 QualityIndex = 0;
		int32 Bitrate = 0;
		double BandwidthScore = 0.0;
		double EstimatedTimeToDownload = 0.0;
		bool bIsCandidate = false;
	};


	struct FQualityMetrics
	{
		FQualityMetrics()
		{
			AverageKbps.Resize(5);
			Reset();
		}
		void Reset()
		{
			AverageKbps.Reset();
			SelectedAtRatio = 0.0;
			SelectedAtTime = 0.0;
			DeselectedAtTime = 0.0;
			DeselectReason = EDeselectReason::None;
			LastEvaluationCount = 0;
			TickleCount = 0;
		}
		enum class EDeselectReason
		{
			None,
			Upswitch,
			Bandwidth,
			BufferLoss,
			DownloadError
		};

		TSimpleMovingAverage<double> AverageKbps;
		double SelectedAtRatio = 0.0;
		double SelectedAtTime = 0.0;
		double DeselectedAtTime = 0.0;
		EDeselectReason DeselectReason = EDeselectReason::None;
		uint32 LastEvaluationCount = 0;
		uint32 TickleCount = 0;
	};

	struct FStreamWorkVars
	{
		enum class EDecision
		{
			AllGood,
			NotConnectedAbort,				// Aborted because not connected to server in time.
			ConnectedAbort,					// Aborted for some reason while receiving data
			Rebuffered,						// Hit by rebuffering
		};

		struct FSegmentInfo
		{
			double BufferDurationAtStart = -1.0;
			double BufferDurationAtEnd = -1.0;
			double ExpectedNetworkLatency = -1.0;
			double SegmentDuration = 0.0;
			double DownloadTime = 0.0;
			double PlayPosAtDownloadEnd = -1.0;
			int32 Bitrate = 0;
			int32 QualityIndex = 0;
			bool bAborted = false;
		};

		FStreamWorkVars()
		{
			const int32 HistorySize = 3;
			AverageBandwidth.Resize(HistorySize);
			AverageThroughput.Resize(HistorySize);
			const int32 LatencyHistorySize = 5;
			AverageLatency.Resize(LatencyHistorySize);
			Reset();
		}

		void ClearForNextDownload()
		{
			BufferContentDurationAtSegmentStart = -1.0;
			ExpectedNetworkLatency = -1.0;
			QualityIndexDownloading = 0;
			BitrateDownloading = 0;
			Decision = EDecision::AllGood;
			bSufferedRebuffering = false;
			bRebufferingPenaltyApplied = false;
			bWentIntoOvertime = false;
			StreamReaderDecision.Reset();
		}
		void Reset()
		{
			ClearForNextDownload();
			AverageSegmentDuration.SetToInvalid();
			HighestQualityStreamBitrate = 0;
			LowestQualityStreamBitrate = 0;
			NumQualities = 0;
			NumConsecutiveSegmentErrors = 0;
			SegmentDownloadHistory.Clear();
			OverDownloadTimeTotal = 0.0;
			QualityMetrics.Empty();
			EvaluationCount = 0;
			NumSegmentsRebuffered = 0;
			// Note: Do not reset AverageBandwidth, AverageLatency and AverageThroughput!
		}

		void ApplyRebufferingPenalty(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, int32 NumLevelsToDrop)
		{
			// This can be called from different threads!
			FScopeLock lock(&Lock);
			if (!bRebufferingPenaltyApplied)
			{
				bRebufferingPenaltyApplied = true;
				int32 DropQDownBitrate =  QualityIndexDownloading >= NumLevelsToDrop ? InCandidates[QualityIndexDownloading - NumLevelsToDrop]->Bitrate : BitrateDownloading;
				AverageThroughput.Reset();
				AverageBandwidth.Reset();
				int64 nv = Utils::Min(BitrateDownloading / 2, DropQDownBitrate);
				AverageThroughput.AddValue(nv);
				AverageBandwidth.AddValue(nv);
			}
		}


		mutable FCriticalSection Lock;

		TMap<int32, FQualityMetrics> QualityMetrics;
		TMediaQueueFixedStaticNoLock<FSegmentInfo, 20> SegmentDownloadHistory;
		TValueHistory<double> AverageLatency;
		TSimpleMovingAverage<int64> AverageBandwidth;
		TSimpleMovingAverage<int64> AverageThroughput;

		FTimeValue AverageSegmentDuration;
		int64 HighestQualityStreamBitrate = 0;
		int64 LowestQualityStreamBitrate = 0;
		int32 NumQualities = 0;
		int32 NumConsecutiveSegmentErrors = 0;
		bool bIsLowLatencyEnabled = false;

		double BufferContentDurationAtSegmentStart = -1.0;
		double ExpectedNetworkLatency = -1.0;
		double OverDownloadTimeTotal = 0.0;
		int32 QualityIndexDownloading = 0;
		int32 BitrateDownloading = 0;

		EDecision Decision = EDecision::AllGood;

		uint32 EvaluationCount = 0;
		uint32 NumSegmentsRebuffered = 0;

		bool bSufferedRebuffering = false;
		bool bRebufferingPenaltyApplied = false;
		bool bWentIntoOvertime = false;

		bool bBufferingWillCompleteOnTime = false;

		FABRDownloadProgressDecision StreamReaderDecision;
		double LastTimePlaylistRefreshRequested = -1.0;
	};


	struct FLatencyExceededVars
	{
		void Reset()
		{
			bLiveSeekRequestIssued = false;
		}
		bool bLiveSeekRequestIssued = false;
	};

	void PrepareLatencyConfiguration();

	int32 GetQualityIndexForBitrate(EStreamType InStreamType, int32 InForBitrate)
	{
		const TArray<TSharedPtrTS<FABRStreamInformation>>& StreamInfos = Info->GetStreamInformations(InStreamType);
		const TSharedPtrTS<FABRStreamInformation>* Stream = StreamInfos.FindByPredicate([InForBitrate](const TSharedPtrTS<FABRStreamInformation>& InInfo) { return InForBitrate == InInfo->Bitrate;} );
		return Stream ? (*Stream)->QualityIndex : -1;
	}

	TSharedPtrTS<FABRStreamInformation> GetStreamInfoForQualityIndex(EStreamType InStreamType, int32 InQualityIndex)
	{
		const TArray<TSharedPtrTS<FABRStreamInformation>>& StreamInfos = Info->GetStreamInformations(InStreamType);
		const TSharedPtrTS<FABRStreamInformation>* Stream = StreamInfos.FindByPredicate([InQualityIndex](const TSharedPtrTS<FABRStreamInformation>& InInfo) { return InQualityIndex == InInfo->QualityIndex;} );
		return Stream ? (*Stream) : nullptr;
	}

	double GetPlayablePlayerDuration(bool& bEOS, EStreamType InStreamType)
	{
		IPlayerABRLiveControl::FABRBufferStats bs;
		Info->ABRGetStreamBufferStats(bs, InStreamType);
		bEOS = bs.bReachedEnd || bs.bEndOfTrack;
		return bs.PlayableContentDuration.GetAsSeconds();
	}

	bool HasStableBuffer(int32& OutGain, int32& OutTrend, EStreamType InStreamType, double ExtraContentDuration);

	EStreamType GetPrimaryStreamType() const
	{
		return VideoWorkVars.HighestQualityStreamBitrate ? EStreamType::Video : EStreamType::Audio;
	}

	FStreamWorkVars* GetWorkVars(EStreamType ForStreamType)
	{
		return ForStreamType == EStreamType::Video ? &VideoWorkVars : ForStreamType == EStreamType::Audio ? &AudioWorkVars : nullptr;
	}

	FString GetFilenameFromURL(const FString& InURL)
	{
		FURL_RFC3986 UrlParser;
		UrlParser.Parse(InURL);
		return UrlParser.GetLastPathComponent();
	}

	void ApplyRebufferingPenalty();

	double CalculateCatchupPlayRate(const FLatencyConfiguration& InCfg, double Distance);
	void SetRenderRateScale(double InNewRate);


	IABRInfoInterface* Info;
	FCriticalSection Lock;
	EMediaFormatType FormatType;
	EABRPresentationType PresentationType;

	FLatencyConfiguration LatencyConfig;

	FStreamWorkVars VideoWorkVars;
	FStreamWorkVars AudioWorkVars;

	int32 ForcedInitialBandwidth = 0;

	FLatencyExceededVars LatencyExceededVars;
	bool bIsBuffering = false;
	bool bIsRebuffering = false;
	bool bIsSeeking = false;
	bool bIsPaused = true;
	bool bFalloffDetected = false;

	// Work variables used throughout the PrepareStreamCandidateList() -> EvaluateForError() -> EvaluateForQuality() -> PerformSelection() chain.
	bool bRetryIfPossible = false;
	bool bSkipWithFiller = false;

	// Configuration
	TMediaOptionalValue<int32> HTTPStatusCodeToDenyStream;
	bool bRebufferingJustResumesLoading = false;
	bool bEmergencyBufferSlowdownPossible = false;

	// Segment duration to use if no value can be obtained.
	const double DefaultAssumedSegmentDuration = 2.0;

	// Factor to scale the available bandwidth factor with. This can be used to be a bit conservative.
	const double UsableBandwidthScaleFactor = 0.85;

	const int32 NumQualityLevelDropsWhenAborting = 2;

	const double TimeUntilNextPlaylistRefresh = 10.0;
	const int32 TriggerPlaylistRefreshAfterNumLateSegments = 3;
	const int32 MaxTimesToRetryLateSegment = 10;

	// When not playing on the Live edge we delay loading of a segment in the lower
	// quality range if the buffer has more data (and the distance to the Live edge is greater).
	// This avoids loading in many poor quality segments and allows for faster upswitching if bandwidth improves.
	const double BufferLowestQualityRange = 0.3;			// if quality in the lower 30%, delay loading
	const double BufferLowQualityHoldbackThreshold = 4.0;	// amount in buffer above which the delay will happen.

	const double ConnectCheckTimeBySegmentDurationScale = 0.5;
	const double ConnectCheckTimeMin = 2.0;
	const double AbortDownloadTimeBySegmentDurationScale = 1.5;
	const double AbortDownloadCheckTimeMaxOverSegmentDuration = 2.0;
	const double SlowdownAfterDownloadTimeBySegmentDurationScale = 1.075;
	const double SlowdownAfterDownloadCheckTimeMaxOverSegmentDuration = 0.2;

	const double BufferingNonLLCompleteStableScaleBySegmentDuration = 0.3;
	const double EmitPartialDataNonLLBufferDurationBelow = 1.0;

	const double DefaultNetworkLatency = 0.4;
	const double SecondsToReconsiderDeselectedStreamAfter = 60.0;
	const uint32 NumTicklesToReconsiderDeselectedStreamAfter = 3;
	const double BandwidthScaleToReconsiderDeselectedStream = 1.5;
	const double ClampBandwidthToMaxStreamBitrateScaleFactor = 2.0;
	const double DownloadOvertimePenaltyScale = 0.5;

	const int32 StableBufferDramaticDropPercentage = -40;
	const double StableBufferNextHitBadDropSegmentDurationScale = 0.6;
	const double StableBufferByNetworkLatencyScale = 2.0;
	const double StableBufferBestBufferDurationByLatencyScale = 0.75;
	const double StableBufferConstantIncreaseBySegmentDuration = 0.75;
	const double StableBufferConstantDecreaseBySegmentDuration = 0.75;
	const double StableBufferRiskyLevelBySegmentDuration = 0.75;
};



IABRLiveStream* IABRLiveStream::Create(IABRInfoInterface* InInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType)
{
	return new FABRLiveStream(InInfo, InFormatType, InPresentationType);
}






FABRLiveStream::FABRLiveStream(IABRInfoInterface* InInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType)
	: Info(InInfo)
	, FormatType(InFormatType)
	, PresentationType(InPresentationType)
{
	bRebufferingJustResumesLoading = Info->GetOptionValue(OptionRebufferingContinuesLoading).SafeGetBool(false);
	if (Info->HaveOptionValue(ABR::OptionKeyABR_CDNSegmentDenyHTTPStatus))
	{
		HTTPStatusCodeToDenyStream.Set((int32) Info->GetOptionValue(ABR::OptionKeyABR_CDNSegmentDenyHTTPStatus).SafeGetInt64(-1));
	}
	PrepareLatencyConfiguration();

	FTimeRange RenderRange = Info->ABRGetSupportedRenderRateScale();
	bEmergencyBufferSlowdownPossible = RenderRange.IsValid() && RenderRange.Start.GetAsSeconds() <= LatencyConfig.LowBufferMinMinPlayRate;

	bIsBuffering = false;
	bIsSeeking = false;
	bIsPaused = Info->ABRGetPlaySpeed() == FTimeValue::GetZero();
	bFalloffDetected = false;
}

FABRLiveStream::~FABRLiveStream()
{
}

void FABRLiveStream::PrepareLatencyConfiguration()
{
	TSharedPtrTS<const FLowLatencyDescriptor> lld = Info->ABRGetLowLatencyDescriptor();
	double TargetLatency = Info->ABRGetDesiredLiveEdgeLatency().GetAsSeconds();
	FTimeRange TimelineRange = Info->ABRGetTimeline();

	FScopeLock lock(&Lock);
	LatencyConfig.Reset();
	LatencyConfig.TargetLatency = TargetLatency;
	// The target latency should be at least half the maximum permitted latency.
	// When the maximum latency is reached a jump to the Live edge may be performed, so
	// some wiggle room is needed to avoid the target to be close to the maximum, especially
	// if the media segment duration is quite large and starting playback may put us further
	// back and possibly over the permitted maximum.
	if (TargetLatency * 2.0 > LatencyConfig.MaxLatency)
	{
		LatencyConfig.MaxLatency = TargetLatency * 2.0;
	}
	if (lld.IsValid())
	{
		LatencyConfig.MinLatency = lld->GetLatencyMin().IsValid() ? lld->GetLatencyMin().GetAsSeconds() : LatencyConfig.TargetLatency;
		LatencyConfig.MaxLatency = lld->GetLatencyMax().IsValid() ? lld->GetLatencyMax().GetAsSeconds() : LatencyConfig.MaxLatency;
		LatencyConfig.MinPlayRate = lld->GetPlayrateMin().IsValid() ? lld->GetPlayrateMin().GetAsSeconds() : LatencyConfig.MinPlayRate;
		LatencyConfig.MaxPlayRate = lld->GetPlayrateMax().IsValid() ? lld->GetPlayrateMax().GetAsSeconds() : LatencyConfig.MaxPlayRate;
	}
	#ifdef ENABLE_LATENCY_OVERRIDE_CVAR
	AsyncTask(ENamedThreads::GameThread, [=]() {(*CVarElectraTL).Set(*LexToString(TargetLatency), EConsoleVariableFlags::ECVF_SetByCode);});
	#endif
}


void FABRLiveStream::ApplyRebufferingPenalty()
{
	const TArray<TSharedPtrTS<FABRStreamInformation>>& Candidates = Info->GetStreamInformations(GetPrimaryStreamType());
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	if (WorkVars)
	{
		WorkVars->ApplyRebufferingPenalty(Candidates, NumQualityLevelDropsWhenAborting);
	}

}


FABRDownloadProgressDecision FABRLiveStream::ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
{
	FStreamWorkVars* WorkVars = GetWorkVars(SegmentDownloadStats.StreamType);
	if (WorkVars)
	{
		uint32 Flags = WorkVars->StreamReaderDecision.Flags;

		if (SegmentDownloadStats.SegmentType == Metrics::ESegmentType::Media)
		{
			bool bEOS = false;
			const double playableSeconds = GetPlayablePlayerDuration(bEOS, SegmentDownloadStats.StreamType);

			FScopeLock lock(&WorkVars->Lock);

			if (WorkVars->bIsLowLatencyEnabled)
			{
				// For low-latency we immediately emit the data as it comes in.
				Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData;

				// Check that after the larger of half the segment's duration and expected network latency
				// spent on downloading we are at least connected to the server.
				double CheckTime = Utils::Min(Utils::Max(SegmentDownloadStats.Duration * ConnectCheckTimeBySegmentDurationScale, WorkVars->ExpectedNetworkLatency), ConnectCheckTimeMin);
				// On the very first segment download the check time is the segment duration as we often see the first
				// segment to take much longer to fetch than the next ones.
				if (WorkVars->SegmentDownloadHistory.Num() == 0)
				{
					CheckTime = SegmentDownloadStats.Duration;
				}
				if (SegmentDownloadStats.TimeToDownload > CheckTime)
				{
					if (SegmentDownloadStats.TimeToFirstByte <= 0.0)
					{
						int32 QualityIndex = GetQualityIndexForBitrate(SegmentDownloadStats.StreamType, SegmentDownloadStats.Bitrate);
						// If not already on the worst quality, abort the download.
						if (QualityIndex > 0)
						{
							Info->LogMessage(IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Not connected to server after %.3fs for \"%s\""), SegmentDownloadStats.TimeToDownload, *GetFilenameFromURL(SegmentDownloadStats.URL.URL)));
							Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload;
							WorkVars->Decision = FStreamWorkVars::EDecision::NotConnectedAbort;
						}
					}
				}

				// Check if the total download is going into overtime such that we should slow down playback
				// to avoid buffer underrun.
				CheckTime = Utils::Min(SegmentDownloadStats.Duration * SlowdownAfterDownloadTimeBySegmentDurationScale, SegmentDownloadStats.Duration + SlowdownAfterDownloadCheckTimeMaxOverSegmentDuration);
				if (SegmentDownloadStats.TimeToDownload > CheckTime)
				{
					WorkVars->bWentIntoOvertime = true;
				}

				// Check for total download duration exceeded.
				CheckTime = Utils::Min(SegmentDownloadStats.Duration * AbortDownloadTimeBySegmentDurationScale, SegmentDownloadStats.Duration + AbortDownloadCheckTimeMaxOverSegmentDuration);
				if (SegmentDownloadStats.TimeToDownload > CheckTime)
				{
					const double missingDuration = SegmentDownloadStats.Duration - SegmentDownloadStats.DurationDelivered;
					bool bCancel = missingDuration > playableSeconds || playableSeconds/missingDuration < 4.0 || playableSeconds < 0.5;
					//Info->LogMessage(IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Total download time %.3fs exceeded for \"%s\", %.3fs missing, %.3fs ready -> %s"), SegmentDownloadStats.TimeToDownload, *GetFilenameFromURL(SegmentDownloadStats.URL), missingDuration, playableSeconds, bCancel?TEXT("CANCEL"):TEXT("KEEP GOING")));
					if (bCancel)
					{
						WorkVars->Decision = FStreamWorkVars::EDecision::ConnectedAbort;
						Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload;
						if (SegmentDownloadStats.DurationDelivered > 0.0)
						{
							Flags |= FABRDownloadProgressDecision::EDecisionFlags::eABR_InsertFillerData;
						}
					}
				}
			}
			else
			{
				double EstimatedTotalDownloadTime = SegmentDownloadStats.Duration;

				const int32 QualityIndex = WorkVars->QualityIndexDownloading;
				const bool bIsEmittingPartial = (Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData) != 0;

				// Estimate how long we will take to complete the download.
				// Base this off the byte sizes?
				if (SegmentDownloadStats.NumBytesDownloaded > 0 && SegmentDownloadStats.ByteSize > 0)
				{
					EstimatedTotalDownloadTime = (SegmentDownloadStats.TimeToDownload / SegmentDownloadStats.NumBytesDownloaded) * SegmentDownloadStats.ByteSize;
				}
				// Otherwise base it off the durations
				else if (SegmentDownloadStats.DurationDownloaded > 0.0 && SegmentDownloadStats.Duration > 0.0)
				{
					EstimatedTotalDownloadTime = (SegmentDownloadStats.TimeToDownload / SegmentDownloadStats.DurationDownloaded) * SegmentDownloadStats.Duration;
				}

				bool bCheck = !bIsBuffering && !bIsPaused;
				// If not already on the worst quality check if we should abort the download.
				if (bCheck && QualityIndex > 0)
				{
					if (SegmentDownloadStats.TimeToDownload > SegmentDownloadStats.Duration)
					{
						if (!bIsEmittingPartial)
						{
							// If we have not yet emitted the data loaded so far we abort when the total download time exceeds what is
							// currently available in the buffer plus what has been downloaded so far.
							if (EstimatedTotalDownloadTime > playableSeconds + SegmentDownloadStats.DurationDownloaded)
							{
								Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload;
								WorkVars->Decision = FStreamWorkVars::EDecision::ConnectedAbort;
							}
						}
						else
						{
							// Since we are already emitting partial segment data we check if an emergency slowdown is possible
							// and if so trigger it. Otherwise we abort if there is not enough content available.
							if (EstimatedTotalDownloadTime > playableSeconds)
							{
								if (bEmergencyBufferSlowdownPossible)
								{
									WorkVars->bWentIntoOvertime = true;
								}
								else
								{
									Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload;
									Flags |= FABRDownloadProgressDecision::EDecisionFlags::eABR_InsertFillerData;
									WorkVars->Decision = FStreamWorkVars::EDecision::ConnectedAbort;
								}
							}
						}
					}

					// If the buffer is low allow the data being streamed in to already be emitted to the buffer unless we are to abort.
					if (!bIsEmittingPartial && (Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload) == 0)
					{
						if (playableSeconds < EmitPartialDataNonLLBufferDurationBelow && SegmentDownloadStats.DurationDownloaded > 0.0 && SegmentDownloadStats.DurationDelivered == 0.0)
						{
							Flags |= FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData;
						}
					}
				}

				// While buffering check when the segment is likely to complete downloading with enough buffered data
				// to allow playback from that point on.
				if (bIsBuffering && (Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload) == 0)
				{
					const double EstimatedDownloadTimeRemaining = EstimatedTotalDownloadTime - SegmentDownloadStats.TimeToDownload;
					if (SegmentDownloadStats.DurationDownloaded / SegmentDownloadStats.Duration > BufferingNonLLCompleteStableScaleBySegmentDuration &&
						playableSeconds > EstimatedDownloadTimeRemaining)
					{
						WorkVars->bBufferingWillCompleteOnTime = true;
					}
					// Emit partial segment data for faster startup. We will not switching the startup quality anyway.
					if (SegmentDownloadStats.DurationDownloaded > 0.0 && SegmentDownloadStats.DurationDelivered == 0.0)
					{
						Flags |= FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData;
					}
				}
			}

			// When aborting give a human readable reason.
			if ((Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload) != 0)
			{
				WorkVars->StreamReaderDecision.Reason = FString::Printf(TEXT("Abort download of %s segment @ %d bps for time %.3f with %.3fs in buffer and %.3fs of %.3fs fetched after %.3fs.")
					, GetStreamTypeName(SegmentDownloadStats.StreamType)
					, SegmentDownloadStats.Bitrate
					, SegmentDownloadStats.PresentationTime
					, playableSeconds
					, SegmentDownloadStats.DurationDownloaded
					, SegmentDownloadStats.Duration
					, SegmentDownloadStats.TimeToDownload);
				//Info->LogMessage(IInfoLog::ELevel::Verbose, WorkVars->StreamReaderDecision.Reason);
			}
		}

		WorkVars->StreamReaderDecision.Flags = (FABRDownloadProgressDecision::EDecisionFlags) Flags;
		return WorkVars->StreamReaderDecision;
	}
	else
	{
		FABRDownloadProgressDecision Decision;
		Decision.Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData;
		return Decision;
	}
}

void FABRLiveStream::ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
{
	if (SegmentDownloadStats.bWasSkipped)
	{
		return;
	}
	else if (SegmentDownloadStats.bWasFalloffSegment)
	{
		bFalloffDetected = true;
		return;
	}

	FStreamWorkVars* WorkVars = GetWorkVars(SegmentDownloadStats.StreamType);

	if (WorkVars && SegmentDownloadStats.SegmentType == Metrics::ESegmentType::Media)
	{
		const int32 QualityIndex = GetQualityIndexForBitrate(SegmentDownloadStats.StreamType, SegmentDownloadStats.Bitrate);
		int64 Throughput = -1;

		FScopeLock lock(&WorkVars->Lock);
		if (!SegmentDownloadStats.bIsCachedResponse)
		{
			if (WorkVars->bSufferedRebuffering)
			{
				++WorkVars->NumSegmentsRebuffered;
				WorkVars->Decision = WorkVars->Decision == FStreamWorkVars::EDecision::AllGood ? FStreamWorkVars::EDecision::Rebuffered : WorkVars->Decision;
			}

			// Track average latency.
			if (SegmentDownloadStats.TimeToFirstByte > 0.0)
			{
				WorkVars->AverageLatency.AddValue(SegmentDownloadStats.TimeToFirstByte);
			}
			WorkVars->OverDownloadTimeTotal += SegmentDownloadStats.TimeToDownload - SegmentDownloadStats.DurationDelivered;

			// Calculate download bandwidth
			int64 Bandwidth = -1;
			if (SegmentDownloadStats.NumBytesDownloaded && SegmentDownloadStats.TimeToDownload > 0.0)
			{
				const double DlBandwidth = SegmentDownloadStats.NumBytesDownloaded * 8 / (SegmentDownloadStats.TimeToDownload - SegmentDownloadStats.TimeToFirstByte);
				if (DlBandwidth > 0.0)
				{
					Bandwidth = Utils::Min((int64)DlBandwidth, (int64)(WorkVars->HighestQualityStreamBitrate * ClampBandwidthToMaxStreamBitrateScaleFactor));
				}
			}

			// Calculate MOOF throughput
			FChunkedDownloadBandwidthCalculator Chunker(SegmentDownloadStats);
			const double ChunkThroughput = Chunker.Calculate();
			if (ChunkThroughput > 0.0)
			{
				Throughput = Utils::Min((int64)ChunkThroughput, (int64)(WorkVars->HighestQualityStreamBitrate * ClampBandwidthToMaxStreamBitrateScaleFactor));
			}

			// Take a penalty if the download had to be aborted, otherwise add the bandwidth and throughput to the trackers.
			if (WorkVars->Decision == FStreamWorkVars::EDecision::NotConnectedAbort || WorkVars->Decision == FStreamWorkVars::EDecision::ConnectedAbort)
			{
				const TArray<TSharedPtrTS<FABRStreamInformation>>& Candidates = Info->GetStreamInformations(SegmentDownloadStats.StreamType);
				WorkVars->ApplyRebufferingPenalty(Candidates, NumQualityLevelDropsWhenAborting);
				Throughput = WorkVars->AverageThroughput.GetLastSample();
			}
			else
			{
				Throughput = Throughput > 0 ? Throughput : Bandwidth;
				if (Throughput > 0)
				{
					WorkVars->AverageThroughput.AddValue(Throughput);
				}
				else
				{
					WorkVars->AverageThroughput.AddValue(WorkVars->AverageThroughput.GetLastSample() / 2);
				}
				if (Bandwidth > 0)
				{
					WorkVars->AverageBandwidth.AddValue(Bandwidth);
				}
			}
		}
		else
		{
			// When we hit the cache we cannot calculate a meaningful bandwidth. We need to update the averages nonetheless
			// to make a quality upswitch possible on one of the next uncached segments.
			Throughput = SegmentDownloadStats.Bitrate;
			const TArray<TSharedPtrTS<FABRStreamInformation>>& StreamInformation = Info->GetStreamInformations(SegmentDownloadStats.StreamType);
			const int64 LastAddedThroughput = WorkVars->AverageThroughput.GetLastSample();
			const int32 NextHighestQualityIndex = QualityIndex + 1 < StreamInformation.Num() ? QualityIndex + 1 : QualityIndex;
			const int32 NextHighestBitrate = StreamInformation[NextHighestQualityIndex]->Bitrate;
			// Last seen bandwidth already higher than what we might want to switch up to next?
			if (LastAddedThroughput > NextHighestBitrate)
			{
				// We want to bring the average down a bit in preparation for the next uncached segment in case the network
				// has degraded. This way we are not overshooting the target.
				Throughput = (LastAddedThroughput + NextHighestBitrate) / 2;
				WorkVars->AverageThroughput.AddValue(Throughput);
				WorkVars->AverageBandwidth.AddValue(Throughput);
			}
			else if (LastAddedThroughput > Throughput)
			{
				// The last bandwidth was somewhere between the bitrate of the cached segment and the next quality.
				// We would love to switch up with the next segment. This might be possible because we got this segment's data
				// at lightning speed and have its duration worth of time to spend on attempting the next segment download.
				// If that doesn't finish in time we can downswitch without hurting things too much.
				Throughput = NextHighestBitrate;
				WorkVars->AverageThroughput.InitializeTo(Throughput);
				WorkVars->AverageBandwidth.InitializeTo(Throughput);
			}
			else
			{
				// Let's try to bring the average up a bit.
				Throughput = (int32)(Throughput * 0.2 + NextHighestBitrate * 0.8);
				WorkVars->AverageThroughput.AddValue(Throughput);
				WorkVars->AverageBandwidth.AddValue(Throughput);
			}
		}

		if (WorkVars->QualityMetrics.Contains(QualityIndex))
		{
			// Update the metrics of the downloaded segment
			FQualityMetrics& segQM = WorkVars->QualityMetrics[QualityIndex];
			segQM.AverageKbps.AddValue((double) Throughput / 1000.0);
		}

		FStreamWorkVars::FSegmentInfo si;
		bool bEOS = false;
		si.BufferDurationAtStart = WorkVars->BufferContentDurationAtSegmentStart;
		si.BufferDurationAtEnd = GetPlayablePlayerDuration(bEOS, SegmentDownloadStats.StreamType);
		si.ExpectedNetworkLatency = WorkVars->ExpectedNetworkLatency;
		si.SegmentDuration = SegmentDownloadStats.Duration;
		si.DownloadTime = SegmentDownloadStats.TimeToDownload;
		si.PlayPosAtDownloadEnd = Info->ABRGetPlayPosition().GetAsSeconds();
		si.Bitrate = SegmentDownloadStats.Bitrate;
		si.QualityIndex = QualityIndex;
		si.bAborted = SegmentDownloadStats.bWasAborted;
		if (WorkVars->SegmentDownloadHistory.IsFull())
		{
			WorkVars->SegmentDownloadHistory.Pop();
		}
		WorkVars->SegmentDownloadHistory.Push(si);
	}
}


void FABRLiveStream::ReportBufferingStart(Metrics::EBufferingReason BufferingReason)
{
	bIsBuffering = true;
	bIsRebuffering = BufferingReason == Metrics::EBufferingReason::Rebuffering;
	bIsSeeking = BufferingReason == Metrics::EBufferingReason::Seeking;

	VideoWorkVars.bBufferingWillCompleteOnTime = false;
	AudioWorkVars.bBufferingWillCompleteOnTime = false;
	if (BufferingReason == Metrics::EBufferingReason::Rebuffering)
	{
		VideoWorkVars.bSufferedRebuffering = true;
		AudioWorkVars.bSufferedRebuffering = true;
	}
}

void FABRLiveStream::ReportBufferingEnd(Metrics::EBufferingReason BufferingReason)
{
	bIsBuffering = false;
	bIsRebuffering = false;
	bIsSeeking = false;
	bFalloffDetected = false;
	LatencyExceededVars.Reset();
}

void FABRLiveStream::ReportPlaybackPaused()
{
	bIsPaused = true;
}

void FABRLiveStream::ReportPlaybackResumed()
{
	bIsPaused = false;
}

void FABRLiveStream::ReportPlaybackEnded()
{
	bIsBuffering = false;
	bIsSeeking = false;
	bIsPaused = true;
	bFalloffDetected = false;
}

void FABRLiveStream::RepresentationsChanged(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPlayPeriod)
{
	FStreamWorkVars* WorkVars = GetWorkVars(InStreamType);
	if (WorkVars)
	{
		TArray<TSharedPtrTS<FABRStreamInformation>> Representations = Info->GetStreamInformations(InStreamType);

		FScopeLock lock(&WorkVars->Lock);
		WorkVars->Reset();
		WorkVars->AverageSegmentDuration.SetToInvalid();
		WorkVars->HighestQualityStreamBitrate = 0;
		WorkVars->LowestQualityStreamBitrate = 1000000000;
		WorkVars->bIsLowLatencyEnabled = false;
		WorkVars->NumQualities = Representations.Num();

		if (Representations.Num())
		{
			InCurrentPlayPeriod->GetAverageSegmentDuration(WorkVars->AverageSegmentDuration, Representations[0]->AdaptationSetUniqueID, Representations[0]->RepresentationUniqueID);
			for(auto &Rep : Representations)
			{
				if (Rep->Bitrate > WorkVars->HighestQualityStreamBitrate)
				{
					WorkVars->HighestQualityStreamBitrate = Rep->Bitrate;
				}
				if (Rep->Bitrate < WorkVars->LowestQualityStreamBitrate)
				{
					WorkVars->LowestQualityStreamBitrate = Rep->Bitrate;
				}
			}
			// We currently assume all representations are either suitable for low-latency or not, so we look only at the first.
			WorkVars->bIsLowLatencyEnabled = Representations[0]->bLowLatencyEnabled;
		}
	}

	PrepareLatencyConfiguration();
}

void FABRLiveStream::SetBandwidth(int64 bitsPerSecond)
{
	ForcedInitialBandwidth = (int32)bitsPerSecond;
}

void FABRLiveStream::SetForcedNextBandwidth(int64 bitsPerSecond, double /*minBufferTimeBeforePlayback*/)
{
	ForcedInitialBandwidth = (int32)bitsPerSecond;
}

int64 FABRLiveStream::GetLastBandwidth()
{
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	return WorkVars ? WorkVars->AverageBandwidth.GetLastSample() : ForcedInitialBandwidth;
}

int64 FABRLiveStream::GetAverageBandwidth()
{
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	return WorkVars ? WorkVars->AverageBandwidth.GetSMA(ForcedInitialBandwidth) : ForcedInitialBandwidth;
}

int64 FABRLiveStream::GetAverageThroughput()
{
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	return WorkVars ? WorkVars->AverageThroughput.GetSMA(ForcedInitialBandwidth) : ForcedInitialBandwidth;
}

double FABRLiveStream::GetAverageLatency()
{
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	return WorkVars ? WorkVars->AverageLatency.GetWeightedMax() : 0.0;
}


void FABRLiveStream::PrepareStreamCandidateList(TArray<TSharedPtrTS<FABRStreamInformation>>& OutCandidates, EStreamType StreamType, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const FTimeValue& TimeNow)
{
	bRetryIfPossible = false;
	bSkipWithFiller = false;

	// Get the list of streams for this type.
	OutCandidates = Info->GetStreamInformations(StreamType);
}

IAdaptiveStreamSelector::ESegmentAction FABRLiveStream::EvaluateForError(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	FStreamWorkVars* WorkVars = GetWorkVars(StreamType);
	double Now = Info->ABRGetWallclockTime().GetAsSeconds();

	// Allow another playlist updates after enough time has elapsed.
	if (WorkVars->LastTimePlaylistRefreshRequested > 0.0 && Now - WorkVars->LastTimePlaylistRefreshRequested > TimeUntilNextPlaylistRefresh)
	{
		WorkVars->LastTimePlaylistRefreshRequested = -1.0;
	}

	if (CurrentSegment.IsValid())
	{
		Metrics::FSegmentDownloadStats _Stats;
		CurrentSegment->GetDownloadStats(_Stats);
		const Metrics::FSegmentDownloadStats& Stats = _Stats;
		// Try to get the stream information for the current downloaded segment. We may not find it on a period transition where the
		// segment is the last of the previous period.
		TSharedPtrTS<FABRStreamInformation> CurrentStreamInfo = Info->GetStreamInformation(Stats);
		if (CurrentStreamInfo.IsValid())
		{
			// Remember stats
			CurrentStreamInfo->Health.LastDownloadStats = Stats;

			if (!Stats.bWasSuccessful)
			{
				if (Stats.bWaitingForRemoteRetryElement)
				{
					return IAdaptiveStreamSelector::ESegmentAction::Retry;
				}

				// If this is the first failure for this segment (not being retried yet) increase
				// the number of consecutively failed segments.
				if (Stats.RetryNumber == 0)
				{
					++WorkVars->NumConsecutiveSegmentErrors;
				}

				// Was not successful. Figure out what to do now.
				// In the case where the segment had an availability window set the stream reader was waiting for
				// to be entered and we got a 404 back, the server did not manage to publish the new segment in time.
				// We try this again after a slight delay.
				if (Stats.AvailibilityDelay && Utils::AbsoluteValue(Stats.AvailibilityDelay) < 0.5)
				{
					//Info->LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Segment \"%s\" not available at announced time. Trying again in 0.15s"), *GetFilenameFromURL(Stats.URL)));
					// For low-latency streams trigger a clock resync as a 404 may be indicative of clock drift.
					if (WorkVars->bIsLowLatencyEnabled)
					{
						Info->ABRTriggerClockSync(IPlayerABRLiveControl::EClockSyncType::Required);
					}
					else
					{
						// For normal Live have the manifest add a backoff. If this gets too large the manifest may trigger a clock resync.
						CurrentPlayPeriod->IncreaseSegmentFetchDelay(FTimeValue(FTimeValue::MillisecondsToHNS(100)));
					}
					OutDelay.SetFromMilliseconds(150);

					if (Stats.RetryNumber > TriggerPlaylistRefreshAfterNumLateSegments && WorkVars->LastTimePlaylistRefreshRequested < 0.0)
					{
						WorkVars->LastTimePlaylistRefreshRequested = Info->ABRGetWallclockTime().GetAsSeconds();
						Info->ABRTriggerPlaylistRefresh();
					}
					return Stats.RetryNumber < MaxTimesToRetryLateSegment ? IAdaptiveStreamSelector::ESegmentAction::Retry : IAdaptiveStreamSelector::ESegmentAction::Fail;
				}

				// Too many failures already? It's unlikely that there is a way that would magically fix itself now.
				if (Stats.RetryNumber >= 3)
				{
					Info->LogMessage(IInfoLog::ELevel::Error, FString::Printf(TEXT("Exceeded permissable number of retries (%d). Failing now."), Stats.RetryNumber));
					return IAdaptiveStreamSelector::ESegmentAction::Fail;
				}
				if (WorkVars->NumConsecutiveSegmentErrors >= 5)
				{
					Info->LogMessage(IInfoLog::ELevel::Error, FString::Printf(TEXT("Exceeded permissable number of consecutive segment failures (%d). Failing now."), WorkVars->NumConsecutiveSegmentErrors));
					return IAdaptiveStreamSelector::ESegmentAction::Fail;
				}

				// Is this an init segment?
				if (Stats.SegmentType == Metrics::ESegmentType::Init)
				{
					bRetryIfPossible = true;
					// A failure other than having aborted in ReportDownloadProgress() ?
					if (!Stats.bWasAborted)
					{
						// Did the init segment fail to be parsed or does the HTTP status match the one used to disable the stream?
						// If so this stream is dead for good.
						if (Stats.bParseFailure || (HTTPStatusCodeToDenyStream.IsSet() && HTTPStatusCodeToDenyStream.Value() == Stats.HTTPStatusCode))
						{
							CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC.SetToInvalid();
						}
						else if (Stats.HTTPStatusCode && Stats.RetryNumber == 0)
						{
							// Do one more try.
							OutDelay.SetFromMilliseconds(150);
						}
						// Take this stream offline for a brief moment.
						else if (CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC == FTimeValue::GetZero())
						{
							CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC = TimeNow + FTimeValue().SetFromMilliseconds(1000);
						}
					}
				}
				// A media segment failure
				else
				{
					// Check if there is a HTTP status code set which, if returned, is used to disable this stream permanently.
					if (HTTPStatusCodeToDenyStream.IsSet() && HTTPStatusCodeToDenyStream.Value() == Stats.HTTPStatusCode)
					{
						CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC.SetToInvalid();
						bRetryIfPossible = true;
						bSkipWithFiller = false;
					}
					else
					{
						bRetryIfPossible = true;
						bSkipWithFiller = true;

						// Did we abort that stream in ReportDownloadProgress() ?
						if (!Stats.bWasAborted)
						{
							// Take a video stream offline for a brief moment unless it is the worst one. Audio is usually the only one with no alternative to switch to, so don't do that!
							if (Stats.StreamType == EStreamType::Video)
							{
								if (CurrentStreamInfo->QualityIndex != 0)
								{
									if (CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC == FTimeValue::GetZero())
									{
										CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC = TimeNow + FTimeValue().SetFromMilliseconds(1000);
									}
								}
								else
								{
									// Need to insert filler data instead.
									bRetryIfPossible = false;
								}
							}
							else
							{
								// For VoD we retry the audio segment once with a small delay before skipping over it, which we do for Live right away
								// since retrying is a luxury we do not have in this case.
								// Note: This here ABR is optimized for Live. For VoD a dedicated Live ABR should be employed!
								if (PresentationType == EABRPresentationType::OnDemand)
								{
									if (Stats.RetryNumber == 0)
									{
										OutDelay.SetFromMilliseconds(150);
										bRetryIfPossible = true;
									}
									else
									{
										bRetryIfPossible = false;
									}
								}
								else
								{
									bRetryIfPossible = false;
								}
							}
						}

						// If any content has already been put out into the buffers we cannot retry the segment on another quality level.
						if (Stats.DurationDelivered > 0.0)
						{
							bRetryIfPossible = false;
							bSkipWithFiller = false;
						}
					}
				}
			}
			else
			{
				if (!Stats.bInsertedFillerData)
				{
					WorkVars->NumConsecutiveSegmentErrors = 0;
				}
			}
		}
	}
	return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
}


IAdaptiveStreamSelector::ESegmentAction FABRLiveStream::EvaluateForQuality(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	/*
		This method doesn't evaluate which exact stream to use. It merely reduces the candidates based on externally
		set limits, like maximum bitrate and resolution.
		The calling ABR will further reduce this list by removing streams that are not healthy.
	*/
	if (StreamType != EStreamType::Video)
	{
		InOutCandidates = Info->GetStreamInformations(StreamType);
		return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
	}

	if (InOutCandidates.Num())
	{
		TArray<TSharedPtrTS<FABRStreamInformation>> PossibleRepresentations;
		PossibleRepresentations.Add(InOutCandidates[0]);
		const int32 MaxAllowedBandwidth = Info->GetBandwidthCeiling();
		const FStreamCodecInformation::FResolution MaxAllowedResolution = Info->GetMaxStreamResolution();
		for(int32 nStr=1; nStr<InOutCandidates.Num(); ++nStr)
		{
			// Check if bitrate and resolution are acceptable
			if (InOutCandidates[nStr]->Bitrate <= MaxAllowedBandwidth && !InOutCandidates[nStr]->Resolution.ExceedsLimit(MaxAllowedResolution))
			{
				PossibleRepresentations.Add(InOutCandidates[nStr]);
			}
		}
		Swap(InOutCandidates, PossibleRepresentations);
	}
	return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
}


IAdaptiveStreamSelector::ESegmentAction FABRLiveStream::PerformSelection(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	if (InCandidates.Num())
	{
		FStreamWorkVars* WorkVars = GetWorkVars(StreamType);
		double Now = Info->ABRGetWallclockTime().GetAsSeconds();

		double BandwidthScore = 1.0;
		int32 NewQualityIndex = -1;

		int32 CurrentStreamQualityIndex = -1;

		int32 BufferTrend = 0;
		int32 BufferGain = 0;
		bool bIsBufferStable = true;

		double ConsumptionRateScale = Info->ABRGetRenderRateScale();
		/*
			When playing slower than normal we don't want to affect bandwidth scoring such that it would
			allow to switch up in quality. Playing slower means that we need to back off because we are
			not getting new data fast enough, so switching up would be quite detrimental.
		*/
		if (ConsumptionRateScale < 1.0)
		{
			ConsumptionRateScale = 1.0;
		}

		bool bEOS = false;
		const double AvailableDuration = GetPlayablePlayerDuration(bEOS, StreamType);
		const double DownloadLatency = WorkVars ? WorkVars->AverageLatency.GetWeightedMax(DefaultNetworkLatency) : DefaultNetworkLatency;

		if (WorkVars && StreamType == EStreamType::Video)
		{
			FScopeLock lock(&WorkVars->Lock);

			int64 AvailableDownloadBPS = 0;
			const int64 DefaultInitialBandwidth = Utils::Max(WorkVars->LowestQualityStreamBitrate, (int64)ForcedInitialBandwidth);
			if (WorkVars->bIsLowLatencyEnabled)
			{
				AvailableDownloadBPS = WorkVars->AverageThroughput.GetWMA(DefaultInitialBandwidth);
			}
			else
			{
				AvailableDownloadBPS = Utils::Min(WorkVars->AverageBandwidth.GetWMA(DefaultInitialBandwidth), WorkVars->AverageThroughput.GetWMA(DefaultInitialBandwidth));
			}
			AvailableDownloadBPS *= UsableBandwidthScaleFactor;

			++WorkVars->EvaluationCount;

			TArray<FDecisionAttributes> QualityDecision;

			bIsBufferStable = HasStableBuffer(BufferGain, BufferTrend, StreamType, 0.0);
			CurrentStreamQualityIndex = WorkVars->SegmentDownloadHistory.Num() ? WorkVars->SegmentDownloadHistory.BackRef().QualityIndex : -1;

			double OvertimePenalty = WorkVars->OverDownloadTimeTotal > 0.0 ? WorkVars->OverDownloadTimeTotal * DownloadOvertimePenaltyScale : 0.0;
			WorkVars->OverDownloadTimeTotal = 0.0;

			const double AvgSegDur = WorkVars->AverageSegmentDuration.GetAsSeconds(DefaultAssumedSegmentDuration);
			for(auto &Can : InCandidates)
			{
				int64 EstimatedSegmentBitSize = Can->Bitrate * AvgSegDur;
				FDecisionAttributes da;
				da.QualityIndex = Can->QualityIndex;
				da.Bitrate = Can->Bitrate;
				da.EstimatedTimeToDownload = DownloadLatency + OvertimePenalty + EstimatedSegmentBitSize / (double)AvailableDownloadBPS;
				da.BandwidthScore = AvgSegDur / (da.EstimatedTimeToDownload * ConsumptionRateScale);
				QualityDecision.Emplace(MoveTemp(da));
			}

			// First segment?
			if (CurrentStreamQualityIndex < 0)
			{
				for(int32 i=QualityDecision.Num()-1; i>=0; --i)
				{
					if (QualityDecision[i].BandwidthScore >= 1.0 || i==0)
					{
						QualityDecision[i].bIsCandidate = true;
						BandwidthScore = QualityDecision[i].BandwidthScore;
						NewQualityIndex = QualityDecision[i].QualityIndex;
						break;
					}
				}
			}
			else
			{
				// Use different bandwidth scale ratios to perform up-/down switching based on buffer trend.
				double UpswitchRatio = 1.2;
				double DownswitchRatio = 1.0;
				int32 MaxQualityUpswitchLevels = 2;
				check(BufferTrend >= -2 && BufferTrend <= 2);
				if (BufferTrend >= -2 && BufferTrend <= 2)
				{
					if (bIsBufferStable)
					{
						const double UpRatios[] = { 100.0, 10.0, 1.3, 1.1, 1.05 };
						const double DownRatios[] = { 1.0, 1.0, 1.0, 0.95, 0.9 };
						const int32 MaxUpLevels[] = { 0, 0, 1, 1, 2 };

						UpswitchRatio = UpRatios[ BufferTrend + 2 ];
						DownswitchRatio = DownRatios[ BufferTrend + 2 ];
						MaxQualityUpswitchLevels = MaxUpLevels[ BufferTrend + 2 ];
					}
					else
					{
						const double UpRatios[] = { 100.0, 10.0, 1.2, 1.2, 1.2 };
						const double DownRatios[] = { 1.1, 1.0, 1.0, 1.0, 1.0 };
						const int32 MaxUpLevels[] = { 0, 0, 1, 1, 1 };

						UpswitchRatio = UpRatios[ BufferTrend + 2 ];
						DownswitchRatio = DownRatios[ BufferTrend + 2 ];
						MaxQualityUpswitchLevels = MaxUpLevels[ BufferTrend + 2 ];
					}
				}

				for(auto &qd : QualityDecision)
				{
					if (qd.QualityIndex > CurrentStreamQualityIndex && qd.BandwidthScore >= UpswitchRatio)
					{
						bool bIsUsable = true;
						if (WorkVars->QualityMetrics.Contains(qd.QualityIndex))
						{
							FQualityMetrics& candQM = WorkVars->QualityMetrics[qd.QualityIndex];
							if (candQM.DeselectReason == FQualityMetrics::EDeselectReason::BufferLoss)
							{
								bIsUsable = false;
								uint32 PrevEvalCount = candQM.LastEvaluationCount;
								candQM.LastEvaluationCount = WorkVars->EvaluationCount;
								if (candQM.LastEvaluationCount - PrevEvalCount == 1)
								{
									++candQM.TickleCount;
								}
								else
								{
									candQM.TickleCount = 0;
								}

								const double TimeSinceLoss = Now - candQM.DeselectedAtTime;
								/*
									After some time or number of consecutive times (tickles) this quality is deemed usable
									we add the stream's bandwidth to the average it was playing at to gradually
									lower the limit at which to consider this stream again.
								*/
								if (TimeSinceLoss > SecondsToReconsiderDeselectedStreamAfter || candQM.TickleCount >= NumTicklesToReconsiderDeselectedStreamAfter)
								{
									candQM.AverageKbps.AddValue(qd.Bitrate / 1000);
								}
								int64 PlayedAt = candQM.AverageKbps.GetAverage();
								if (AvailableDownloadBPS/1000.0 > PlayedAt * BandwidthScaleToReconsiderDeselectedStream)
								{
									bIsUsable = true;
								}
							}
						}

						// If the quality upswitch exceeds the number of steps we are allowed to switch up, then we do not use this stream.
						if (qd.QualityIndex - CurrentStreamQualityIndex > MaxQualityUpswitchLevels)
						{
							bIsUsable = false;
						}

						qd.bIsCandidate = bIsUsable;
					}
					if (qd.QualityIndex <= CurrentStreamQualityIndex && qd.BandwidthScore >= DownswitchRatio)
					{
						qd.bIsCandidate = true;
					}
				}
			}

			for(int32 i=QualityDecision.Num()-1; i>=0; --i)
			{
				// If the current stream had a download issue do not use it or any better one again in this iteration.
				if (QualityDecision[i].QualityIndex >= CurrentStreamQualityIndex &&
					(WorkVars->Decision == FStreamWorkVars::EDecision::NotConnectedAbort ||
					 WorkVars->Decision == FStreamWorkVars::EDecision::ConnectedAbort ||
					 WorkVars->Decision == FStreamWorkVars::EDecision::Rebuffered))
				{
					QualityDecision[i].bIsCandidate = false;
				}

				if (QualityDecision[i].bIsCandidate || i==0)
				{
					BandwidthScore = QualityDecision[i].BandwidthScore;
					NewQualityIndex = QualityDecision[i].QualityIndex;
					break;
				}
			}


			/*
				Check if we are in the lower configured range of all qualities and if so, hold back the download
				for a while to avoid polluting the buffer with low quality data in case the bandwidth will recover soon
				and we could get better quality then.
			*/
			if (!Info->ABRShouldPlayOnLiveEdge() && Info->ABRGetLatency().GetAsSeconds() > BufferLowQualityHoldbackThreshold)
			{
				if (WorkVars->NumQualities && NewQualityIndex < (int32)(WorkVars->NumQualities * BufferLowestQualityRange + 0.5))
				{
					if (AvailableDuration > BufferLowQualityHoldbackThreshold)
					{
						OutDelay.SetFromSeconds(AvailableDuration - BufferLowQualityHoldbackThreshold);
						//Info->LogMessage(IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Delaying segment by %.3fs"), OutDelay.GetAsSeconds()));
					}
				}
			}
		}
		else
		{
			if (WorkVars)
			{
				FScopeLock lock(&WorkVars->Lock);

				TArray<FDecisionAttributes> QualityDecision;

				CurrentStreamQualityIndex = WorkVars->SegmentDownloadHistory.Num() ? WorkVars->SegmentDownloadHistory.BackRef().QualityIndex : -1;

				for(auto &Can : InCandidates)
				{
					FDecisionAttributes da;
					da.QualityIndex = Can->QualityIndex;
					da.Bitrate = Can->Bitrate;
					da.bIsCandidate = true;
					QualityDecision.Emplace(MoveTemp(da));
				}

				for(int32 i=QualityDecision.Num()-1; i>=0; --i)
				{
					if (QualityDecision[i].bIsCandidate || i==0)
					{
						BandwidthScore = QualityDecision[i].BandwidthScore;
						NewQualityIndex = QualityDecision[i].QualityIndex;
						break;
					}
				}
			}
		}

		check(NewQualityIndex >= 0);
		TSharedPtrTS<FABRStreamInformation> Candidate = GetStreamInfoForQualityIndex(StreamType, NewQualityIndex);
		if (Candidate.IsValid())
		{
			if (WorkVars)
			{
				if (NewQualityIndex != CurrentStreamQualityIndex)
				{
					FQualityMetrics& newQM = WorkVars->QualityMetrics.FindOrAdd(NewQualityIndex);
					newQM.AverageKbps.Reset();
					newQM.SelectedAtRatio = BandwidthScore;
					newQM.SelectedAtTime = Now;
					newQM.DeselectedAtTime = Now;
					newQM.DeselectReason = FQualityMetrics::EDeselectReason::None;

					if (CurrentStreamQualityIndex >= 0)
					{
						FQualityMetrics& leaveQM = WorkVars->QualityMetrics.FindOrAdd(CurrentStreamQualityIndex);
						leaveQM.DeselectedAtTime = Now;

						FQualityMetrics::EDeselectReason SwitchReason = FQualityMetrics::EDeselectReason::None;
						if (NewQualityIndex < CurrentStreamQualityIndex)
						{
							if (WorkVars->Decision == FStreamWorkVars::EDecision::AllGood)
							{
								if (!bIsBufferStable && BufferTrend < 0)
								{
									SwitchReason = FQualityMetrics::EDeselectReason::BufferLoss;
								}
								else
								{
									SwitchReason = FQualityMetrics::EDeselectReason::Bandwidth;
								}
							}
							else
							{
								SwitchReason = FQualityMetrics::EDeselectReason::DownloadError;
							}
						}
						else
						{
							/*
								When switching up we need to change all the lower quality stream deselect reason
								from "buffer loss" to "bandwidth" to allow them to be selected again in the next downswitch.
							*/
							for(auto &qm : WorkVars->QualityMetrics)
							{
								if (qm.Key < NewQualityIndex)
								{
									qm.Value.DeselectReason = qm.Value.DeselectReason == FQualityMetrics::EDeselectReason::BufferLoss ? FQualityMetrics::EDeselectReason::Bandwidth : qm.Value.DeselectReason;
								}
							}
							SwitchReason = FQualityMetrics::EDeselectReason::Upswitch;
						}
						leaveQM.DeselectReason = SwitchReason;
					}
				}

				WorkVars->ClearForNextDownload();
				WorkVars->QualityIndexDownloading = NewQualityIndex;
				WorkVars->BitrateDownloading = Candidate->Bitrate;
				WorkVars->BufferContentDurationAtSegmentStart = AvailableDuration;
				WorkVars->ExpectedNetworkLatency = DownloadLatency;

				// For low-latency we want to fetch the init segments of the streams one quality level up and below
				// in case we can/have to switch there on the next iteration. We do not want to take the penalty of
				// downloading them just in time then.
				if (WorkVars->bIsLowLatencyEnabled)
				{
					auto GetNeighbour = [&](bool bUp) -> TSharedPtrTS<FABRStreamInformation>
					{
						for(int32 i=0; i<InCandidates.Num(); ++i)
						{
							if (InCandidates[i]->QualityIndex == NewQualityIndex)
							{
								if (bUp && i+1 < InCandidates.Num())
								{
									return InCandidates[i+1];
								}
								else if (!bUp && i>0)
								{
									return InCandidates[i-1];
								}
							}
						}
						return nullptr;
					};

					TArray<IManifest::IPlayPeriod::FInitSegmentPreload> InitSegmentsToPreload;
					TSharedPtrTS<FABRStreamInformation> OneUp = GetNeighbour(true);
					TSharedPtrTS<FABRStreamInformation> OneDown = GetNeighbour(false);
					if (OneUp.IsValid() || OneDown.IsValid())
					{
						IManifest::IPlayPeriod::FInitSegmentPreload pl;
						if (OneUp.IsValid())
						{
							pl.AdaptationSetID = OneUp->AdaptationSetUniqueID;
							pl.RepresentationID = OneUp->RepresentationUniqueID;
							InitSegmentsToPreload.Emplace(MoveTemp(pl));
						}
						if (OneDown.IsValid())
						{
							pl.AdaptationSetID = OneDown->AdaptationSetUniqueID;
							pl.RepresentationID = OneDown->RepresentationUniqueID;
							InitSegmentsToPreload.Emplace(MoveTemp(pl));
						}
						CurrentPlayPeriod->TriggerInitSegmentPreload(InitSegmentsToPreload);
					}
				}
			}

			CurrentPlayPeriod->SelectStream(Candidate->AdaptationSetUniqueID, Candidate->RepresentationUniqueID, Candidate->QualityIndex, Info->GetStreamInformations(StreamType).Num()-1);
			if (bRetryIfPossible)
			{
				return IAdaptiveStreamSelector::ESegmentAction::Retry;
			}
			else if (bSkipWithFiller)
			{
				return IAdaptiveStreamSelector::ESegmentAction::Fill;
			}
			else
			{
				return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
			}
		}
	}
	else
	{
		// Note: Are there representations we could lift the temporary denylist from?
		if (bSkipWithFiller)
		{
			return IAdaptiveStreamSelector::ESegmentAction::Fill;
		}
	}
	return IAdaptiveStreamSelector::ESegmentAction::Fail;
}







bool FABRLiveStream::HasStableBuffer(int32& OutGain, int32& OutTrend, EStreamType InStreamType, double ExtraContentDuration)
{
	OutTrend = 0;
	OutGain = 0;

	const FStreamWorkVars* WorkVars = GetWorkVars(InStreamType);
	if (!WorkVars)
	{
		return true;
	}
	FScopeLock lock(&WorkVars->Lock);

	bool bEOS = false;
	const double AvailableDuration = GetPlayablePlayerDuration(bEOS, InStreamType) + ExtraContentDuration;
	const double DesiredLatency = LatencyConfig.TargetLatency;

	/*
		Get a reference segment duration.

		It is possible for the segment duration to be larger than the desired latency, espcially for low-latency
		Live streams where segments are consumed as they are produced. Large segment durations can't really be
		used to gauge buffer stability since it will not really be possible to gather that much data ahead of time.
	*/
	const double SegmentDuration = Utils::Min(WorkVars->AverageSegmentDuration.GetAsSeconds(DefaultAssumedSegmentDuration), DesiredLatency);

	// Try to determine if the buffer is stable.
	if (WorkVars && WorkVars->SegmentDownloadHistory.Num() > 1)
	{
		const FStreamWorkVars::FSegmentInfo& Last = WorkVars->SegmentDownloadHistory.BackRef();
		const double NetworkLatency = WorkVars->AverageLatency.GetSMA();

		// No data at all?
		if (Last.BufferDurationAtEnd == 0.0 || Last.BufferDurationAtStart == 0.0)
		{
			return false;
		}


		// Buffer level
		double r = Last.BufferDurationAtEnd / Last.BufferDurationAtStart;
		OutGain = (int32)(r >= 1.0 ? (r - 1.0) * 100.0 : (1.0 - r) * -100.0);
		// If we just took a large buffer loss we say the buffer is not stable, regardless of how much content is in there.
		if (OutGain < StableBufferDramaticDropPercentage)
		{
			OutTrend = -2;
			return false;
		}

		// If we took a hit and the same hit again would put us under a given segment duration's worth then we are not stable.
		if (OutGain < 0 && Last.BufferDurationAtEnd * r < SegmentDuration * StableBufferNextHitBadDropSegmentDurationScale)
		{
			OutTrend = -2;
			return false;
		}

		// Because of constant jitter we cannot really expect to get more data buffered than that
		const double BestExpectedBufferedDuration = DesiredLatency * StableBufferBestBufferDurationByLatencyScale;

		// If we have what we can expect then we are stable.
		if (AvailableDuration >= BestExpectedBufferedDuration)
		{
			OutTrend = 2;
			return true;
		}

		// Do we currently have more than the network latency?
		const bool bMoreThanNetworkLatency = AvailableDuration >= NetworkLatency * StableBufferByNetworkLatencyScale;
		const bool bConsiderStable = bMoreThanNetworkLatency;

		// We need to look at the past buffer ratios to see how they behave.
		const int32 kLastN = 3;
		const int32 kLastI = WorkVars->SegmentDownloadHistory.Num();
		const int32 NumRecents = Utils::Min(kLastN, kLastI);
		int32 SignChanges = 0;
		double PrevChange = 0.0;
		TValueHistory<double> AvgBuffDiffs;
		AvgBuffDiffs.Resize(NumRecents - 1);
		for(int32 i=kLastI-NumRecents; i<kLastI-1; ++i)
		{
			double dx = WorkVars->SegmentDownloadHistory[i+1].BufferDurationAtEnd - WorkVars->SegmentDownloadHistory[i].BufferDurationAtEnd;
			AvgBuffDiffs.AddValue(dx);
			PrevChange = i==kLastI-NumRecents ? dx : PrevChange;
			SignChanges += PrevChange * dx < 0.0 ? 1 : 0;
			PrevChange = dx;
		}
		const double tdiff = Last.BufferDurationAtEnd - WorkVars->SegmentDownloadHistory[kLastI-NumRecents].BufferDurationAtEnd;
		// No sign changes? (constant increase or decrease)
		if (SignChanges == 0 && tdiff > 0.0)
		{
			if (AvailableDuration > SegmentDuration * StableBufferConstantIncreaseBySegmentDuration)
			{
				OutTrend = 1;
				return bConsiderStable;
			}
		}
		if (SignChanges == 0 && -tdiff > SegmentDuration * StableBufferConstantDecreaseBySegmentDuration)
		{
			OutTrend = -1;
			return false;
		}

		// Alternating gain and loss. Is the average change positive (gaining)?
		const double avgChange = AvgBuffDiffs.GetWMA(tdiff);
		if (avgChange > 0.0 && AvailableDuration > SegmentDuration)
		{
			OutTrend = 1;
			return bConsiderStable;
		}

		// Assume no trend
		OutTrend = 0;
		// If average change is negative, or the total difference between the first and last inspected history
		// samples is a loss, or the most recent gain is a loss set the trend to be declining.
		if (avgChange < 0.0 || tdiff < 0.0 || OutGain < 0)
		{
			OutTrend = -1;
		}
		// If still enough content available we say that we are still stable at the moment.
		if (AvailableDuration > SegmentDuration * StableBufferRiskyLevelBySegmentDuration)
		{
			return bConsiderStable;
		}
	}
	return false;
}


FTimeValue FABRLiveStream::GetMinBufferTimeForPlayback(IAdaptiveStreamSelector::EMinBufferType InBufferingType, FTimeValue InDefaultMBT)
{
	if (FormatType == EMediaFormatType::DASH)
	{
		const FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
		if (WorkVars)
		{
			// For low latency we return a value small enough to start pretty much immediately
			// but require a tiny amount of data to be buffered to minimize the chance of an immediate rebuffer.
			if (WorkVars->bIsLowLatencyEnabled)
			{
				return FTimeValue(0.3);
			}
			// When it has been determined that buffering will soon be done return a value small enough
			// to just start now.
			else if (WorkVars->bBufferingWillCompleteOnTime)
			{
				return FTimeValue(0.1);
			}
		}
	}
	return FTimeValue();
}

IAdaptiveStreamSelector::FRebufferAction FABRLiveStream::GetRebufferAction()
{
	IAdaptiveStreamSelector::FRebufferAction Action;

	if (PresentationType == EABRPresentationType::OnDemand)
	{
		Action.Action = bRebufferingJustResumesLoading ? IAdaptiveStreamSelector::FRebufferAction::EAction::ContinueLoading : IAdaptiveStreamSelector::FRebufferAction::EAction::Restart;
	}
	else
	{
		if (FormatType == EMediaFormatType::DASH)
		{
			const FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
			if (WorkVars && WorkVars->bIsLowLatencyEnabled)
			{
				Action.Action = IAdaptiveStreamSelector::FRebufferAction::EAction::ContinueLoading;
			}
			else
			{
				ApplyRebufferingPenalty();
				Action.Action = Info->ABRShouldPlayOnLiveEdge() ? IAdaptiveStreamSelector::FRebufferAction::EAction::GoToLive : IAdaptiveStreamSelector::FRebufferAction::EAction::Restart;
			}
		}
		else
		{
			ApplyRebufferingPenalty();
			Action.Action = Info->ABRShouldPlayOnLiveEdge() ? IAdaptiveStreamSelector::FRebufferAction::EAction::GoToLive : IAdaptiveStreamSelector::FRebufferAction::EAction::Restart;
			//Action.Action = IAdaptiveStreamSelector::FRebufferAction::EAction::GoToLive;
		}
	}
	return Action;
}


IAdaptiveStreamSelector::EHandlingAction FABRLiveStream::PeriodicHandle()
{
	IAdaptiveStreamSelector::EHandlingAction NextAction = IAdaptiveStreamSelector::EHandlingAction::None;

	Lock.Lock();
	#ifdef ENABLE_LATENCY_OVERRIDE_CVAR
		LatencyConfig.TargetLatency = CVarElectraTL.GetValueOnAnyThread();
	#endif
	FLatencyConfiguration lc(LatencyConfig);
	Lock.Unlock();

	// Perform catch up / slow down of playback when playing (not paused and not buffering).
	if (!bIsPaused && !bIsBuffering && Info->ABRGetPlaySpeed() != FTimeValue::GetZero())
	{
		if (bFalloffDetected)
		{
			bFalloffDetected = false;
			if (!LatencyExceededVars.bLiveSeekRequestIssued)
			{
				LatencyExceededVars.bLiveSeekRequestIssued = true;
				NextAction = IAdaptiveStreamSelector::EHandlingAction::SeekToLive;
				Info->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Timeline fall off detected, requesting seek to Live edge")));
			}
		}
		else
		{
			const EStreamType StreamType = GetPrimaryStreamType();
			const FStreamWorkVars* WorkVars = GetWorkVars(StreamType);
			if (WorkVars)
			{
				bool bEOS = false;
				const double AvailableBufferedDuration = GetPlayablePlayerDuration(bEOS, StreamType);
				const double CurrentLatency = Info->ABRGetLatency().GetAsSeconds();
				const double TargetLatency = lc.TargetLatency;
				const double NetworkLatency = WorkVars->AverageLatency.GetWeightedMax(DefaultNetworkLatency);
				const double Distance = CurrentLatency - TargetLatency;
				const double AbsDistance = Utils::AbsoluteValue(Distance);
				const double CurrentPlayRate = Info->ABRGetRenderRateScale();
				const bool bOvertime = WorkVars->bWentIntoOvertime;

				int32 Gain, Trend;
				const bool bStable = HasStableBuffer(Gain, Trend, StreamType, 0.0);
				const bool bMaybeSpeedUp = bStable || Trend>=0;

				// Slow down because of insufficient buffered data?
				bool bBufferSlowDown = bOvertime || (!bEOS && AvailableBufferedDuration < Utils::Max(lc.LowBufferContentBackoff, NetworkLatency));
				// Do not slow down if we are supposed to play on the Live edge and are already too far behind.
				if (Info->ABRShouldPlayOnLiveEdge() && Distance > lc.LowBufferMaxTargetLatencyDistance)
				{
					bBufferSlowDown = false;
				}
				if (bBufferSlowDown)
				{
					const double NewRate = lc.LowBufferMinMinPlayRate;
					SetRenderRateScale(NewRate);
				}
				// Should we be playing on the Live edge?
				else if (Info->ABRShouldPlayOnLiveEdge() && AbsDistance > lc.ActivationThreshold * TargetLatency)
				{
					double NewRate = CalculateCatchupPlayRate(lc, Distance);
					if (Utils::AbsoluteValue(NewRate - CurrentPlayRate) > lc.MinRateChangeUseThreshold)
					{
						// If we are to speed up to catch up with the Live edge we do so only if the buffer is stable.
						if (NewRate > 1.0 && !bMaybeSpeedUp)
						{
							NewRate = 1.0;
						}
						SetRenderRateScale(NewRate);
					}
				}
				else if (CurrentPlayRate != 1.0)
				{
					SetRenderRateScale(1.0);
				}

				// Is the maximum latency exceeded and a jump to the Live edge required?
				if (Info->ABRShouldPlayOnLiveEdge() && CurrentLatency > lc.MaxLatency)
				{
					if (!LatencyExceededVars.bLiveSeekRequestIssued)
					{
						LatencyExceededVars.bLiveSeekRequestIssued = true;
						NextAction = IAdaptiveStreamSelector::EHandlingAction::SeekToLive;
						Info->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Maximum latency exceeded, requesting seek to Live edge")));
					}
				}
			}
		}
	}
	return NextAction;
}

void FABRLiveStream::SetRenderRateScale(double InNewRate)
{
	FTimeRange pr = Info->ABRGetSupportedRenderRateScale();
	if (pr.IsValid())
	{
		double MinAllowed = pr.Start.GetAsSeconds(1.0);
		double MaxAllowed = pr.End.GetAsSeconds(1.0);
		Info->ABRSetRenderRateScale(Utils::Min(Utils::Max(MinAllowed, InNewRate), MaxAllowed));
	}
}

double FABRLiveStream::CalculateCatchupPlayRate(const FLatencyConfiguration& InCfg, double Distance)
{
	const double PlayRate = Distance >= 0.0 ? InCfg.MaxPlayRate - 1.0 : 1.0 - InCfg.MinPlayRate;
	/*
		Use a nice Sigmoid function ( https://en.wikipedia.org/wiki/Sigmoid_function ),
		specifically the Logistics function, to determine the play rate.
		x is scaled by a factor to make the "S" steeper so that a larger rate is maintained for longer
		before starting to fall off. We need to perform catch up relatively quickly and not drag this
		out forever. We also center it around 1 so we can use the value directly.
	*/
	const double Rate = (1.0 - PlayRate) + (2.0 * PlayRate) / (1.0 + FMath::Pow(DOUBLE_EULERS_NUMBER, Distance * -5.0));
	return Rate;
}



void FABRLiveStream::DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...))
{
	const EStreamType StreamType = GetPrimaryStreamType();
	const FStreamWorkVars* WorkVars = GetWorkVars(StreamType);

	pPrintFN(pThat, "===== (LL) Live ABR state =====");

	pPrintFN(pThat, "Buffering: %d", bIsBuffering);
	pPrintFN(pThat, "Rebuffering: %d", bIsRebuffering);
	pPrintFN(pThat, "Seeking: %d", bIsSeeking);
	pPrintFN(pThat, "Paused: %d", bIsPaused);

	pPrintFN(pThat, "   Low latency: %s", WorkVars->bIsLowLatencyEnabled ? "Yes" : "No");
	pPrintFN(pThat, "  Live latency: %.3f", Info->ABRGetLatency().GetAsSeconds());
	pPrintFN(pThat, "Target latency: %.3f", LatencyConfig.TargetLatency);
	pPrintFN(pThat, "  Play on edge: %s", Info->ABRShouldPlayOnLiveEdge() ? "Yes" : "No");
	pPrintFN(pThat, "  Catchup rate: %.3f", Info->ABRGetRenderRateScale());
	pPrintFN(pThat, "Network latency: %.3f", WorkVars->AverageLatency.GetWeightedMax(DefaultNetworkLatency));
	pPrintFN(pThat, " Num rebuffers: %d", WorkVars->NumSegmentsRebuffered);
	bool bEOS = false;
	double Dur = GetPlayablePlayerDuration(bEOS, GetPrimaryStreamType());
	pPrintFN(pThat, "Duration avail: %.3f, EOS=%d", Dur, bEOS);
#if 0
	int32 Gain, Trend;
	bool bStable = HasStableBuffer(Gain, Trend, StreamType, 0.0);
	const char* TrendChars[5] = { "dropping", "draining", "same", "gaining", "rising" };
	pPrintFN(pThat, " Buffer stable: %d %s(%d) %d%% gain", bStable, TrendChars[Trend+2], Trend, Gain);
#endif
	pPrintFN(pThat, " ll min: %.3f, max: %.3f", LatencyConfig.MinLatency, LatencyConfig.MaxLatency);
	pPrintFN(pThat, " pr min: %.3f, max: %.3f", LatencyConfig.MinPlayRate, LatencyConfig.MaxPlayRate);

}



} // namespace Electra
