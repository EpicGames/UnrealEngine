// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{
namespace ABR
{
	//! (bool) true to just finish the currently loading segment when rebuffering. false to start over with.
	//const FName OptionKeyABR_RebufferingContinuesLoading(TEXT("abr:rebuffering_continues_loading"));

	//! (int) HTTP status code if returned by CDN for any segment request will take that stream permanently offline.
	const FName OptionKeyABR_CDNSegmentDenyHTTPStatus(TEXT("abr:cdn_deny_httpstatus"));
}
} // namespace Electra


