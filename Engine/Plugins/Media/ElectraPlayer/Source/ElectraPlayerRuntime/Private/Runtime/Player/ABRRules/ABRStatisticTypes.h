// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>


namespace Electra
{


template <typename T>
class TRollingAverage
{
public:
	TRollingAverage()
	{ Reset(); }
	
	void Reset()
	{
		Avg = Min = Max = T(0);
		N = 0;
	}

	T AddValue(T InNewValue)
	{
		if (++N == 1)
		{
			Avg = Min = Max = InNewValue;
		}
		else
		{
			if (InNewValue < Min)
			{
				Min = InNewValue;
			}
			if (InNewValue > Max)
			{
				Max = InNewValue;
			}
			Avg = Avg + ((InNewValue - Avg) / N);
		}
		return Avg;
	}

	T GetAverage() const
	{
		return Avg;
	}

	T GetMin() const
	{
		return Min;
	}
	
	T GetMax() const
	{
		return Max;
	}
	
	int32 GetNumSamples() const
	{
		return N;
	}
private:
	T Avg;
	T Min;
	T Max;
	int32 N;
};


template <typename T>
class TSimpleMovingAverage
{
public:
	TSimpleMovingAverage(int32 maxNumSMASamples = kMaxSMASamples)
		: MaxSamples(maxNumSMASamples)
	{
		SimpleMovingAverage.Resize(MaxSamples);
		LastSample = T(0);
	}

	TSimpleMovingAverage(const TSimpleMovingAverage& Other)
	{
		InternalCopy(Other);
	}

	void Resize(int32 maxSMASamples)
	{
		FScopeLock lock(&CriticalSection);
		MaxSamples = maxSMASamples;
		SimpleMovingAverage.Clear();
		SimpleMovingAverage.Resize(MaxSamples);
	}

	void Reset()
	{
		FScopeLock lock(&CriticalSection);
		SimpleMovingAverage.Clear();
	}

	void InitializeTo(const T& v)
	{
		FScopeLock lock(&CriticalSection);
		Reset();
		AddValue(v);
	}

	//! Adds a sample to the mean.
	void AddValue(const T& value)
	{
		FScopeLock lock(&CriticalSection);

		LastSample = value;

		if (SimpleMovingAverage.Capacity())
		{
			if (SimpleMovingAverage.Num() >= MaxSamples)
			{
				SimpleMovingAverage.Pop();
			}
			check(!SimpleMovingAverage.IsFull());
			SimpleMovingAverage.Push(LastSample);
		}
	}

	//! Returns the sample value that was added last, if one exists
	T GetLastSample(T V=T(0)) const
	{
		FScopeLock lock(&CriticalSection);
		return SimpleMovingAverage.Num() ? LastSample : V;
	}

	/*
		Returns the simple moving average from all the collected history samples.
		See: https://en.wikipedia.org/wiki/Moving_average#Simple_moving_average
	*/
	T GetSMA(T V=T(0)) const
	{
		FScopeLock lock(&CriticalSection);
		if (SimpleMovingAverage.Num())
		{
			T sum = 0;
			for(int32 i=0; i<SimpleMovingAverage.Num(); ++i)
			{
				sum += SimpleMovingAverage[i];
			}
			return T(sum / SimpleMovingAverage.Num());
		}
		return V;
	}
	T GetAverage(T V=T(0)) const
	{
		return GetSMA(V);
	}

	/*
		Returns the weighted moving average from all the collected history samples.
		See: https://en.wikipedia.org/wiki/Moving_average#Weighted_moving_average
	*/
	T GetWMA(T V=T(0)) const
	{
		FScopeLock lock(&CriticalSection);
		int32 iMax=SimpleMovingAverage.Num();
		if (iMax)
		{
			T sum = 0;
			for(int32 i=0; i<iMax; ++i)
			{
				sum += SimpleMovingAverage[i] * (i+1);
			}
			return T(sum / ((iMax * (iMax+1) / 2)));
		}
		return V;
	}

private:
	enum
	{
		kMaxSMASamples = 5,		//!< Default amount of SMA samples
	};

	void InternalCopy(const TSimpleMovingAverage& Other)
	{
		FScopeLock l1(&CriticalSection);
		FScopeLock l2(&Other.CriticalSection);

		MaxSamples = Other.kMaxSMASamples;
		LastSample = Other.LastSample;
		SimpleMovingAverage.Resize(MaxSamples);
		SimpleMovingAverage.AppendFirstElements(Other.SimpleMovingAverage);
	}

	mutable FCriticalSection	CriticalSection;
	TMediaQueueNoLock<T>		SimpleMovingAverage;	//!< Simple moving average
	T							LastSample;				//!< The last sample that was added.
	int32						MaxSamples;				//!< Limit number of SMA samples (0 < mMaxSMASamples <= kMaxSMASamples)
};




template <typename T>
class TValueHistory
{
public:
	TValueHistory(int32 InMaxSamples = 5)
	{
		check(InMaxSamples);
		MaxSamples = InMaxSamples;
		Samples.Reserve(MaxSamples);
	}


	TValueHistory(const TValueHistory& Other)
	{
		InternalCopy(Other);
	}


	void Resize(int32 InMaxSamples)
	{
		check(InMaxSamples);
		FScopeLock lock(&Lock);
		Samples.Reset(InMaxSamples);
		MaxSamples = InMaxSamples;
		NextIndex = 0;
		CurrentSortOrder = ESortOrder::Index;
	}

	void Reset()
	{
		FScopeLock lock(&Lock);
		Samples.Reset();
		NextIndex = 0;
		CurrentSortOrder = ESortOrder::Index;
	}

	void InitializeTo(const T& v)
	{
		FScopeLock lock(&Lock);
		Reset();
		AddValue(v);
	}

	void AddValue(const T& value)
	{
		FScopeLock lock(&Lock);

		SortByIndex();
		if (MaxSamples)
		{
			while(Samples.Num() >= MaxSamples)
			{
				Samples.RemoveAt(0);
			}
			FSample s;
			s.Value = value;
			s.Index = ++NextIndex;
			Samples.Emplace(MoveTemp(s));
		}
	}

	T GetWeightedMax(T V=T(0)) const
	{
		FScopeLock lock(&Lock);
		SortByValueDescending();
		return InternalWMA(V);
	}

	T GetSMA(T V=T(0)) const
	{
		FScopeLock lock(&Lock);
		int32 iMax=Samples.Num();
		if (iMax)
		{
			T sum = 0;
			for(int32 i=0; i<iMax; ++i)
			{
				sum += Samples[i].Value;
			}
			return T(sum / iMax);
		}
		return V;
	}

	T GetWMA(T V=T(0)) const
	{
		FScopeLock lock(&Lock);
		SortByIndex();
		return InternalWMA(V);
	}

private:
	struct FSample
	{
		T Value;
		uint64 Index;
	};
	enum class ESortOrder
	{
		Index,
		Value
	};

	void InternalCopy(const TValueHistory& Other)
	{
		FScopeLock lock1(&Lock);
		FScopeLock lock2(&Other.Lock);

		Samples = Other.Samples;
		NextIndex = Other.NextIndex;
		MaxSamples = Other.MaxSamples;
		CurrentSortOrder = Other.CurrentSortOrder;
	}

	T InternalWMA(T V=T(0)) const
	{
		int32 iMax=Samples.Num();
		if (iMax)
		{
			T sum = 0;
			for(int32 i=0; i<iMax; ++i)
			{
				sum += Samples[i].Value * (i+1);
			}
			return T(sum / ((iMax * (iMax+1) / 2)));
		}
		return V;
	}

	void SortByIndex() const
	{
		if (CurrentSortOrder != ESortOrder::Index)
		{
			CurrentSortOrder = ESortOrder::Index;
			Samples.Sort([](const FSample& a, const FSample&b) { return a.Index < b.Index; });
		}
	}

	void SortByValueDescending() const
	{
		if (CurrentSortOrder != ESortOrder::Value)
		{
			CurrentSortOrder = ESortOrder::Value;
			Samples.Sort([](const FSample& a, const FSample&b) { return b.Value < a.Value; });
		}
	}

	mutable FCriticalSection Lock;
	mutable TArray<FSample> Samples;
	mutable ESortOrder CurrentSortOrder = ESortOrder::Index;
	int64 NextIndex = 0;
	int32 MaxSamples = 0;
};




class FChunkedDownloadBandwidthCalculator
{
public:
	FChunkedDownloadBandwidthCalculator(const Metrics::FSegmentDownloadStats& InSegmentStats)
	: SegmentStats(InSegmentStats)
	{ }

	double Calculate()
	{
		struct FE
		{
			double T;
			double Bps;
			const double GetY() const
			{ return Bps; }
		};
		TArray<FE> Throughput;

		TimeUntilFirstByte = SegmentStats.TimeToFirstByte;
		LastAvailByteCount = SegmentStats.TimingTraces.Num() ? SegmentStats.TimingTraces.Last().TotalBytesAdded : 0;
		int32 StartTrace = 0;
		bool bDone = false;
		for(int32 nC=0; !bDone && nC<SegmentStats.MovieChunkInfos.Num(); )
		{
			int32 firstNC = nC;
			int32 chkIdx0 = GetDownloadChunkIndexForPos(SegmentStats.MovieChunkInfos[nC].PayloadStartOffset, StartTrace);
			if (chkIdx0 < 0)
			{
				break;
			}
			int32 chkIdx1 = -1;
			while(!bDone && nC<SegmentStats.MovieChunkInfos.Num())
			{
				chkIdx1 = GetDownloadChunkIndexForPos(SegmentStats.MovieChunkInfos[nC].PayloadEndOffset, chkIdx0);
				if (chkIdx1 < 0)
				{
					bDone = true;
					break;
				}
				int32 nextC;
				for(nextC=nC+1; nextC<SegmentStats.MovieChunkInfos.Num(); ++nextC)
				{
					int32 chkIdxN = GetDownloadChunkIndexForPos(SegmentStats.MovieChunkInfos[nextC].PayloadEndOffset, chkIdx1);
					if (chkIdxN != chkIdx1)
					{
						break;
					}
				}

				const double minDl = 0.001;
				double dlStart = chkIdx1 != chkIdx0 ? SegmentStats.TimingTraces[chkIdx0].TimeSinceStart : chkIdx0 ? SegmentStats.TimingTraces[chkIdx0 - 1].TimeSinceStart : SegmentStats.TimeToFirstByte;
				double dlDur = SegmentStats.TimingTraces[chkIdx1].TimeSinceStart - dlStart;
				nC = nextC;
				if (dlDur > minDl)
				{
					int64 dlNum = SegmentStats.TimingTraces[chkIdx1].TotalBytesAdded - SegmentStats.MovieChunkInfos[firstNC].HeaderOffset;
					FE fe;
					fe.T = dlStart - SegmentStats.TimeToFirstByte;
					fe.Bps = dlNum * 8.0 / dlDur;
					Throughput.Emplace(MoveTemp(fe));
					break;
				}
				else
				{
					int x=0;
				}
			}
			StartTrace = chkIdx1;
		}


		class FInterquartileRange
		{
		public:
			void Apply(TArray<FE>& Out, const TArray<FE>& In)
			{
				Out = In;
				const int32 N = Out.Num();
				if (N > 2)
				{
					Out.Sort([](const FE& e1, const FE& e2) { return e1.GetY() < e2.GetY(); });
					double Q1, Q3;
					int32 R = N % 4;
					if (R > 1)
					{
						Q1 = Out[N / 4].GetY();
						Q3 = Out[3*N / 4].GetY();
					}
					else
					{
						int32 N1 = N / 4;
						int32 N3 = 3*N / 4;
						Q1 = (Out[N1 - 1].GetY() + Out[N1].GetY()) / 2.0;
						Q3 = (R == 0 ? (Out[N3-1].GetY() + Out[N3].GetY()) : (Out[N3].GetY() + Out[N3+1].GetY())) / 2.0;
					}
					Out.Reset();
					for(auto& E : In)
					{
						if (E.GetY() >= Q1 && E.GetY() <= Q3)
						{
							Out.Add(E);
						}
					}
					check(Out.Num());
				}
			}
		};


		TArray<FE> Filtered;
		FInterquartileRange Filter;
		Filter.Apply(Filtered, Throughput);
//			Swap(Filtered, Throughput);

#if 0
		double sx = 0.0;
		double sy = 0.0;
		double sxy = 0.0;
		double sx2 = 0.0;
		double N = Throughput.Num();
		for(auto& tp : Throughput)
		{
			sx += tp.T;
			sy += tp.Bps;
			sxy += tp.T * tp.Bps;
			sx2 = tp.T * tp.T;
		}
		double M = (N * sxy - sx * sy) / (N * sx2 - sx * sx);
		double B = (sy - M * sx) / N;

		double lrls = B + M * 0.5;
		return lrls;
			

#elif 1
		double Average = -1.0;
		if (Throughput.Num())
		{
			Average = 0.0;
			for(int32 i=0; i<Throughput.Num(); ++i)
			{
				Average += Throughput[i].Bps;
			}
			Average = Average / Throughput.Num();
		}
		return Average;

#elif 1
		double HarmonicMean = -1.0;
		if (Throughput.Num())
		{
			HarmonicMean = 0.0;
			for(int32 i=0; i<Throughput.Num(); ++i)
			{
				HarmonicMean += 1.0 / Throughput[i].Bps;
			}
			HarmonicMean = Throughput.Num() / HarmonicMean;
		}
		return HarmonicMean;
#endif
	}

private:
	int32 GetDownloadChunkIndexForPos(int64 InByteOffset, int32 InStartAtTrace)
	{
		if (InByteOffset <= LastAvailByteCount)
		{
			for(int32 i=InStartAtTrace; i<SegmentStats.TimingTraces.Num(); ++i)
			{
				if (InByteOffset <= SegmentStats.TimingTraces[i].TotalBytesAdded)
				{
					return i;
				}
			}
		}
		return -1;
	}

	const Metrics::FSegmentDownloadStats& SegmentStats;
	int64 LastAvailByteCount = 0;
	double TimeUntilFirstByte = 0.0;
};


} // namespace Electra
