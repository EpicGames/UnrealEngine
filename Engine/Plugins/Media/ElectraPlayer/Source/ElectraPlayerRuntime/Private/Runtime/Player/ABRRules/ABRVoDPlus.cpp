// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/ABRRules/ABRVoDPlus.h"
#include "Player/ABRRules/ABRStatisticTypes.h"
#include "Player/ABRRules/ABROptionKeynames.h"
#include "Player/AdaptivePlayerOptionKeynames.h"

#include "Player/Manifest.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Utilities/Utilities.h"
#include "Utilities/URLParser.h"

#define ENABLE_CYCLE_CVAR 0

#if ENABLE_CYCLE_CVAR
#include "HAL/IConsoleManager.h"
static TAutoConsoleVariable<int32> CVarElectraABRVODCycle(
	TEXT("Electra.ABR.VoD.CycleQualities"),
	0,
	TEXT("Cycles through all quality levels even if not sustainable.\n")
	TEXT(" 0: do not cycle; 1 to n: cycle up to quality level n."),
	ECVF_Default);
static TAutoConsoleVariable<int32> CVarElectraABRVODFixedStream(
	TEXT("Electra.ABR.VoD.FixedStream"),
	0,
	TEXT("Forces use of a particular quality even if not sustainable.\n")
	TEXT(" 0: do not lock; 1 to n: lock to quality level n-1."),
	ECVF_Default);
#endif



namespace Electra
{

class FABROnDemandPlus : public IABROnDemandPlus
{
public:
	FABROnDemandPlus(IABRInfoInterface* InInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType);
	virtual ~FABROnDemandPlus();

	FABRDownloadProgressDecision ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override;
	void ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override;
	void ReportBufferingStart(Metrics::EBufferingReason BufferingReason) override;
	void ReportBufferingEnd(Metrics::EBufferingReason BufferingReason) override;
	void ReportPlaybackPaused() override;
	void ReportPlaybackResumed() override;
	void ReportPlaybackEnded() override;

	FTimeValue GetMinBufferTimeForPlayback(IAdaptiveStreamSelector::EMinBufferType InBufferingType, FTimeValue InDefaultMBT) override;
	IAdaptiveStreamSelector::FRebufferAction GetRebufferAction() override;
	IAdaptiveStreamSelector::EHandlingAction PeriodicHandle() override;
	void DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...)) override;

	void RepresentationsChanged(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPlayPeriod) override;
	void SetBandwidth(int64 bitsPerSecond) override;
	void SetForcedNextBandwidth(int64 bitsPerSecond, double minBufferTimeBeforePlayback) override;
	int64 GetLastBandwidth() override;
	int64 GetAverageBandwidth() override;
	int64 GetAverageThroughput() override;
	double GetAverageLatency() override;

	void PrepareStreamCandidateList(TArray<TSharedPtrTS<FABRStreamInformation>>& OutCandidates, EStreamType StreamType, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction EvaluateForError(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction EvaluateForQuality(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;
	IAdaptiveStreamSelector::ESegmentAction PerformSelection(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow) override;

private:
	struct FStreamWorkVars
	{
		enum class EDecision
		{
			AllGood,
			NotConnectedAbort,				// Aborted because not connected to server in time.
			ConnectedAbort,					// Aborted for some reason while receiving data
			Rebuffered,						// Hit by rebuffering
		};

		FStreamWorkVars()
		{
			const int32 HistorySize = 5;
			AverageBandwidth.Resize(HistorySize);
			const int32 LatencyHistorySize = 5;
			AverageLatency.Resize(LatencyHistorySize);
			Reset();
		}

		void ClearForNextDownload()
		{
			QualityIndexDownloading = 0;
			BitrateDownloading = 0;
			Decision = EDecision::AllGood;
			bSufferedRebuffering = false;
			bRebufferingPenaltyApplied = false;
			bWentIntoOvertime = false;
			StreamReaderDecision.Reset();
		}
		void Reset()
		{
			ClearForNextDownload();
			AverageSegmentDuration.SetToInvalid();
			LowestQualityStreamBitrate = 0;
			HighestQualityStreamBitrate = 0;
			NumQualities = 0;
			NumConsecutiveSegmentErrors = 0;
			// Note: Do not reset AverageBandwidth and AverageLatency!
		}

		void ApplyRebufferingPenalty(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, int32 NumLevelsToDrop)
		{
			// This can be called from different threads!
			FScopeLock lock(&Lock);
			if (!bRebufferingPenaltyApplied)
			{
				bRebufferingPenaltyApplied = true;
				int32 DropQDownBitrate =  QualityIndexDownloading >= NumLevelsToDrop ? InCandidates[QualityIndexDownloading - NumLevelsToDrop]->Bitrate : BitrateDownloading;
				AverageBandwidth.Reset();
				int64 nv = Utils::Min(BitrateDownloading / 2, DropQDownBitrate);
				AverageBandwidth.AddValue(nv);
			}
		}


		mutable FCriticalSection Lock;

		TValueHistory<double> AverageLatency;
		TSimpleMovingAverage<int64> AverageBandwidth;

		FTimeValue AverageSegmentDuration;
		int64 LowestQualityStreamBitrate = 0;
		int64 HighestQualityStreamBitrate = 0;
		int32 NumQualities = 0;
		int32 NumConsecutiveSegmentErrors = 0;

		int32 QualityIndexDownloading = 0;
		int32 BitrateDownloading = 0;

		EDecision Decision = EDecision::AllGood;

		bool bSufferedRebuffering = false;
		bool bRebufferingPenaltyApplied = false;
		bool bWentIntoOvertime = false;

		FABRDownloadProgressDecision StreamReaderDecision;
	};


	int32 GetQualityIndexForBitrate(EStreamType InStreamType, int32 InForBitrate)
	{
		const TArray<TSharedPtrTS<FABRStreamInformation>>& StreamInfos = Info->GetStreamInformations(InStreamType);
		const TSharedPtrTS<FABRStreamInformation>* Stream = StreamInfos.FindByPredicate([InForBitrate](const TSharedPtrTS<FABRStreamInformation>& InInfo) { return InForBitrate == InInfo->Bitrate;} );
		return Stream ? (*Stream)->QualityIndex : -1;
	}

	TSharedPtrTS<FABRStreamInformation> GetStreamInfoForQualityIndex(EStreamType InStreamType, int32 InQualityIndex)
	{
		const TArray<TSharedPtrTS<FABRStreamInformation>>& StreamInfos = Info->GetStreamInformations(InStreamType);
		const TSharedPtrTS<FABRStreamInformation>* Stream = StreamInfos.FindByPredicate([InQualityIndex](const TSharedPtrTS<FABRStreamInformation>& InInfo) { return InQualityIndex == InInfo->QualityIndex;} );
		return Stream ? (*Stream) : nullptr;
	}

	double GetPlayablePlayerDuration(bool& bEOS, EStreamType InStreamType)
	{
		IPlayerABRLiveControl::FABRBufferStats bs;
		Info->ABRGetStreamBufferStats(bs, InStreamType);
		bEOS = bs.bReachedEnd || bs.bEndOfTrack;
		return bs.PlayableContentDuration.GetAsSeconds();
	}

	EStreamType GetPrimaryStreamType() const
	{
		return VideoWorkVars.HighestQualityStreamBitrate ? EStreamType::Video : EStreamType::Audio;
	}

	FStreamWorkVars* GetWorkVars(EStreamType ForStreamType)
	{
		return ForStreamType == EStreamType::Video ? &VideoWorkVars : ForStreamType == EStreamType::Audio ? &AudioWorkVars : nullptr;
	}

	FString GetFilenameFromURL(const FString& InURL)
	{
		FURL_RFC3986 UrlParser;
		UrlParser.Parse(InURL);
		return UrlParser.GetLastPathComponent();
	}

	void ApplyRebufferingPenalty();

	void SetRenderRateScale(double InNewRate);

	bool IsVideoOnly() const
	{
		return AudioWorkVars.NumQualities == 0;
	}

	bool IsAudioOnly() const
	{
		return VideoWorkVars.NumQualities == 0;
	}

	IABRInfoInterface* Info;
	FCriticalSection Lock;
	EMediaFormatType FormatType;
	EABRPresentationType PresentationType;

	FStreamWorkVars VideoWorkVars;
	FStreamWorkVars AudioWorkVars;

	int32 ForcedInitialBandwidth = 0;
	double ForcedInitialBandwidthUntilSecondsBuffered = 0.0;

	bool bIsBuffering = false;
	bool bIsRebuffering = false;
	bool bIsPaused = true;

	// Work variables used throughout the PrepareStreamCandidateList() -> EvaluateForError() -> EvaluateForQuality() -> PerformSelection() chain.
	bool bRetryIfPossible = false;
	bool bSkipWithFiller = false;

	// Configuration
	TMediaOptionalValue<int32> HTTPStatusCodeToDenyStream;
	bool bRebufferingJustResumesLoading = false;
	bool bEmergencyBufferSlowdownPossible = false;

	// Simulation
	const double SimulateForwardDuration = 20.0;
	const double BufferLowWatermark = 3.0;
	const double BufferHighWatermark = 8.0;
	const double BufferLowQualityHoldbackThreshold = 4.0;	// must be > BufferLowWatermark
	const double BufferLowestQualityRange = 0.3;			// if quality in the lower 30%, delay loading

	// Segment duration to use if no value can be obtained.
	const double DefaultAssumedSegmentDuration = 2.0;

	// Factor to scale the available bandwidth factor with. This can be used to be a bit conservative.
	const double UsableBandwidthScaleFactor = 0.85;

	// Number of quality levels to drop when aborting a download.
	const int32 NumQualityLevelDropsWhenAborting = 2;

	// Threshold below which to artificially slow down playback to avoid a buffer underrun.
	const double SlowDownWhenBufferDurationUnder = 0.6;
	// Playback rate to drop down to in that case.
	const double SlowDownRateWhenBufferDurationUnder = 0.8;

	// Duration in the buffer below which partially downloaded segment data will be emitted.
	const double EmitPartialDataWhenBufferDurationBelow = 1.0;

	// Initially assumed network latency.
	const double DefaultNetworkLatency = 0.4;

	// Scale of highest stream bitrate to clamp bandwidth to so it does not get ridiculously large.
	const double ClampBandwidthToMaxStreamBitrateScaleFactor = 2.0;
	// Clamp to this rate unless the max stream bitrate after applying the scaling factor is already higher.
	const double ClampBandwidthToMaxNonStreamBitrate = 20000000.0;

#if ENABLE_CYCLE_CVAR
	bool bForceQualityCycling = false;
	int32 QualityCyclingIndex = 0;
	int32 QualityCyclingLimit = 0;
	bool bForceQualityLock = false;
	int32 QualityLockIndex = 0;
#endif
};



IABROnDemandPlus* IABROnDemandPlus::Create(IABRInfoInterface* InInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType)
{
	return new FABROnDemandPlus(InInfo, InFormatType, InPresentationType);
}






FABROnDemandPlus::FABROnDemandPlus(IABRInfoInterface* InInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType)
	: Info(InInfo)
	, FormatType(InFormatType)
	, PresentationType(InPresentationType)
{
	bRebufferingJustResumesLoading = Info->GetOptionValue(OptionRebufferingContinuesLoading).SafeGetBool(false);
	if (Info->HaveOptionValue(ABR::OptionKeyABR_CDNSegmentDenyHTTPStatus))
	{
		HTTPStatusCodeToDenyStream.Set((int32) Info->GetOptionValue(ABR::OptionKeyABR_CDNSegmentDenyHTTPStatus).SafeGetInt64(-1));
	}

	FTimeRange RenderRange = Info->ABRGetSupportedRenderRateScale();
	bEmergencyBufferSlowdownPossible = RenderRange.IsValid();

	bIsBuffering = false;
	bIsRebuffering = false;
	bIsPaused = Info->ABRGetPlaySpeed() == FTimeValue::GetZero();
}

FABROnDemandPlus::~FABROnDemandPlus()
{
}

void FABROnDemandPlus::ApplyRebufferingPenalty()
{
	const TArray<TSharedPtrTS<FABRStreamInformation>>& Candidates = Info->GetStreamInformations(GetPrimaryStreamType());
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	if (WorkVars)
	{
		WorkVars->ApplyRebufferingPenalty(Candidates, NumQualityLevelDropsWhenAborting);
	}
}


FABRDownloadProgressDecision FABROnDemandPlus::ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
{
	FStreamWorkVars* WorkVars = GetWorkVars(SegmentDownloadStats.StreamType);
	if (WorkVars)
	{
		uint32 Flags = WorkVars->StreamReaderDecision.Flags;

		if (SegmentDownloadStats.SegmentType == Metrics::ESegmentType::Media)
		{
			FScopeLock lock(&WorkVars->Lock);

			double EstimatedTotalDownloadTime = SegmentDownloadStats.Duration;

			const int32 QualityIndex = WorkVars->QualityIndexDownloading;
			bool bEOS = false;
			const double playableSeconds = GetPlayablePlayerDuration(bEOS, SegmentDownloadStats.StreamType);
			const bool bIsEmittingPartial = (Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData) != 0;

			// Estimate how long we will take to complete the download.
			// Base this off the byte sizes?
			if (SegmentDownloadStats.NumBytesDownloaded > 0 && SegmentDownloadStats.ByteSize > 0)
			{
				EstimatedTotalDownloadTime = (SegmentDownloadStats.TimeToDownload / SegmentDownloadStats.NumBytesDownloaded) * SegmentDownloadStats.ByteSize;
			}
			// Otherwise base it off the durations
			else if (SegmentDownloadStats.DurationDownloaded > 0.0 && SegmentDownloadStats.Duration > 0.0)
			{
				EstimatedTotalDownloadTime = (SegmentDownloadStats.TimeToDownload / SegmentDownloadStats.DurationDownloaded) * SegmentDownloadStats.Duration;
			}

			bool bCheck = !bIsBuffering && !bIsPaused;
			// If not already on the worst quality check if we should abort the download.
			if (bCheck && QualityIndex > 0)
			{
				double MaxSegmentDownloadDuration = SegmentDownloadStats.Duration;
				// If forced to use a bitrate or currently buffering, allow for a longer timeout.
				if (bIsBuffering || (SegmentDownloadStats.StreamType == EStreamType::Video && ForcedInitialBandwidth && ForcedInitialBandwidthUntilSecondsBuffered > 0.0))
				{
					MaxSegmentDownloadDuration *= 2.0;
				}
				if (SegmentDownloadStats.TimeToDownload > MaxSegmentDownloadDuration)
				{
					if (!bIsEmittingPartial)
					{
						// If we have not yet emitted the data loaded so far we abort when the total download time exceeds what is
						// currently available in the buffer plus what has been downloaded so far.
						if (EstimatedTotalDownloadTime > playableSeconds + SegmentDownloadStats.DurationDownloaded)
						{
							Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload;
							WorkVars->Decision = FStreamWorkVars::EDecision::ConnectedAbort;
						}
					}
					else
					{
						// Since we are already emitting partial segment data we check if an emergency slowdown is possible
						// and if so trigger it. Otherwise we abort if there is not enough content available.
						if (EstimatedTotalDownloadTime > playableSeconds)
						{
							if (bEmergencyBufferSlowdownPossible)
							{
								WorkVars->bWentIntoOvertime = true;
							}
							else
							{
								Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload;
								Flags |= FABRDownloadProgressDecision::EDecisionFlags::eABR_InsertFillerData;
								WorkVars->Decision = FStreamWorkVars::EDecision::ConnectedAbort;
							}
						}
					}
				}

				// If the buffer is low allow the data being streamed in to already be emitted to the buffer unless we are to abort.
				if (!bIsEmittingPartial && (Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload) == 0)
				{
					if (playableSeconds < EmitPartialDataWhenBufferDurationBelow && SegmentDownloadStats.DurationDownloaded > 0.0 && SegmentDownloadStats.DurationDelivered == 0.0)
					{
						Flags |= FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData;
					}
				}

				// When aborting give a human readable reason.
				if ((Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload) != 0)
				{
					WorkVars->StreamReaderDecision.Reason = FString::Printf(TEXT("Abort download of %s segment @ %d bps for time %.3f with %.3fs in buffer and %.3fs of %.3fs fetched after %.3fs.")
						, GetStreamTypeName(SegmentDownloadStats.StreamType)
						, SegmentDownloadStats.Bitrate
						, SegmentDownloadStats.PresentationTime
						, playableSeconds
						, SegmentDownloadStats.DurationDownloaded
						, SegmentDownloadStats.Duration
						, SegmentDownloadStats.TimeToDownload);
					//Info->LogMessage(IInfoLog::ELevel::Verbose, WorkVars->StreamReaderDecision.Reason);
				}
			}

			// While buffering check when the segment is likely to complete downloading with enough buffered data
			// to allow playback from that point on.
			if (bIsBuffering && (Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload) == 0)
			{
				// Emit partial segment data for faster startup. We will not switching the startup quality anyway.
				if (SegmentDownloadStats.DurationDownloaded > 0.0 && SegmentDownloadStats.DurationDelivered == 0.0)
				{
					Flags |= FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData;
				}
			}
		}

		WorkVars->StreamReaderDecision.Flags = (FABRDownloadProgressDecision::EDecisionFlags) Flags;
		return WorkVars->StreamReaderDecision;
	}
	else
	{
		FABRDownloadProgressDecision Decision;
		Decision.Flags = FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData;
		return Decision;
	}
}

void FABROnDemandPlus::ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
{
	if (SegmentDownloadStats.bWasSkipped)
	{
		return;
	}
	FStreamWorkVars* WorkVars = GetWorkVars(SegmentDownloadStats.StreamType);

	if (WorkVars && SegmentDownloadStats.SegmentType == Metrics::ESegmentType::Media)
	{
		const int32 QualityIndex = GetQualityIndexForBitrate(SegmentDownloadStats.StreamType, SegmentDownloadStats.Bitrate);
		FScopeLock lock(&WorkVars->Lock);
		if (!SegmentDownloadStats.bIsCachedResponse)
		{
			if (WorkVars->bSufferedRebuffering)
			{
				WorkVars->Decision = WorkVars->Decision == FStreamWorkVars::EDecision::AllGood ? FStreamWorkVars::EDecision::Rebuffered : WorkVars->Decision;
			}

			// Track average latency.
			if (SegmentDownloadStats.TimeToFirstByte > 0.0)
			{
				WorkVars->AverageLatency.AddValue(SegmentDownloadStats.TimeToFirstByte);
			}

			// Calculate download bandwidth
			int64 Bandwidth = -1;
			if (SegmentDownloadStats.NumBytesDownloaded)
			{
				double ttdl = SegmentDownloadStats.TimeToDownload - SegmentDownloadStats.TimeToFirstByte;
				const double DlBandwidth = SegmentDownloadStats.NumBytesDownloaded * 8 / (ttdl > 0.0 ? ttdl : 0.1);
				if (DlBandwidth > 0.0)
				{
					Bandwidth = (int64) Utils::Min(DlBandwidth, Utils::Max(WorkVars->HighestQualityStreamBitrate * ClampBandwidthToMaxStreamBitrateScaleFactor, ClampBandwidthToMaxNonStreamBitrate));
				}
			}

			// Take a penalty if the download had to be aborted, otherwise add the bandwidth and throughput to the trackers.
			if (WorkVars->Decision == FStreamWorkVars::EDecision::NotConnectedAbort || WorkVars->Decision == FStreamWorkVars::EDecision::ConnectedAbort)
			{
				const TArray<TSharedPtrTS<FABRStreamInformation>>& Candidates = Info->GetStreamInformations(SegmentDownloadStats.StreamType);
				WorkVars->ApplyRebufferingPenalty(Candidates, NumQualityLevelDropsWhenAborting);
			}
			else if (Bandwidth > 0)
			{
				WorkVars->AverageBandwidth.AddValue(Bandwidth);
			}
		}
		else
		{
			// When we hit the cache we cannot calculate a meaningful bandwidth. We need to update the averages nonetheless
			// to make a quality upswitch possible on one of the next uncached segments.
			int64 ThisBitrate = SegmentDownloadStats.Bitrate;
			const TArray<TSharedPtrTS<FABRStreamInformation>>& StreamInformation = Info->GetStreamInformations(SegmentDownloadStats.StreamType);
			const int64 LastAddedBandwidth = WorkVars->AverageBandwidth.GetLastSample();
			const int32 NextHighestQualityIndex = QualityIndex + 1 < StreamInformation.Num() ? QualityIndex + 1 : QualityIndex;
			const int32 NextHighestBitrate = StreamInformation[NextHighestQualityIndex]->Bitrate;
			const double InverseUsableBandwidthScaleFactor = IsAudioOnly() ? 2.0 / UsableBandwidthScaleFactor : 1.0 / UsableBandwidthScaleFactor;
			const double PretendedLatency = IsAudioOnly() ? 0.06 : 0.12;

			// Last seen bandwidth already higher than what we might want to switch up to next?
			if (LastAddedBandwidth > NextHighestBitrate)
			{
				// We want to bring the average down a bit in preparation for the next uncached segment in case the network
				// has degraded. This way we are not overshooting the target.
				ThisBitrate = (LastAddedBandwidth + NextHighestBitrate) / 2;
				WorkVars->AverageBandwidth.AddValue(ThisBitrate * InverseUsableBandwidthScaleFactor);
			}
			else if (LastAddedBandwidth > ThisBitrate)
			{
				// The last bandwidth was somewhere between the bitrate of the cached segment and the next quality.
				// We would love to switch up with the next segment. This might be possible because we got this segment's data
				// at lightning speed and have its duration worth of time to spend on attempting the next segment download.
				// If that doesn't finish in time we can downswitch without hurting things too much.
				ThisBitrate = NextHighestBitrate;
				WorkVars->AverageBandwidth.InitializeTo(ThisBitrate * InverseUsableBandwidthScaleFactor);
			}
			else
			{
				// Let's try to bring the average up a bit.
				ThisBitrate = (int32)(ThisBitrate * 0.2 + NextHighestBitrate * 0.8);
				WorkVars->AverageBandwidth.AddValue(ThisBitrate * InverseUsableBandwidthScaleFactor);
			}
			WorkVars->AverageLatency.AddValue(PretendedLatency);
		}

		// Adjust any forced bitrate duration.
		if (ForcedInitialBandwidthUntilSecondsBuffered > 0.0 && (SegmentDownloadStats.StreamType == EStreamType::Video || IsAudioOnly()))
		{
			ForcedInitialBandwidthUntilSecondsBuffered -= SegmentDownloadStats.DurationDownloaded;
		}
	}
}


void FABROnDemandPlus::ReportBufferingStart(Metrics::EBufferingReason BufferingReason)
{
	bIsBuffering = true;

	if (BufferingReason == Metrics::EBufferingReason::Rebuffering)
	{
		bIsRebuffering = true;
		VideoWorkVars.bSufferedRebuffering = true;
		AudioWorkVars.bSufferedRebuffering = true;
	}
}

void FABROnDemandPlus::ReportBufferingEnd(Metrics::EBufferingReason BufferingReason)
{
	bIsBuffering = false;
	bIsRebuffering = false;
}

void FABROnDemandPlus::ReportPlaybackPaused()
{
	bIsPaused = true;
}

void FABROnDemandPlus::ReportPlaybackResumed()
{
	bIsPaused = false;
}

void FABROnDemandPlus::ReportPlaybackEnded()
{
	bIsBuffering = false;
	bIsRebuffering = false;
	bIsPaused = true;
}

void FABROnDemandPlus::RepresentationsChanged(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPlayPeriod)
{
	FStreamWorkVars* WorkVars = GetWorkVars(InStreamType);
	if (WorkVars)
	{
		TArray<TSharedPtrTS<FABRStreamInformation>> Representations = Info->GetStreamInformations(InStreamType);

		FScopeLock lock(&WorkVars->Lock);
		WorkVars->Reset();
		WorkVars->AverageSegmentDuration.SetToInvalid();
		WorkVars->HighestQualityStreamBitrate = 0;
		WorkVars->LowestQualityStreamBitrate = 1000000000;
		WorkVars->NumQualities = Representations.Num();
		if (Representations.Num())
		{
			InCurrentPlayPeriod->GetAverageSegmentDuration(WorkVars->AverageSegmentDuration, Representations[0]->AdaptationSetUniqueID, Representations[0]->RepresentationUniqueID);
			for(auto &Rep : Representations)
			{
				if (Rep->Bitrate > WorkVars->HighestQualityStreamBitrate)
				{
					WorkVars->HighestQualityStreamBitrate = Rep->Bitrate;
				}
				if (Rep->Bitrate < WorkVars->LowestQualityStreamBitrate)
				{
					WorkVars->LowestQualityStreamBitrate = Rep->Bitrate;
				}
			}
		}
	}
}

void FABROnDemandPlus::SetBandwidth(int64 bitsPerSecond)
{
	ForcedInitialBandwidth = (int32)bitsPerSecond;
}

void FABROnDemandPlus::SetForcedNextBandwidth(int64 bitsPerSecond, double minBufferTimeBeforePlayback)
{
	ForcedInitialBandwidth = (int32)bitsPerSecond;
	ForcedInitialBandwidthUntilSecondsBuffered = minBufferTimeBeforePlayback;
}

int64 FABROnDemandPlus::GetLastBandwidth()
{
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	return WorkVars ? WorkVars->AverageBandwidth.GetLastSample() : ForcedInitialBandwidth;
}

int64 FABROnDemandPlus::GetAverageBandwidth()
{
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	return WorkVars ? WorkVars->AverageBandwidth.GetSMA(ForcedInitialBandwidth) : ForcedInitialBandwidth;
}

int64 FABROnDemandPlus::GetAverageThroughput()
{
	return GetAverageBandwidth();
}

double FABROnDemandPlus::GetAverageLatency()
{
	FStreamWorkVars* WorkVars = GetWorkVars(GetPrimaryStreamType());
	return WorkVars ? WorkVars->AverageLatency.GetWeightedMax() : 0.0;
}


void FABROnDemandPlus::PrepareStreamCandidateList(TArray<TSharedPtrTS<FABRStreamInformation>>& OutCandidates, EStreamType StreamType, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const FTimeValue& TimeNow)
{
	bRetryIfPossible = false;
	bSkipWithFiller = false;

	// Get the list of streams for this type.
	OutCandidates = Info->GetStreamInformations(StreamType);
}

IAdaptiveStreamSelector::ESegmentAction FABROnDemandPlus::EvaluateForError(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	FStreamWorkVars* WorkVars = GetWorkVars(StreamType);

	if (CurrentSegment.IsValid())
	{
		Metrics::FSegmentDownloadStats Stats;
		CurrentSegment->GetDownloadStats(Stats);
		// Try to get the stream information for the current downloaded segment. We may not find it on a period transition where the
		// segment is the last of the previous period.
		TSharedPtrTS<FABRStreamInformation> CurrentStreamInfo = Info->GetStreamInformation(Stats);
		if (CurrentStreamInfo.IsValid())
		{
			// Remember stats
			CurrentStreamInfo->Health.LastDownloadStats = Stats;

			if (!Stats.bWasSuccessful)
			{
				if (Stats.bWaitingForRemoteRetryElement)
				{
					// Switching to another representation that needs to be made ready resets the consecutive errors so we get
					// a chance to switch to a different representation if several are being blocked by the CDN to force
					// multiple downswitches.
					WorkVars->NumConsecutiveSegmentErrors = 0;
					return IAdaptiveStreamSelector::ESegmentAction::Retry;
				}

				// If this is the first failure for this segment (not being retried yet) increase
				// the number of consecutively failed segments.
				if (Stats.RetryNumber == 0)
				{
					++WorkVars->NumConsecutiveSegmentErrors;
				}
				// Was not successful. Figure out what to do now.
				// In the case where the segment had an availability window set the stream reader was waiting for
				// to be entered and we got a 404 back, the server did not manage to publish the new segment in time.
				// We try this again after a slight delay.
				// This should only ever happen with a Live presentation for which this ABR should not be used.
				// We handle the case nonetheless.
				if (Stats.AvailibilityDelay && Utils::AbsoluteValue(Stats.AvailibilityDelay) < 0.5)
				{
					//Info->LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Segment \"%s\" not available at announced time. Trying again in 0.15s"), *GetFilenameFromURL(Stats.URL)));
					CurrentPlayPeriod->IncreaseSegmentFetchDelay(FTimeValue(FTimeValue::MillisecondsToHNS(100)));
					OutDelay.SetFromMilliseconds(150);
					return Stats.RetryNumber < 3 ? IAdaptiveStreamSelector::ESegmentAction::Retry : IAdaptiveStreamSelector::ESegmentAction::Fail;
				}

				// Too many failures already? It's unlikely that there is a way that would magically fix itself now.
				if (Stats.RetryNumber >= 3)
				{
					Info->LogMessage(IInfoLog::ELevel::Error, FString::Printf(TEXT("Exceeded permissable number of retries (%d). Failing now."), Stats.RetryNumber));
					return IAdaptiveStreamSelector::ESegmentAction::Fail;
				}
				if (WorkVars->NumConsecutiveSegmentErrors >= 3)
				{
					Info->LogMessage(IInfoLog::ELevel::Error, FString::Printf(TEXT("Exceeded permissable number of consecutive segment failures (%d). Failing now."), WorkVars->NumConsecutiveSegmentErrors));
					return IAdaptiveStreamSelector::ESegmentAction::Fail;
				}

				// Is this an init segment?
				if (Stats.SegmentType == Metrics::ESegmentType::Init)
				{
					bRetryIfPossible = true;
					// A failure other than having aborted in ReportDownloadProgress() ?
					if (!Stats.bWasAborted)
					{
						// Did the init segment fail to be parsed or does the HTTP status match the one used to disable the stream?
						// If so this stream is dead for good.
						if (Stats.bParseFailure || (HTTPStatusCodeToDenyStream.IsSet() && HTTPStatusCodeToDenyStream.Value() == Stats.HTTPStatusCode))
						{
							CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC.SetToInvalid();
						}
						else if (Stats.HTTPStatusCode && Stats.RetryNumber == 0)
						{
							// Do one more try.
							OutDelay.SetFromMilliseconds(150);
						}
						// Take this stream offline for a brief moment.
						else if (CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC == FTimeValue::GetZero())
						{
							CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC = TimeNow + FTimeValue().SetFromMilliseconds(1000);
						}
					}
				}
				// A media segment failure
				else
				{
					// Check if there is a HTTP status code set which, if returned, is used to disable this stream permanently.
					if (HTTPStatusCodeToDenyStream.IsSet() && HTTPStatusCodeToDenyStream.Value() == Stats.HTTPStatusCode)
					{
						CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC.SetToInvalid();
						bRetryIfPossible = true;
						bSkipWithFiller = false;
					}
					else
					{
						bRetryIfPossible = true;
						bSkipWithFiller = true;

						// Did we abort that stream in ReportDownloadProgress() ?
						if (!Stats.bWasAborted)
						{
							// Take a stream offline for a brief moment unless it is the worst one.
							if (Stats.StreamType == EStreamType::Video || IsAudioOnly())
							{
								if (CurrentStreamInfo->QualityIndex != 0)
								{
									if (CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC == FTimeValue::GetZero())
									{
										CurrentStreamInfo->Health.BecomesAvailableAgainAtUTC = TimeNow + FTimeValue().SetFromMilliseconds(1000);
									}
								}
								else if (IsAudioOnly() && Stats.RetryNumber < 2)
								{
									OutDelay.SetFromMilliseconds(400);
									bRetryIfPossible = true;
								}
								else
								{
									// Insert filler data instead.
									bRetryIfPossible = false;
								}
							}
							else
							{
								// For VoD we retry the audio segment once with a small delay before skipping over it, which we do for Live right away
								// since retrying is a luxury we do not have in this case.
								// Note: This here ABR is optimized for VoD. For Live a dedicated Live ABR should be employed!
								if (PresentationType == EABRPresentationType::OnDemand)
								{
									if (Stats.RetryNumber == 0)
									{
										OutDelay.SetFromMilliseconds(150);
										bRetryIfPossible = true;
									}
									else
									{
										bRetryIfPossible = false;
									}
								}
								else
								{
									bRetryIfPossible = false;
								}
							}
						}
					}

					// If any content has already been put out into the buffers we cannot retry the segment on another quality level.
					if (Stats.DurationDelivered > 0.0)
					{
						bRetryIfPossible = false;
						bSkipWithFiller = false;
					}
				}
			}
			else
			{
				if (!Stats.bInsertedFillerData)
				{
					WorkVars->NumConsecutiveSegmentErrors = 0;
				}
			}
		}
	}
	return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
}


IAdaptiveStreamSelector::ESegmentAction FABROnDemandPlus::EvaluateForQuality(TArray<TSharedPtrTS<FABRStreamInformation>>& InOutCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	/*
		This method doesn't evaluate which exact stream to use. It merely reduces the candidates based on externally
		set limits, like maximum bitrate and resolution.
		The calling ABR will further reduce this list by removing streams that are not healthy.
	*/
	if (InOutCandidates.Num())
	{
		if (StreamType == EStreamType::Video)
		{
			TArray<TSharedPtrTS<FABRStreamInformation>> PossibleRepresentations;
			PossibleRepresentations.Add(InOutCandidates[0]);
			const int32 MaxAllowedBandwidth = Info->GetBandwidthCeiling();
			const FStreamCodecInformation::FResolution MaxAllowedResolution = Info->GetMaxStreamResolution();
			for(int32 nStr=1; nStr<InOutCandidates.Num(); ++nStr)
			{
				// Check if bitrate and resolution are acceptable
				if (InOutCandidates[nStr]->Bitrate <= MaxAllowedBandwidth && !InOutCandidates[nStr]->Resolution.ExceedsLimit(MaxAllowedResolution))
				{
					PossibleRepresentations.Add(InOutCandidates[nStr]);
				}
			}
			Swap(InOutCandidates, PossibleRepresentations);
		}
		else if (StreamType == EStreamType::Audio)
		{
			if (IsAudioOnly())
			{
				TArray<TSharedPtrTS<FABRStreamInformation>> PossibleRepresentations;
				PossibleRepresentations.Add(InOutCandidates[0]);
				const int32 MaxAllowedBandwidth = Info->GetBandwidthCeiling();
				for(int32 nStr=1; nStr<InOutCandidates.Num(); ++nStr)
				{
					// Check if bitrate is acceptable
					if (InOutCandidates[nStr]->Bitrate <= MaxAllowedBandwidth)
					{
						PossibleRepresentations.Add(InOutCandidates[nStr]);
					}
				}
				Swap(InOutCandidates, PossibleRepresentations);
			}
			else
			{
				InOutCandidates = Info->GetStreamInformations(StreamType);
			}
		}
	}
	return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
}


IAdaptiveStreamSelector::ESegmentAction FABROnDemandPlus::PerformSelection(const TArray<TSharedPtrTS<FABRStreamInformation>>& InCandidates, EStreamType StreamType, FTimeValue& OutDelay, const TSharedPtrTS<IManifest::IPlayPeriod>& CurrentPlayPeriod, const TSharedPtrTS<const IStreamSegment>& CurrentSegment, const FTimeValue& TimeNow)
{
	if (InCandidates.Num())
	{
		FStreamWorkVars* WorkVars = GetWorkVars(StreamType);

		int32 NewQualityIndex = -1;

		bool bEOS = false;
		const double AvailableDuration = GetPlayablePlayerDuration(bEOS, StreamType);
		const double DownloadLatency = WorkVars ? WorkVars->AverageLatency.GetWeightedMax(DefaultNetworkLatency) : DefaultNetworkLatency;

		if (WorkVars && (StreamType == EStreamType::Video || IsAudioOnly()))
		{
			FScopeLock lock(&WorkVars->Lock);

			const int64 DefaultInitialBandwidth = Utils::Max(WorkVars->LowestQualityStreamBitrate, (int64)ForcedInitialBandwidth);
			double AverageDownloadBPS = (double) WorkVars->AverageBandwidth.GetWMA(DefaultInitialBandwidth) * UsableBandwidthScaleFactor;
			double LastDownloadBPS = (double) WorkVars->AverageBandwidth.GetLastSample(DefaultInitialBandwidth) * UsableBandwidthScaleFactor;

			const double AvgSegDur = WorkVars->AverageSegmentDuration.GetAsSeconds(DefaultAssumedSegmentDuration);

			int32 NumSimulationSegments = (int32)(SimulateForwardDuration / AvgSegDur + 0.5);
			if (NumSimulationSegments == 0)
			{
				NumSimulationSegments = 1;
			}
			for(auto &Can : InCandidates)
			{
				/*
					Simulate buffer progression for the next n seconds.
					Use a fixed segment size for this. While we could get the exact segment byte size in case of DASH VoD
					that uses SegmentBase addressing through the SIDX, this is not always how segments are accessible.
					This also tends to cause quality upswitches when some segments are significantly smaller than the average
					(as with very little motion, black or still frames) that may not be warranted.
				*/
				const int64 EstimatedSegmentBitSize = Can->Bitrate * AvgSegDur;
				const double EstimatedSegmentDownloadTime = DownloadLatency + EstimatedSegmentBitSize / AverageDownloadBPS;
				const double GainPerSegment = AvgSegDur - EstimatedSegmentDownloadTime;
				const double DurationGained = NumSimulationSegments * GainPerSegment;
				const double SmallestGain = GainPerSegment;
				const double SecondsInBufferAfterNext = AvailableDuration + AvgSegDur - (DownloadLatency + EstimatedSegmentBitSize / LastDownloadBPS);

				const bool bCond1 = SmallestGain > 0.0 && AvailableDuration < BufferLowWatermark;
				const bool bCond2 = SmallestGain + SecondsInBufferAfterNext > BufferLowWatermark;
				const bool bCond3 = DurationGained + AvailableDuration > BufferHighWatermark;
				bool bIsFeasible = (bCond1 || bCond2) && bCond3;

				// If there is an enforced bitrate for the first x seconds then force the stream feasible.
				if (ForcedInitialBandwidthUntilSecondsBuffered > 0.0 && ForcedInitialBandwidth)
				{
					bIsFeasible = Can->Bitrate <= ForcedInitialBandwidth;
				}
				if (bIsFeasible)
				{
					NewQualityIndex = Can->QualityIndex;
				}
			}

			// None of the candidates is deemed feasible. This may happen if all low quality streams have issues and ended up
			// on the deny list. Pick the lowest quality one, even if this is one that may not be playable without rebuffering.
			if (NewQualityIndex < 0)
			{
				NewQualityIndex = InCandidates[0]->QualityIndex;
			}

			/*
				Check if we are in the lower configured range of all qualities and if so, hold back the download
				for a while to avoid polluting the buffer with low quality data in case the bandwidth will recover soon
				and we could get better quality then.
			*/
			if (WorkVars->NumQualities && NewQualityIndex < (int32)(WorkVars->NumQualities * BufferLowestQualityRange + 0.5))
			{
				if (AvailableDuration > BufferLowQualityHoldbackThreshold)
				{
					OutDelay.SetFromSeconds(AvailableDuration - BufferLowQualityHoldbackThreshold);
					//Info->LogMessage(IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Delaying segment by %.3fs"), OutDelay.GetAsSeconds()));
				}
			}

#if ENABLE_CYCLE_CVAR
			if (bForceQualityCycling)
			{
				if (QualityCyclingIndex >= Utils::Min(InCandidates.Num(), QualityCyclingLimit))
				{
					QualityCyclingIndex = 0;
				}
				NewQualityIndex = InCandidates[QualityCyclingIndex++]->QualityIndex;
				TSharedPtrTS<FABRStreamInformation> Cnd = GetStreamInfoForQualityIndex(StreamType, NewQualityIndex);
				if (Cnd.IsValid())
				{
					Info->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Cycling to quality index %d (%d*%d @ %d bps)"), NewQualityIndex, Cnd->Resolution.Width, Cnd->Resolution.Height, Cnd->Bitrate));
				}
			}
			if (bForceQualityLock)
			{
				NewQualityIndex = Utils::Min(InCandidates.Num(), QualityLockIndex) - 1;
				TSharedPtrTS<FABRStreamInformation> Cnd = GetStreamInfoForQualityIndex(StreamType, NewQualityIndex);
				if (Cnd.IsValid())
				{
					Info->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Forcing quality index %d (%d*%d @ %d bps)"), NewQualityIndex, Cnd->Resolution.Width, Cnd->Resolution.Height, Cnd->Bitrate));
				}
			}
#endif
		}
		else
		{
//			FScopeLock lock(&WorkVars->Lock);
			struct FDecisionAttributes
			{
				int32 Bitrate = 0;
				int32 QualityIndex = 0;
			};
			TArray<FDecisionAttributes> QualityLevels;
			for(auto &Can : InCandidates)
			{
				FDecisionAttributes& da = QualityLevels.Emplace_GetRef();
				da.QualityIndex = Can->QualityIndex;
				da.Bitrate = Can->Bitrate;
			}
			QualityLevels.StableSort([](const FDecisionAttributes& a, const FDecisionAttributes& b){ return a.Bitrate > b.Bitrate; });
			NewQualityIndex = QualityLevels[0].QualityIndex;
		}

		check(NewQualityIndex >= 0);
		TSharedPtrTS<FABRStreamInformation> Candidate = GetStreamInfoForQualityIndex(StreamType, NewQualityIndex);
		if (Candidate.IsValid())
		{
			if (WorkVars)
			{
				WorkVars->ClearForNextDownload();
				WorkVars->QualityIndexDownloading = NewQualityIndex;
				WorkVars->BitrateDownloading = Candidate->Bitrate;
			}

			CurrentPlayPeriod->SelectStream(Candidate->AdaptationSetUniqueID, Candidate->RepresentationUniqueID, NewQualityIndex, Info->GetStreamInformations(StreamType).Num()-1);
			if (bRetryIfPossible)
			{
				return IAdaptiveStreamSelector::ESegmentAction::Retry;
			}
			else if (bSkipWithFiller)
			{
				return IAdaptiveStreamSelector::ESegmentAction::Fill;
			}
			else
			{
				return IAdaptiveStreamSelector::ESegmentAction::FetchNext;
			}
		}
	}
	else
	{
		// Note: Are there representations we could lift the temporary denylist from?
		if (bSkipWithFiller)
		{
			return IAdaptiveStreamSelector::ESegmentAction::Fill;
		}
	}
	return IAdaptiveStreamSelector::ESegmentAction::Fail;
}




FTimeValue FABROnDemandPlus::GetMinBufferTimeForPlayback(IAdaptiveStreamSelector::EMinBufferType InBufferingType, FTimeValue InDefaultMBT)
{
	return FTimeValue();
}

IAdaptiveStreamSelector::FRebufferAction FABROnDemandPlus::GetRebufferAction()
{
	IAdaptiveStreamSelector::FRebufferAction Action;
	Action.Action = bRebufferingJustResumesLoading ? IAdaptiveStreamSelector::FRebufferAction::EAction::ContinueLoading : IAdaptiveStreamSelector::FRebufferAction::EAction::Restart;
	ApplyRebufferingPenalty();
	return Action;
}


IAdaptiveStreamSelector::EHandlingAction FABROnDemandPlus::PeriodicHandle()
{
	IAdaptiveStreamSelector::EHandlingAction NextAction = IAdaptiveStreamSelector::EHandlingAction::None;

	// Perform slow down when buffer is about to run dry?
	if (!bIsPaused && !bIsBuffering && Info->ABRGetPlaySpeed() != FTimeValue::GetZero())
	{
		const EStreamType StreamType = GetPrimaryStreamType();
		const FStreamWorkVars* WorkVars = GetWorkVars(StreamType);
		if (WorkVars)
		{
			bool bEOS = false;
			const double AvailableBufferedDuration = GetPlayablePlayerDuration(bEOS, StreamType);
			const double NetworkLatency = WorkVars->AverageLatency.GetWeightedMax(DefaultNetworkLatency);
			const double CurrentPlayRate = Info->ABRGetRenderRateScale();
			const bool bOvertime = WorkVars->bWentIntoOvertime;

			// Slow down because of insufficient buffered data?
			bool bBufferSlowDown = bOvertime || (!bEOS && AvailableBufferedDuration < Utils::Max(SlowDownWhenBufferDurationUnder, NetworkLatency));
			if (bBufferSlowDown)
			{
				SetRenderRateScale(SlowDownRateWhenBufferDurationUnder);
			}
			else if (CurrentPlayRate != 1.0)
			{
				SetRenderRateScale(1.0);
			}
		}
	}

#if ENABLE_CYCLE_CVAR
	int32 CycleQuality = CVarElectraABRVODCycle.GetValueOnAnyThread();
	if (CycleQuality && CycleQuality!=QualityCyclingLimit)
	{
		QualityCyclingIndex = bForceQualityCycling ? QualityCyclingIndex : 0;
		QualityCyclingLimit = CycleQuality;
		bForceQualityCycling = true;
		Info->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Enabling quality cycling up to quality %d"), CycleQuality));
	}
	else if (CycleQuality == 0 && bForceQualityCycling)
	{
		bForceQualityCycling = false;
		QualityCyclingLimit = 0;
		Info->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Diable quality cycling")));
	}

	int32 LockQuality = CVarElectraABRVODFixedStream.GetValueOnAnyThread();
	if (LockQuality && LockQuality!=QualityLockIndex)
	{
		bForceQualityLock = true;
		QualityLockIndex = LockQuality;
		Info->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Locking quality to index %d"), LockQuality));
	}
	else if (LockQuality == 0 && bForceQualityLock)
	{
		bForceQualityLock = false;
		QualityLockIndex = 0;
		Info->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Diable quality locking")));
	}
#endif

	return NextAction;
}

void FABROnDemandPlus::SetRenderRateScale(double InNewRate)
{
	if (!IsAudioOnly())
	{
		FTimeRange pr = Info->ABRGetSupportedRenderRateScale();
		if (pr.IsValid())
		{
			double MinAllowed = pr.Start.GetAsSeconds(1.0);
			double MaxAllowed = pr.End.GetAsSeconds(1.0);
			Info->ABRSetRenderRateScale(Utils::Min(Utils::Max(MinAllowed, InNewRate), MaxAllowed));
		}
	}
}




void FABROnDemandPlus::DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...))
{
	pPrintFN(pThat, "===== VoD+ ABR state =====");

	pPrintFN(pThat, "Buffering: %d", bIsBuffering);
	pPrintFN(pThat, "Rebuffering: %d", bIsRebuffering);
	pPrintFN(pThat, "Paused: %d", bIsPaused);

	pPrintFN(pThat, "  Catchup rate: %.3f", Info->ABRGetRenderRateScale());
	pPrintFN(pThat, "Network latency: %.3f", VideoWorkVars.AverageLatency.GetWeightedMax(DefaultNetworkLatency));
	bool bEOS = false;
	double Dur = GetPlayablePlayerDuration(bEOS, GetPrimaryStreamType());
	pPrintFN(pThat, "Duration avail: %.3f, EOS=%d", Dur, bEOS);
}



} // namespace Electra
