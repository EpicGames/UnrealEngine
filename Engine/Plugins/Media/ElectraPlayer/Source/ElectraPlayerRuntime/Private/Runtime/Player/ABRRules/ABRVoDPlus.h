// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Player/ABRRules/ABRInternal.h"

namespace Electra
{

class IABROnDemandPlus : public IABRRule
{
public:
	static IABROnDemandPlus* Create(IABRInfoInterface* InIInfo, EMediaFormatType InFormatType, EABRPresentationType InPresentationType);
	virtual ~IABROnDemandPlus() = default;
};

} // namespace Electra
