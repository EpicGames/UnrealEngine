// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{
	const FName MediaInfoStartTimecodeValue(TEXT("StartTimecodeValue"));				// String
	const FName MediaInfoStartTimecodeFrameRate(TEXT("StartTimecodeFrameRate"));		// String
	const FName MediaInfoVideoDecoderProvider(TEXT("VideoDecoderProvider"));			// String
	const FName MediaInfoAudioDecoderProvider(TEXT("AudioDecoderProvider"));			// String

	/** Bitrate of video currently being displayed. */
	const FLazyName MediaInfoVideoBitrateCurrent(TEXT("VideoBitrateCurrent"));			// int64, bits per second
	/** Bitrate of video currently being buffered ahead of time. */
	const FLazyName MediaInfoVideoBitrateBuffering(TEXT("VideoBitrateBuffering"));		// int64, bits per second

	/** Bitrate of audio currently being heard. */
	const FLazyName MediaInfoAudioBitrateCurrent(TEXT("AudioBitrateCurrent"));			// int64, bits per second
	/** Bitrate of audio currently being buffered ahead of time. */
	const FLazyName MediaInfoAudioBitrateBuffering(TEXT("AudioBitrateBuffering"));		// int64, bits per second

} // namespace Electra


