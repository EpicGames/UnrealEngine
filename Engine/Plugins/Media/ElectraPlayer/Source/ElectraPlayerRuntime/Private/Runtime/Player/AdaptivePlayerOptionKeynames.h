// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{
	//! (FString) An ID uniquely identifying the content. Must not exceed 64 characters.
	const FName OptionKeyContentID(TEXT("content_id"));

	//! (FString) A session ID, in UUID format. Limited to 64 characters if not UUID format.
	const FName OptionKeySessionID(TEXT("session_id"));

	//! (int64) ID of the platform specific output texture pool registered with the decoder platform resources.
	const FName OptionKeyOutputTexturePoolID(TEXT("output_texturepool_id"));

	//! (FString) Use of worker threads. ("shared", "worker" or "worker_and_events")
	const FName OptionKeyWorkerThreads(TEXT("worker_threads"));

	//! (FString) mime type of URL to load
	const FName OptionKeyMimeType(TEXT("mime_type"));

	//! (int64) value indicating the maximum vertical resolution to be used for any video stream.
	const FName OptionKeyMaxVerticalResolution(TEXT("max_resoY"));

	//! (int64) value indicating the maximum vertical resolution to be used for video streams of more than 30fps.
	const FName OptionKeyMaxVerticalResolutionAbove30fps(TEXT("max_resoY_above_30fps"));

	//! (int64) value indicating the bitrate to start playback with (initial start).
	const FName OptionKeyInitialBitrate(TEXT("initial_bitrate"));

	//! (int64) value indicating the bitrate to start buffering with when seeking (not at playback start)
	const FName OptionKeySeekStartBitrate(TEXT("seekstart_bitrate"));

	//! (int64) value indicating the bitrate to start rebuffering with
	const FName OptionKeyRebufferStartBitrate(TEXT("rebufferstart_bitrate"));

	//! (FTimeValue) value specifying how many seconds away from the Live media timeline the seekable range should end.
	const FName OptionKeyLiveSeekableEndOffset(TEXT("seekable_range_live_end_offset"));

	//! (bool) true to just finish the currently loading segment when rebuffering. false to start over with.
	const FName OptionRebufferingContinuesLoading(TEXT("rebuffering_continues_loading"));

	//! (bool) true to throw a playback error when rebuffering occurs, false to continue normally.
	const FName OptionThrowErrorWhenRebuffering(TEXT("throw_error_when_rebuffering"));

	//! (bool) true to perform frame accurate seeks (slow as decoding and discarding data from a preceeding keyframe is required)
	const FName OptionKeyFrameAccurateSeek(TEXT("frame_accurate_seeking"));

	//! (bool) true to not truncate the media segment access units at the end of the presentation. Must only be used without a set playback range end!
	const FName OptionKeyDoNotTruncateAtPresentationEnd(TEXT("do_not_truncate_at_presentation_end"));

	const FName OptionKeyCurrentAvgStartingVideoBitrate(TEXT("current:avg_video_bitrate"));

	const FName OptionKeyExcludedCodecsVideo(TEXT("excluded_codecs_video"));
	const FName OptionKeyExcludedCodecsAudio(TEXT("excluded_codecs_audio"));
	const FName OptionKeyExcludedCodecsSubtitles(TEXT("excluded_codecs_subtitles"));

	const FName OptionKeyPreferredCodecsVideo(TEXT("preferred_codecs_video"));
	const FName OptionKeyPreferredCodecsAudio(TEXT("preferred_codecs_audio"));
	const FName OptionKeyPreferredCodecsSubtitles(TEXT("preferred_codecs_subtitles"));

	const FName OptionKeyResponseCacheMaxEntries(TEXT("httpcache_max_entries"));
	const FName OptionKeyResponseCacheMaxByteSize(TEXT("httpcache_max_bytesize"));

	const FName OptionKeyParseTimecodeInfo(TEXT("parse_timecode_info"));

	const FName OptionKeyPlaylistProperties(TEXT("playlist_properties"));

	const FName OptionKeyCMCDConfiguration(TEXT("cmcd_configuration"));

	const FName OptionKeyFrameStatInterval(TEXT("framestat_interval"));

	namespace CustomOptions
	{
		const TCHAR* const Custom_EpicStaticStart = TEXT("EpicStaticStart");
		const TCHAR* const Custom_EpicDynamicStart = TEXT("EpicDynamicStart");
		const TCHAR* const Custom_EpicUTCUrl = TEXT("EpicUTCUrl");
		const TCHAR* const Custom_EpicUTCNow = TEXT("EpicUTCNow");
	}
} // namespace Electra


