// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/AdaptiveStreamingPlayerInternal.h"
#include "ElectraPlayerPrivate.h"
#include "PlayerRuntimeGlobal.h"

#include "Stats/Stats.h"
#include "StreamAccessUnitBuffer.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/PlayerLicenseKey.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/AdaptivePlayerMetadataKeynames.h"
#include "Utilities/Utilities.h"
#include "Utilities/BCP47-Helpers.h"
#include "Player/DRM/DRMManager.h"

#include "HAL/LowLevelMemTracker.h"


DECLARE_CYCLE_STAT(TEXT("FAdaptiveStreamingPlayer::WorkerThread"), STAT_ElectraPlayer_AdaptiveWorker, STATGROUP_ElectraPlayer);
DECLARE_CYCLE_STAT(TEXT("FAdaptiveStreamingPlayer::EventThread"), STAT_ElectraPlayer_EventWorker, STATGROUP_ElectraPlayer);



namespace Electra
{
FAdaptiveStreamingPlayer *FAdaptiveStreamingPlayer::PointerToLatestPlayer;

TSharedPtr<IAdaptiveStreamingPlayer, ESPMode::ThreadSafe> IAdaptiveStreamingPlayer::Create(const IAdaptiveStreamingPlayer::FCreateParam& InCreateParameters)
{
	return MakeShareable(new FAdaptiveStreamingPlayer(InCreateParameters), FAdaptiveStreamingPlayer::TDeleter());
}

void IAdaptiveStreamingPlayer::DebugHandle(void* pPlayer, void (*debugDrawPrintf)(void* pPlayer, const char *pFmt, ...))
{
	FAdaptiveStreamingPlayer::DebugHandle(pPlayer, debugDrawPrintf);
}

//---------------------------------------------------------------------------------------------------------------------------------
//---------------------------------------------------------------------------------------------------------------------------------
//---------------------------------------------------------------------------------------------------------------------------------


//-----------------------------------------------------------------------------
/**
 * CTOR
 */
FAdaptiveStreamingPlayer::FAdaptiveStreamingPlayer(const IAdaptiveStreamingPlayer::FCreateParam& InCreateParameters)
{
	Electra::AddActivePlayerInstance();

	CurrentMediaSamples = MakeShared<FMediaSamples, ESPMode::ThreadSafe>();
	EmptyMediaSamples = MakeShared<FMediaSamples, ESPMode::ThreadSafe>();
	VideoOutput.OutputHandler = MakeShared<FOutputHandlerVideo, ESPMode::ThreadSafe>();
	VideoOutput.OutputHandler->SetOutputTexturePool(InCreateParameters.OutputTexturePool);
	VideoOutput.OutputHandler->SetMediaSamples(CurrentMediaSamples);
	AudioOutput.OutputHandler = MakeShared<FOutputHandlerAudio, ESPMode::ThreadSafe>();
	AudioOutput.OutputHandler->SetOutputAudioSamplePool(InCreateParameters.OutputAudioSamplePool);
	AudioOutput.OutputHandler->SetMediaSamples(CurrentMediaSamples);

	ExternalPlayerGUID = InCreateParameters.ExternalPlayerGUID;
	UseSharedWorkerThreads = InCreateParameters.WorkerThreads;
	CurrentState = EPlayerState::eState_Idle;
	PipelineState = EPipelineState::ePipeline_Stopped;
	DecoderState = EDecoderState::eDecoder_Paused;
	StreamState = EStreamState::eStream_Running;
	RenderRateScale = 1.0;
	StreamReaderHandler = nullptr;
	bStreamingHasStarted = false;
	RebufferCause = ERebufferCause::None;
	Manifest = nullptr;
	LastBufferingState = EPlayerState::eState_Buffering;
	bIsClosing = false;

	RebufferDetectedAtPlayPos.SetToInvalid();

	CurrentPlaybackSequenceID[0] = 0;
	CurrentPlaybackSequenceID[1] = 0;
	CurrentPlaybackSequenceID[2] = 0;
	CurrentPlaybackSequenceID[3] = 0;
	CurrentPlaybackSequenceState.Reset();

	bIsVideoDeselected = false;
	bIsAudioDeselected = false;
	bIsTextDeselected = false;

	DataAvailabilityStateVid.StreamType = EStreamType::Video;
	DataAvailabilityStateVid.Availability = Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable;
	DataAvailabilityStateAud.StreamType = EStreamType::Audio;
	DataAvailabilityStateAud.Availability = Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable;
	DataAvailabilityStateTxt.StreamType = EStreamType::Subtitle;
	DataAvailabilityStateTxt.Availability = Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable;

	// Subtitles need to be explicitly selected by the user.
	SelectedStreamAttributesTxt.Deselect();


	// Create the UTC time handler for this player session
	SynchronizedUTCTime = ISynchronizedUTCTime::Create();
	SynchronizedUTCTime->SetTime(MEDIAutcTime::Current());

	// Create the render clock
	RenderClock = MakeSharedTS<FMediaRenderClock>();

	// Create the AEMS handler. This is needed early on for the client to register receivers on.
	AEMSEventHandler = IAdaptiveStreamingPlayerAEMSHandler::Create();

	bShouldBePaused  = false;
	bShouldBePlaying = false;

	SeekVars.Reset();
	CurrentLoopState = {};
	NextLoopStates.Empty();
	bFirstSegmentRequestIsForLooping = false;

	BitrateCeiling = 0;
	VideoResolutionLimitWidth = 0;
	VideoResolutionLimitHeight = 0;

	TMediaInterlockedExchangePointer(PointerToLatestPlayer, this);

	StartWorkerThread();
}



//-----------------------------------------------------------------------------
/**
 * DTOR
 */
FAdaptiveStreamingPlayer::~FAdaptiveStreamingPlayer()
{
	TMediaInterlockedExchangePointer(PointerToLatestPlayer, (FAdaptiveStreamingPlayer*)0);

	FMediaEvent doneSig;
	WorkerThread.SendCloseMessage(&doneSig);
	doneSig.Wait();
	StopWorkerThread();

	MetricListenerCriticalSection.Lock();
	MetricListeners.Empty();
	MetricListenerCriticalSection.Unlock();

	RenderClock.Reset();
	delete SynchronizedUTCTime;

	delete AEMSEventHandler;

	Electra::RemoveActivePlayerInstance();
}



//-----------------------------------------------------------------------------
/**
 * Returns the error event. If there is no error an empty pointer will be returned.
 *
 * @return
 */
FErrorDetail FAdaptiveStreamingPlayer::GetError() const
{
	FScopeLock lock(&DiagnosticsCriticalSection);
	return LastErrorDetail;
}









void FAdaptiveStreamingPlayer::GetStreamBufferUtilization(FAccessUnitBufferInfo& OutInfo, const FAccessUnit* InForThis)
{
	FScopeLock lock(&DataBuffersCriticalSection);
	bool bFirst = true;
	if (ActiveDataOutputBuffers.IsValid())
	{
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> MultiBuffer = ActiveDataOutputBuffers->GetBuffer(InForThis->ESType);
		TSharedPtrTS<FAccessUnitBuffer> TrackBuffer = MultiBuffer ? MultiBuffer->FindBufferFor(InForThis->BufferSourceInfo) : nullptr;
		if (TrackBuffer)
		{
			TrackBuffer->GetStats(OutInfo);
			bFirst = false;
		}
	}
	for(int32 i=0,iMax=NextDataBuffers.Num(); i<iMax; ++i)
	{
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> MultiBuffer = NextDataBuffers[i]->GetBuffer(InForThis->ESType);
		TSharedPtrTS<FAccessUnitBuffer> TrackBuffer = MultiBuffer ? MultiBuffer->FindBufferFor(InForThis->BufferSourceInfo) : nullptr;
		if (TrackBuffer)
		{
			if (bFirst)
			{
				TrackBuffer->GetStats(OutInfo);
				bFirst = false;
			}
			else
			{
				FAccessUnitBufferInfo info;
				TrackBuffer->GetStats(info);
				OutInfo.PlayableDuration += info.PlayableDuration;
				OutInfo.CurrentMemInUse += info.CurrentMemInUse;
				OutInfo.NumCurrentAccessUnits += info.NumCurrentAccessUnits;
				OutInfo.bEndOfData |= info.bEndOfData;
				if (i == iMax-1)
				{
					OutInfo.bLastPushWasBlocked = info.bLastPushWasBlocked;
				}
			}
		}
	}
}

void FAdaptiveStreamingPlayer::GetStreamBufferUtilization(FAccessUnitBufferInfo& OutInfo, EStreamType BufferType)
{
	FScopeLock lock(&DataBuffersCriticalSection);

	// Start with the current buffer.
	bool bFirst = true;
	if (ActiveDataOutputBuffers.IsValid())
	{
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> Buffer = ActiveDataOutputBuffers->GetBuffer(BufferType);
		if (Buffer.IsValid())
		{
			Buffer->GetStats(OutInfo);
			bFirst = false;
		}
	}
	// Then add all the upcoming buffers. This list includes the current write buffer at the end
	// so we must not add that one as well.
	for(int32 i=0,iMax=NextDataBuffers.Num(); i<iMax; ++i)
	{
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> Buffer = NextDataBuffers[i]->GetBuffer(BufferType);
		if (Buffer.IsValid())
		{
			if (bFirst)
			{
				Buffer->GetStats(OutInfo);
				bFirst = false;
			}
			else
			{
				FAccessUnitBufferInfo info;
				Buffer->GetStats(info);
				OutInfo.PlayableDuration += info.PlayableDuration;
				OutInfo.CurrentMemInUse += info.CurrentMemInUse;
				OutInfo.NumCurrentAccessUnits += info.NumCurrentAccessUnits;
				// Propagate any end of data state to the overall stats. This is important
				// to determine when the current buffer has finished playing.
				if (info.bEndOfData)
				{
					OutInfo.bEndOfData = info.bEndOfData;
				}
				if (i == iMax-1)
				{
					OutInfo.bLastPushWasBlocked = info.bLastPushWasBlocked;
				}
			}
		}
	}
}



//-----------------------------------------------------------------------------
/**
 * Updates the internally kept diagnostics value with the current
 * decoder live values.
 */
void FAdaptiveStreamingPlayer::UpdateDiagnostics()
{
	FAccessUnitBufferInfo vidInf;
	FAccessUnitBufferInfo audInf;
	FAccessUnitBufferInfo txtInf;
	GetStreamBufferUtilization(vidInf, EStreamType::Video);
	GetStreamBufferUtilization(audInf, EStreamType::Audio);
	GetStreamBufferUtilization(txtInf, EStreamType::Subtitle);
	int64 tNow = MEDIAutcTime::CurrentMSec();
	DiagnosticsCriticalSection.Lock();
	VideoBufferStats.StreamBuffer = vidInf;
	AudioBufferStats.StreamBuffer = audInf;
	TextBufferStats.StreamBuffer = txtInf;
	VideoBufferStats.UpdateStalledDuration(tNow);
	AudioBufferStats.UpdateStalledDuration(tNow);
	TextBufferStats.UpdateStalledDuration(tNow);
	DiagnosticsCriticalSection.Unlock();
}

void FAdaptiveStreamingPlayer::UpdateFrameStats(bool bFinal)
{
	bool bSendReport = bFinal;
	// Track frame stats only while playing.
	if (CurrentState == EPlayerState::eState_Paused)
	{
		if (DroppedFrameStats.LastProcessedTime >= 0 && DroppedFrameStats.PausedDelta < 0)
		{
			DroppedFrameStats.PausedDelta = MEDIAutcTime::CurrentMSec() - DroppedFrameStats.LastProcessedTime;
		}
	}
	else if (CurrentState == EPlayerState::eState_Playing)
	{
		// Check if any video frames have been dropped.
		uint64 NumNewDroppedVideoFrames = VideoOutput.PeekNewlyNumDroppedSamples();
		bSendReport |= NumNewDroppedVideoFrames != 0;
		if (DroppedFrameStats.ReportingIntervalMs > 0)
		{
			// Add an entry to the dropped frame stats.
			if (NumNewDroppedVideoFrames)
			{
				FDroppedFrameStats::FEntry& NewEntry = DroppedFrameStats.Entries.Emplace_GetRef();
				NewEntry.DropCount = NumNewDroppedVideoFrames;
			}
			int64 tNow = MEDIAutcTime::CurrentMSec();
			// First time init?
			if (DroppedFrameStats.LastProcessedTime < 0)
			{
				DroppedFrameStats.LastProcessedTime = tNow;
			}
			// Back to playing from previously paused?
			if (DroppedFrameStats.PausedDelta > 0)
			{
				DroppedFrameStats.LastProcessedTime = tNow - DroppedFrameStats.PausedDelta;
				DroppedFrameStats.PausedDelta = -1;
			}
			int64 tElapsed = tNow - DroppedFrameStats.LastProcessedTime;
			if (bFinal || tElapsed >= DroppedFrameStats.ReportingIntervalMs)
			{
				// Sum up and remove entries up to the end of the current reporting interval.
				uint64 DropsInInterval = 0;
				for(auto& e : DroppedFrameStats.Entries)
				{
					DropsInInterval += e.DropCount;
				}
				DroppedFrameStats.Entries.Empty();
				if (DropsInInterval)
				{
					DroppedFrameStats.AverageDropsPerInterval.AddValue((int64)DropsInInterval);
					++DroppedFrameStats.NumIntervalsWithDrops;
				}
				DroppedFrameStats.LastProcessedTime = tNow;
				++DroppedFrameStats.NumIntervals;
				bSendReport = true;
			}
		}
	}
	if (bSendReport)
	{
		DispatchEvent(FMetricEvent::ReportVideoFrameStats(VideoOutput.GetTotalAddedSamples(), VideoOutput.GetTotalNumDroppedSamples(), VideoOutput.GetNewlyNumDroppedSamples(), DroppedFrameStats));
	}
	// When this is the final report we stop the interval
	if (bFinal)
	{
		DroppedFrameStats.ReportingIntervalMs = -1;
	}
}




//-----------------------------------------------------------------------------
/**
 * Initializes the player instance.
 *
 * @param Options
 */
void FAdaptiveStreamingPlayer::Initialize(const FParamDict& Options)
{
	PlayerOptions = Options;
	WorkerThread.SendInitializeMessage();
}

void FAdaptiveStreamingPlayer::ModifyOptions(const FParamDict& InOptionsToSetOrChange, const FParamDict& InOptionsToClear)
{
	WorkerThread.SendOptionChangeMessage(InOptionsToSetOrChange, InOptionsToClear);
}

FVariantValue FAdaptiveStreamingPlayer::GetMediaInfo(FName InKey) const
{
	return MediaInfoDictionary.GetValue(InKey);	// thread safe
}

void FAdaptiveStreamingPlayer::SetStaticResourceProviderCallback(const TSharedPtr<IAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe>& InStaticResourceProvider)
{
	StaticResourceProvider = InStaticResourceProvider;
}

void FAdaptiveStreamingPlayer::AddMetricsReceiver(IAdaptiveStreamingPlayerMetrics* InMetricsReceiver)
{
	if (InMetricsReceiver)
	{
		FScopeLock lock(&MetricListenerCriticalSection);
		if (MetricListeners.Find(InMetricsReceiver) == INDEX_NONE)
		{
			MetricListeners.Push(InMetricsReceiver);
		}
	}
}

void FAdaptiveStreamingPlayer::RemoveMetricsReceiver(IAdaptiveStreamingPlayerMetrics* InMetricsReceiver)
{
	if (InMetricsReceiver)
	{
		FScopeLock lock(&MetricListenerCriticalSection);
		/*bool bRemoved =*/ MetricListeners.Remove(InMetricsReceiver);
	}
}

void FAdaptiveStreamingPlayer::AddAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode)
{
	AEMSEventHandler->AddAEMSReceiver(InReceiver, InForSchemeIdUri, InForValue, InDispatchMode, true);
}

void FAdaptiveStreamingPlayer::RemoveAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode)
{
	AEMSEventHandler->RemoveAEMSReceiver(InReceiver, InForSchemeIdUri, InForValue, InDispatchMode);
}





//-----------------------------------------------------------------------------
/**
 * Dispatches an event to the listeners.
 *
 * @param pEvent
 */
void FAdaptiveStreamingPlayer::DispatchEvent(TSharedPtrTS<FMetricEvent> Event)
{
	if (EventDispatcher.IsValid())
	{
		if (DoesSharedInstanceExist())
		{
			Event->Player = SharedThis(this);
		}
		EventDispatcher->DispatchEvent(Event);
	}
}

void FAdaptiveStreamingPlayer::FireSyncEvent(TSharedPtrTS<FMetricEvent> Event)
{
	LockMetricsReceivers();
	const TArray<IAdaptiveStreamingPlayerMetrics*, TInlineAllocator<4>>& Listeners = GetMetricsReceivers();
	for(int32 i=0; i<Listeners.Num(); ++i)
	{
		switch(Event->Type)
		{
			case FMetricEvent::EType::OpenSource:
				Listeners[i]->ReportOpenSource(Event->Param.URL);
				break;
			case FMetricEvent::EType::ReceivedMainPlaylist:
				Listeners[i]->ReportReceivedMainPlaylist(Event->Param.URL);
				break;
			case FMetricEvent::EType::ReceivedPlaylists:
				Listeners[i]->ReportReceivedPlaylists();
				break;
			case FMetricEvent::EType::TracksChanged:
				Listeners[i]->ReportTracksChanged();
				break;
			case FMetricEvent::EType::BufferingStart:
				Listeners[i]->ReportBufferingStart(Event->Param.BufferingReason);
				break;
			case FMetricEvent::EType::BufferingEnd:
				Listeners[i]->ReportBufferingEnd(Event->Param.BufferingReason);
				break;
			case FMetricEvent::EType::Bandwidth:
				Listeners[i]->ReportBandwidth(Event->Param.Bandwidth.EffectiveBps, Event->Param.Bandwidth.ThroughputBps, Event->Param.Bandwidth.Latency);
				break;
			case FMetricEvent::EType::BufferUtilization:
				Listeners[i]->ReportBufferUtilization(Event->Param.BufferStats);
				break;
			case FMetricEvent::EType::PlaylistDownload:
				Listeners[i]->ReportPlaylistDownload(Event->Param.PlaylistStats);
				break;
			case FMetricEvent::EType::SegmentDownload:
				Listeners[i]->ReportSegmentDownload(Event->Param.SegmentStats);
				break;
			case FMetricEvent::EType::LicenseKey:
				Listeners[i]->ReportLicenseKey(Event->Param.LicenseKeyStats);
				break;
			case FMetricEvent::EType::DataAvailabilityChange:
				Listeners[i]->ReportDataAvailabilityChange(Event->Param.DataAvailability);
				break;
			case FMetricEvent::EType::VideoQualityChange:
				Listeners[i]->ReportVideoQualityChange(Event->Param.QualityChange.NewBitrate, Event->Param.QualityChange.PrevBitrate, Event->Param.QualityChange.bIsDrastic);
				break;
			case FMetricEvent::EType::AudioQualityChange:
				Listeners[i]->ReportAudioQualityChange(Event->Param.QualityChange.NewBitrate, Event->Param.QualityChange.PrevBitrate, Event->Param.QualityChange.bIsDrastic);
				break;
			case FMetricEvent::EType::CodecFormatChange:
				Listeners[i]->ReportDecodingFormatChange(Event->Param.CodecFormatChange);
				break;
			case FMetricEvent::EType::PrerollStart:
				Listeners[i]->ReportPrerollStart();
				break;
			case FMetricEvent::EType::PrerollEnd:
				Listeners[i]->ReportPrerollEnd();
				break;
			case FMetricEvent::EType::PlaybackStart:
				Listeners[i]->ReportPlaybackStart();
				break;
			case FMetricEvent::EType::PlaybackPaused:
				Listeners[i]->ReportPlaybackPaused();
				break;
			case FMetricEvent::EType::PlaybackResumed:
				Listeners[i]->ReportPlaybackResumed();
				break;
			case FMetricEvent::EType::PlaybackEnded:
				Listeners[i]->ReportPlaybackEnded();
				break;
			case FMetricEvent::EType::PlaybackJumped:
				Listeners[i]->ReportJumpInPlayPosition(Event->Param.TimeJump.ToNewTime, Event->Param.TimeJump.FromTime, Event->Param.TimeJump.Reason);
				break;
			case FMetricEvent::EType::PlaybackStopped:
				Listeners[i]->ReportPlaybackStopped();
				break;
			case FMetricEvent::EType::SeekCompleted:
				Listeners[i]->ReportSeekCompleted();
				break;
			case FMetricEvent::EType::MediaMetadataChanged:
				Listeners[i]->ReportMediaMetadataChanged(Event->Param.MediaMetadataChange.NewMetadata);
				break;
			case FMetricEvent::EType::Errored:
				Listeners[i]->ReportError(Event->Param.ErrorDetail.GetPrintable());
				break;
			case FMetricEvent::EType::LogMessage:
				Listeners[i]->ReportLogMessage(Event->Param.LogMessage.Level, Event->Param.LogMessage.Message, Event->Param.LogMessage.AtMillis);
				break;
			case FMetricEvent::EType::VideoFrameStats:
				Listeners[i]->ReportVideoFrameStats(Event->Param.FrameStats);
				break;
		}
	}
	UnlockMetricsReceivers();
	if (Event->EventSignal)
	{
		Event->EventSignal->Signal();
	}
}


//-----------------------------------------------------------------------------
/**
 * Starts the worker thread.
 */
void FAdaptiveStreamingPlayer::StartWorkerThread()
{
	// Get us an event dispatcher and add ourselves to the shared worker thread.
	if (!EventDispatcher.IsValid())
	{
		EventDispatcher = FAdaptiveStreamingPlayerEventHandler::Create(UseSharedWorkerThreads != IAdaptiveStreamingPlayer::FCreateParam::EWorkerThreads::DedicatedWorkerAndEventDispatch);
	}
	if (!SharedWorkerThread.IsValid())
	{
		SharedWorkerThread = FAdaptiveStreamingPlayerWorkerThread::Create(UseSharedWorkerThreads == IAdaptiveStreamingPlayer::FCreateParam::EWorkerThreads::Shared);
		WorkerThread.SetSharedWorkerThread(SharedWorkerThread);
		SharedWorkerThread->AddPlayerInstance(this);
	}
}


//-----------------------------------------------------------------------------
/**
 * Stops the worker thread.
 */
void FAdaptiveStreamingPlayer::StopWorkerThread()
{
	if (SharedWorkerThread.IsValid())
	{
		WorkerThread.SetSharedWorkerThread(nullptr);
		SharedWorkerThread->RemovePlayerInstance(this);
		SharedWorkerThread.Reset();
	}
	EventDispatcher.Reset();
}


//-----------------------------------------------------------------------------
/**
 * Enables or disables frame accurate seeking.
 */
void FAdaptiveStreamingPlayer::EnableFrameAccurateSeeking(bool bEnabled)
{
	PlayerOptions.Set(OptionKeyFrameAccurateSeek, FVariantValue(bEnabled));
}


//-----------------------------------------------------------------------------
/**
 * Sets the playback range.
 */
void FAdaptiveStreamingPlayer::SetPlaybackRange(const FPlaybackRange& InPlaybackRange)
{
	PlaybackState.SetPlayRange(InPlaybackRange);
}


//-----------------------------------------------------------------------------
/**
 * Returns the currently set playback range.
 */
void FAdaptiveStreamingPlayer::GetPlaybackRange(FPlaybackRange& OutPlaybackRange)
{
	PlaybackState.GetPlayRange(OutPlaybackRange);
}


TRangeSet<double> FAdaptiveStreamingPlayer::GetSupportedRates(EPlaybackRateType InForPlayRateType)
{
	return PlaybackState.GetPlaybackRates(InForPlayRateType);
}

void FAdaptiveStreamingPlayer::SetPlayRate(double InDesiredPlayRate, const FTrickplayParams& InParameters)
{
	PlaybackState.SetDesiredPlayRate(InDesiredPlayRate, InParameters);
}

double FAdaptiveStreamingPlayer::GetPlayRate() const
{
	// Return the *desired* play rate.
	return PlaybackState.GetDesiredPlayRate();
}


//-----------------------------------------------------------------------------
/**
 * Starts loading the manifest / main playlist.
 */
void FAdaptiveStreamingPlayer::LoadManifest(const FString& InManifestURL)
{
	WorkerThread.SendLoadManifestMessage(InManifestURL, FString());
}

//-----------------------------------------------------------------------------
/**
 * Returns the pointer to the current media samples. Because this will get
 * dereferenced and used _as a reference_(!!) it must never be a nullptr.
 */
TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> FAdaptiveStreamingPlayer::GetMediaSamples()
{
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> ms(CurrentMediaSamples);
	return ms.IsValid() ? ms : EmptyMediaSamples;
}



//-----------------------------------------------------------------------------
/**
 * Pauses playback
 */
void FAdaptiveStreamingPlayer::Pause()
{
	WorkerThread.SendPauseMessage();
}



//-----------------------------------------------------------------------------
/**
 * Resumes playback
 */
void FAdaptiveStreamingPlayer::Resume()
{
	WorkerThread.SendResumeMessage();
}



//-----------------------------------------------------------------------------
/**
 * Seek to a new position and play from there. This includes first playstart.
 *
 * @param NewPosition
 */
void FAdaptiveStreamingPlayer::SeekTo(const FSeekParam& NewPosition)
{
	FTimeRange NewPlayRange = PlaybackState.GetPlayRange();
	FScopeLock lock(&SeekVars.Lock);
	SeekVars.PendingRequest = NewPosition;
	SeekVars.PlayrangeOnRequest = NewPlayRange;
	WorkerThread.TriggerSharedWorkerThread();
}



//-----------------------------------------------------------------------------
/**
 * Stops playback. Playback cannot be resumed. Final player events will be sent to registered listeners.
 */
void FAdaptiveStreamingPlayer::Stop()
{
	FMediaEvent doneSig;
	WorkerThread.SendCloseMessage(&doneSig);
	doneSig.Wait();
}


//-----------------------------------------------------------------------------
/**
 * Puts playback into loop mode if possible. Live streams cannot be made to loop as they have infinite duration.
 *
 * @param InLoopParams
 */
void FAdaptiveStreamingPlayer::SetLooping(const FLoopParam& InLoopParams)
{
	PlaybackState.RequestLooping(InLoopParams);
}


//-----------------------------------------------------------------------------
/**
 * Returns whether or not a manifest has been loaded and assigned yet.
 *
 * @return
 */
bool FAdaptiveStreamingPlayer::HaveMetadata() const
{
	return PlaybackState.GetHaveMetadata();
}


//-----------------------------------------------------------------------------
/**
 * Returns the duration of the video. Returns invalid time when there is nothing to play. Returns positive infinite for live streams.
 *
 * @return
 */
FTimeValue FAdaptiveStreamingPlayer::GetDuration() const
{
	return PlaybackState.GetDuration();
}

//-----------------------------------------------------------------------------
/**
 * Returns the current play position. Returns -1.0 when there is nothing to play.
 *
 * @return
 */
FTimeValue FAdaptiveStreamingPlayer::GetPlayPosition() const
{
	return PlaybackState.GetPlayPosition();
}


//-----------------------------------------------------------------------------
/**
 * Returns the seekable range.
 *
 * @param OutRange
 */
void FAdaptiveStreamingPlayer::GetSeekableRange(FTimeRange& OutRange) const
{
	PlaybackState.GetSeekableRange(OutRange);
}

//-----------------------------------------------------------------------------
/**
 * Returns the timeline range.
 *
 * @param OutRange
 */
void FAdaptiveStreamingPlayer::GetTimelineRange(FTimeRange& OutRange) const
{
	PlaybackState.GetTimelineRange(OutRange);
}

//-----------------------------------------------------------------------------
/**
 * Returns true when playback has finished.
 *
 * @return
 */
bool FAdaptiveStreamingPlayer::HasEnded() const
{
	return PlaybackState.GetHasEnded();
}

//-----------------------------------------------------------------------------
/**
 * Returns true when seeking is in progress. False if not.
 *
 * @return
 */
bool FAdaptiveStreamingPlayer::IsSeeking() const
{
	return PlaybackState.GetIsSeeking();
}

//-----------------------------------------------------------------------------
/**
 * Returns true when data is being buffered/rebuffered, false otherwise.
 *
 * @return
 */
bool FAdaptiveStreamingPlayer::IsBuffering() const
{
	return PlaybackState.GetIsBuffering();
}

//-----------------------------------------------------------------------------
/**
 * Returns true when playing back, false if not.
 *
 * @return
 */
bool FAdaptiveStreamingPlayer::IsPlaying() const
{
	return PlaybackState.GetIsPlaying();
}

//-----------------------------------------------------------------------------
/**
 * Returns true when paused, false if not.
 *
 * @return
 */
bool FAdaptiveStreamingPlayer::IsPaused() const
{
	return PlaybackState.GetIsPaused();
}


//-----------------------------------------------------------------------------
/**
 * Returns the current loop state.
 *
 * @param OutLoopState
 */
void FAdaptiveStreamingPlayer::GetLoopState(FLoopState& OutLoopState) const
{
	PlaybackState.GetLoopState(OutLoopState);

	// Check if there is a pending seek. A seek resets the loop count, but the seek itself may not be executing
	// and the count not being reset yet.

	/* Not needed since we are only accessing a bool
	FScopeLock lock(&SeekVars.Lock);
	*/
	if (SeekVars.PendingRequest.IsSet())
	{
		OutLoopState.Count = 0;
	}
}

void FAdaptiveStreamingPlayer::GetReplayEventState(FEventReplayState& OutReplayEventState) const
{
	OutReplayEventState.bIsReplayEvent = Manifest.IsValid() ? Manifest->GetReplayType() != IManifest::EReplayEventType::NoReplay : EventReplayParams.HaveReplayParams();
}


//-----------------------------------------------------------------------------
/**
 * Returns track metadata of the currently active play period.
 *
 * @param OutTrackMetadata
 * @param StreamType
 */
void FAdaptiveStreamingPlayer::GetTrackMetadata(TArray<FTrackMetadata>& OutTrackMetadata, EStreamType StreamType) const
{
	TArray<FTrackMetadata> dummy1, dummy2;
	if (StreamType == EStreamType::Video)
	{
		PlaybackState.GetTrackMetadata(OutTrackMetadata, dummy1, dummy2);
	}
	else if (StreamType == EStreamType::Audio)
	{
		PlaybackState.GetTrackMetadata(dummy1, OutTrackMetadata, dummy2);
	}
	else if (StreamType == EStreamType::Subtitle)
	{
		PlaybackState.GetTrackMetadata(dummy1, dummy2, OutTrackMetadata);
	}
	else
	{
		OutTrackMetadata.Empty();
	}
}


#if 0
//-----------------------------------------------------------------------------
/**
 * Returns the track metadata of the currently active track. If nothing is selected yet the TOptional<> will be unset.
 *
 * @param OutSelectedTrackMetadata
 * @param StreamType
 */
void FAdaptiveStreamingPlayer::GetSelectedTrackMetadata(TOptional<FTrackMetadata>& OutSelectedTrackMetadata, EStreamType StreamType) const
{
}
#endif

void FAdaptiveStreamingPlayer::GetSelectedTrackAttributes(FStreamSelectionAttributes& OutAttributes, EStreamType StreamType) const
{
	TSharedPtrTS<FBufferSourceInfo> BufferInfo;

	auto SetOutAttributes = [&](const TSharedPtrTS<FBufferSourceInfo>& BufferInfo, const FStreamSelectionAttributes& CurrentAttr, bool bIsValidInfo) -> void
	{
		if (BufferInfo.IsValid())
		{
			OutAttributes.Kind = BufferInfo->Kind;
			FString Lang = BufferInfo->LanguageTag.Get();
			if (!Lang.IsEmpty())
			{
				OutAttributes.Language_RFC4647 = MoveTemp(Lang);
			}
			else
			{
				OutAttributes.Language_RFC4647.Reset();
			}
			OutAttributes.Codec = BufferInfo->Codec;
			OutAttributes.OverrideIndex = BufferInfo->HardIndex;
		}
		else
		{
			OutAttributes = CurrentAttr;
			// If the information is known to be valid then the stream is currently not available.
			// We set the override index to reflect this and return the intended selection.
			if (bIsValidInfo)
			{
				OutAttributes.OverrideIndex = -1;
			}
		}
	};

	FTimeValue Now = PlaybackState.GetPlayPosition();
	bool bHavePeriods;
	bool bFoundTime;
	BufferInfo = GetStreamBufferInfoAtTime(bHavePeriods, bFoundTime, StreamType, Now);
	bool bBufferInfoValid = bHavePeriods && bFoundTime;

	if (StreamType == EStreamType::Video)
	{
		SetOutAttributes(BufferInfo, SelectedStreamAttributesVid, bBufferInfoValid);
	}
	else if (StreamType == EStreamType::Audio)
	{
		SetOutAttributes(BufferInfo, SelectedStreamAttributesAud, bBufferInfoValid);
	}
	else if (StreamType == EStreamType::Subtitle)
	{
		SetOutAttributes(BufferInfo, SelectedStreamAttributesTxt, bBufferInfoValid);
	}
}


//-----------------------------------------------------------------------------
/**
 * Sets the highest bitrate when selecting a candidate stream.
 *
 * @param highestSelectableBitrate
 */
void FAdaptiveStreamingPlayer::SetBitrateCeiling(int32 highestSelectableBitrate)
{
	WorkerThread.SendBitrateMessage(EStreamType::Video, highestSelectableBitrate, 1);
}


//-----------------------------------------------------------------------------
/**
 * Sets the maximum resolution to use. Set both to 0 to disable, set only one to limit widht or height only.
 * Setting both will limit on either width or height, whichever limits first.
 *
 * @param MaxWidth
 * @param MaxHeight
 */
void FAdaptiveStreamingPlayer::SetMaxResolution(int32 MaxWidth, int32 MaxHeight)
{
	WorkerThread.SendResolutionMessage(MaxWidth, MaxHeight);
}


void FAdaptiveStreamingPlayer::SetInitialStreamAttributes(EStreamType StreamType, FStreamSelectionAttributes InitialSelection)
{
	bool bInitiallyDisabled = InitialSelection.OverrideIndex.IsSet() && InitialSelection.OverrideIndex.GetValue() < 0;
	if (bInitiallyDisabled)
	{
		InitialSelection.ClearOverrideIndex();
	}
	switch(StreamType)
	{
		case EStreamType::Video:
		{
			StreamSelectionAttributesVid = InitialSelection;
			break;
		}
		case EStreamType::Audio:
		{
			StreamSelectionAttributesAud = InitialSelection;
			break;
		}
		case EStreamType::Subtitle:
		{
			StreamSelectionAttributesTxt = InitialSelection;
			// If there is an initial subtitle selection (whether it can be fulfilled or not)
			// we enable them. They were turned off implicitly in the constructor.
			if (StreamSelectionAttributesTxt.IsSet())
			{
				SelectedStreamAttributesTxt.Select();
			}
			break;
		}
		default:
		{
			break;
		}
	}
	if (bInitiallyDisabled)
	{
		InternalDeselectStream(StreamType);
	}
}

#if 0
//-----------------------------------------------------------------------------
/**
 * Selects a track based from one of the array members returned by GetStreamMetadata().
 *
 * @param StreamType
 * @param TrackMetadata
 */
void FAdaptiveStreamingPlayer::SelectTrackByMetadata(EStreamType StreamType, const FTrackMetadata& TrackMetadata)
{
	WorkerThread.SendTrackSelectByMetadataMessage(StreamType, TrackMetadata);
}
#endif

void FAdaptiveStreamingPlayer::SelectTrackByAttributes(EStreamType StreamType, const FStreamSelectionAttributes& Attributes)
{
	WorkerThread.SendTrackSelectByAttributeMessage(StreamType, Attributes);
}


//-----------------------------------------------------------------------------
/**
 * Deselect track. The stream will continue to stream to allow for immediate selection/activation but no data will be fed to the decoder.
 *
 * @param StreamType
 */
void FAdaptiveStreamingPlayer::DeselectTrack(EStreamType StreamType)
{
	WorkerThread.SendTrackDeselectMessage(StreamType);
}

bool FAdaptiveStreamingPlayer::IsTrackDeselected(EStreamType StreamType)
{
	switch(StreamType)
	{
		case EStreamType::Video:
			return bIsVideoDeselected;
		case EStreamType::Audio:
			return bIsAudioDeselected;
		case EStreamType::Subtitle:
			return bIsTextDeselected;
		default:
			return true;
	}
}


void FAdaptiveStreamingPlayer::QueryStreamBufferInfo(FStreamBufferInfo& OutStreamBufferInfo, EStreamType InStreamType)
{
	OutStreamBufferInfo.TimeAvailable.Empty();
	OutStreamBufferInfo.TimeRequested.Empty();
	OutStreamBufferInfo.TimeEnqueued.Empty();
	OutStreamBufferInfo.bIsBufferActive = IsExpectedToStreamNow(InStreamType);
	FTimeRange dl = PlaybackState.GetCurrentDownloadRequestTimeRange(InStreamType);
	if (dl.IsValid())
	{
		OutStreamBufferInfo.TimeRequested.Emplace(MoveTemp(dl));
	}
	if (OutStreamBufferInfo.bIsBufferActive)
	{
		switch(InStreamType)
		{
			case EStreamType::Video:
			case EStreamType::Audio:
			{
				auto AddBufferInfo = [](TArray<FTimeRange>& OutRanges, const TSharedPtrTS<FMultiTrackAccessUnitBuffer>& InBuffer) -> void
				{
					if (InBuffer.IsValid())
					{
						FAccessUnitBufferInfo bi;
						InBuffer->GetStats(bi);
						if (bi.SmallestPTS.IsValid())
						{
							FTimeRange tr;
							tr.Start = bi.SmallestPTS;
							tr.End = bi.LargestPTSPlusDur;
							OutRanges.Emplace(MoveTemp(tr));
						}
					}
				};

				// Ask the renderers what they know they have released into the media sample queue.
				TArray<IOutputHandlerBase::FEnqueuedSampleInfo> Enqueued;
				if (InStreamType == EStreamType::Video && VideoOutput.OutputHandler.IsValid())
				{
					VideoOutput.OutputHandler->GetEnqueuedSampleInfo(Enqueued);
				}
				else if (InStreamType == EStreamType::Audio && AudioOutput.OutputHandler.IsValid())
				{
					AudioOutput.OutputHandler->GetEnqueuedSampleInfo(Enqueued);
				}
				if (Enqueued.Num())
				{
					// TBD: Do we need to break them down into separate ranges based on the sequence index?
					FTimeRange tr;
					tr.Start.SetFromTimespan(Enqueued[0].Timestamp.GetTime());
					tr.End.SetFromTimespan(Enqueued.Last().Timestamp.GetTime() + Enqueued.Last().Duration);
					OutStreamBufferInfo.TimeEnqueued.Emplace(MoveTemp(tr));
				}

				// Get the time range of the access units the respective decoder has in its internal pipeline
				// These AUs are not in the player buffer any more.
				{
					FScopeLock lock(&DiagnosticsCriticalSection);
					// Video or audio buffer? (see switch-case we are in above)
					IDecoderOutputBufferListener::FDecodeReadyStats* Ready = InStreamType == EStreamType::Video ? &VideoBufferStats.DecoderOutputBuffer : &AudioBufferStats.DecoderOutputBuffer;
					if (Ready && Ready->InDecoderTimeRangePTS.IsValid())
					{
						OutStreamBufferInfo.TimeEnqueued.Emplace(Ready->InDecoderTimeRangePTS);
					}
				}

				// Get the time range of AUs in the current active and future buffers (loops).
				{
					FScopeLock lock(&DataBuffersCriticalSection);
					if (ActiveDataOutputBuffers.IsValid())
					{
						AddBufferInfo(OutStreamBufferInfo.TimeAvailable, ActiveDataOutputBuffers->GetBuffer(InStreamType));
					}
					for(int32 i=0,iMax=NextDataBuffers.Num(); i<iMax; ++i)
					{
						AddBufferInfo(OutStreamBufferInfo.TimeAvailable, NextDataBuffers[i]->GetBuffer(InStreamType));
					}
				}
				break;
			}
		}
	}
}


void FAdaptiveStreamingPlayer::SuspendOrResumeDecoders(bool bSuspend, const FParamDict& InOptions)
{
	VideoDecoder.SuspendOrResume(bSuspend, InOptions);
	AudioDecoder.SuspendOrResume(bSuspend, InOptions);
}

//-----------------------------------------------------------------------------
/**
 * Starts the renderers.
 *
 * Call this once enough renderable data (both audio and video) is present.
 */
void FAdaptiveStreamingPlayer::StartRendering()
{
	RenderClock->Start();

	if (VideoOutput.OutputHandler)
	{
		VideoOutput.OutputHandler->StartOutput();
	}
	if (AudioOutput.OutputHandler)
	{
		AudioOutput.OutputHandler->StartOutput();
	}
}

//-----------------------------------------------------------------------------
/**
 * Stops renderers
 */
void FAdaptiveStreamingPlayer::StopRendering()
{
	RenderClock->Stop();

	if (VideoOutput.OutputHandler)
	{
		VideoOutput.OutputHandler->StopOutput();
	}
	if (AudioOutput.OutputHandler)
	{
		AudioOutput.OutputHandler->StopOutput();
	}
}

//-----------------------------------------------------------------------------
/**
 * Create the necessary AV renderers.
 *
 * @return
 */
int32 FAdaptiveStreamingPlayer::CreateRenderers()
{
	double CurrentRate = PlaybackState.GetCurrentPlayRate();
	double IntendedRate = PlaybackState.GetDesiredPlayRate();
	const bool bCurrentlyPaused = PlaybackState.GetIsPaused();

	// Set the render clock with the renderes.
	if (VideoOutput.OutputHandler)
	{
		VideoOutput.OutputHandler->SetClock(RenderClock);
	}
	if (AudioOutput.OutputHandler)
	{
		AudioOutput.OutputHandler->SetClock(RenderClock);
		AudioOutput.OutputHandler->SetPlaybackRate(CurrentRate, IntendedRate, bCurrentlyPaused);
	}

	return 0;
}


//-----------------------------------------------------------------------------
/**
 * Destroys the renderers and decoders.
 */
void FAdaptiveStreamingPlayer::DestroyRenderers()
{
	// Decoders must be gone already
	check(!VideoDecoder.Decoder);
	check(!AudioDecoder.Decoder);
	AudioOutput.Close();
	VideoOutput.Close();
}





void FAdaptiveStreamingPlayer::PostLog(Facility::EFacility FromFacility, IInfoLog::ELevel InLogLevel, const FString &Message)
{
	int64 millisNow = SynchronizedUTCTime->GetTime().GetAsMilliseconds();
	FString msg = FString::Printf(TEXT("%s: %s"), Facility::GetName(FromFacility), *Message);
	DispatchEvent(FMetricEvent::ReportLogMessage(InLogLevel, msg, millisNow));
}

void FAdaptiveStreamingPlayer::PostError(const FErrorDetail& InError)
{
	TSharedPtrTS<FErrorDetail> Error(new FErrorDetail(InError));
	ErrorQueue.Push(Error);
}

void FAdaptiveStreamingPlayer::SendMessageToPlayer(TSharedPtrTS<IPlayerMessage> PlayerMessage)
{
	WorkerThread.SendPlayerSessionMessage(PlayerMessage);
}

//-----------------------------------------------------------------------------
/**
 * Returns the external GUID identifying this player and its associated external texture.
 */
void FAdaptiveStreamingPlayer::GetExternalGuid(FGuid& OutExternalGuid)
{
	OutExternalGuid = ExternalPlayerGUID;
}

ISynchronizedUTCTime* FAdaptiveStreamingPlayer::GetSynchronizedUTCTime()
{
	return SynchronizedUTCTime;
}

TSharedPtr<IAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe> FAdaptiveStreamingPlayer::GetStaticResourceProvider()
{
	return StaticResourceProvider.Pin();
}

TSharedPtrTS<IElectraHttpManager> FAdaptiveStreamingPlayer::GetHTTPManager()
{
	return HttpManager;
}

TSharedPtrTS<IAdaptiveStreamSelector> FAdaptiveStreamingPlayer::GetStreamSelector()
{
	return StreamSelector;
}

IPlayerABRLiveControl* FAdaptiveStreamingPlayer::GetPlayerARBControl()
{
	return this;
}

TSharedPtrTS<IPlaylistReader> FAdaptiveStreamingPlayer::GetManifestReader()
{
	return ManifestReader;
}

TSharedPtrTS<FContentSteeringHandler> FAdaptiveStreamingPlayer::GetContentSteeringHandler()
{
	return ContentSteeringHandler;
}

TSharedPtrTS<FCMCDHandler> FAdaptiveStreamingPlayer::GetCMCDHandler()
{
	return CMCDHandler;
}

TSharedPtrTS<IPlayerEntityCache> FAdaptiveStreamingPlayer::GetEntityCache()
{
	return EntityCache;
}

TSharedPtrTS<IHTTPResponseCache> FAdaptiveStreamingPlayer::GetHTTPResponseCache()
{
	return HttpResponseCache;
}

IAdaptiveStreamingPlayerAEMSHandler* FAdaptiveStreamingPlayer::GetAEMSEventHandler()
{
	return AEMSEventHandler;
}
FParamDictTS& FAdaptiveStreamingPlayer::GetMediaInfoDictionary()
{
	return MediaInfoDictionary;
}
FParamDictTS& FAdaptiveStreamingPlayer::GetMutableOptions()
{
	return PlayerOptions;
}
bool FAdaptiveStreamingPlayer::HaveOptionValue(const FName& InOption)
{
	return PlayerOptions.HaveKey(InOption);
}
const FVariantValue FAdaptiveStreamingPlayer::GetOptionValue(const FName& InOption)
{
	return PlayerOptions.GetValue(InOption);
}

TSharedPtrTS<FDRMManager> FAdaptiveStreamingPlayer::GetDRMManager()
{
	return DrmManager;
}

void FAdaptiveStreamingPlayer::SetPlaybackEnd(const FTimeValue& InEndAtTime, IPlayerSessionServices::EPlayEndReason InEndingReason, TSharedPtrTS<IPlayerSessionServices::IPlayEndReason> InCustomManifestObject)
{
	WorkerThread.SendPlaybackEndMessage(InEndAtTime, InEndingReason, InCustomManifestObject);
}



//-----------------------------------------------------------------------------
/**
 * Notified when the given fragment will be opened.
 *
 * @param pRequest
 */
void FAdaptiveStreamingPlayer::OnFragmentOpen(TSharedPtrTS<IStreamSegment> pRequest)
{
	WorkerThread.Enqueue(FWorkerThreadMessages::FMessage::EType::FragmentOpen, pRequest);
}


//-----------------------------------------------------------------------------
/**
 * Fragment access unit received callback.
 *
 * NOTE: This must be done in place and cannot be deferred to a worker thread
 *       since we must return whether or not we could store the AU in our buffer right away.
 *
 * @param InAccessUnit
 *
 * @return
 */
bool FAdaptiveStreamingPlayer::OnFragmentAccessUnitReceived(FAccessUnit* InAccessUnit)
{
	// When shutting down we don't accept anything new, but we return we did so the stream reader won't keep pummeling us with new data.
	if (bIsClosing)
	{
		return true;
	}
	TSharedPtrTS<FMultiTrackAccessUnitBuffer> Buffer = GetCurrentReceiveStreamBuffer(InAccessUnit->ESType);
	if (!Buffer.IsValid())
	{
		return true;
	}

	// Get the current buffer stats. Although they are updated regularly once per tick this would not reflect the current
	// state until the the next tick, which would allow the limits to be ignored as new access units arrive quickly.
	FAccessUnitBufferInfo bufferInfo;
	GetStreamBufferUtilization(bufferInfo, InAccessUnit);
	const FAccessUnitBuffer::FConfiguration* BufferConfig = InAccessUnit->ESType == EStreamType::Video ? &PlayerConfig.StreamBufferConfigVideo :
															InAccessUnit->ESType == EStreamType::Audio ? &PlayerConfig.StreamBufferConfigAudio :
															InAccessUnit->ESType == EStreamType::Subtitle ? &PlayerConfig.StreamBufferConfigText : nullptr;
	FAccessUnitBuffer::FExternalBufferInfo TotalUtilization;
	TotalUtilization.Duration = bufferInfo.PlayableDuration;
	// Try to push the data into the receiving buffer
	return Buffer->Push(InAccessUnit, BufferConfig, &TotalUtilization);
}

//-----------------------------------------------------------------------------
/**
 * Fragment reached end-of-stream callback.
 * No additional access units will be received for this fragment.
 *
 * @param InStreamType
 * @param InStreamSourceInfo
 */
void FAdaptiveStreamingPlayer::OnFragmentReachedEOS(EStreamType InStreamType, TSharedPtr<const FBufferSourceInfo, ESPMode::ThreadSafe> InStreamSourceInfo)
{
	if (!bIsClosing)
	{
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> Buffer = GetCurrentReceiveStreamBuffer(InStreamType);
		if (Buffer.IsValid())
		{
			Buffer->PushEndOfDataFor(InStreamSourceInfo);
		}
	}
}


//-----------------------------------------------------------------------------
/**
 * Notified when the current fragment is closed.
 * This is called regardless of the error state and can always
 * be used as an indication that the fragment has finished processing.
 *
 * @param InRequest
 */
void FAdaptiveStreamingPlayer::OnFragmentClose(TSharedPtrTS<IStreamSegment> InRequest)
{
	WorkerThread.Enqueue(FWorkerThreadMessages::FMessage::EType::FragmentClose, InRequest);
}




void FAdaptiveStreamingPlayer::InternalHandleOnce()
{
	if (!bIsClosing)
	{
		// Update the play position we return to the interested caller.
		if (!PlaybackState.GetHasEnded())
		{
			if (RenderClock->IsRunning())
			{
				IMediaRenderClock::ERendererType ClockSource = IMediaRenderClock::ERendererType::Audio;
				if (DataAvailabilityStateAud.Availability == Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable)
				{
					ClockSource = IMediaRenderClock::ERendererType::Video;
				}
				FTimeValue playPos = RenderClock->GetInterpolatedRenderTime(ClockSource);
				PlaybackState.SetPlayPosition(ClampTimeToCurrentRange(playPos, false, true));
			}
		}
		else
		{
			// When playback has ended we lock the position to the end of the timeline.
			// This is only in case the application checks for the play position to reach the end of the timeline
			// ie. calling GetPlayPosition() to compare against the end of GetTimelineRange()
			// instead of using the dedicated HasEnded() method.
			PlaybackState.SetPlayPosition(ClampTimeToCurrentRange(FTimeValue::GetPositiveInfinity(), false, true));
		}

		// Handle state changes to match the user request.
		HandlePlayStateChanges();
		// Update diag buffers.
		UpdateDiagnostics();
		// Update frame statistics.
		UpdateFrameStats(false);
		// Handle seek requests.
		HandleSeeking();
		// Check for end of stream.
		CheckForStreamEnd();
		// Check the error queue for new arrivals.
		CheckForErrors();
		// Handle pending media segment requests.
		HandlePendingMediaSegmentRequests();
		// Handle changes in metadata, like timeline changes or track availability.
		HandleMetadataChanges();
		// Handle AEMS events
		HandleAEMSEvents();
		// Handle subtitle decoder
		HandleSubtitleDecoder();
		// Handle buffer level changes
		HandleNewBufferedData();
		// Handle new output data (finish prerolling)
		HandleNewOutputData();
		// Handle data buffers from deselected tracks to align with selected ones.
		HandleDeselectedBuffers();
		// Handle decoder changes
		HandleDecoderChanges();
		// Handle entity cache expirations.
		if (EntityCache.IsValid())
		{
			EntityCache->HandleEntityExpiration();
		}
		// Handle HTTP response cache expirations.
		if (HttpResponseCache.IsValid())
		{
			HttpResponseCache->HandleEntityExpiration();
		}
		// Handle completed DRM requests.
		if (DrmManager.IsValid())
		{
			DrmManager->Tick();
		}
		// Handle ABR.
		if (StreamSelector.IsValid())
		{
			IAdaptiveStreamSelector::EHandlingAction ABRAction = StreamSelector->PeriodicHandle();
			if (ABRAction == IAdaptiveStreamSelector::EHandlingAction::SeekToLive)
			{
				InternalStartoverAtLiveEdge();
			}
		}
		// Handle content steering
		if (ContentSteeringHandler.IsValid())
		{
			ContentSteeringHandler->PeriodicHandle();
		}
		// Handle CMCD
		if (CMCDHandler.IsValid())
		{
			CMCDHandler->PeriodicHandle();
		}
	}
}

bool FAdaptiveStreamingPlayer::InternalHandleThreadMessages()
{
	bool bGotMessage = false;
	FWorkerThreadMessages::FMessage msg;
	while(WorkerThread.WorkMessages.Dequeue(msg))
	{
		// While closed ignore all messages.
		if (bIsClosing)
		{
			if (msg.Data.IsType<FWorkerThreadMessages::FMessage::FEvent>())
			{
				FWorkerThreadMessages::FMessage::FEvent& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FEvent>();
				if (ev.Event)
				{
					ev.Event->Signal();
				}
			}
			continue;
		}

		bGotMessage = true;
		// What is it?
		switch(msg.Type)
		{
			case FWorkerThreadMessages::FMessage::EType::Initialize:
			{
				FAdaptiveStreamingPlayer::InternalInitialize();
				break;
			}
			case FWorkerThreadMessages::FMessage::EType::ChangeOptions:
			{
				FWorkerThreadMessages::FMessage::FOptionChange& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FOptionChange>();
				TArray<FName> Keys;
				ev.OptionsToClear.GetKeys(Keys);
				for(auto &Key : Keys)
				{
					PlayerOptions.Remove(Key);
				}
				ev.OptionsToSetOrChange.GetKeys(Keys);
				for(auto &Key : Keys)
				{
					FVariantValue Value = ev.OptionsToSetOrChange.GetValue(Key);
					PlayerOptions.Set(Key, Value);
				}
				break;
			}
			case FWorkerThreadMessages::FMessage::EType::LoadManifest:
			{
				FWorkerThreadMessages::FMessage::FLoadManifest& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FLoadManifest>();
				InternalLoadManifest(ev.URL, ev.MimeType);
				break;
			}
			case FWorkerThreadMessages::FMessage::EType::Pause:
			{
				bShouldBePaused = true;
				bShouldBePlaying = false;
				break;
			}
			case FWorkerThreadMessages::FMessage::EType::Resume:
			{
				bShouldBePaused = false;
				bShouldBePlaying = true;
				break;
			}
			case FWorkerThreadMessages::FMessage::EType::Close:
			{
				if (!bIsClosing)
				{
					bIsClosing = true;
					InternalStop(false);
					InternalClose();
					DispatchEvent(FMetricEvent::ReportPlaybackStopped());
				}
				FWorkerThreadMessages::FMessage::FEvent& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FEvent>();
				if (ev.Event)
				{
					ev.Event->Signal();
				}
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::ChangeBitrate:
			{
				FWorkerThreadMessages::FMessage::FBitrate& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FBitrate>();
				BitrateCeiling = ev.Value;
				StreamSelector->SetBandwidthCeiling(BitrateCeiling);
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::LimitResolution:
			{
				FWorkerThreadMessages::FMessage::FResolution& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FResolution>();
				VideoResolutionLimitWidth  = ev.Width;
				VideoResolutionLimitHeight = ev.Height;
				UpdateStreamResolutionLimit();
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::SelectTrackByAttributes:
			{
				if (!bIsClosing)
				{
					FWorkerThreadMessages::FMessage::FMetadataTrackSelection& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FMetadataTrackSelection>();
					switch(ev.StreamType)
					{
						case EStreamType::Video:
						{
							PendingTrackSelectionVid = MakeSharedTS<FStreamSelectionAttributes>(ev.TrackAttributes);
							bIsVideoDeselected = false;
							break;
						}
						case EStreamType::Audio:
						{
							PendingTrackSelectionAud = MakeSharedTS<FStreamSelectionAttributes>(ev.TrackAttributes);
							bIsAudioDeselected = false;
							break;
						}
						case EStreamType::Subtitle:
						{
							PendingTrackSelectionTxt = MakeSharedTS<FStreamSelectionAttributes>(ev.TrackAttributes);
							bIsTextDeselected = false;
							break;
						}
						default:
						{
							break;
						}
					}
				}
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::SelectTrackByMetadata:
			{
				// Currently not used. May be used again later.
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::DeselectTrack:
			{
				FWorkerThreadMessages::FMessage::FMetadataTrackSelection& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FMetadataTrackSelection>();
				InternalDeselectStream(ev.StreamType);
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::FragmentOpen:
			{
				FWorkerThreadMessages::FMessage::FStreamReader& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FStreamReader>();
				TSharedPtrTS<IStreamSegment> pRequest = ev.Request;
				EStreamType reqType = ev.Request->GetType();
				// Check that the request is for this current playback sequence and not an outdated one.
				if (pRequest.IsValid() && pRequest->GetPlaybackSequenceID() == CurrentPlaybackSequenceID[StreamTypeToArrayIndex(reqType)])
				{
					PlaybackState.SetCurrentDownloadRequestTimeRange(reqType, pRequest->GetTimeRange());

					DispatchBufferUtilizationEvent(ev.Request->GetType());

					// Video bitrate change?
					if (ev.Request->GetType() == EStreamType::Video)
					{
						int32 SegmentBitrate = pRequest->GetBitrate();
						int32 SegmentQualityLevel = pRequest->GetQualityIndex();
						if (SegmentBitrate != CurrentVideoStreamBitrate.Bitrate)
						{
							bool bDrastic = CurrentVideoStreamBitrate.Bitrate && SegmentQualityLevel < CurrentVideoStreamBitrate.QualityLevel-1;
							DispatchEvent(FMetricEvent::ReportVideoQualityChange(SegmentBitrate, CurrentVideoStreamBitrate.Bitrate, bDrastic));
							CurrentVideoStreamBitrate.Bitrate = SegmentBitrate;
							CurrentVideoStreamBitrate.QualityLevel = SegmentQualityLevel;
							GetMediaInfoDictionary().Set(MediaInfoVideoBitrateBuffering.Resolve(), FVariantValue((int64)SegmentBitrate));
						}
					}
					// Audio bitrate change?
					else if (ev.Request->GetType() == EStreamType::Audio)
					{
						int32 SegmentBitrate = pRequest->GetBitrate();
						int32 SegmentQualityLevel = pRequest->GetQualityIndex();
						if (SegmentBitrate != CurrentAudioStreamBitrate.Bitrate)
						{
							bool bDrastic = CurrentAudioStreamBitrate.Bitrate && SegmentQualityLevel < CurrentAudioStreamBitrate.QualityLevel-1;
							DispatchEvent(FMetricEvent::ReportAudioQualityChange(SegmentBitrate, CurrentAudioStreamBitrate.Bitrate, bDrastic));
							CurrentAudioStreamBitrate.Bitrate = SegmentBitrate;
							CurrentAudioStreamBitrate.QualityLevel = SegmentQualityLevel;
							GetMediaInfoDictionary().Set(MediaInfoAudioBitrateBuffering.Resolve(), FVariantValue((int64)SegmentBitrate));
						}
					}
				}
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::FragmentClose:
			{
				FWorkerThreadMessages::FMessage::FStreamReader& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FStreamReader>();
				TSharedPtrTS<IStreamSegment> pRequest = ev.Request;
				EStreamType reqType = ev.Request->GetType();
				// Check that the request is for this current playback sequence and not an outdated one.
				if (pRequest.IsValid() && pRequest->GetPlaybackSequenceID() == CurrentPlaybackSequenceID[StreamTypeToArrayIndex(reqType)])
				{
					PlaybackState.ClearCurrentDownloadRequestTimeRange(reqType);

					Metrics::FSegmentDownloadStats stats;
					pRequest->GetDownloadStats(stats);

					// Inform content steering
					FContentSteeringHandler::FStreamParams streamParams;
					streamParams.bActiveVideo = IsExpectedToStreamNow(EStreamType::Video);
					streamParams.bActiveAudio = IsExpectedToStreamNow(EStreamType::Audio);
					streamParams.bActiveSubtitles = IsExpectedToStreamNow(EStreamType::Subtitle);
					GetContentSteeringHandler()->FinishedDownloadRequestOn(stats, streamParams);

					// Dispatch download event
					DispatchEvent(FMetricEvent::ReportSegmentDownload(stats));

					// Dispatch buffer utilization
					DispatchBufferUtilizationEvent(reqType);

					// Dispatch average throughput, bandwidth and latency event for video segments only.
					if (reqType == EStreamType::Video)
					{
						DispatchEvent(FMetricEvent::ReportBandwidth(StreamSelector->GetAverageBandwidth(), StreamSelector->GetAverageThroughput(), StreamSelector->GetAverageLatency()));
					}

					// Add to the list of completed requests for which we need to find the next or retry segment to fetch.
					FPendingSegmentRequest NextReq;
					NextReq.Request = pRequest;
					NextPendingSegmentRequests->Enqueue(MoveTemp(NextReq));
				}
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::BufferUnderrun:
			{
				InternalRebuffer();
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::PlayerSession:
			{
				FWorkerThreadMessages::FMessage::FSession& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FSession>();
				HandleSessionMessage(ev.PlayerMessage);
				break;
			}

			case FWorkerThreadMessages::FMessage::EType::EndPlaybackAt:
			{
				FWorkerThreadMessages::FMessage::FEndPlaybackAt& ev = msg.Data.Get<FWorkerThreadMessages::FMessage::FEndPlaybackAt>();
				// With a defined end time we just play out until then.
				PlaybackState.SetShouldPlayOnLiveEdge(false);
				PlaybackState.SetPlaybackEndAtTime(ev.EndAtTime);
				break;
			}

			default:
			{
				checkNoEntry();
				break;
			}
		}

		InternalHandleOnce();
	}
	return bGotMessage;
}

//-----------------------------------------------------------------------------
/**
 * Handling function called by the shared worker thread.
 */
void FAdaptiveStreamingPlayer::HandleOnce()
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_AdaptiveWorker);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, AdaptiveStreamingPlayer_Worker);

	if (!InternalHandleThreadMessages())
	{
		InternalHandleOnce();
	}
	// Handle the components that are bound to add new worker messages.
	// This should not be done while processing the messages that were enqueued earlier,
	// especially not since handling those will result in handling all state checks
	// in InternalHandleOnce() again!
	InternalHandleManifestReader();
}



void FAdaptiveStreamingPlayer::HandlePlayStateChanges()
{
	if (CurrentState == EPlayerState::eState_Error)
	{
		return;
	}

	// Should the player be paused?
	if (bShouldBePaused)
	{
		// Are we not paused and in a state in which we can pause?
		if (CurrentState == EPlayerState::eState_Playing)
		{
			PlaybackState.SetShouldPlayOnLiveEdge(false);
			InternalPause();
		}
	}

	// Should the player be playing?
	if (bShouldBePlaying)
	{
		// Are we paused and in a state in which we can play?
		if (CurrentState == EPlayerState::eState_Paused)
		{
			InternalResume();
		}
	}

	// Handle pending loop type changes.
	InternalHandlePendingLoopRequest();

	// Did the play range or loop state change such that we need to start over?
	if (PlaybackState.GetPlayRangeHasChanged() || PlaybackState.GetLoopStateHasChanged())
	{
		// We do not do this when paused as setting the range usually happens only when paused for seeking or scrubbing.
		if (CurrentState == EPlayerState::eState_Playing)
		{
			InternalStartoverAtCurrentPosition();
		}
	}

	// Update the current live latency.
	PlaybackState.SetCurrentLiveLatency(CalculateCurrentLiveLatency(false));

	// Inform the renderers about the current playback rate.
	const double CurrentRate = PlaybackState.GetCurrentPlayRate();
	const double IntendedRate = PlaybackState.GetDesiredPlayRate();
	const bool bCurrentlyPaused = PlaybackState.GetIsPaused();
	if (AudioOutput.OutputHandler)
	{
		AudioOutput.OutputHandler->SetPlaybackRate(CurrentRate, IntendedRate, bCurrentlyPaused);
	}
}


void FAdaptiveStreamingPlayer::HandleSeeking()
{
	// Are we in a state where a seek is possible?
	if (CurrentState == EPlayerState::eState_Idle || CurrentState == EPlayerState::eState_ParsingManifest || CurrentState == EPlayerState::eState_PreparingStreams || CurrentState == EPlayerState::eState_Error ||
		!Manifest.IsValid())
	{
		// No, return.
		return;
	}

	FScopeLock lock(&SeekVars.Lock);
	// Is there a pending request?
	if (SeekVars.PendingRequest.IsSet())
	{
		// Not playing on the Live edge. This may get set to true later on handling the start segment.
		PlaybackState.SetShouldPlayOnLiveEdge(false);

		// Adjust seek index as necessary.
		if (SeekVars.PendingRequest.GetValue().NewSequenceIndex.IsSet())
		{
			CurrentPlaybackSequenceState.PrimaryIndex = SeekVars.PendingRequest.GetValue().NewSequenceIndex.GetValue();
		}
		// Contractually a seek implies resetting the loop count, which is represented in the 2ndary index.
		CurrentPlaybackSequenceState.SecondaryIndex = 0;
		// Reset the loop counter in the state as well.
		FInternalLoopState LoopStateNow;
		PlaybackState.GetLoopState(LoopStateNow);
		LoopStateNow.Count = 0;
		PlaybackState.SetLoopState(LoopStateNow);
		CurrentLoopState.Count = 0;

		// Trigger the seek.
		FSeekParam SeekParam = SeekVars.PendingRequest.GetValue();
		FTimeRange SeekPlayrangeOnRequest = SeekVars.PlayrangeOnRequest;
		SeekVars.PendingRequest.Reset();
		SeekVars.ClearWorkVars();

		if (SeekVars.bIsPlayStart)
		{
			CurrentState = EPlayerState::eState_Buffering;
		}
		else
		{
			// Stop everything.
			// We need to unlock the seek vars lock here to allow the renderer's call to Flush() in InternalStop()
			// to call out to the user code for which we must not hold any state locks.
			SeekVars.Lock.Unlock();
			InternalStop(PlayerConfig.bHoldLastFrameDuringSeek);
			if (Manifest.IsValid())
			{
				Manifest->UpdateDynamicRefetchCounter();
			}
			CurrentState = EPlayerState::eState_Seeking;
			SeekVars.Lock.Lock();

			// When seeking (other than the initial playstart seek) any default end time that
			// was set via the URL #t,e parameter no longer applies.
			if (Manifest.IsValid())
			{
				Manifest->ClearDefaultEndTime();
			}
		}
		SeekVars.ActiveRequest = SeekParam;
		lock.Unlock();
		InternalStartAt(SeekVars.ActiveRequest.GetValue(), &SeekPlayrangeOnRequest);
	}
}



//-----------------------------------------------------------------------------
/**
 * Handle changes in metadata, like timeline changes or track availability.
 */
void FAdaptiveStreamingPlayer::HandleMetadataChanges()
{
	// The timeline can change dynamically. Refresh it on occasion.
	if (Manifest.IsValid() && Manifest->ReloadsPlaylistPeriodically())
	{
		PlaybackState.SetSeekableRange(Manifest->GetSeekableTimeRange());
		PlaybackState.SetTimelineRange(Manifest->GetTotalTimeRange());
		PlaybackState.SetDuration(Manifest->GetDuration());
	}

	FTimeValue CurrentTime = PlaybackState.GetPlayPosition();

	// See if we may be reaching the next period.
	if (UpcomingPeriods.Num() && CurrentTime.IsValid() && CurrentTime != MetadataHandlingState.LastHandlingTime)
	{
		// Locate the period the position falls into.
		for(int32 i=0; i<UpcomingPeriods.Num(); ++i)
		{
			if (UpcomingPeriods[i].TimeRange.Contains(CurrentTime))
			{
				if (!UpcomingPeriods[i].ID.Equals(MetadataHandlingState.LastSentPeriodID))
				{
					MetadataHandlingState.LastSentPeriodID = UpcomingPeriods[i].ID;

					TArray<FTrackMetadata> MetadataVideo;
					TArray<FTrackMetadata> MetadataAudio;
					TArray<FTrackMetadata> MetadataSubtitle;
					UpcomingPeriods[i].Period->GetMetaData(MetadataVideo, EStreamType::Video);
					UpcomingPeriods[i].Period->GetMetaData(MetadataAudio, EStreamType::Audio);
					UpcomingPeriods[i].Period->GetMetaData(MetadataSubtitle, EStreamType::Subtitle);

					bool bMetadataChanged = PlaybackState.SetTrackMetadata(MetadataVideo, MetadataAudio, MetadataSubtitle);
					PlaybackState.SetHaveMetadata(true);
					if (bMetadataChanged)
					{
						DispatchEvent(FMetricEvent::ReportTracksChanged());
					}

					// Update the data availability for this period.
					UpdateDataAvailabilityState(DataAvailabilityStateVid, MetadataVideo.Num() ? Metrics::FDataAvailabilityChange::EAvailability::DataAvailable : Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable);
					UpdateDataAvailabilityState(DataAvailabilityStateAud, MetadataAudio.Num() ? Metrics::FDataAvailabilityChange::EAvailability::DataAvailable : Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable);
					UpdateDataAvailabilityState(DataAvailabilityStateTxt, MetadataSubtitle.Num() ? Metrics::FDataAvailabilityChange::EAvailability::DataAvailable : Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable);

					// Inform AEMS handler of the period transition.
					AEMSEventHandler->PlaybackPeriodTransition(UpcomingPeriods[i].ID, UpcomingPeriods[i].TimeRange);
				}
				break;
			}
		}

		// Remove the periods that we have passed for which the loop counter is not in the future.
		// With looping playback the periods to loop back to will be added/updated and must not be removed
		// if they are used in a future loop cycle.
		for(int32 i=0; i<UpcomingPeriods.Num(); ++i)
		{
			if ((UpcomingPeriods[i].TimeRange.End.IsValid() && CurrentTime >= UpcomingPeriods[i].TimeRange.End && UpcomingPeriods[i].LoopCount <= CurrentTime.GetSequenceIndex()) ||
				(UpcomingPeriods[i].TimeRange.Start.IsValid() && CurrentTime < UpcomingPeriods[i].TimeRange.Start && UpcomingPeriods[i].LoopCount < CurrentTime.GetSequenceIndex()))
			{
				UpcomingPeriods.RemoveAt(i);
				--i;
			}
		}

		MetadataHandlingState.LastHandlingTime = CurrentTime;
	}

	// Dump periods that are no longer active because we passed their end with some threshold
	// and whose loop count is also not expired.
	if (CurrentTime.IsValid())
	{
		FTimeValue PastTime = CurrentTime - FTimeValue().SetFromMilliseconds(1000);
		FScopeLock Lock(&ActivePeriodCriticalSection);
		for(int32 i=0; i<ActivePeriods.Num(); ++i)
		{
			if (((ActivePeriods[i].TimeRange.End.IsValid() && PastTime > ActivePeriods[i].TimeRange.End) ||
				(!ActivePeriods[i].TimeRange.End.IsValid() && (i+1) < ActivePeriods.Num() && PastTime >= ActivePeriods[i+1].TimeRange.Start)) &&
				ActivePeriods[i].LoopCount <= CurrentTime.GetSequenceIndex())
			{
				// Do not remove the only active period.
				if (ActivePeriods.Num() > 1)
				{
					ActivePeriods.RemoveAt(i);
					--i;
				}
			}
		}
	}

	// Handle media metadata updates.
	FMediaMetadataUpdate::EResult MetaResult = MediaMetadataUpdates.Handle(CurrentTime);
	if (MetaResult != FMediaMetadataUpdate::EResult::NoChange)
	{
		if (MetaResult == FMediaMetadataUpdate::EResult::ChangedAndUpdate && Manifest.IsValid())
		{
			Manifest->UpdateRunningMetaData(MediaMetadataUpdates.GetActive());
			if (ActivePeriods.Num())
			{
				TArray<FTrackMetadata> MetadataVideo;
				TArray<FTrackMetadata> MetadataAudio;
				TArray<FTrackMetadata> MetadataSubtitle;
				// FIXME: We are currently assuming that this happens only for audio casts where
				//        we won't have more than just one period.
				ActivePeriods[0].Period->GetMetaData(MetadataVideo, EStreamType::Video);
				ActivePeriods[0].Period->GetMetaData(MetadataAudio, EStreamType::Audio);
				ActivePeriods[0].Period->GetMetaData(MetadataSubtitle, EStreamType::Subtitle);
				PlaybackState.SetTrackMetadata(MetadataVideo, MetadataAudio, MetadataSubtitle);
				PlaybackState.SetHaveMetadata(true);
			}
		}
		DispatchEvent(FMetricEvent::ReportMediaMetadataChanged(MediaMetadataUpdates.GetActive()));
	}
}


//-----------------------------------------------------------------------------
/**
 * Handles Application Event or Metadata Stream (AEMS) events triggering on current playback position.
 *
 * @param SessionMessage
 */
void FAdaptiveStreamingPlayer::HandleAEMSEvents()
{
	if (CurrentState == EPlayerState::eState_Playing)
	{
		FTimeValue Current = PlaybackState.GetPlayPosition();
		AEMSEventHandler->ExecutePendingEvents(Current);
	}
}


//-----------------------------------------------------------------------------
/**
 * Handles a player session message sent by one of the player sub-systems.
 *
 * @param SessionMessage
 */
void FAdaptiveStreamingPlayer::HandleSessionMessage(TSharedPtrTS<IPlayerMessage> SessionMessage)
{
	// Playlist downloaded (successful or not)?
	if (SessionMessage->GetType() == IPlaylistReader::PlaylistDownloadMessage::Type())
	{
		IPlaylistReader::PlaylistDownloadMessage* pMsg = static_cast<IPlaylistReader::PlaylistDownloadMessage *>(SessionMessage.Get());
		Metrics::FPlaylistDownloadStats stats;
		stats.bWasSuccessful = !pMsg->GetConnectionInfo().StatusInfo.ErrorDetail.IsSet();
		stats.bWasAborted = pMsg->GetConnectionInfo().bWasAborted;
		stats.bDidTimeout = pMsg->GetConnectionInfo().StatusInfo.ErrorCode == HTTP::EStatusErrorCode::ERRCODE_HTTP_CONNECTION_TIMEOUT;
		stats.ListType = pMsg->GetListType();
		stats.LoadType = pMsg->GetLoadType();
		stats.Url.URL = pMsg->GetConnectionInfo().EffectiveURL;
		stats.FailureReason = pMsg->GetConnectionInfo().StatusInfo.ErrorDetail.GetMessage();
		stats.HTTPStatusCode = pMsg->GetConnectionInfo().StatusInfo.HTTPStatus;
		stats.RetryNumber = Utils::Max(pMsg->GetAttempts() - 1, 0);
		DispatchEvent(FMetricEvent::ReportPlaylistDownload(stats));
	}
	// Playlist fetched & parsed?
	else if (SessionMessage->GetType() == IPlaylistReader::PlaylistLoadedMessage::Type())
	{
		if (ManifestReader.IsValid())
		{
			IPlaylistReader::PlaylistLoadedMessage* pMsg = static_cast<IPlaylistReader::PlaylistLoadedMessage *>(SessionMessage.Get());
			const FErrorDetail& Result = pMsg->GetResult();
			// Note: Right now only successful messages should get here. We check for failure anyway in case this changes in the future.
			if (!Result.IsError())
			{
				if (!Result.WasAborted())
				{
					if (pMsg->GetListType() == Playlist::EListType::Main && pMsg->GetLoadType() == Playlist::ELoadType::Initial)
					{
						DispatchEvent(FMetricEvent::ReportReceivedMainPlaylist(pMsg->GetConnectionInfo().EffectiveURL));
					}
					else if (pMsg->GetLoadType() == Playlist::ELoadType::Initial)
					{
						SelectManifest();
						DispatchEvent(FMetricEvent::ReportReceivedPlaylists());
					}
					else
					{
						UpdateManifest();
					}
				}
			}
			else
			{
				PostError(Result);
			}
		}
	}
	// Media metadata changed?
	else if (SessionMessage->GetType() == FPlaylistMetadataUpdateMessage::Type())
	{
		FPlaylistMetadataUpdateMessage* pMsg = static_cast<FPlaylistMetadataUpdateMessage*>(SessionMessage.Get());
		MediaMetadataUpdates.AddEntry(pMsg->GetValidFrom(), pMsg->GetMetadata(), pMsg->GetTriggerInternalRefresh());
	}
	// License key?
	else if (SessionMessage->GetType() == FLicenseKeyMessage::Type())
	{
		FLicenseKeyMessage* pMsg = static_cast<FLicenseKeyMessage*>(SessionMessage.Get());
		Metrics::FLicenseKeyStats stats;
		stats.bWasSuccessful = !pMsg->GetResult().IsError();
		stats.URL   		 = pMsg->GetConnectionInfo().EffectiveURL;
		stats.FailureReason  = pMsg->GetResult().IsError() ? pMsg->GetResult().GetPrintable() : FString();
		DispatchEvent(FMetricEvent::ReportLicenseKey(stats));
		if (pMsg->GetResult().IsError())
		{
			PostError(pMsg->GetResult());
		}
	}
	// Decoder message?
	else if (SessionMessage->GetType() == FDecoderMessage::Type())
	{
		FDecoderMessage* pMsg = static_cast<FDecoderMessage*>(SessionMessage.Get());
		if (pMsg->GetStreamType() == EStreamType::Video)
		{
			VideoDecoder.bDrainingForCodecChangeDone = true;
		}
	}
	else
	{
		checkNoEntry();
	}
}




FTimeValue FAdaptiveStreamingPlayer::CalculateCurrentLiveLatency(bool bViaLatencyElement)
{
	FTimeValue LiveLatency;
	if (Manifest.IsValid())
	{
		LiveLatency = Manifest->CalculateCurrentLiveLatency(PlaybackState.GetPlayPosition(), PlaybackState.GetEncoderLatency(), bViaLatencyElement);
	}
	return LiveLatency;
}


//-----------------------------------------------------------------------------
/**
 * Returns the required duration to be available in the buffer before playback can begin.
 */
double FAdaptiveStreamingPlayer::GetMinBufferTimeBeforePlayback()
{
	// Yes. Check if we have enough data buffered up to begin handing off data to the decoders.
	double kMinBufferBeforePlayback = LastBufferingState == EPlayerState::eState_Seeking ? PlayerConfig.SeekBufferMinTimeAvailBeforePlayback :
										LastBufferingState == EPlayerState::eState_Rebuffering ? PlayerConfig.RebufferMinTimeAvailBeforePlayback : PlayerConfig.InitialBufferMinTimeAvailBeforePlayback;

	FTimeValue mbtAbr = StreamSelector.IsValid() ? StreamSelector->GetMinBufferTimeForPlayback(LastBufferingState == EPlayerState::eState_Seeking ? IAdaptiveStreamSelector::EMinBufferType::Seeking
																							 : LastBufferingState == EPlayerState::eState_Rebuffering ? IAdaptiveStreamSelector::EMinBufferType::Rebuffering
																							 : IAdaptiveStreamSelector::EMinBufferType::Initial, Manifest->GetMinBufferTime()) : FTimeValue();
	kMinBufferBeforePlayback = mbtAbr.IsValid() ? mbtAbr.GetAsSeconds() : kMinBufferBeforePlayback;
	return kMinBufferBeforePlayback;
}

bool FAdaptiveStreamingPlayer::IsExpectedToStreamNow(EStreamType InType)
{
	if (!StreamReaderHandler)
	{
		return false;
	}
	if (InType == EStreamType::Video)
	{
		return CurrentPlayPeriodVideo.IsValid() ? CurrentPlayPeriodVideo->GetSelectedStreamBufferSourceInfo(InType).IsValid() : false;
	}
	else if (InType == EStreamType::Audio)
	{
		return CurrentPlayPeriodAudio.IsValid() ? CurrentPlayPeriodAudio->GetSelectedStreamBufferSourceInfo(InType).IsValid() : false;
	}
	else if (InType == EStreamType::Subtitle)
	{
		return CurrentPlayPeriodText.IsValid() ? CurrentPlayPeriodText->GetSelectedStreamBufferSourceInfo(InType).IsValid() : false;
	}
	return false;
}


bool FAdaptiveStreamingPlayer::HaveEnoughBufferedDataToStartPlayback()
{
	double kMinBufferBeforePlayback = GetMinBufferTimeBeforePlayback();

	DiagnosticsCriticalSection.Lock();

	bool bHaveEnoughVideo = true;
	if (IsExpectedToStreamNow(EStreamType::Video))
	{
		bHaveEnoughVideo = false;
		// If the video stream has reached the end then there won't be any new data and whatever we have is all there is.
		if (VideoBufferStats.StreamBuffer.bEndOfData)
		{
			bHaveEnoughVideo = true;
		}
		else
		{
			// Does the buffer have the requested amount of data?
			if (VideoBufferStats.StreamBuffer.PlayableDuration.GetAsSeconds() >= kMinBufferBeforePlayback)
			{
				bHaveEnoughVideo = true;
			}
			else
			{
				// There is a chance that the input stream buffer is too small to hold the requested amount of material.
				if (VideoBufferStats.StreamBuffer.bLastPushWasBlocked)
				{
					// We won't be able to receive any more, so we have enough as it is.
					bHaveEnoughVideo = true;
				}
			}
		}
	}

	bool bHaveEnoughAudio = true;
	if (IsExpectedToStreamNow(EStreamType::Audio))
	{
		bHaveEnoughAudio = false;
		// If the audio stream has reached the end then there won't be any new data and whatever we have is all there is.
		if (AudioBufferStats.StreamBuffer.bEndOfData)
		{
			bHaveEnoughAudio = true;
		}
		else
		{
			// Does the buffer have the requested amount of data?
			if (AudioBufferStats.StreamBuffer.PlayableDuration.GetAsSeconds() >= kMinBufferBeforePlayback)
			{
				bHaveEnoughAudio = true;
			}
			else
			{
				// There is a chance that the input stream buffer is too small to hold the requested amount of material.
				if (AudioBufferStats.StreamBuffer.bLastPushWasBlocked)
				{
					// We won't be able to receive any more, so we have enough as it is.
					bHaveEnoughAudio = true;
				}
			}
		}
	}

	// When we are dealing with a single multiplexed stream and one buffer is blocked then essentially all buffers
	// must be considered blocked since demuxing cannot continue.
	if (StreamReaderHandler && StreamReaderHandler->LoadsMultiplexedTracksSequentially())
	{
		if (VideoBufferStats.StreamBuffer.bLastPushWasBlocked ||
			AudioBufferStats.StreamBuffer.bLastPushWasBlocked ||
			TextBufferStats.StreamBuffer.bLastPushWasBlocked)
		{
			bHaveEnoughVideo = bHaveEnoughAudio = true;
		}
	}

	DiagnosticsCriticalSection.Unlock();

	return bHaveEnoughVideo && bHaveEnoughAudio;
}


void FAdaptiveStreamingPlayer::PrepareForPrerolling()
{
	PrerollVars.StartTime = MEDIAutcTime::CurrentMSec();

	// We need to check if there is any data available in the buffers that can be decoded.
	// It may be possible that playback was starting on the last segment that did not bring in any playable
	// data or none at all if it was past end of stream.
	DiagnosticsCriticalSection.Lock();

	// Check video buffer
	if (IsExpectedToStreamNow(EStreamType::Video))
	{
		if (VideoBufferStats.StreamBuffer.bEndOfData && (VideoBufferStats.StreamBuffer.PlayableDuration == FTimeValue::GetZero() || VideoBufferStats.StreamBuffer.NumCurrentAccessUnits == 0))
		{
			PrerollVars.bHaveEnoughVideo = true;
			VideoBufferStats.DecoderInputBuffer.bEODSignaled = true;
			VideoBufferStats.DecoderInputBuffer.bEODReached = true;
			VideoBufferStats.DecoderOutputBuffer.bEODreached = true;
		}
	}

	// Check audio buffer
	if (IsExpectedToStreamNow(EStreamType::Audio))
	{
		if (AudioBufferStats.StreamBuffer.bEndOfData && (AudioBufferStats.StreamBuffer.PlayableDuration == FTimeValue::GetZero() || AudioBufferStats.StreamBuffer.NumCurrentAccessUnits == 0))
		{
			PrerollVars.bHaveEnoughAudio = true;
			AudioBufferStats.DecoderInputBuffer.bEODSignaled = true;
			AudioBufferStats.DecoderInputBuffer.bEODReached = true;
			AudioBufferStats.DecoderOutputBuffer.bEODreached = true;
		}
	}

	DiagnosticsCriticalSection.Unlock();
}


//-----------------------------------------------------------------------------
/**
 * Checks if buffers have enough data to advance the play state.
 */
void FAdaptiveStreamingPlayer::HandleNewBufferedData()
{
	// Are we in buffering state? (this includes rebuffering)
	if (bStreamingHasStarted && CurrentState == EPlayerState::eState_Buffering && DecoderState == EDecoderState::eDecoder_Paused)
	{
		// Seek for scrubbing is not possible at playstart so we can skip a few things here.
		if (!SeekVars.bPrerollDone)
		{
			DataBuffersCriticalSection.Lock();
			if (!ActiveDataOutputBuffers.IsValid())
			{
				TSharedPtrTS<FStreamDataBuffers> NewOutputBuffers;
				if (NextDataBuffers.Num())
				{
					NewOutputBuffers = NextDataBuffers[0];
					NextDataBuffers.RemoveAt(0);
				}
				ActiveDataOutputBuffers = MoveTemp(NewOutputBuffers);
			}
			bool bHaveActiveOutputBuffer = ActiveDataOutputBuffers.IsValid();
			DataBuffersCriticalSection.Unlock();
			if (bHaveActiveOutputBuffer)
			{
				PrepareForPrerolling();
				PipelineState = EPipelineState::ePipeline_Prerolling;
				DecoderState = EDecoderState::eDecoder_Running;
				PostrollVars.Clear();
				DispatchEvent(FMetricEvent::ReportPrerollStart());
			}
		}
	}
}


//-----------------------------------------------------------------------------
/**
 * Handle data in deselected buffers.
 * Since deselected buffers do not feed a decoder the data must be discarded as playback progresses
 * to avoid buffer overflows.
 */
void FAdaptiveStreamingPlayer::HandleDeselectedBuffers()
{
	if (bIsVideoDeselected || bIsAudioDeselected || bIsTextDeselected)
	{
		FAccessUnitBufferInfo BufferStats;
		// Get the current playback position in case nothing is selected.
		FTimeValue DiscardUntilTime = PlaybackState.GetPlayPosition();

		TSharedPtrTS<FMultiTrackAccessUnitBuffer> VidOutBuffer = GetCurrentOutputStreamBuffer(EStreamType::Video);
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> AudOutBuffer = GetCurrentOutputStreamBuffer(EStreamType::Audio);
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> TxtOutBuffer = GetCurrentOutputStreamBuffer(EStreamType::Subtitle);

		// Get last popped video PTS if video track is selected.
		if (!bIsVideoDeselected && VidOutBuffer.IsValid())
		{
			FTimeValue v = VidOutBuffer->GetLastPoppedPTS();
			if (v.IsValid())
			{
				DiscardUntilTime = v;
			}
			VidOutBuffer->GetStats(BufferStats);
			if (BufferStats.bEndOfData && BufferStats.NumCurrentAccessUnits == 0)
			{
				DiscardUntilTime.SetToPositiveInfinity();
			}
		}
		// Get last popped audio PTS if audio track is selected.
		if (!bIsAudioDeselected && AudOutBuffer.IsValid())
		{
			FTimeValue v = AudOutBuffer->GetLastPoppedPTS();
			if (v.IsValid())
			{
				DiscardUntilTime = v;
			}
			AudOutBuffer->GetStats(BufferStats);
			if (BufferStats.bEndOfData && BufferStats.NumCurrentAccessUnits == 0)
			{
				DiscardUntilTime.SetToPositiveInfinity();
			}
		}
		// FIXME: do we need to consider subtitle tracks as being the only selected ones??

		if (DiscardUntilTime.IsValid())
		{
			if (bIsVideoDeselected && VidOutBuffer.IsValid())
			{
				FMultiTrackAccessUnitBuffer::FScopedLock lock(VidOutBuffer);
				VidOutBuffer->PopDiscardUntil(DiscardUntilTime);
			}
			if (bIsAudioDeselected && AudOutBuffer.IsValid())
			{
				FMultiTrackAccessUnitBuffer::FScopedLock lock(AudOutBuffer);
				AudOutBuffer->PopDiscardUntil(DiscardUntilTime);
			}
			if (bIsTextDeselected && TxtOutBuffer.IsValid())
			{
				FMultiTrackAccessUnitBuffer::FScopedLock lock(TxtOutBuffer);
				TxtOutBuffer->PopDiscardUntil(DiscardUntilTime);
			}
		}
	}
}


//-----------------------------------------------------------------------------
/**
 * Checks if the player state can transition to the next state with available decoder output.
 */
void FAdaptiveStreamingPlayer::HandleNewOutputData()
{
	int64 timeNow = MEDIAutcTime::CurrentMSec();

	// Are we currently pre-rolling the media pipeline?
	if (PipelineState == EPipelineState::ePipeline_Prerolling)
	{
		// Yes. Is it "warm" enough to start rendering?
		DiagnosticsCriticalSection.Lock();

		// Check video
		bool bHaveEnoughVideo = PrerollVars.bHaveEnoughVideo;
		if (IsExpectedToStreamNow(EStreamType::Video))
		{
			// If the video decoder output buffer is stalled then we have enough video output. There won't be any more coming in.
			// Also when the stream has reached the end and that has propagated through the decoder's buffer.
			if (VideoBufferStats.DecoderOutputBuffer.bOutputStalled ||
				(VideoBufferStats.StreamBuffer.bEndOfData && VideoBufferStats.DecoderInputBuffer.bEODSignaled && VideoBufferStats.DecoderInputBuffer.bEODReached) ||
				bIsVideoDeselected)
			{
				bHaveEnoughVideo = true;
			}
		}
		else
		{
			bHaveEnoughVideo = true;
		}

		// Check audio
		bool bHaveEnoughAudio = PrerollVars.bHaveEnoughAudio;
		if (IsExpectedToStreamNow(EStreamType::Audio))
		{
			// If the audio decoder output buffer is stalled then we have enough video output. There won't be any more coming in.
			// Also when the stream has reached the end and that has propagated through the decoder's buffer.
			if (AudioBufferStats.DecoderOutputBuffer.bOutputStalled ||
				(AudioBufferStats.StreamBuffer.bEndOfData && AudioBufferStats.DecoderInputBuffer.bEODSignaled && AudioBufferStats.DecoderInputBuffer.bEODReached) ||
				bIsAudioDeselected)
			{
				bHaveEnoughAudio = true;
			}
		}
		else
		{
			bHaveEnoughAudio = true;
		}

		DiagnosticsCriticalSection.Unlock();

		// Keep current decision around to prime the next check.
		if (bHaveEnoughVideo)
		{
			PrerollVars.bHaveEnoughVideo = true;
		}
		if (bHaveEnoughAudio)
		{
			PrerollVars.bHaveEnoughAudio = true;
		}

		// Ready to go?
		if (bHaveEnoughVideo && bHaveEnoughAudio)
		{
			// The first time around we send the seek complete event and stop the feeding of the decoders
			// to prevent them consuming and sending output to the renderers while we await the buffers
			// to fill to the amount we need.
			if (!SeekVars.bPrerollDone)
			{
				SeekVars.bPrerollDone = true;
				SeekVars.SetFinished();
				DispatchEvent(FMetricEvent::ReportSeekCompleted());
				// Pause feeding the decoder to stop draining buffers we want to fill.
				DecoderState = EDecoderState::eDecoder_Paused;
			}

			if (HaveEnoughBufferedDataToStartPlayback())
			{
				PlaybackState.SetIsBuffering(false);
				PlaybackState.SetIsSeeking(false);

				SeekVars.ClearWorkVars();
				PipelineState = EPipelineState::ePipeline_Stopped;
				CurrentState = EPlayerState::eState_Paused;
				DecoderState = EDecoderState::eDecoder_Running;
				PrerollVars.Clear();
				DispatchBufferingEvent(false, LastBufferingState);
				DispatchEvent(FMetricEvent::ReportPrerollEnd());
				if (Manifest.IsValid())
				{
					Manifest->ReachedStableBuffer();
				}
			}
		}
	}
}



void FAdaptiveStreamingPlayer::InternalHandlePendingStartRequest(const FTimeValue& CurrentTime)
{
	// Is there a start request?
	if (PendingStartRequest.IsValid())
	{
		FPlayStartPosition StartAt = PendingStartRequest->StartAt;

		// Need to have a manifest.
		if (Manifest.IsValid())
		{
			if (PendingStartRequest->RetryAtTime.IsValid() && CurrentTime < PendingStartRequest->RetryAtTime)
			{
				return;
			}

			// For sanities sake disable any loop flag that might have been set before loading the playlist if the presentation isn't VoD
			// and is also not a replay event.
			if (Manifest->GetPresentationType() != IManifest::EType::OnDemand && Manifest->GetReplayType() == IManifest::EReplayEventType::NoReplay)
			{
				CurrentLoopParam.bEnableLooping = false;
			}

			// If the starting time has not been set we now check if we are dealing with a VoD or a Live stream and choose a starting point.
			if (!StartAt.Time.IsValid())
			{
				// Use the presentation provided start time, if it has one.
				StartAt.Time = Manifest->GetDefaultStartTime();
				if (!StartAt.Time.IsValid())
				{
					FTimeRange Seekable = Manifest->GetSeekableTimeRange();
					if (Manifest->GetPresentationType() == IManifest::EType::OnDemand)
					{
						check(Seekable.Start.IsValid());
						StartAt.Time = Seekable.Start;
					}
					else
					{
						/*
							We need to wait for the timeline to become valid. It may not be yet if the presentation
							is scheduled to start in the future or if the intended distance to the Live edge can not
							be maintained yet when the presentation has just begun.
						*/
						if (Seekable.End - Seekable.Start <= FTimeValue::GetZero())
						{
							return;
						}
						check(Seekable.End.IsValid());

						// If this is a looping replay event and our start time is not set then we are to join 'now'.
						if (CurrentLoopParam.bEnableLooping && Manifest->GetReplayType() != IManifest::EReplayEventType::NoReplay)
						{
							StartAt.Time = GetSynchronizedUTCTime()->GetTime();
							FTimeValue LiveLatency = Manifest->GetDesiredLiveLatency();
							if (LiveLatency.IsValid())
							{
								StartAt.Time -= LiveLatency;
							}
						}
						else
						{
							StartAt.Time = Seekable.End;
						}
						switch(Manifest->GetLiveEdgePlayMode())
						{
							case IManifest::ELiveEdgePlayMode::Default:
							case IManifest::ELiveEdgePlayMode::Always:
							{
								PlaybackState.SetShouldPlayOnLiveEdge(true);
								break;
							}
							default:
							{
								break;
							}
						}
					}
				}
			}
			// Set the current playback range with the start options and clamp the start time to the range.
			SetPlaystartOptions(StartAt.Options);
			ClampStartRequestTime(StartAt.Time);


			// Do we have a play period yet?
			if (!InitialPlayPeriod.IsValid())
			{
				IManifest::FResult Result = Manifest->FindPlayPeriod(InitialPlayPeriod, StartAt, PendingStartRequest->SearchType);
				switch(Result.GetType())
				{
					case IManifest::FResult::EType::TryAgainLater:
					{
						PendingStartRequest->RetryAtTime = Result.GetRetryAgainAtTime();
						break;
					}
					case IManifest::FResult::EType::Found:
					{
						// NOTE: Do *not* reset the pending start request yet. It is still needed.
						break;
					}
					case IManifest::FResult::EType::PastEOS:
					{
						// If the initial start time is beyond the end of the presentation
						// but looping is enabled we need to handle this accordingly.
						bool bEnded = !CurrentLoopParam.bEnableLooping;
						if (CurrentLoopParam.bEnableLooping)
						{
							if (Manifest->GetReplayType() == IManifest::EReplayEventType::NoReplay)
							{
								// Not a replay, start at the beginning.
								FTimeRange Seekable = Manifest->GetSeekableTimeRange();
								PendingStartRequest->SearchType = IManifest::ESearchType::Closest;
								StartAt.Time = Seekable.Start;
								ClampStartRequestTime(StartAt.Time);
								PendingStartRequest->StartAt.Time = StartAt.Time;
							}
							else
							{
								// For a looping replay we need to check if the start is beyond the event range and
								// then adjust the timeline accordingly.
								FTimeRange TotalRange = Manifest->GetTotalTimeRange();
								if (TotalRange.IsValid())
								{
									const FTimeValue RangeDuration = TotalRange.End - TotalRange.Start;
									const FTimeValue OverTime = StartAt.Time - TotalRange.Start;
									int32 NumWraps = OverTime.GetAsHNS() / RangeDuration.GetAsHNS();
									// Apply a batch of wraps only when there is more than one to avoid inaccuracies
									// that may result in slightly overshooting the range.
									if (NumWraps > 1)
									{
										Manifest->PrepareForLooping(NumWraps - 1);
									}
									// Advance single iterations (zero or one)
									while(StartAt.Time >= Manifest->GetTotalTimeRange().End)
									{
										Manifest->PrepareForLooping(1);
									}
								}
								else
								{
									bEnded = true;
								}
							}
						}
						if (bEnded)
						{
							PendingStartRequest.Reset();
							InternalSetPlaybackEnded();
						}
						break;
					}
					case IManifest::FResult::EType::NotFound:
					case IManifest::FResult::EType::BeforeStart:
					case IManifest::FResult::EType::NotLoaded:
					{
						// Throw a playback error for now.
						FErrorDetail err;
						err.SetFacility(Facility::EFacility::Player);
						err.SetMessage(TEXT("Could not locate the playback period to begin playback at."));
						err.SetCode(INTERR_COULD_NOT_LOCATE_START_PERIOD);
						PostError(err);
						InitialPlayPeriod.Reset();
						PendingStartRequest.Reset();
						break;
					}
					case IManifest::FResult::EType::Error:
					default:
					{
						FErrorDetail err;
						err.SetFacility(Facility::EFacility::Player);
						err.SetMessage(FString::Printf(TEXT("Error selecting the initial set of streams (%s)."), *Result.GetErrorDetail().GetMessage()));
						err.SetCode(INTERR_GENERAL_FAILURE);
						PostError(err);
						PendingStartRequest.Reset();
						InitialPlayPeriod.Reset();
						break;
					}
				}
			}
			// Do we have a starting play period now?
			if (InitialPlayPeriod.IsValid())
			{
				// Is it ready yet?
				switch(InitialPlayPeriod->GetReadyState())
				{
					case IManifest::IPlayPeriod::EReadyState::NotLoaded:
					{
						// If there are already pending track selections update the initial selections with them and clear the pending ones.
						if (PendingTrackSelectionVid.IsValid())
						{
							StreamSelectionAttributesVid = *PendingTrackSelectionVid;
							PendingTrackSelectionVid.Reset();
						}
						if (PendingTrackSelectionAud.IsValid())
						{
							StreamSelectionAttributesAud = *PendingTrackSelectionAud;
							PendingTrackSelectionAud.Reset();
						}
						if (PendingTrackSelectionTxt.IsValid())
						{
							StreamSelectionAttributesTxt = *PendingTrackSelectionTxt;
							PendingTrackSelectionTxt.Reset();
						}

						// Prepare the play period. This must immediately change the state away from NotReady
						InitialPlayPeriod->SetStreamPreferences(EStreamType::Video, StreamSelectionAttributesVid);
						InitialPlayPeriod->SetStreamPreferences(EStreamType::Audio, StreamSelectionAttributesAud);
						InitialPlayPeriod->SetStreamPreferences(EStreamType::Subtitle, StreamSelectionAttributesTxt);
						InitialPlayPeriod->Load();
						break;
					}
					case IManifest::IPlayPeriod::EReadyState::Loading:
					{
						// Period is not yet ready. Check again on the next iteration.
						break;
					}
					case IManifest::IPlayPeriod::EReadyState::Loaded:
					{
						// Period is loaded and configured according to the stream preferences.
						// Prepare the initial quality/bitrate for playback.
						int64 StartingBitrate = -1;
						bool bForceInitial = false;
						bool bForceStartRate = false;
						if (PendingStartRequest->StartingBitrate.IsSet())
						{
							// This is used for rebuffering in which case we enforce the starting bitrate limits.
							StartingBitrate = PendingStartRequest->StartingBitrate.GetValue();
							bForceInitial = true;
							bForceStartRate = true;
						}
						else
						{
							StartingBitrate = StreamSelector->GetAverageBandwidth();
							if (PendingStartRequest->StartType == FPendingStartRequest::EStartType::PlayStart)
							{
								StartingBitrate = PlayerOptions.GetValue(OptionKeyInitialBitrate).SafeGetInt64(StartingBitrate);
								bForceInitial = true;
								bForceStartRate = true;
							}
							else if (PendingStartRequest->StartType == FPendingStartRequest::EStartType::Seeking)
							{
								if (PlayerOptions.HaveKey(OptionKeySeekStartBitrate))
								{
									StartingBitrate = PlayerOptions.GetValue(OptionKeySeekStartBitrate).SafeGetInt64(StartingBitrate);
									bForceInitial = true;
									bForceStartRate = true;
								}
							}
						}
						// If not set or set to an invalid value try to use defaults.
						if (StartingBitrate <= 0)
						{
							// Ask the manifest for the default starting bitrate, which is usually the first stream listed.
							StartingBitrate = InitialPlayPeriod->GetDefaultStartingBitrate();
							// If still nothing for whatever reason use a reasonable value.
							if (StartingBitrate <= 0)
							{
								StartingBitrate = 1000000;
							}
						}
						if (bForceStartRate)
						{
							StreamSelector->SetBandwidth(StartingBitrate);
						}
						if (bForceInitial)
						{
							StreamSelector->SetForcedNextBandwidth(StartingBitrate, GetMinBufferTimeBeforePlayback());
						}

						// Set the current average video bitrate in the player options for the period to retrieve if necessary.
						PlayerOptions.Set(OptionKeyCurrentAvgStartingVideoBitrate, FVariantValue(StreamSelector->GetAverageBandwidth()));

						InitialPlayPeriod->PrepareForPlay();
						break;
					}
					case IManifest::IPlayPeriod::EReadyState::Preparing:
					{
						// Period is not yet ready. Check again on the next iteration.
						break;
					}
					case IManifest::IPlayPeriod::EReadyState::IsReady:
					{
						// With the period being ready we can now get the initial media segment request.
						check(PendingStartRequest.IsValid());	// must not have been released yet

						// At playback start all streams begin in the same period.
						CurrentPlayPeriodVideo = InitialPlayPeriod;
						CurrentPlayPeriodAudio = InitialPlayPeriod;
						CurrentPlayPeriodText = InitialPlayPeriod;

						// Tell the ABR the current playback period. At playback start it is the same for all stream types.
						StreamSelector->SetCurrentPlaybackPeriod(EStreamType::Video, CurrentPlayPeriodVideo);
						StreamSelector->SetCurrentPlaybackPeriod(EStreamType::Audio, CurrentPlayPeriodAudio);

						TSharedPtrTS<FBufferSourceInfo> BufferSourceInfoVid = CurrentPlayPeriodVideo->GetSelectedStreamBufferSourceInfo(EStreamType::Video);
						TSharedPtrTS<FBufferSourceInfo> BufferSourceInfoAud = CurrentPlayPeriodAudio->GetSelectedStreamBufferSourceInfo(EStreamType::Audio);
						TSharedPtrTS<FBufferSourceInfo> BufferSourceInfoTxt = CurrentPlayPeriodText->GetSelectedStreamBufferSourceInfo(EStreamType::Subtitle);
						if (BufferSourceInfoVid.IsValid())
						{
							SelectedStreamAttributesVid.UpdateWith(BufferSourceInfoVid->Kind, BufferSourceInfoVid->LanguageTag, BufferSourceInfoVid->Codec, BufferSourceInfoVid->HardIndex);
							if (StreamReaderHandler && !StreamReaderHandler->LoadsMultiplexedTracksSequentially())
							{
								StreamSelectionAttributesVid.UpdateIfOverrideSet(BufferSourceInfoVid->Kind, BufferSourceInfoVid->LanguageTag, BufferSourceInfoVid->Codec);
							}
						}
						if (BufferSourceInfoAud.IsValid())
						{
							SelectedStreamAttributesAud.UpdateWith(BufferSourceInfoAud->Kind, BufferSourceInfoAud->LanguageTag, BufferSourceInfoAud->Codec, BufferSourceInfoAud->HardIndex);
							if (StreamReaderHandler && !StreamReaderHandler->LoadsMultiplexedTracksSequentially())
							{
								StreamSelectionAttributesAud.UpdateIfOverrideSet(BufferSourceInfoAud->Kind, BufferSourceInfoAud->LanguageTag, BufferSourceInfoAud->Codec);
							}
						}
						if (BufferSourceInfoTxt.IsValid())
						{
							SelectedStreamAttributesTxt.UpdateWith(BufferSourceInfoTxt->Kind, BufferSourceInfoTxt->LanguageTag, BufferSourceInfoTxt->Codec, BufferSourceInfoTxt->HardIndex);
							if (StreamReaderHandler && !StreamReaderHandler->LoadsMultiplexedTracksSequentially())
							{
								StreamSelectionAttributesTxt.UpdateIfOverrideSet(BufferSourceInfoTxt->Kind, BufferSourceInfoTxt->LanguageTag, BufferSourceInfoTxt->Codec);
							}
						}

						// Apply any set resolution limit at the start of the new play period.
						UpdateStreamResolutionLimit();

						// Have the ABR select the initial bandwidth. Pass it an empty segment request to indicate this.
						FTimeValue ActionDelay;
						IAdaptiveStreamSelector::ESegmentAction Action = StreamSelector->SelectSuitableStreams(ActionDelay, TSharedPtrTS<IStreamSegment>());
						check(Action == IAdaptiveStreamSelector::ESegmentAction::FetchNext || Action == IAdaptiveStreamSelector::ESegmentAction::Fail);
						if (Action != IAdaptiveStreamSelector::ESegmentAction::FetchNext)
						{
							FErrorDetail err;
							err.SetFacility(Facility::EFacility::Player);
							err.SetMessage(TEXT("All streams have failed. There is nothing to play any more."));
							err.SetCode(INTERR_ALL_STREAMS_HAVE_FAILED);
							PostError(err);
							PendingStartRequest.Reset();
							break;
						}

						if (!PendingStartRequest->RetryAtTime.IsValid() || CurrentTime >= PendingStartRequest->RetryAtTime)
						{
							TSharedPtrTS<IStreamSegment> FirstSegmentRequest;
							IManifest::FResult Result;
							if (PendingStartRequest->StartType != FPendingStartRequest::EStartType::LoopPoint)
							{
								// Get the current loop state to fetch the last published loop count.
								// The internal counter in CurrentLoopState may be some iterations ahead since it could have fetched segments from the loop point already.
								FInternalLoopState LoopStateNow;
								PlaybackState.GetLoopState(LoopStateNow);
								CurrentLoopState.Count = LoopStateNow.Count;

								Result = InitialPlayPeriod->GetStartingSegment(FirstSegmentRequest, CurrentPlaybackSequenceState, StartAt, PendingStartRequest->SearchType);
								bFirstSegmentRequestIsForLooping = false;
							}
							else
							{
								FPlayerSequenceState NextPlaybackState = CurrentPlaybackSequenceState;
								++NextPlaybackState.SecondaryIndex;
								Result = InitialPlayPeriod->GetLoopingSegment(FirstSegmentRequest, NextPlaybackState, StartAt, PendingStartRequest->SearchType);
								if (Result.GetType() == IManifest::FResult::EType::Found)
								{
									CurrentPlaybackSequenceState = NextPlaybackState;
								}
								bFirstSegmentRequestIsForLooping = true;
							}
							switch(Result.GetType())
							{
								case IManifest::FResult::EType::TryAgainLater:
								{
									PendingStartRequest->RetryAtTime = Result.GetRetryAgainAtTime();
									break;
								}
								case IManifest::FResult::EType::Found:
								{
									// Now that we have a valid first-ever play start established we clear out the internal
									// default start time so we can seek backwards/forward from it to a different time.
									Manifest->ClearDefaultStartTime();

									// For DASH Live check if it would be better to start on the next segment.
									if (Manifest->GetMediaFormatType() == EMediaFormatType::DASH &&
										Manifest->GetPresentationType() != IManifest::EType::OnDemand)
									{
										struct FSegTime { FTimeValue Start, Into, Duration; };
										FSegTime t0;
										if (FirstSegmentRequest->GetStartupDelay(t0.Start, t0.Into, t0.Duration))
										{
											TSharedPtrTS<IStreamSegment> NextStartSegmentRequest;
											FPlayStartPosition LaterStart = StartAt;
											FTimeValue RemainingSegmentDuration = t0.Duration - t0.Into;
											LaterStart.Time += RemainingSegmentDuration;
											LaterStart.Options.bFrameAccuracy = false;
											IManifest::FResult LaterResult = InitialPlayPeriod->GetStartingSegment(NextStartSegmentRequest, CurrentPlaybackSequenceState, LaterStart, IManifest::ESearchType::Closest);
											if (LaterResult.GetType() == IManifest::FResult::EType::Found && NextStartSegmentRequest.IsValid())
											{
												// Are we skipping more than half the segment's duration?
												if (t0.Into >= t0.Duration/2)
												{
													// Start with the next segment if this segment has less than 1s of usable duration.
													bool bStartWithNextSegment = RemainingSegmentDuration.GetAsSeconds() < 1.0;

													// If using the next segment would put us under the minimum required latency we cannot start with it
													// if we are to play on the Live edge.
													if (ABRShouldPlayOnLiveEdge())
													{
														double NextSegStartLiveLatency = (ABRGetWallclockTime() - NextStartSegmentRequest->GetFirstPTS()).GetAsSeconds();
														double MinLiveLatency = 0.0;
														TSharedPtrTS<const FLowLatencyDescriptor> llDesc = ABRGetLowLatencyDescriptor();
														MinLiveLatency = llDesc.IsValid() ? llDesc->GetLatencyMin().GetAsSeconds(MinLiveLatency) : MinLiveLatency;
														if (NextSegStartLiveLatency < MinLiveLatency)
														{
															bStartWithNextSegment = false;
														}
													}

													if (bStartWithNextSegment)
													{
														PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Using next segment since it has a nearer start time.")));
														FirstSegmentRequest = NextStartSegmentRequest;
													}
												}
											}
										}
									}

									// Add this period to the list of upcoming ones. This ensures the period metadata gets reported
									// if the period playback starts in is not the first for which metadata was initially reported.
									AddUpcomingPeriod(InitialPlayPeriod);
									// Remember the selected buffer source infos.
									UpdatePeriodStreamBufferSourceInfo(InitialPlayPeriod, EStreamType::Video, BufferSourceInfoVid);
									UpdatePeriodStreamBufferSourceInfo(InitialPlayPeriod, EStreamType::Audio, BufferSourceInfoAud);
									UpdatePeriodStreamBufferSourceInfo(InitialPlayPeriod, EStreamType::Subtitle, BufferSourceInfoTxt);

									PendingFirstSegmentRequest = FirstSegmentRequest;
									// No longer need the start request.
									PendingStartRequest.Reset();
									// Also no longer need the initial play period.
									InitialPlayPeriod.Reset();
									break;
								}
								case IManifest::FResult::EType::NotFound:
								case IManifest::FResult::EType::NotLoaded:
								{
									// Reset the current play period and start over with the initial period selection.
									PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Info, FString::Printf(TEXT("Period start segments not found. Reselecting start period.")));
									InitialPlayPeriod.Reset();
									break;
								}
								case IManifest::FResult::EType::BeforeStart:
								case IManifest::FResult::EType::PastEOS:
								{
									PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Info, FString::Printf(TEXT("miss(%d,a=%d,l=%d) for %lld/%d in range %lld/%d - %lld/%d"), (int32)Result.GetType(), StartAt.Options.bFrameAccuracy, PendingStartRequest->StartType == FPendingStartRequest::EStartType::LoopPoint,
											(long long int) StartAt.Time.GetAsHNS(), StartAt.Time.IsValid(),
											(long long int) StartAt.Options.PlaybackRange.Start.GetAsHNS(),
											StartAt.Options.PlaybackRange.Start.IsValid(),
											(long long int) StartAt.Options.PlaybackRange.End.GetAsHNS(),
											StartAt.Options.PlaybackRange.End.IsValid()));
									FTimeRange Seekable = Manifest->GetSeekableTimeRange();
									PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Info, FString::Printf(TEXT("timeline is %lld/%d - %lld/%d"),
											(long long int) Seekable.Start.GetAsHNS(), Seekable.Start.IsValid(),
											(long long int) Seekable.End.GetAsHNS(), Seekable.End.IsValid()));

									FErrorDetail err;
									err.SetFacility(Facility::EFacility::Player);
									err.SetMessage(TEXT("Could not locate the stream segment to begin playback at."));
									err.SetCode(INTERR_COULD_NOT_LOCATE_START_SEGMENT);
									PostError(err);
									PendingStartRequest.Reset();
									InitialPlayPeriod.Reset();
									break;
								}
								case IManifest::FResult::EType::Error:
								default:
								{
									FErrorDetail err;
									err.SetFacility(Facility::EFacility::Player);
									err.SetMessage(FString::Printf(TEXT("Error selecting the initial set of streams (%s)."), *Result.GetErrorDetail().GetMessage()));
									err.SetCode(INTERR_GENERAL_FAILURE);
									PostError(err);
									PendingStartRequest.Reset();
									InitialPlayPeriod.Reset();
									break;
								}
							}
						}
						break;
					}
				}
			}
		}
		else
		{
			// No manifest, no start request. Those are the rules.
			PendingStartRequest.Reset();
		}
	}
}


void FAdaptiveStreamingPlayer::InternalHandlePendingFirstSegmentRequest(const FTimeValue& CurrentTime)
{
	// Is there a play start initial segment download request?
	if (PendingFirstSegmentRequest.IsValid())
	{
		check(StreamReaderHandler);
		if (StreamReaderHandler)
		{
			// Create the new stream data receive buffer
			TSharedPtrTS<FBufferSourceInfo> BufferSourceInfoVid = CurrentPlayPeriodVideo.IsValid() ? CurrentPlayPeriodVideo->GetSelectedStreamBufferSourceInfo(EStreamType::Video) : nullptr;
			TSharedPtrTS<FBufferSourceInfo> BufferSourceInfoAud = CurrentPlayPeriodAudio.IsValid() ? CurrentPlayPeriodAudio->GetSelectedStreamBufferSourceInfo(EStreamType::Audio) : nullptr;
			TSharedPtrTS<FBufferSourceInfo> BufferSourceInfoTxt = CurrentPlayPeriodText.IsValid() ? CurrentPlayPeriodText->GetSelectedStreamBufferSourceInfo(EStreamType::Subtitle) : nullptr;

			TSharedPtrTS<FStreamDataBuffers> NewReceiveBuffers = MakeSharedTS<FStreamDataBuffers>();
			NewReceiveBuffers->VidBuffer = MakeSharedTS<FMultiTrackAccessUnitBuffer>(EStreamType::Video);
			NewReceiveBuffers->AudBuffer = MakeSharedTS<FMultiTrackAccessUnitBuffer>(EStreamType::Audio);
			NewReceiveBuffers->TxtBuffer = MakeSharedTS<FMultiTrackAccessUnitBuffer>(EStreamType::Subtitle);

			// Some formats have multiple tracks in a single stream that provide data all at the same time.
			if (StreamReaderHandler && StreamReaderHandler->LoadsMultiplexedTracksSequentially())
			{
				NewReceiveBuffers->VidBuffer->SetParallelTrackMode();
				NewReceiveBuffers->AudBuffer->SetParallelTrackMode();
				NewReceiveBuffers->TxtBuffer->SetParallelTrackMode();
			}
			NewReceiveBuffers->VidBuffer->SelectTrackWhenAvailable(CurrentPlaybackSequenceID[StreamTypeToArrayIndex(EStreamType::Video)], BufferSourceInfoVid);
			NewReceiveBuffers->AudBuffer->SelectTrackWhenAvailable(CurrentPlaybackSequenceID[StreamTypeToArrayIndex(EStreamType::Audio)], BufferSourceInfoAud);
			NewReceiveBuffers->TxtBuffer->SelectTrackWhenAvailable(CurrentPlaybackSequenceID[StreamTypeToArrayIndex(EStreamType::Subtitle)], BufferSourceInfoTxt);
			// Flag tracks that will not deliver any data accordingly.
			if (!BufferSourceInfoVid.IsValid())
			{
				NewReceiveBuffers->VidBuffer->SetEndOfTrackAll();
			}
			if (!BufferSourceInfoAud.IsValid())
			{
				NewReceiveBuffers->AudBuffer->SetEndOfTrackAll();
			}
			if (!BufferSourceInfoTxt.IsValid())
			{
				NewReceiveBuffers->TxtBuffer->SetEndOfTrackAll();
			}

			// Add the new receive buffer to the list.
			DataBuffersCriticalSection.Lock();
			NextDataBuffers.Emplace(NewReceiveBuffers);
			// And for easier access set it as the current receive buffer to which all new data gets added.
			CurrentDataReceiveBuffers = MoveTemp(NewReceiveBuffers);
			DataBuffersCriticalSection.Unlock();

			if (!bFirstSegmentRequestIsForLooping)
			{
				FTimeValue CurrentPlayPos = PlaybackState.GetPlayPosition();
				FTimeValue NewPlayPos = PendingFirstSegmentRequest->GetFirstPTS();
				// If the PTS is offset by an edit list it could be negative. We do not want to
				// use negative values so we set it to zero instead. This is not a problem.
				if (NewPlayPos < FTimeValue::GetZero())
				{
					NewPlayPos.SetToZero();
				}
				PlaybackState.SetPlayPosition(NewPlayPos);
				if (CurrentState == EPlayerState::eState_Seeking)
				{
					// Get the current loop state to fetch the last published loop count.
					// The internal counter in CurrentLoopState may be some iterations ahead since it could have fetched segments from the loop point.
					FInternalLoopState LoopStateNow;
					PlaybackState.GetLoopState(LoopStateNow);
					CurrentLoopState.Count = LoopStateNow.Count;
				}
				PlaybackState.SetLoopState(CurrentLoopState);

				check(CurrentState == EPlayerState::eState_Buffering || CurrentState == EPlayerState::eState_Rebuffering || CurrentState == EPlayerState::eState_Seeking);
				if (CurrentState == EPlayerState::eState_Seeking)
				{
					DispatchEvent(FMetricEvent::ReportJumpInPlayPosition(NewPlayPos, CurrentPlayPos, Metrics::ETimeJumpReason::UserSeek));
				}

				RenderClock->SetCurrentTime(IMediaRenderClock::ERendererType::Audio, NewPlayPos);
				RenderClock->SetCurrentTime(IMediaRenderClock::ERendererType::Video, NewPlayPos);

				PlaybackState.SetIsBuffering(true);
				PlaybackState.SetIsSeeking(true);

				// Send QoS buffering event
				DispatchBufferingEvent(true, CurrentState);

				// Remember why we are buffering so we can send the proper QoS event when buffering is done.
				LastBufferingState = CurrentState;

				// Whether we were seeking or rebuffering, we're now just buffering.
				CurrentState = EPlayerState::eState_Buffering;
			}
			else
			{
				PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Enqueuing stream loop start request.")));
				// Add the updated loop state to the upcoming loop queue.
				CurrentLoopState.To = PendingFirstSegmentRequest->GetFirstPTS();
				NextLoopStates.Enqueue(CurrentLoopState);
			}

			// Get the stream types that have already reached EOS in this start request because the streams are of different duration.
			// The streams are there in principle so we need to set them up (in case we loop back into them) but we won't see any data arriving.
			TArray<TSharedPtrTS<IStreamSegment>> AlreadyEndedStreams;
			PendingFirstSegmentRequest->GetEndedStreams(AlreadyEndedStreams);
			for(int32 i=0; i<AlreadyEndedStreams.Num(); ++i)
			{
				CompletedSegmentRequests.Add(AlreadyEndedStreams[i]->GetType(), AlreadyEndedStreams[i]);
			}

			// Get the requested streams from the initial request (which may be a wrapper for several individual streams)
			// and add them to the request queue.
			TArray<TSharedPtrTS<IStreamSegment>> RequestedStreams;
			PendingFirstSegmentRequest->GetRequestedStreams(RequestedStreams);
			for(auto &RequestedStream : RequestedStreams)
			{
				ReadyWaitingSegmentRequests->Enqueue(RequestedStream);
			}
		}
		PendingFirstSegmentRequest.Reset();
		bStreamingHasStarted = true;
	}
}


void FAdaptiveStreamingPlayer::InternalHandleCompletedSegmentRequests(const FTimeValue& CurrentTime)
{
	// Check the streams that reached EOS. If all of them are done decide what to do, end playback or loop.
	if (CompletedSegmentRequests.Num())
	{
		bool bAllAtEOS = true;

		auto CheckStreamTypeEOS = [this](bool& bInOutAllEOS, EStreamType InStreamType) -> void
		{
			TSharedPtrTS<FMultiTrackAccessUnitBuffer> RcvBuffer = GetCurrentReceiveStreamBuffer(InStreamType);
			if (IsExpectedToStreamNow(InStreamType))
			{
				if (!CompletedSegmentRequests.Contains(InStreamType))
				{
					bInOutAllEOS = false;
				}
				else if (RcvBuffer.IsValid())
				{
					RcvBuffer->SetEndOfTrackAll();
					RcvBuffer->PushEndOfDataAll();
				}
			}
			else if (RcvBuffer.IsValid())
			{
				RcvBuffer->SetEndOfTrackAll();
				RcvBuffer->PushEndOfDataAll();
			}
		};

		CheckStreamTypeEOS(bAllAtEOS, EStreamType::Video);
		CheckStreamTypeEOS(bAllAtEOS, EStreamType::Audio);
		CheckStreamTypeEOS(bAllAtEOS, EStreamType::Subtitle);

		if (bAllAtEOS)
		{
			// Move the finished requests into a local work map, emptying the original.
			TMultiMap<EStreamType, TSharedPtrTS<IStreamSegment>> LocalFinishedRequests(MoveTemp(CompletedSegmentRequests));

			// Looping enabled?
			if (CurrentLoopParam.bEnableLooping)
			{
				if (Manifest.IsValid())
				{
					// We will be looping now. If the manifest needs internal updating, let it know to do so now.
					Manifest->PrepareForLooping(1);

					FTimeRange Seekable = Manifest->GetSeekableTimeRange();

					PendingStartRequest = MakeSharedTS<FPendingStartRequest>();
					// Loop back to the beginning. If a play range has been set that has a larger in-point
					// the time will be clamped to that. The in-point cannot be before the start point.
					PendingStartRequest->SearchType = IManifest::ESearchType::Closest;
					PendingStartRequest->StartAt.Time = Seekable.Start;
					SetPlaystartOptions(PendingStartRequest->StartAt.Options);
					PendingStartRequest->StartType = FPendingStartRequest::EStartType::LoopPoint;
					// Increase the internal loop count
					++CurrentLoopState.Count;
					CurrentLoopState.To = ClampTimeToCurrentRange(Seekable.Start, true, false);
					CurrentLoopState.From = ClampTimeToCurrentRange(FTimeValue::GetPositiveInfinity(), false, true);
				}
			}
		}
	}
}


void FAdaptiveStreamingPlayer::RequestNewPeriodStreams(EStreamType InType, FPendingSegmentRequest& InOutCurrentRequest)
{
	// Check if the next period has streams the current one does not have.
	// We need to trigger segment requests for those.
	if (!InOutCurrentRequest.bDidRequestNewPeriodStreams)
	{
		InOutCurrentRequest.bDidRequestNewPeriodStreams = true;

		auto RequestIfNeeded = [this, &InOutCurrentRequest](EStreamType InStreamType)
		{
			if (!IsExpectedToStreamNow(InStreamType))
			{
				FPendingSegmentRequest NextReq;
				NextReq.Period = InOutCurrentRequest.Period;
				NextReq.StreamType = InStreamType;
				NextReq.StartoverPosition.Time = InOutCurrentRequest.Period->GetMediaAsset()->GetTimeRange().Start;
				SetPlaystartOptions(NextReq.StartoverPosition.Options);
				NextReq.bStartOver = true;
				NextReq.bDidRequestNewPeriodStreams = true;
				NextPendingSegmentRequests->Enqueue(MoveTemp(NextReq));
			}
		};

		if (InType == EStreamType::Video)
		{
			RequestIfNeeded(EStreamType::Audio);
			RequestIfNeeded(EStreamType::Subtitle);
		}
		else if (InType == EStreamType::Audio)
		{
			RequestIfNeeded(EStreamType::Video);
			RequestIfNeeded(EStreamType::Subtitle);
		}
		else if (InType == EStreamType::Subtitle)
		{
			RequestIfNeeded(EStreamType::Video);
			RequestIfNeeded(EStreamType::Audio);
		}
	}
}


//-----------------------------------------------------------------------------
/**
 * Handles pending media segment requests.
 */
void FAdaptiveStreamingPlayer::HandlePendingMediaSegmentRequests()
{
	if (CurrentState == EPlayerState::eState_Error)
	{
		return;
	}

	FTimeValue UTCNow = GetSynchronizedUTCTime()->GetTime();
	FTimeValue CurrentTime = MEDIAutcTime::Current();
	InternalHandlePendingStartRequest(CurrentTime);
	InternalHandlePendingFirstSegmentRequest(CurrentTime);
	InternalHandleSegmentTrackChanges(CurrentTime);

	FTimeValue CurrentPlaybackPos = GetPlayPosition();

	// Are there completed downloads for which we need to find the next segment?
	TUniqueObj<TQueue<FPendingSegmentRequest>> pendingRequests;
	Swap(pendingRequests, NextPendingSegmentRequests);
	while(!pendingRequests->IsEmpty())
	{
		FPendingSegmentRequest FinishedReq;
		pendingRequests->Dequeue(FinishedReq);

		TSharedPtrTS<IManifest::IPlayPeriod> SegmentPlayPeriod;

		EStreamType StreamType = FinishedReq.StreamType != EStreamType::Unsupported ? FinishedReq.StreamType : FinishedReq.Request.IsValid() ? FinishedReq.Request->GetType() : EStreamType::Unsupported;
		switch(StreamType)
		{
			default:
			{
				break;
			}
			case EStreamType::Video:
			{
				SegmentPlayPeriod = CurrentPlayPeriodVideo;
				break;
			}
			case EStreamType::Audio:
			{
				SegmentPlayPeriod = CurrentPlayPeriodAudio;
				break;
			}
			case EStreamType::Subtitle:
			{
				SegmentPlayPeriod = CurrentPlayPeriodText;
				break;
			}
		}

		// If not yet ready to check again put the request back into the list.
		if (FinishedReq.AtTime.IsValid() && FinishedReq.AtTime > CurrentTime)
		{
			NextPendingSegmentRequests->Enqueue(MoveTemp(FinishedReq));
		}
		else if (SegmentPlayPeriod.IsValid())
		{
			// Is this a startover for a new period?
			if ((FinishedReq.bStartOver || FinishedReq.bPlayPosAutoReselect) && !FinishedReq.Period.IsValid())
			{
				// Locate the period the startover time is in.
				TSharedPtrTS<IManifest::IPlayPeriod> StartoverPeriod;
				IManifest::FResult PeriodResult = Manifest->FindPlayPeriod(StartoverPeriod, FinishedReq.StartoverPosition, IManifest::ESearchType::After);
				// When found or at the end of the stream
				if (PeriodResult.GetType() == IManifest::FResult::EType::Found)
				{
					FinishedReq.Period = MoveTemp(StartoverPeriod);
				}
				else if (PeriodResult.GetType() == IManifest::FResult::EType::PastEOS)
				{
					// At the end of the stream we just don't do anything. Just ignore the switch.
					continue;
				}
				else
				{
					FErrorDetail err;
					err.SetFacility(Facility::EFacility::Player);
					err.SetMessage(TEXT("Could not locate startover period for track switch"));
					err.SetCode(INTERR_COULD_NOT_LOCATE_START_PERIOD);
					PostError(err);
					break;
				}
			}

			// Did we move into a new or a startover period that needs to be made ready?
			if (FinishedReq.Period.IsValid())
			{
				IManifest::IPlayPeriod::EReadyState PeriodState = FinishedReq.Period->GetReadyState();
				if (PeriodState == IManifest::IPlayPeriod::EReadyState::NotLoaded)
				{
					FinishedReq.Period->SetStreamPreferences(EStreamType::Video, StreamSelectionAttributesVid);
					FinishedReq.Period->SetStreamPreferences(EStreamType::Audio, StreamSelectionAttributesAud);
					FinishedReq.Period->SetStreamPreferences(EStreamType::Subtitle, StreamSelectionAttributesTxt);
					FinishedReq.Period->Load();
					NextPendingSegmentRequests->Enqueue(MoveTemp(FinishedReq));
					continue;
				}
				else if (PeriodState == IManifest::IPlayPeriod::EReadyState::Loading)
				{
					NextPendingSegmentRequests->Enqueue(MoveTemp(FinishedReq));
					continue;
				}
				else if (PeriodState == IManifest::IPlayPeriod::EReadyState::Loaded)
				{
					// Set the current average video bitrate in the player options for the period to retrieve if necessary.
					PlayerOptions.Set(OptionKeyCurrentAvgStartingVideoBitrate, FVariantValue(StreamSelector->GetAverageBandwidth()));
					FinishedReq.Period->PrepareForPlay();
					NextPendingSegmentRequests->Enqueue(MoveTemp(FinishedReq));
					continue;
				}
				else if (PeriodState == IManifest::IPlayPeriod::EReadyState::Preparing)
				{
					NextPendingSegmentRequests->Enqueue(MoveTemp(FinishedReq));
					continue;
				}
				else if (PeriodState == IManifest::IPlayPeriod::EReadyState::IsReady)
				{
					RequestNewPeriodStreams(StreamType, FinishedReq);

					TSharedPtrTS<FMultiTrackAccessUnitBuffer> StreamRcvBuffer = GetCurrentReceiveStreamBuffer(StreamType);
					if (StreamType == EStreamType::Video)
					{
						SegmentPlayPeriod = FinishedReq.Period;
						CurrentPlayPeriodVideo = FinishedReq.Period;
						StreamSelector->SetCurrentPlaybackPeriod(EStreamType::Video, CurrentPlayPeriodVideo);
						UpdateStreamResolutionLimit();
					}
					else if (StreamType == EStreamType::Audio)
					{
						SegmentPlayPeriod = FinishedReq.Period;
						CurrentPlayPeriodAudio = FinishedReq.Period;
						StreamSelector->SetCurrentPlaybackPeriod(EStreamType::Audio, CurrentPlayPeriodAudio);
					}
					else if (StreamType == EStreamType::Subtitle)
					{
						SegmentPlayPeriod = FinishedReq.Period;
						CurrentPlayPeriodText = FinishedReq.Period;
						//StreamSelector->SetCurrentPlaybackPeriod(EStreamType::Subtitle, CurrentPlayPeriodText);
					}
					// Add this to the upcoming periods the play position will move into and metadata will need to be updated then
					// unless this is a startover request.
					if (!FinishedReq.bStartOver || FinishedReq.bPlayPosAutoReselect)
					{
						AddUpcomingPeriod(SegmentPlayPeriod);
					}
					UpdatePeriodStreamBufferSourceInfo(SegmentPlayPeriod, StreamType, SegmentPlayPeriod->GetSelectedStreamBufferSourceInfo(StreamType));
				}
			}

			// Evaluate ABR to select the next stream quality.
			FTimeValue ActionDelay(FTimeValue::GetZero());
			IAdaptiveStreamSelector::ESegmentAction Action = StreamSelector->SelectSuitableStreams(ActionDelay, FinishedReq.Request);
			if (Action == IAdaptiveStreamSelector::ESegmentAction::Fail)
			{
				FErrorDetail err;
				err.SetFacility(Facility::EFacility::Player);
				err.SetMessage(TEXT("All streams have failed. There is nothing to play any more."));
				err.SetCode(INTERR_ALL_STREAMS_HAVE_FAILED);
				PostError(err);
				break;
			}

			TSharedPtrTS<IStreamSegment> NextSegment;
			IManifest::FResult Result;
			if (!FinishedReq.bStartOver && !FinishedReq.bPlayPosAutoReselect)
			{
				FPlayStartOptions Options;
				SetPlaystartOptions(Options);
				if (Action == IAdaptiveStreamSelector::ESegmentAction::FetchNext)
				{
					Result = SegmentPlayPeriod->GetNextSegment(NextSegment, FinishedReq.Request, Options);
				}
				else if (Action == IAdaptiveStreamSelector::ESegmentAction::Retry || Action == IAdaptiveStreamSelector::ESegmentAction::Fill)
				{
					Result = SegmentPlayPeriod->GetRetrySegment(NextSegment, FinishedReq.Request, Options, Action == IAdaptiveStreamSelector::ESegmentAction::Fill);
				}
			}
			else
			{
				Result = SegmentPlayPeriod->GetContinuationSegment(NextSegment, StreamType, CurrentPlaybackSequenceState, FinishedReq.StartoverPosition, IManifest::ESearchType::After);
			}
			switch(Result.GetType())
			{
				case IManifest::FResult::EType::TryAgainLater:
				{
					FinishedReq.AtTime = Result.GetRetryAgainAtTime();
					NextPendingSegmentRequests->Enqueue(MoveTemp(FinishedReq));
					break;
				}
				case IManifest::FResult::EType::Found:
				{
					// Switching tracks?
					if (FinishedReq.bStartOver)
					{
						// Tell the multi stream buffer where to switch to and update the selection defaults.
						TSharedPtrTS<FBufferSourceInfo> BufferSourceInfo = SegmentPlayPeriod->GetSelectedStreamBufferSourceInfo(StreamType);
						if (BufferSourceInfo.IsValid())
						{
							TSharedPtrTS<FMultiTrackAccessUnitBuffer> StreamRcvBuffer = GetCurrentReceiveStreamBuffer(StreamType);
							if (StreamRcvBuffer.IsValid())
							{
								StreamRcvBuffer->SelectTrackWhenAvailable(CurrentPlaybackSequenceID[StreamTypeToArrayIndex(StreamType)], BufferSourceInfo);
							}
							switch(StreamType)
							{
								case EStreamType::Video:
								{
									SelectedStreamAttributesVid.UpdateWith(BufferSourceInfo->Kind, BufferSourceInfo->LanguageTag, BufferSourceInfo->Codec, BufferSourceInfo->HardIndex);
									if (Manifest->OrderOfTracksCanChangeDynamically())
									{
										StreamSelectionAttributesVid.ClearOverrideIndex();
									}
									break;
								}
								case EStreamType::Audio:
								{
									SelectedStreamAttributesAud.UpdateWith(BufferSourceInfo->Kind, BufferSourceInfo->LanguageTag, BufferSourceInfo->Codec, BufferSourceInfo->HardIndex);
									if (Manifest->OrderOfTracksCanChangeDynamically())
									{
										StreamSelectionAttributesAud.ClearOverrideIndex();
									}
									break;
								}
								case EStreamType::Subtitle:
								{
									SelectedStreamAttributesTxt.UpdateWith(BufferSourceInfo->Kind, BufferSourceInfo->LanguageTag, BufferSourceInfo->Codec, BufferSourceInfo->HardIndex);
									if (Manifest->OrderOfTracksCanChangeDynamically())
									{
										StreamSelectionAttributesTxt.ClearOverrideIndex();
									}
									break;
								}
								default:
								{
									break;
								}
							}
						}
						UpdatePeriodStreamBufferSourceInfo(SegmentPlayPeriod, StreamType, BufferSourceInfo);
					}

					NextSegment->SetExecutionDelay(UTCNow, ActionDelay);
					ReadyWaitingSegmentRequests->Enqueue(NextSegment);
					break;
				}
				case IManifest::FResult::EType::PastEOS:
				{
					TSharedPtrTS<IManifest::IPlayPeriod> NextPeriod;
					IManifest::FResult NextPeriodResult = Manifest->FindNextPlayPeriod(NextPeriod, FinishedReq.Request);
					switch(NextPeriodResult.GetType())
					{
						case IManifest::FResult::EType::TryAgainLater:
						{
							// Next period is either not there yet or is being resolved right now.
							FinishedReq.AtTime = Result.GetRetryAgainAtTime();
							NextPendingSegmentRequests->Enqueue(MoveTemp(FinishedReq));
							break;
						}
						case IManifest::FResult::EType::Found:
						{
							// Get the next period ready for playback. Put it back into the queue.
							FinishedReq.Period = NextPeriod;
							NextPendingSegmentRequests->Enqueue(MoveTemp(FinishedReq));
							break;
						}
						case IManifest::FResult::EType::PastEOS:
						case IManifest::FResult::EType::NotFound:
						case IManifest::FResult::EType::BeforeStart:
						case IManifest::FResult::EType::NotLoaded:
						case IManifest::FResult::EType::Error:
						{
							// Get the dependent stream types, if any.
							// This is mainly for multiplexed streams like an .mp4 at this point.
							// All stream types have reached EOS now.
							// NOTE: It is possible that there is no actual request here. This happens when a stream
							// gets enabled (like a subtitle stream) but is immediately at EOS and thus has no actual segment request!
							if (FinishedReq.Request.IsValid())
							{
								TArray<TSharedPtrTS<IStreamSegment>> DependentStreams;
								FinishedReq.Request->GetDependentStreams(DependentStreams);
								for(int32 i=0; i<DependentStreams.Num(); ++i)
								{
									CompletedSegmentRequests.Add(DependentStreams[i]->GetType(), FinishedReq.Request);
								}
							}
							// Add the primary stream type to the list as well.
							CompletedSegmentRequests.Add(StreamType, FinishedReq.Request);
							break;
						}
					}
					break;
				}
				case IManifest::FResult::EType::NotFound:
				{
					// Not found here should indicate that the type of stream is not available (or not selected) in the new period.
					TSharedPtrTS<ITimelineMediaAsset> ThisPeriod = SegmentPlayPeriod->GetMediaAsset();
					if (ThisPeriod.IsValid())
					{
						UpdatePeriodStreamBufferSourceInfo(SegmentPlayPeriod, StreamType, TSharedPtrTS<FBufferSourceInfo>());
						TSharedPtrTS<FMultiTrackAccessUnitBuffer> RcvBuffer = GetCurrentReceiveStreamBuffer(StreamType);
						if (RcvBuffer.IsValid())
						{
							RcvBuffer->SetEndOfTrackAll();
						}
					}
					break;
				}
				case IManifest::FResult::EType::BeforeStart:
				case IManifest::FResult::EType::NotLoaded:
				case IManifest::FResult::EType::Error:
				default:
				{
					// Throw a playback error for now.
					FErrorDetail err;
					err.SetFacility(Facility::EFacility::Player);
					err.SetMessage(FString::Printf(TEXT("Could not locate next segment (%s)"), *Result.GetErrorDetail().GetMessage()));
					err.SetCode(INTERR_FRAGMENT_NOT_AVAILABLE);
					PostError(err);
					break;
				}
			}
		}
	}


	// Go through the queue of ready-to-fetch segments to see if they should be fetched now.
	TUniqueObj<TQueue<TSharedPtrTS<IStreamSegment>>> readyRequests;
	Swap(readyRequests, ReadyWaitingSegmentRequests);
	while(!readyRequests->IsEmpty())
	{
		TSharedPtrTS<IStreamSegment> ReadyReq;
		readyRequests->Dequeue(ReadyReq);

		bool bExecuteNow = true;
		bool bSkipOver = false;

		// Does this request have an availability window?
		FTimeValue When = ReadyReq->GetExecuteAtUTCTime();
		if (When.IsValid() && When > UTCNow)
		{
			bExecuteNow = false;
		}

		// Subtitles need to be throttled based on play position.
		// Due them being sparse and small they would essentially all get fetched right away.
		if (ReadyReq->GetType() == EStreamType::Subtitle)
		{
			// Fetch up to 20 seconds ahead of time, but only for matching sequence indices to avoid
			// fetching segments when looping.
			FTimeValue NextPTS = ReadyReq->GetFirstPTS();
			if (CurrentPlaybackPos.IsValid() && NextPTS.IsValid())
			{
				const FTimeValue AdvanceFetchThreshold(FTimeValue::MillisecondsToHNS(1000 * 20));
				if (NextPTS.GetSequenceIndex() != CurrentPlaybackPos.GetSequenceIndex() || (CurrentPlaybackPos + AdvanceFetchThreshold) < NextPTS)
				{
					bExecuteNow = false;
				}
			}
			// When subtitles are disabled we do not actually fetch the segment.
			bSkipOver = SelectedStreamAttributesTxt.IsDeselected();
		}

		if (bExecuteNow)
		{
			if (!bSkipOver)
			{
				IStreamReader::EAddResult ReqResult = StreamReaderHandler->AddRequest(CurrentPlaybackSequenceID[StreamTypeToArrayIndex(ReadyReq->GetType())], ReadyReq);
				if (ReqResult != IStreamReader::EAddResult::Added)
				{
					FErrorDetail err;
					err.SetFacility(Facility::EFacility::Player);
					err.SetMessage(TEXT("Failed to add stream segment request to reader"));
					err.SetCode(INTERR_FRAGMENT_READER_REQUEST);
					PostError(err);
					break;
				}
			}
			else
			{
				// Skip over this segment. Add to the list of completed requests for which we need to find the next segment to fetch.
				FPendingSegmentRequest NextReq;
				NextReq.Request = ReadyReq;
				NextPendingSegmentRequests->Enqueue(MoveTemp(NextReq));
			}
		}
		else
		{
			ReadyWaitingSegmentRequests->Enqueue(ReadyReq);
		}
	}

	InternalHandleCompletedSegmentRequests(CurrentTime);
}

void FAdaptiveStreamingPlayer::CancelPendingMediaSegmentRequests(EStreamType StreamType)
{
	if (StreamReaderHandler)
	{
		StreamReaderHandler->CancelRequest(StreamType, true);
	}
	// Go through the queue of ready-to-fetch segments and remove those that are to be canceled.
	TUniqueObj<TQueue<TSharedPtrTS<IStreamSegment>>> readyRequests;
	Swap(readyRequests, ReadyWaitingSegmentRequests);
	while(!readyRequests->IsEmpty())
	{
		TSharedPtrTS<IStreamSegment> ReadyReq;
		readyRequests->Dequeue(ReadyReq);
		if (ReadyReq->GetType() != StreamType)
		{
			ReadyWaitingSegmentRequests->Enqueue(ReadyReq);
		}
	}
	// Likewise for pending segments that hadn't been made ready yet.
	TUniqueObj<TQueue<FPendingSegmentRequest>> pendingRequests;
	Swap(pendingRequests, NextPendingSegmentRequests);
	while(!pendingRequests->IsEmpty())
	{
		FPendingSegmentRequest PendingReq;
		pendingRequests->Dequeue(PendingReq);
		if (PendingReq.StreamType != StreamType)
		{
			NextPendingSegmentRequests->Enqueue(PendingReq);
		}
	}
}


void FAdaptiveStreamingPlayer::InternalDeselectStream(EStreamType StreamType)
{
	switch(StreamType)
	{
		case EStreamType::Video:
		{
			bIsVideoDeselected = true;
			SelectedStreamAttributesVid.Deselect();
			VideoDecoder.Flush();
			VideoOutput.Flush(false);
			break;
		}
		case EStreamType::Audio:
		{
			bIsAudioDeselected = true;
			SelectedStreamAttributesAud.Deselect();
			AudioDecoder.Flush();
			AudioOutput.Flush();
			break;
		}
		case EStreamType::Subtitle:
		{
			bIsTextDeselected = true;
			SelectedStreamAttributesTxt.Deselect();
			SubtitleDecoder.Flush();
			break;
		}
		default:
		{
			break;
		}
	}
}

void FAdaptiveStreamingPlayer::InternalStartoverAtCurrentPosition()
{
	{
		// Check if a seek is currently pending. If so then let us perform the seek instead.
		FScopeLock lock(&SeekVars.Lock);
		// Is there a pending request?
		if (SeekVars.PendingRequest.IsSet())
		{
			// To ensure the seek will execute we clear out the active requests.
			SeekVars.ActiveRequest.Reset();
			return;
		}
	}
	FSeekParam NewPosition;
	NewPosition.Time = PlaybackState.GetPlayPosition();
	InternalStop(PlayerConfig.bHoldLastFrameDuringSeek);
	CurrentState = EPlayerState::eState_Seeking;
	// Get the current loop state back from the playback state to keep the loop counter as it is now.
	PlaybackState.GetLoopState(CurrentLoopState);
	InternalStartAt(NewPosition, nullptr);
}

void FAdaptiveStreamingPlayer::InternalStartoverAtLiveEdge()
{
	FSeekParam NewPosition;
	InternalStop(PlayerConfig.bHoldLastFrameDuringSeek);
	CurrentState = EPlayerState::eState_Seeking;
	PlaybackState.GetLoopState(CurrentLoopState);
	InternalStartAt(NewPosition, nullptr);
}


void FAdaptiveStreamingPlayer::InternalHandleSegmentTrackChanges(const FTimeValue& CurrentTime)
{
	bool bChangeMade = false;
	bool bForceStartover = false;

	if (Manifest.IsValid() && Manifest->SupportsMultipleTracksPerKind() && StreamReaderHandler)
	{
		if (StreamReaderHandler->LoadsMultiplexedTracksSequentially())
		{
			auto HandleMultiplexedTrackSelection = [&bChangeMade, this](FStreamSelectionAttributes& OutSelectionAttributes, FInternalStreamSelectionAttributes& InOutSelectedAttributes, EStreamType InType,
																	TSharedPtrTS<FStreamSelectionAttributes>& InPendingSelection, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPeriod) -> void
			{
				if (InPendingSelection.IsValid() && InCurrentPeriod.IsValid())
				{
					IManifest::IPlayPeriod::ETrackChangeResult Result = InCurrentPeriod->ChangeTrackStreamPreference(InType, *InPendingSelection);
					TSharedPtrTS<FBufferSourceInfo> BufferSourceInfo = InCurrentPeriod->GetSelectedStreamBufferSourceInfo(InType);
					if (Result == IManifest::IPlayPeriod::ETrackChangeResult::Changed)
					{
						TSharedPtrTS<FMultiTrackAccessUnitBuffer> StreamRcvBuf = GetCurrentReceiveStreamBuffer(InType);
						if (StreamRcvBuf.IsValid())
						{
							StreamRcvBuf->SelectTrackWhenAvailable(CurrentPlaybackSequenceID[StreamTypeToArrayIndex(InType)], BufferSourceInfo);
						}
						UpdatePeriodStreamBufferSourceInfo(InCurrentPeriod, InType, BufferSourceInfo);
						OutSelectionAttributes = *InPendingSelection;
						bChangeMade = true;
					}
					InOutSelectedAttributes.UpdateWith(BufferSourceInfo->Kind, BufferSourceInfo->LanguageTag, BufferSourceInfo->Codec, BufferSourceInfo->HardIndex);
					InOutSelectedAttributes.Select();
					InPendingSelection.Reset();
				}
			};

			HandleMultiplexedTrackSelection(StreamSelectionAttributesVid, SelectedStreamAttributesVid, EStreamType::Video, PendingTrackSelectionVid, CurrentPlayPeriodVideo);
			HandleMultiplexedTrackSelection(StreamSelectionAttributesAud, SelectedStreamAttributesAud, EStreamType::Audio, PendingTrackSelectionAud, CurrentPlayPeriodAudio);
			HandleMultiplexedTrackSelection(StreamSelectionAttributesTxt, SelectedStreamAttributesTxt, EStreamType::Subtitle, PendingTrackSelectionTxt, CurrentPlayPeriodText);
		}
		else
		{
			if (PendingStartRequest.IsValid() || PendingFirstSegmentRequest.IsValid())
			{
				return;
			}

			auto HandleElementaryStreamTrackSelection = [&bChangeMade, &bForceStartover, this](FStreamSelectionAttributes& OutSelectionAttributes, FInternalStreamSelectionAttributes& InOutSelectedAttributes, EStreamType InType,
																 TSharedPtrTS<FStreamSelectionAttributes>& InPendingSelection, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPeriod) -> void
			{
				if (InPendingSelection.IsValid() && InCurrentPeriod.IsValid())
				{
					if (InOutSelectedAttributes.IsDeselected() || !InOutSelectedAttributes.IsCompatibleWith(*InPendingSelection))
					{
						IManifest::IPlayPeriod::ETrackChangeResult TrackChangeResult = InCurrentPeriod->ChangeTrackStreamPreference(InType, *InPendingSelection);
						if (TrackChangeResult == IManifest::IPlayPeriod::ETrackChangeResult::NewPeriodNeeded)
						{
							// Make sure any finishing current requests will be ignored by increasing the sequence ID.
							++CurrentPlaybackSequenceID[StreamTypeToArrayIndex(InType)];
							// Cancel any ongoing segment download now.
							CancelPendingMediaSegmentRequests(InType);

							OutSelectionAttributes = *InPendingSelection;
							InOutSelectedAttributes.Select();

							FPendingSegmentRequest NextReq;
							NextReq.bStartOver = true;
							NextReq.StartoverPosition.Time = PlaybackState.GetPlayPosition();
							SetPlaystartOptions(NextReq.StartoverPosition.Options);
							NextReq.StreamType = InType;
							NextPendingSegmentRequests->Enqueue(MoveTemp(NextReq));

							bChangeMade = true;
						}
						else if (TrackChangeResult == IManifest::IPlayPeriod::ETrackChangeResult::StartOver)
						{
							OutSelectionAttributes = *InPendingSelection;
							InOutSelectedAttributes.Select();
							bChangeMade = true;
							bForceStartover = true;
						}
					}
					InOutSelectedAttributes.Select();
					InPendingSelection.Reset();
				}
			};

			HandleElementaryStreamTrackSelection(StreamSelectionAttributesVid, SelectedStreamAttributesVid, EStreamType::Video, PendingTrackSelectionVid, CurrentPlayPeriodVideo);
			HandleElementaryStreamTrackSelection(StreamSelectionAttributesAud, SelectedStreamAttributesAud, EStreamType::Audio, PendingTrackSelectionAud, CurrentPlayPeriodAudio);
			HandleElementaryStreamTrackSelection(StreamSelectionAttributesTxt, SelectedStreamAttributesTxt, EStreamType::Subtitle, PendingTrackSelectionTxt, CurrentPlayPeriodText);
		}
	}
	else
	{
		PendingTrackSelectionVid.Reset();
		PendingTrackSelectionAud.Reset();
		PendingTrackSelectionTxt.Reset();
	}

	// Check if a change was made when there are already future buffers enqueued in which case a switch is
	// no longer possible as we cannot switch on the current buffer.
	if (bChangeMade)
	{
		DataBuffersCriticalSection.Lock();
		bool bNeedStartover = bForceStartover || (ActiveDataOutputBuffers.IsValid() && NextDataBuffers.Num() > 0) || NextDataBuffers.Num() > 1;
		DataBuffersCriticalSection.Unlock();
		if (bNeedStartover)
		{
			InternalStartoverAtCurrentPosition();
		}
	}
}


//-----------------------------------------------------------------------------
/**
 * Adds the given period to the list of periods playback will eventually move into
 * and a metadata change event may need to be generated.
 */
void FAdaptiveStreamingPlayer::AddUpcomingPeriod(TSharedPtrTS<IManifest::IPlayPeriod> InUpcomingPeriod)
{
	/*
		We maintain two lists of periods.
		 - ActivePeriods
		     holds all the periods for which segments are to be requested. As the play position moves forward
			 periods in this get removed when they have fallen out of range with a threshold.
			 This list is iterated when searching for codec information and selected buffer source information.
		 - UpcomingPeriods
		     this is used solely to report changes in metadata to the user as playback progresses.
			 entries here are removed immediately without threshold, so this list tends to be shorter than
			 ActivePeriods.
	*/

	TSharedPtrTS<ITimelineMediaAsset> Period = InUpcomingPeriod->GetMediaAsset();
	if (Period.IsValid())
	{
		FString PeriodID = Period->GetUniqueIdentifier();
		int64 LoopCountNow = CurrentLoopState.Count;

		// Check if this period is in the active and/or upcoming list already and update the
		// loop counter in the list if it is so it does not get removed when the play position
		// of an earlier loop iteration passes it.
		bool bHaveAsActive = false;
		ActivePeriodCriticalSection.Lock();
		for(auto& P : ActivePeriods)
		{
			if (P.ID.Equals(PeriodID))
			{
				bHaveAsActive = true;
				P.LoopCount = LoopCountNow;
			}
		}
		ActivePeriodCriticalSection.Unlock();

		bool bHaveAsUpcoming = false;
		for(auto& P : UpcomingPeriods)
		{
			if (P.ID.Equals(PeriodID))
			{
				bHaveAsUpcoming = true;
				P.LoopCount = LoopCountNow;
			}
		}

		FPeriodInformation Next;
		Next.ID = MoveTemp(PeriodID);
		Next.TimeRange = Period->GetTimeRange();
		Next.Period = MoveTemp(Period);
		Next.LoopCount = LoopCountNow;

		if (!bHaveAsActive)
		{
			// Tell the AEMS handler that a new period will be coming up. It needs this information to cut overlapping events.
			AEMSEventHandler->NewUpcomingPeriod(Next.ID, Next.TimeRange);

			ActivePeriodCriticalSection.Lock();
			ActivePeriods.Emplace(Next);
			ActivePeriods.Sort([](const FPeriodInformation& e1, const FPeriodInformation& e2)
			{
				return e1.TimeRange.Start < e2.TimeRange.Start;
			});
			ActivePeriodCriticalSection.Unlock();
		}

		if (!bHaveAsUpcoming)
		{
			UpcomingPeriods.Emplace(Next);
			UpcomingPeriods.Sort([](const FPeriodInformation& e1, const FPeriodInformation& e2)
			{
				return e1.TimeRange.Start < e2.TimeRange.Start;
			});
		}
	}
}


//-----------------------------------------------------------------------------
/**
 * Updates the active or upcoming period's selected buffer source information
 * for the given stream type.
 */
void FAdaptiveStreamingPlayer::UpdatePeriodStreamBufferSourceInfo(TSharedPtrTS<IManifest::IPlayPeriod> InForPeriod, EStreamType InStreamType, TSharedPtrTS<FBufferSourceInfo> InBufferSourceInfo)
{
	TSharedPtrTS<ITimelineMediaAsset> Period = InForPeriod->GetMediaAsset();
	if (Period.IsValid())
	{
		FString PeriodID = Period->GetUniqueIdentifier();

		auto UpdatePeriods = [&](TArray<FPeriodInformation>& Periods) -> void
		{
			for(auto &P : Periods)
			{
				if (P.ID.Equals(PeriodID))
				{
					switch(InStreamType)
					{
						case EStreamType::Video:
						{
							P.BufferSourceInfoVid = InBufferSourceInfo;
							break;
						}
						case EStreamType::Audio:
						{
							P.BufferSourceInfoAud = InBufferSourceInfo;
							break;
						}
						case EStreamType::Subtitle:
						{
							P.BufferSourceInfoTxt = InBufferSourceInfo;
							break;
						}
						default:
						{
							break;
						}
					}
				}
			}
		};

		// Check if we have to update upcoming periods
		UpdatePeriods(UpcomingPeriods);

		// Update the active periods if necessary.
		ActivePeriodCriticalSection.Lock();
		UpdatePeriods(ActivePeriods);
		ActivePeriodCriticalSection.Unlock();
	}
}



TSharedPtrTS<FBufferSourceInfo> FAdaptiveStreamingPlayer::GetStreamBufferInfoAtTime(bool& bOutHavePeriods, bool& bOutFoundTime, EStreamType InStreamType, const FTimeValue& InAtTime) const
{
	TSharedPtrTS<FBufferSourceInfo> BufferSourceInfo;
	// Locate the period for the specified time.
	ActivePeriodCriticalSection.Lock();
	bOutHavePeriods = ActivePeriods.Num() != 0;
	bOutFoundTime = false;
	for(int32 i=0; i<ActivePeriods.Num(); ++i)
	{
		if (InAtTime >= ActivePeriods[i].TimeRange.Start && InAtTime < (ActivePeriods[i].TimeRange.End.IsValid() ? ActivePeriods[i].TimeRange.End : FTimeValue::GetPositiveInfinity()))
		{
			bOutFoundTime = true;
			switch(InStreamType)
			{
				case EStreamType::Video:
				{
					BufferSourceInfo = ActivePeriods[i].BufferSourceInfoVid;
					break;
				}
				case EStreamType::Audio:
				{
					BufferSourceInfo = ActivePeriods[i].BufferSourceInfoAud;
					break;
				}
				case EStreamType::Subtitle:
				{
					BufferSourceInfo = ActivePeriods[i].BufferSourceInfoTxt;
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
	ActivePeriodCriticalSection.Unlock();
	return BufferSourceInfo;
}


void FAdaptiveStreamingPlayer::SetPlaystartOptions(FPlayStartOptions& OutOptions)
{
	FTimeRange ActiveRange = PlaybackState.GetActivePlayRange();

	OutOptions.PlaybackRange.Start = ActiveRange.Start.IsValid() ? ActiveRange.Start : FTimeValue::GetZero();
	OutOptions.PlaybackRange.End = ActiveRange.End.IsValid() ? ActiveRange.End : FTimeValue::GetPositiveInfinity();

	OutOptions.bFrameAccuracy = PlayerOptions.GetValue(OptionKeyFrameAccurateSeek).SafeGetBool(false);
}

void FAdaptiveStreamingPlayer::ClampStartRequestTime(FTimeValue& InOutTimeToClamp)
{
	if (PendingStartRequest.IsValid())
	{
		// Clamp to within any set playback range
		FTimeValue RangeStart = PendingStartRequest->StartAt.Options.PlaybackRange.Start;
		FTimeValue RangeEnd = PendingStartRequest->StartAt.Options.PlaybackRange.End;
		if (InOutTimeToClamp < RangeStart)
		{
			InOutTimeToClamp = RangeStart;
		}
		if (InOutTimeToClamp > RangeEnd)
		{
			InOutTimeToClamp = RangeEnd;
			// Going to the end will of course result in an 'end reached' and end of playback.
			// If the player is set to loop then we should start from the beginning.
			if (CurrentLoopParam.bEnableLooping)
			{
				InOutTimeToClamp = RangeStart;
			}
		}
	}
}

FTimeValue FAdaptiveStreamingPlayer::ClampTimeToCurrentRange(const FTimeValue& InTime, bool bClampToStart, bool bClampToEnd)
{
	FTimeValue Out(InTime);

	FTimeRange TimelineRange, ActiveRange = PlaybackState.GetActivePlayRange();
	PlaybackState.GetTimelineRange(TimelineRange);
	if (bClampToEnd)
	{
		if (ActiveRange.End.IsValid() && ActiveRange.End < TimelineRange.End)
		{
			TimelineRange.End = ActiveRange.End;
		}
		if (Out > TimelineRange.End)
		{
			Out = TimelineRange.End;
		}
	}
	if (bClampToStart)
	{
		if (ActiveRange.Start.IsValid() && ActiveRange.Start > TimelineRange.Start)
		{
			TimelineRange.Start = ActiveRange.Start;
		}
		if (Out < TimelineRange.Start)
		{
			Out = TimelineRange.Start;
		}
	}
	return Out;
}



//-----------------------------------------------------------------------------
/**
 * Checks the error collectors for any errors thrown by the decoders.
 */
void FAdaptiveStreamingPlayer::CheckForErrors()
{
	while(!ErrorQueue.IsEmpty())
	{
		TSharedPtrTS<FErrorDetail> Error = ErrorQueue.Pop();
		// Do this only once in case there will be several decoder errors thrown.
		if (CurrentState != EPlayerState::eState_Error)
		{
			// Pause before setting up error.
			InternalPause();

			CurrentState = EPlayerState::eState_Error;
			// Only keep the first error, not any errors after that which may just be the avalanche and not the cause.
			if (!LastErrorDetail.IsSet())
			{
				DiagnosticsCriticalSection.Lock();
				LastErrorDetail = *Error;
				DiagnosticsCriticalSection.Unlock();
				DispatchEvent(FMetricEvent::ReportError(LastErrorDetail));
			}
			// In error state we do not need any periodic manifest refetches any more.
			InternalCloseManifestReader();
		}
	}
}



//-----------------------------------------------------------------------------
/**
 * Updates the data availability state if it has changed and dispatches the
 * corresponding metrics event.
 *
 * @param DataAvailabilityState
 * @param NewAvailability
 */
void FAdaptiveStreamingPlayer::UpdateDataAvailabilityState(Metrics::FDataAvailabilityChange& DataAvailabilityState, Metrics::FDataAvailabilityChange::EAvailability NewAvailability)
{
	if (DataAvailabilityState.Availability != NewAvailability)
	{
		DataAvailabilityState.Availability = NewAvailability;
		DispatchEvent(FMetricEvent::ReportDataAvailabilityChange(DataAvailabilityState));
	}
}


//-----------------------------------------------------------------------------
/**
 * Updates the ABR with maximum stream resolution limits.
 */
void FAdaptiveStreamingPlayer::UpdateStreamResolutionLimit()
{
	StreamSelector->SetMaxVideoResolution(VideoResolutionLimitWidth, VideoResolutionLimitHeight);
}


//-----------------------------------------------------------------------------
/**
 * Checks if the stream readers, decoders and renderers are finished.
 */
void FAdaptiveStreamingPlayer::CheckForStreamEnd()
{
	if (CurrentState == EPlayerState::eState_Playing || CurrentState == EPlayerState::eState_Paused)
	{
		if (StreamState == EStreamState::eStream_Running)
		{
			// First check if there is an end time set at which we need to stop.
			// Do this only when playing, not in paused state.
			// We run through this code here when paused to ensure that if looping is
			// enabled we switch over to the new buffer set to keep the decoders busy.
			// This is intended for scrubbing across the loop point.
			if (CurrentState == EPlayerState::eState_Playing)
			{
				FTimeValue EndAtTime = PlaybackState.GetPlaybackEndAtTime();
				FTimeValue InitialEndTime = Manifest.IsValid() ? Manifest->GetDefaultEndTime() : FTimeValue();
				if (EndAtTime.IsValid() && InitialEndTime.IsValid())
				{
					EndAtTime = EndAtTime < InitialEndTime ? EndAtTime : InitialEndTime;
				}
				else if (InitialEndTime.IsValid())
				{
					EndAtTime = InitialEndTime;
				}
				if (EndAtTime.IsValid())
				{
					if (PlaybackState.GetPlayPosition() >= EndAtTime)
					{
						// A forced end time is intended to stop playback. We do NOT look at the loop state here.
						InternalSetPlaybackEnded();
						DataBuffersCriticalSection.Lock();
						NextDataBuffers.Empty();
						DataBuffersCriticalSection.Unlock();
						return;
					}
				}
			}

			DiagnosticsCriticalSection.Lock();
			FBufferStats vidStats = VideoBufferStats;
			FBufferStats audStats = AudioBufferStats;
			DiagnosticsCriticalSection.Unlock();

			// As long as no buffer is at EOD we don't need to check further
			if (!vidStats.StreamBuffer.bEndOfData && !audStats.StreamBuffer.bEndOfData)
			{
				return;
			}

			// Look at the last period's buffer source infos to see what streams we are expecting to be currently active.
			bool bHaveVid = false;
			bool bHaveAud = false;
			bool bHavetxt = false;
			ActivePeriodCriticalSection.Lock();
			if (ActivePeriods.Num())
			{
				bHaveVid = ActivePeriods.Last().BufferSourceInfoVid.IsValid();
				bHaveAud = ActivePeriods.Last().BufferSourceInfoAud.IsValid();
				bHavetxt = ActivePeriods.Last().BufferSourceInfoTxt.IsValid();
			}
			ActivePeriodCriticalSection.Unlock();

			// For simplicity we assume all streams are done and check for those that are active at the moment.
			bool bEndVid = true;
			bool bEndAud = true;
			bool bEndTxt = true;
			int64 VidStalled = 0;
			int64 AudStalled = 0;

			// Check for end of video stream
			if (bHaveVid)
			{
				// Source buffer and decoder input buffer at end of data?
				// We do not check the decoder output buffer since it is possible that no decodable output was produced
				// which tends to happen at the end of the stream or when there is no video any more when audio is longer.
				bEndVid = (vidStats.StreamBuffer.bEndOfData && vidStats.DecoderInputBuffer.bEODSignaled && vidStats.DecoderInputBuffer.bEODReached);
				VidStalled = vidStats.StreamBuffer.bEndOfData ? vidStats.GetStalledDurationMillisec() : 0;
#if NOTIFY_DATA_AVAILABILITY_AT_START_AND_END
				if (bEndVid)
				{
					UpdateDataAvailabilityState(DataAvailabilityStateVid, Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable);
				}
#endif
			}

			// Check for end of audio stream
			if (bHaveAud)
			{
				// Source buffer and decoder input buffer at end of data?
				// We do not check the decoder output buffer since it is possible that no decodable output was produced
				// which tends to happen at the end of the stream or when there is no audio any more when video is longer.
				bEndAud = (audStats.StreamBuffer.bEndOfData && audStats.DecoderInputBuffer.bEODSignaled && audStats.DecoderInputBuffer.bEODReached);
				AudStalled = audStats.StreamBuffer.bEndOfData ? audStats.GetStalledDurationMillisec() : 0;
#if NOTIFY_DATA_AVAILABILITY_AT_START_AND_END
				if (bEndAud)
				{
					UpdateDataAvailabilityState(DataAvailabilityStateAud, Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable);
				}
#endif
			}

			// Text stream
			// ...


			// If either primary stream has reliably ended do a check if the other may have stalled because the application is no longer
			// consuming decoder output, which will prevent the other stream from ending.
			if ((bHaveVid && bHaveAud) && (bEndVid || bEndAud))
			{
				int64 OtherStallTime = bEndAud ? VidStalled : AudStalled;
				if (OtherStallTime > 500)
				{
					bEndVid = bEndAud = true;
				}
			}

			// Everything at EOD?
			if (bEndVid && bEndAud && bEndTxt)
			{
				// When not looping we are done.
				if (!CurrentLoopParam.bEnableLooping)
				{
					InternalSetPlaybackEnded();
					DataBuffersCriticalSection.Lock();
					NextDataBuffers.Empty();
					DataBuffersCriticalSection.Unlock();
				}
				else
				{
					// With loop data the decoders will receive more data. Clear their end of data states.
					ClearAllDecoderEODs();
					// There needs to be another data buffer with the data at the loop point.
					DataBuffersCriticalSection.Lock();
					if (NextDataBuffers.Num())
					{
						TSharedPtrTS<FStreamDataBuffers> NewOutputBuffers;
						NewOutputBuffers = NextDataBuffers[0];
						NextDataBuffers.RemoveAt(0);
						DataBuffersCriticalSection.Unlock();

						// Check if the video stream is seamlessly decodable (next AU is a keyframe and it will also be rendered and not cut off)
						// Note: at this point we assume audio is always decodable on every AU.
						bool bSeamlessSwitchPossible = bHaveVid ? IsSeamlessBufferSwitchPossible(EStreamType::Video, NewOutputBuffers) : true;
						if (!bSeamlessSwitchPossible)
						{
							StopRendering();
							PipelineState = EPipelineState::ePipeline_Prerolling;
							PrerollVars.Clear();
							PrerollVars.bIsMidSequencePreroll = true;
							PrerollVars.StartTime = MEDIAutcTime::CurrentMSec();
							PostrollVars.Clear();
						}

						DataBuffersCriticalSection.Lock();
						ActiveDataOutputBuffers = MoveTemp(NewOutputBuffers);
						DataBuffersCriticalSection.Unlock();

						// Update the current loop state.
						FInternalLoopState newLoopState;
						if (NextLoopStates.Peek(newLoopState))
						{
							NextLoopStates.Pop();
							PlaybackState.SetLoopState(newLoopState);
							DispatchEvent(FMetricEvent::ReportJumpInPlayPosition(newLoopState.To, newLoopState.From, Metrics::ETimeJumpReason::Looping));
						}
						// Update the metadata state again.
						MetadataHandlingState.Reset();
					}
					else
					{
						DataBuffersCriticalSection.Unlock();
					}
					// Update the buffer statistics immediately to reflect the end of data states.
					UpdateDiagnostics();
				}
			}
		}
		else if (StreamState == EStreamState::eStream_ReachedEnd)
		{
			// ... anything special we could still do here?
		}
	}
}


//-----------------------------------------------------------------------------
/**
 * Finds fragments for a given time, creates the stream readers
 * and issues the first fragment request.
 *
 * @param NewPosition
 * @param InTimeRange
 */
void FAdaptiveStreamingPlayer::InternalStartAt(const FSeekParam& NewPosition, const FTimeRange* InTimeRange)
{
	uint32 NextPlaybackID = Utils::Max(Utils::Max(CurrentPlaybackSequenceID[0], CurrentPlaybackSequenceID[1]), CurrentPlaybackSequenceID[2]) + 1;
	CurrentPlaybackSequenceID[0] = NextPlaybackID;
	CurrentPlaybackSequenceID[1] = NextPlaybackID;
	CurrentPlaybackSequenceID[2] = NextPlaybackID;

	StreamState = EStreamState::eStream_Running;

	PlaybackState.SetHasEnded(false);
	PlaybackState.SetLoopStateHasChanged(false);
	PlaybackState.SetEncoderLatency(FTimeValue::GetInvalid());

	VideoBufferStats.Clear();
	AudioBufferStats.Clear();
	TextBufferStats.Clear();
	DroppedFrameStats.Clear();

#if NOTIFY_DATA_AVAILABILITY_AT_START_AND_END
	// Update data availability states in case this wasn't done yet.
	UpdateDataAvailabilityState(DataAvailabilityStateVid, Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable);
	UpdateDataAvailabilityState(DataAvailabilityStateAud, Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable);
	UpdateDataAvailabilityState(DataAvailabilityStateTxt, Metrics::FDataAvailabilityChange::EAvailability::DataNotAvailable);
#endif

	// (Re-)configure the stream selector
	StreamSelector->SetBandwidthCeiling(BitrateCeiling);
	StreamSelector->SetMaxVideoResolution(VideoResolutionLimitWidth, VideoResolutionLimitHeight);

	// Create the stream reader
	check(StreamReaderHandler == nullptr);
	StreamReaderHandler = Manifest->CreateStreamReaderHandler();
	check(StreamReaderHandler);
	IStreamReader::CreateParam srhParam;
	srhParam.EventListener  = this;
	srhParam.MemoryProvider = this;
	if (StreamReaderHandler->Create(this, srhParam) != UEMEDIA_ERROR_OK)
	{
		FErrorDetail err;
		err.SetFacility(Facility::EFacility::Player);
		err.SetMessage(TEXT("Failed to create stream reader"));
		err.SetCode(INTERR_CREATE_FRAGMENT_READER);
		PostError(err);
		return;
	}

	check(!PendingStartRequest.IsValid());
	check(!InitialPlayPeriod.IsValid());
	check(!PendingFirstSegmentRequest.IsValid());
	check(NextPendingSegmentRequests->IsEmpty());
	check(ReadyWaitingSegmentRequests->IsEmpty());
	check(CompletedSegmentRequests.Num() == 0);
	NextPendingSegmentRequests->Empty();
	ReadyWaitingSegmentRequests->Empty();
	PendingFirstSegmentRequest.Reset();
	CompletedSegmentRequests.Empty();
	InitialPlayPeriod.Reset();
	CurrentPlayPeriodVideo.Reset();
	CurrentPlayPeriodAudio.Reset();
	CurrentPlayPeriodText.Reset();
	PendingStartRequest = MakeSharedTS<FPendingStartRequest>();
	PendingStartRequest->SearchType = IManifest::ESearchType::Closest;
	PendingStartRequest->StartAt.Time = NewPosition.Time;
	PendingStartRequest->StartingBitrate = NewPosition.StartingBitrate;
	PendingStartRequest->StartType = SeekVars.bIsPlayStart ? FPendingStartRequest::EStartType::PlayStart : FPendingStartRequest::EStartType::Seeking;
	PlaybackState.ActivateNewPlayRange(InTimeRange);
	SetPlaystartOptions(PendingStartRequest->StartAt.Options);

	// The fragment should be the closest to the time stamp unless we are rebuffering in which case it should be the one we failed on.
/*
	if (mLastBufferingState == eState_Rebuffering)
	{
		mpPendingStartRequest->SearchType = IManifest::ESearchType::Before;
	}
	else
*/
	{
		PendingStartRequest->SearchType = IManifest::ESearchType::Closest;
	}
	bFirstSegmentRequestIsForLooping = false;
	bStreamingHasStarted = false;

	// Let the AEMS handler know that we are starting up.
	AEMSEventHandler->PlaybackStartingUp();

	// From this point on any further start will not be the very first one any more.
	SeekVars.bIsPlayStart = false;
}


void FAdaptiveStreamingPlayer::InternalSetPlaybackEnded()
{
	PostrollVars.Clear();

	FTimeRange TimelineRange;
	PlaybackState.GetTimelineRange(TimelineRange);
	FTimeValue RangeEnd = PlaybackState.GetActivePlayRange().End;
	if (RangeEnd.IsValid() && RangeEnd < TimelineRange.End)
	{
		TimelineRange.End = RangeEnd;
	}
	if (Manifest.IsValid() && Manifest->GetDefaultEndTime().IsValid())
	{
		if (Manifest->GetDefaultEndTime() < TimelineRange.End)
		{
			TimelineRange.End = Manifest->GetDefaultEndTime();
		}
	}

	PlaybackState.SetPlayPosition(TimelineRange.End);

	if (LastBufferingState == EPlayerState::eState_Seeking)
	{
		DispatchEvent(FMetricEvent::ReportSeekCompleted());
	}
	SeekVars.Reset();

	PlaybackState.SetHasEnded(true);
	PlaybackState.SetShouldPlayOnLiveEdge(false);
	StreamState = EStreamState::eStream_ReachedEnd;

	// Go to pause mode now.
	InternalPause();
	// Report the frame stats.
	UpdateFrameStats(true);
	DispatchEvent(FMetricEvent::ReportPlaybackEnded());
	// In case a seek back into the stream will happen we reset the first start state.
	PrerollVars.bIsVeryFirstStart = true;
	// Let the AEMS handler know as well.
	AEMSEventHandler->PlaybackReachedEnd(TimelineRange.End);
}


//-----------------------------------------------------------------------------
/**
 * Pauses playback.
 */
void FAdaptiveStreamingPlayer::InternalPause()
{
	// Pause decoders to stop feeding them data.
	// NOTE: A decoder may be in InputNeeded() / FeedDecoder() at this point!
	//       This is to prevent their next ask for new data.
	DecoderState = EDecoderState::eDecoder_Paused;
	SubtitleDecoder.Stop();

	// Stop rendering.
	StopRendering();
	PipelineState = EPipelineState::ePipeline_Stopped;

	// Pause playing.
	if (CurrentState != EPlayerState::eState_Error)
	{
		CurrentState = EPlayerState::eState_Paused;
		PlaybackState.SetPausedAndPlaying(true, false);
		DispatchEvent(FMetricEvent::ReportPlaybackPaused());
	}
}


//-----------------------------------------------------------------------------
/**
 * Resumes playback.
 */
void FAdaptiveStreamingPlayer::InternalResume()
{
	// Cannot resume when in error state or when stream has reached the end.
	if (CurrentState != EPlayerState::eState_Error && StreamState != EStreamState::eStream_ReachedEnd)
	{
		// Start rendering.
		check(PipelineState == EPipelineState::ePipeline_Stopped);
		StartRendering();
		PipelineState = EPipelineState::ePipeline_Running;

		// Resume decoders.
		DecoderState = EDecoderState::eDecoder_Running;
		SubtitleDecoder.Start();

		if (PrerollVars.bIsVeryFirstStart)
		{
			PrerollVars.bIsVeryFirstStart = false;
			// Send play start event
			DispatchEvent(FMetricEvent::ReportPlaybackStart());
		}

		// Resume playing.
		CurrentState = EPlayerState::eState_Playing;
		PlaybackState.SetPausedAndPlaying(false, true);
		PostrollVars.Clear();
		DispatchEvent(FMetricEvent::ReportPlaybackResumed());
	}
}





void FAdaptiveStreamingPlayer::InternalStop(bool bHoldCurrentFrame)
{
	// Increase the playback sequence ID. This will cause all async segment request message
	// to be discarded when they are received.
	++CurrentPlaybackSequenceID[0];
	++CurrentPlaybackSequenceID[1];
	++CurrentPlaybackSequenceID[2];

	// Pause decoders to stop feeding them data.
	// NOTE: A decoder may be in InputNeeded() / FeedDecoder() at this point!
	//       This is to prevent their next ask for new data.
	DecoderState = EDecoderState::eDecoder_Paused;

	// Stop rendering.
	StopRendering();
	PipelineState = EPipelineState::ePipeline_Stopped;

	// Destroy the stream reader handler.
	IStreamReader* CurrentStreamReader = TMediaInterlockedExchangePointer(StreamReaderHandler, (IStreamReader*)nullptr);
	if (CurrentStreamReader)
	{
		CurrentStreamReader->Close();
	}
	delete CurrentStreamReader;
	bStreamingHasStarted = false;

	// Release any pending segment requests.
	PendingStartRequest.Reset();
	PendingFirstSegmentRequest.Reset();
	NextPendingSegmentRequests->Empty();
	ReadyWaitingSegmentRequests->Empty();
	CompletedSegmentRequests.Empty();
	InitialPlayPeriod.Reset();
	CurrentPlayPeriodVideo.Reset();
	CurrentPlayPeriodAudio.Reset();
	CurrentPlayPeriodText.Reset();
	ActivePeriodCriticalSection.Lock();
	ActivePeriods.Empty();
	ActivePeriodCriticalSection.Unlock();
	UpcomingPeriods.Empty();
	MetadataHandlingState.Reset();
	MediaMetadataUpdates.Reset();
	NextLoopStates.Empty();
	SeekVars.Reset();

	CurrentState = EPlayerState::eState_Paused;
	PlaybackState.SetPausedAndPlaying(false, false);
	PlaybackState.SetPlaybackEndAtTime(FTimeValue::GetInvalid());

	// Flush all access unit buffers.
	DataBuffersCriticalSection.Lock();
	CurrentDataReceiveBuffers.Reset();
	NextDataBuffers.Empty();
	ActiveDataOutputBuffers.Reset();
	DataBuffersCriticalSection.Unlock();

	// Flush the renderers once before flushing the decoders.
	// In case the media samples hold references to the decoder (because of shared frame resources) it could be a possibility
	// that the decoder cannot flush as long as those references are "out there".
	AudioOutput.Flush();
	VideoOutput.Flush(bHoldCurrentFrame);

	// Flush the decoders.
	AudioDecoder.Flush();
	VideoDecoder.Flush();
	SubtitleDecoder.Flush();
	// Decoders are no longer at end of data now.
	ClearAllDecoderEODs();

	// Flush the renderers again, this time to discard everything the decoders may have emitted while being flushed.
	AudioOutput.Flush();
	VideoOutput.Flush(bHoldCurrentFrame);

	// Flush dynamic events from the AEMS handler.
	AEMSEventHandler->FlushDynamic();
}


void FAdaptiveStreamingPlayer::InternalInitialize()
{
	// Get the HTTP manager. This is a shared instance for all players.
	HttpManager = IElectraHttpManager::Create();

	// Create the DRM manager.
	DrmManager = FDRMManager::Create(this);

	// Create an entity cache.
	EntityCache = IPlayerEntityCache::Create(this);

	// Create an HTTP response cache.
	HttpResponseCache = IHTTPResponseCache::Create(GetOptionValue(OptionKeyResponseCacheMaxByteSize).SafeGetInt64(0), (int32)GetOptionValue(OptionKeyResponseCacheMaxEntries).SafeGetInt64(8192), nullptr);

	// Create the ABR stream selector.
	StreamSelector = IAdaptiveStreamSelector::Create(this);
	AddMetricsReceiver(StreamSelector.Get());

	// Create the content steering handler.
	ContentSteeringHandler = MakeSharedTS<FContentSteeringHandler>(this);

	// Create the CMCD handler.
	CMCDHandler = MakeSharedTS<FCMCDHandler>(this);
	CMCDHandler->Initialize(GetOptionValue(OptionKeyContentID).SafeGetFString(), GetOptionValue(OptionKeySessionID).SafeGetFString(), GetOptionValue(OptionKeyCMCDConfiguration).SafeGetFString(FString()));
	AddMetricsReceiver(CMCDHandler.Get());

	// Create renderers.
	CreateRenderers();

	// Check for codecs that are not to be used as per the user's choice, even if the device supports them.
	auto GetExcludedCodecPrefixes = [](TArray<FString>& OutList, const FParamDictTS& InOptions, const FName& InKey) -> void
	{
		if (InOptions.HaveKey(InKey))
		{
			const TCHAR* const CommaDelimiter = TEXT(",");
			TArray<FString> Prefixes;
			FString Value = InOptions.GetValue(InKey).GetFString();
			Value.TrimQuotesInline();
			Value.ParseIntoArray(Prefixes, CommaDelimiter, true);
			for (auto& Prefix : Prefixes)
			{
				OutList.Emplace(Prefix.TrimStartAndEnd().TrimQuotes().TrimStartAndEnd());
			}
		}
	};
	GetExcludedCodecPrefixes(ExcludedVideoDecoderPrefixes, PlayerOptions, OptionKeyExcludedCodecsVideo);
	GetExcludedCodecPrefixes(ExcludedAudioDecoderPrefixes, PlayerOptions, OptionKeyExcludedCodecsAudio);
	GetExcludedCodecPrefixes(ExcludedSubtitleDecoderPrefixes, PlayerOptions, OptionKeyExcludedCodecsSubtitles);

	auto GetCodecSelectionPriorities = [this](FCodecSelectionPriorities& OutPriorities, const FParamDictTS& InOptions, const FName& InKey, const TCHAR* const InType) -> void
	{
		if (InOptions.HaveKey(InKey))
		{
			if (!OutPriorities.Initialize(InOptions.GetValue(InKey).GetFString()))
			{
				PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Failed to parse %s codec selection priority configuration"), InType));
			}
		}
	};
	GetCodecSelectionPriorities(CodecPrioritiesVideo, PlayerOptions, OptionKeyPreferredCodecsVideo, TEXT("video"));
	GetCodecSelectionPriorities(CodecPrioritiesAudio, PlayerOptions, OptionKeyPreferredCodecsAudio, TEXT("audio"));
	GetCodecSelectionPriorities(CodecPrioritiesSubtitles, PlayerOptions, OptionKeyPreferredCodecsSubtitles, TEXT("subtitle"));

	// Unless already specified in the options to either value, enable frame accurate seeking now.
	if (!PlayerOptions.HaveKey(OptionKeyFrameAccurateSeek))
	{
		PlayerOptions.Set(OptionKeyFrameAccurateSeek, FVariantValue(true));
	}

	// Get the custom attributes to extract from the main playlist, if any.
	if (PlayerOptions.HaveKey(OptionKeyPlaylistProperties))
	{
		if (!PlaylistPropertyHandler.ConfigureFromJSON(PlayerOptions.GetValue(OptionKeyPlaylistProperties).GetFString()))
		{
			PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Failed to parse playlist property JSON configuration, ignoring.")));
		}
	}

	// Set the interval to aggregate (dropped) frame statistics and report them.
	DroppedFrameStats.ReportingIntervalMs = GetOptionValue(OptionKeyFrameStatInterval).SafeGetInt64(4000);

	// Initialize decoder limits.
	// NOTE: This is to be deprecated as soon as possible.
	Deprecate_InternalInitializeDecoderLimits();
}


void FAdaptiveStreamingPlayer::InternalClose()
{
	// No longer need the manifest reader/updater
	InternalCancelLoadManifest();
	InternalCloseManifestReader();

	DestroyDecoders();
	DestroyRenderers();
	CurrentMediaSamples.Reset();

	// Do not clear the playback state to allow GetPlayPosition() to query the last play position.
	// This also means queries like IsBuffering() will return the last state but calling those
	// after a Stop() can be considered at least weird practice.
	//PlaybackState.Reset();
	Manifest.Reset();

	if (DrmManager.IsValid())
	{
		DrmManager->Close();
		DrmManager.Reset();
	}

	RemoveMetricsReceiver(StreamSelector.Get());
	StreamSelector.Reset();
	ContentSteeringHandler.Reset();
	RemoveMetricsReceiver(CMCDHandler.Get());
	CMCDHandler.Reset();

	HttpManager.Reset();
	EntityCache.Reset();
	HttpResponseCache.Reset();

	// Reset remaining internal state
	CurrentState = EPlayerState::eState_Idle;
	StreamState = EStreamState::eStream_Running;
	RebufferCause = ERebufferCause::None;
	LastBufferingState = EPlayerState::eState_Buffering;
	RebufferDetectedAtPlayPos.SetToInvalid();
	PrerollVars.Clear();
	PostrollVars.Clear();

	bShouldBePaused = false;
	bShouldBePlaying = false;
	CurrentLoopState = {};
	NextLoopStates.Empty();

	CurrentPlaybackSequenceState.Reset();

	PendingTrackSelectionVid.Reset();
	PendingTrackSelectionAud.Reset();
	PendingTrackSelectionTxt.Reset();

	// Flush all events from the AEMS handler
	AEMSEventHandler->FlushEverything();

	// Clear error state.
	LastErrorDetail.Clear();
	ErrorQueue.Clear();

	// Clear diagnostics.
	DiagnosticsCriticalSection.Lock();
	VideoBufferStats.Clear();
	AudioBufferStats.Clear();
	TextBufferStats.Clear();
	DiagnosticsCriticalSection.Unlock();
	DroppedFrameStats.Clear();
}


void FAdaptiveStreamingPlayer::InternalHandlePendingLoopRequest()
{
	TUniquePtr<FLoopParam> NewLoopParam(PlaybackState.GetPendingLoopRequest());
	if (NewLoopParam.IsValid())
	{
		// Looping only makes sense for on demand presentations since Live does not have an end to loop at.
		// In case this is called that early on that we can't determine the presentation type yet we allow
		// looping to be set. It won't get evaluated anyway since the Live presentation doesn't end.
		// The only issue with this is that until a manifest is loaded querying loop enable will return true.
		if (!Manifest || Manifest->GetPresentationType() == IManifest::EType::OnDemand || Manifest->GetReplayType() == IManifest::EReplayEventType::IsReplay)
		{
			bool bStartOver = false;
			// When looping is to be enabled we need to check if we can do this without having to start over.
			if (Manifest && !CurrentLoopParam.bEnableLooping && NewLoopParam->bEnableLooping)
			{
				// If there are any completed segment requests the loop state will be checked when all reach
				// the end. Otherwise, if any of the receive buffers has the end-of-data flag set that
				// that streams reached the end and will not loop, so we need to start over.
				if (CompletedSegmentRequests.Num() == 0)
				{
					TSharedPtrTS<FMultiTrackAccessUnitBuffer> RcvBuffer;
					bool bAnyReceiveBufferAtEOD = false;
					RcvBuffer = GetCurrentReceiveStreamBuffer(EStreamType::Video);
					bAnyReceiveBufferAtEOD |= RcvBuffer.IsValid() ? RcvBuffer->IsEODFlagSet() || RcvBuffer->IsEndOfTrack() : false;
					RcvBuffer = GetCurrentReceiveStreamBuffer(EStreamType::Audio);
					bAnyReceiveBufferAtEOD |= RcvBuffer.IsValid() ? RcvBuffer->IsEODFlagSet() || RcvBuffer->IsEndOfTrack() : false;
					RcvBuffer = GetCurrentReceiveStreamBuffer(EStreamType::Subtitle);
					bAnyReceiveBufferAtEOD |= RcvBuffer.IsValid() ? RcvBuffer->IsEODFlagSet() || RcvBuffer->IsEndOfTrack() : false;

					bStartOver = bAnyReceiveBufferAtEOD;
				}
			}
			CurrentLoopParam = *NewLoopParam;
			// For the lack of better knowledge update the current state immediately.
			CurrentLoopState.bIsEnabled = NewLoopParam->bEnableLooping;
			PlaybackState.SetLoopStateEnable(NewLoopParam->bEnableLooping);
			if (bStartOver)
			{
				PlaybackState.SetLoopStateHasChanged(true);
			}
		}
	}
}



//-----------------------------------------------------------------------------
/**
 * Rebuffers at the current play position.
 */
void FAdaptiveStreamingPlayer::InternalRebuffer()
{
	IAdaptiveStreamSelector::FRebufferAction Action;
	check(StreamSelector.IsValid());
	if (StreamSelector.IsValid())
	{
		Action = StreamSelector->GetRebufferAction();
	}

	// Check if we are configured to throw a playback error instead of doing a rebuffer.
	// This can be used by a higher logic to force playback at a different position instead.
	if (Action.Action == IAdaptiveStreamSelector::FRebufferAction::EAction::ThrowError)
	{
		FErrorDetail err;
		err.SetFacility(Facility::EFacility::Player);
		err.SetMessage(TEXT("Rebuffering is set to generate a playback error"));
		err.SetCode(INTERR_REBUFFER_SHALL_THROW_ERROR);
		PostError(err);
	}
	else
	{
		// Pause decoders to stop feeding them data.
		// NOTE: A decoder may be in InputNeeded() / FeedDecoder() at this point!
		//       This is to prevent their next ask for new data.
		DecoderState = EDecoderState::eDecoder_Paused;

		// Stop rendering.
		StopRendering();
		PipelineState = EPipelineState::ePipeline_Stopped;

		CurrentState = EPlayerState::eState_Rebuffering;
		LastBufferingState = EPlayerState::eState_Rebuffering;

		// If rebuffering triggered while switching tracks then the spacing of sync frames may be too big and we
		// have none to switch over to the new track yet. To overcome this we restart at the current position.
		if (RebufferCause == ERebufferCause::TrackswitchUnderrun && Action.Action == IAdaptiveStreamSelector::FRebufferAction::EAction::ContinueLoading)
		{
			Action.Action = IAdaptiveStreamSelector::FRebufferAction::EAction::Restart;
			PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Info, FString::Printf(TEXT("Rebuffering during track change switches rebuffering policy from continue to restart.")));
		}
		RebufferCause = ERebufferCause::None;

		// Check if we should just continue loading data
		if (Action.Action == IAdaptiveStreamSelector::FRebufferAction::EAction::ContinueLoading)
		{
			RebufferDetectedAtPlayPos.SetToInvalid();
			PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Info, FString::Printf(TEXT("Rebuffering is waiting for new data.")));
			DispatchBufferingEvent(true, CurrentState);
			CurrentState = EPlayerState::eState_Buffering;
		}
		else
		{
			// We do not wait on the current segment that caused the rebuffering to complete and start over.
			FSeekParam StartAtTime;
			if (Action.Action == IAdaptiveStreamSelector::FRebufferAction::EAction::GoToLive)
			{
				StartAtTime.Time.SetToInvalid();
				PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Info, FString::Printf(TEXT("Rebuffering jumps to the live edge")));
			}
			else
			{
				if (!RebufferDetectedAtPlayPos.IsValid())
				{
					RebufferDetectedAtPlayPos = PlaybackState.GetPlayPosition();
				}
				StartAtTime.Time = RebufferDetectedAtPlayPos;
				PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Info, FString::Printf(TEXT("Rebuffering starts over at current position")));
			}
			RebufferDetectedAtPlayPos.SetToInvalid();
			InternalStop(PlayerConfig.bHoldLastFrameDuringSeek);
			CurrentState = EPlayerState::eState_Rebuffering;
			if (Action.SuggestedRestartBitrate > 0)
			{
				StartAtTime.StartingBitrate = Action.SuggestedRestartBitrate;
			}

			InternalStartAt(StartAtTime, nullptr);
		}
	}
}








FTimeValue FAdaptiveStreamingPlayer::ABRGetPlayPosition() const
{
	return PlaybackState.GetPlayPosition();
}
FTimeRange FAdaptiveStreamingPlayer::ABRGetTimeline() const
{
	FTimeRange Timeline;
	PlaybackState.GetTimelineRange(Timeline);
	return Timeline;
}
FTimeValue FAdaptiveStreamingPlayer::ABRGetWallclockTime() const
{
	return SynchronizedUTCTime->GetTime();
}
bool FAdaptiveStreamingPlayer::ABRIsLive() const
{
	TSharedPtrTS<IManifest> Current = Manifest;
	return Current.IsValid() ? Current->GetPresentationType() == IManifest::EType::Live : false;
}
bool FAdaptiveStreamingPlayer::ABRShouldPlayOnLiveEdge() const
{
	return PlaybackState.GetShouldPlayOnLiveEdge();
}
TSharedPtrTS<const FLowLatencyDescriptor> FAdaptiveStreamingPlayer::ABRGetLowLatencyDescriptor() const
{
	TSharedPtrTS<IManifest> Current = Manifest;
	return Current.IsValid() ? Current->GetLowLatencyDescriptor() : nullptr;
}
FTimeValue FAdaptiveStreamingPlayer::ABRGetDesiredLiveEdgeLatency() const
{
	TSharedPtrTS<IManifest> Current = Manifest;
	return Current.IsValid() ? Current->GetDesiredLiveLatency() : FTimeValue();
}
FTimeValue FAdaptiveStreamingPlayer::ABRGetLatency() const
{
	return PlaybackState.GetCurrentLiveLatency();
}
FTimeValue FAdaptiveStreamingPlayer::ABRGetPlaySpeed() const
{
	return FTimeValue(RenderClock->IsRunning() ? 1.0 : 0.0);
}
void FAdaptiveStreamingPlayer::ABRGetStreamBufferStats(IPlayerABRLiveControl::FABRBufferStats& OutBufferStats, EStreamType ForStream)
{
	UpdateDiagnostics();
	DiagnosticsCriticalSection.Lock();
	switch(ForStream)
	{
		case EStreamType::Video:
		{
			OutBufferStats.PlayableContentDuration = VideoBufferStats.StreamBuffer.PlayableDuration;
			OutBufferStats.bReachedEnd = VideoBufferStats.StreamBuffer.bEndOfData;
			OutBufferStats.bEndOfTrack = VideoBufferStats.StreamBuffer.bEndOfTrack;
			break;
		}
		case EStreamType::Audio:
		{
			OutBufferStats.PlayableContentDuration = AudioBufferStats.StreamBuffer.PlayableDuration;
			OutBufferStats.bReachedEnd = AudioBufferStats.StreamBuffer.bEndOfData;
			OutBufferStats.bEndOfTrack = AudioBufferStats.StreamBuffer.bEndOfTrack;
			break;
		}
		case EStreamType::Subtitle:
		{
			OutBufferStats.PlayableContentDuration = TextBufferStats.StreamBuffer.PlayableDuration;
			OutBufferStats.bReachedEnd = TextBufferStats.StreamBuffer.bEndOfData;
			OutBufferStats.bEndOfTrack = TextBufferStats.StreamBuffer.bEndOfTrack;
			break;
		}
		default:
		{
			break;
		}
	}
	DiagnosticsCriticalSection.Unlock();
	if (ForStream == EStreamType::Video && VideoOutput.OutputHandler.IsValid())
	{
		OutBufferStats.PlayableContentDuration += FTimeValue().SetFromTimespan(VideoOutput.OutputHandler->GetEnqueuedSampleDuration());
	}
	else if (ForStream == EStreamType::Audio && AudioOutput.OutputHandler.IsValid())
	{
		OutBufferStats.PlayableContentDuration += FTimeValue().SetFromTimespan(AudioOutput.OutputHandler->GetEnqueuedSampleDuration());
	}
}
void FAdaptiveStreamingPlayer::ABRSetRenderRateScale(double InRenderRateScale)
{
	RenderRateScale = InRenderRateScale;
	if (AudioOutput.OutputHandler.IsValid())
	{
		AudioOutput.OutputHandler->SetPlayRateScale(InRenderRateScale);
	}
}
FTimeRange FAdaptiveStreamingPlayer::ABRGetSupportedRenderRateScale()
{
	FTimeRange Range;
	if (AudioOutput.OutputHandler.IsValid())
	{
		Range = AudioOutput.OutputHandler->GetSupportedRenderRateScale();
	}
	return Range;
}

double FAdaptiveStreamingPlayer::ABRGetRenderRateScale() const
{
	return RenderRateScale;
}

void FAdaptiveStreamingPlayer::ABRTriggerClockSync(IPlayerABRLiveControl::EClockSyncType InClockSyncType)
{
	if (Manifest.IsValid())
	{
		Manifest->TriggerClockSync(InClockSyncType == IPlayerABRLiveControl::EClockSyncType::Required ?
									IManifest::EClockSyncType::Required : IManifest::EClockSyncType::Recommended);
	}
}

void FAdaptiveStreamingPlayer::ABRTriggerPlaylistRefresh()
{
	if (Manifest.IsValid())
	{
		Manifest->TriggerPlaylistRefresh();
	}
}



//-----------------------------------------------------------------------------
/**
 * Debug prints information to the screen
 *
 * @param pPrintFN
 */
void FAdaptiveStreamingPlayer::DebugPrint(void* pPlayer, void (*pPrintFN)(void* pPlayer, const char *pFmt, ...))
{
#if 1
	DiagnosticsCriticalSection.Lock();
	const FAccessUnitBufferInfo *pD = nullptr;
	if(1)
	{
		pD = &VideoBufferStats.StreamBuffer;
		pPrintFN(pPlayer, "Video buffer  : EOT %d; EOS %d; %3u AUs; %8lld bytes in; %#7.4fs", pD->bEndOfTrack, pD->bEndOfData, (uint32)pD->NumCurrentAccessUnits, (long long int)pD->CurrentMemInUse, pD->PlayableDuration.GetAsSeconds());
		pPrintFN(pPlayer, "Video decoder : %2u in decoder, %zu total; EOD in %d; EOD out %d; %s", (uint32)VideoBufferStats.DecoderOutputBuffer.NumElementsInDecoder, (uint32)VideoBufferStats.DecoderOutputBuffer.OutputBufferPoolSize, VideoBufferStats.DecoderInputBuffer.bEODReached, VideoBufferStats.DecoderOutputBuffer.bEODreached, VideoBufferStats.DecoderOutputBuffer.bOutputStalled?"stalled":"not stalled");
		pPrintFN(pPlayer, "Video renderer: %#7.4fs enqueued", VideoOutput.OutputHandler.IsValid() ? VideoOutput.OutputHandler->GetEnqueuedSampleDuration().GetTotalSeconds() : 0.0);
	}
	if(1)
	{
		pD = &AudioBufferStats.StreamBuffer;
		pPrintFN(pPlayer, "Audio buffer  : EOT %d; EOS %d; %3u AUs; %8lld bytes in; %#7.4fs", pD->bEndOfTrack, pD->bEndOfData, (uint32)pD->NumCurrentAccessUnits, (long long int)pD->CurrentMemInUse, pD->PlayableDuration.GetAsSeconds());
		pPrintFN(pPlayer, "Audio decoder : %2u in decoder, %2u total; EOD in %d; EOD out %d; %s", (uint32)AudioBufferStats.DecoderOutputBuffer.NumElementsInDecoder, (uint32)AudioBufferStats.DecoderOutputBuffer.OutputBufferPoolSize, AudioBufferStats.DecoderInputBuffer.bEODReached, AudioBufferStats.DecoderOutputBuffer.bEODreached, AudioBufferStats.DecoderOutputBuffer.bOutputStalled?"stalled":"not stalled");
		pPrintFN(pPlayer, "Audio renderer: %#7.4fs enqueued", AudioOutput.OutputHandler.IsValid() ? AudioOutput.OutputHandler->GetEnqueuedSampleDuration().GetTotalSeconds() : 0.0);
	}
	if(1)
	{
		pD = &TextBufferStats.StreamBuffer;
		pPrintFN(pPlayer, "Text buffer   : EOT %d; EOS %d; %3u AUs; %8lld bytes in; %#7.4fs", pD->bEndOfTrack, pD->bEndOfData, (uint32)pD->NumCurrentAccessUnits, (long long int)pD->CurrentMemInUse, pD->PlayableDuration.GetAsSeconds());
	}
	DiagnosticsCriticalSection.Unlock();

	pPrintFN(pPlayer, "Player state  : %s", GetPlayerStateName(CurrentState));
	pPrintFN(pPlayer, "Decoder state : %s", GetDecoderStateName(DecoderState));
	pPrintFN(pPlayer, "Pipeline state: %s", GetPipelineStateName(PipelineState));
	pPrintFN(pPlayer, "Stream state  : %s", GetStreamStateName(StreamState));
	pPrintFN(pPlayer, "-----------------------------------");
	if (auto csh = ContentSteeringHandler)
	{
		pPrintFN(pPlayer, "CDN priority: %s", (ANSICHAR*)StringCast<ANSICHAR>(static_cast<const TCHAR*>( *ContentSteeringHandler->GetCurrentCDNListForDisplay() )).Get());
		pPrintFN(pPlayer, "CDN access  : %s", (ANSICHAR*)StringCast<ANSICHAR>(static_cast<const TCHAR*>( *ContentSteeringHandler->GetRecentlyAccessedCDNListForDisplay() )).Get());
	}
	FTimeRange ActiveRange = PlaybackState.GetActivePlayRange();
	FTimeRange seekable, timeline;
	GetSeekableRange(seekable);
	PlaybackState.GetTimelineRange(timeline);
	pPrintFN(pPlayer, " have metadata: %s", HaveMetadata() ? "Yes" : "No");
	pPrintFN(pPlayer, "      duration: %.3f", GetDuration().GetAsSeconds());
	pPrintFN(pPlayer, "timeline range: %.3f - %.3f", timeline.Start.GetAsSeconds(), timeline.End.GetAsSeconds());
	pPrintFN(pPlayer, "seekable range: %.3f - %.3f", seekable.Start.GetAsSeconds(), seekable.End.GetAsSeconds());
	pPrintFN(pPlayer, "playback range: %.3f - %.3f", ActiveRange.Start.GetAsSeconds(), ActiveRange.End.GetAsSeconds());
	pPrintFN(pPlayer, " play position: [%d] %.3f", (int32) GetPlayPosition().GetSequenceIndex(), GetPlayPosition().GetAsSeconds());
	pPrintFN(pPlayer, "  is buffering: %s", IsBuffering() ? "Yes" : "No");
	pPrintFN(pPlayer, "    is playing: %s", IsPlaying() ? "Yes" : "No");
	pPrintFN(pPlayer, "     is paused: %s", IsPaused() ? "Yes" : "No");
	pPrintFN(pPlayer, "    is seeking: %s", IsSeeking() ? "Yes" : "No");
	pPrintFN(pPlayer, "     has ended: %s", HasEnded() ? "Yes" : "No");
#endif
	if (auto strstel = StreamSelector)
	{
		StreamSelector->DebugPrint(pPlayer, pPrintFN);
	}
}
void FAdaptiveStreamingPlayer::DebugHandle(void* pPlayer, void (*debugDrawPrintf)(void* pPlayer, const char *pFmt, ...))
{
	if (PointerToLatestPlayer)
	{
		PointerToLatestPlayer->DebugPrint(pPlayer, debugDrawPrintf);
	}
}



//---------------------------------------------------------------------------------------------------------------------------------
//---------------------------------------------------------------------------------------------------------------------------------
//---------------------------------------------------------------------------------------------------------------------------------


TWeakPtrTS<FAdaptiveStreamingPlayerWorkerThread>	FAdaptiveStreamingPlayerWorkerThread::SingletonSelf;
FCriticalSection									FAdaptiveStreamingPlayerWorkerThread::SingletonLock;

TSharedPtrTS<FAdaptiveStreamingPlayerWorkerThread> FAdaptiveStreamingPlayerWorkerThread::Create(bool bUseSharedWorkerThread)
{
	if (bUseSharedWorkerThread)
	{
		FScopeLock lock(&SingletonLock);
		TSharedPtrTS<FAdaptiveStreamingPlayerWorkerThread> Self = SingletonSelf.Pin();
		if (!Self.IsValid())
		{
			FAdaptiveStreamingPlayerWorkerThread* Handler = new FAdaptiveStreamingPlayerWorkerThread;
			Handler->bIsDedicatedWorker = !bUseSharedWorkerThread;
			Handler->StartWorkerThread();
			Self = MakeShareable(Handler);
			SingletonSelf = Self;
		}
		return Self;
	}
	else
	{
		FAdaptiveStreamingPlayerWorkerThread* Handler = new FAdaptiveStreamingPlayerWorkerThread;
		Handler->bIsDedicatedWorker = !bUseSharedWorkerThread;
		Handler->StartWorkerThread();
		return MakeShareable(Handler);
	}
}

FAdaptiveStreamingPlayerWorkerThread::~FAdaptiveStreamingPlayerWorkerThread()
{
	StopWorkerThread();
}

void FAdaptiveStreamingPlayerWorkerThread::StartWorkerThread()
{
	if (bIsDedicatedWorker)
	{
		WorkerThread.ThreadSetName("ElectraPlayer::Worker");
	}
	else
	{
		WorkerThread.ThreadSetName("ElectraPlayer::SharedWorker");
	}
	WorkerThread.ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FAdaptiveStreamingPlayerWorkerThread::WorkerThreadFN));
}

void FAdaptiveStreamingPlayerWorkerThread::StopWorkerThread()
{
	bTerminate = true;
	HaveWorkSignal.Signal();
	WorkerThread.ThreadWaitDone();
	WorkerThread.ThreadReset();
}

void FAdaptiveStreamingPlayerWorkerThread::WorkerThreadFN()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	auto RemoveInstances = [this]() -> void
	{
		FInstanceToRemove DelInstance;
		while(InstancesToRemove.Dequeue(DelInstance))
		{
			PlayerInstances.Remove(DelInstance.Player);
			if (DelInstance.DoneSignal)
			{
				DelInstance.DoneSignal->Signal();
			}
		}
	};

	while(!bTerminate)
	{
		HaveWorkSignal.WaitTimeoutAndReset(1000 * 20);

		FAdaptiveStreamingPlayer* NewInstance;
		while(InstancesToAdd.Dequeue(NewInstance))
		{
			PlayerInstances.Add(NewInstance);
		}

		for(int32 i=0; i<PlayerInstances.Num(); ++i)
		{
			PlayerInstances[i]->HandleOnce();
		}

		RemoveInstances();
	}
	RemoveInstances();
}


//---------------------------------------------------------------------------------------------------------------------------------
//---------------------------------------------------------------------------------------------------------------------------------
//---------------------------------------------------------------------------------------------------------------------------------

TWeakPtrTS<FAdaptiveStreamingPlayerEventHandler>	FAdaptiveStreamingPlayerEventHandler::SingletonSelf;
FCriticalSection									FAdaptiveStreamingPlayerEventHandler::SingletonLock;

TSharedPtrTS<FAdaptiveStreamingPlayerEventHandler> FAdaptiveStreamingPlayerEventHandler::Create(bool bUseSharedWorkerThread)
{
	if (bUseSharedWorkerThread)
	{
		FScopeLock lock(&SingletonLock);
		TSharedPtrTS<FAdaptiveStreamingPlayerEventHandler> Self = SingletonSelf.Pin();
		if (!Self.IsValid())
		{
			FAdaptiveStreamingPlayerEventHandler* Handler = new FAdaptiveStreamingPlayerEventHandler;
			Handler->StartWorkerThread();
			Self = MakeShareable(Handler);
			SingletonSelf = Self;
		}
		return Self;
	}
	else
	{
		FAdaptiveStreamingPlayerEventHandler* Handler = new FAdaptiveStreamingPlayerEventHandler;
		Handler->StartWorkerThread();
		return MakeShareable(Handler);
	}
}

void FAdaptiveStreamingPlayerEventHandler::DispatchEvent(TSharedPtrTS<FMetricEvent> InEvent)
{
	EventQueue.SendMessage(InEvent);
}

FAdaptiveStreamingPlayerEventHandler::~FAdaptiveStreamingPlayerEventHandler()
{
	StopWorkerThread();
}

void FAdaptiveStreamingPlayerEventHandler::StartWorkerThread()
{
	WorkerThread.ThreadSetName("ElectraPlayer::EventDispatch");
	WorkerThread.ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FAdaptiveStreamingPlayerEventHandler::WorkerThreadFN));
}

void FAdaptiveStreamingPlayerEventHandler::StopWorkerThread()
{
	EventQueue.SendMessage(TSharedPtrTS<FMetricEvent>());
	WorkerThread.ThreadWaitDone();
	WorkerThread.ThreadReset();
}

void FAdaptiveStreamingPlayerEventHandler::WorkerThreadFN()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);
	while(1)
	{
		TSharedPtrTS<FMetricEvent> pEvt = EventQueue.ReceiveMessage();
		if (!pEvt.IsValid())
		{
			break;
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_EventWorker);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, AdaptiveStreamingPlayer_Event);

			// Get the player that sends the event. If it no longer exists ignore the event.
			TSharedPtrTS<FAdaptiveStreamingPlayer>	Player = pEvt->Player.Pin();
			if (Player.IsValid())
			{
				Player->FireSyncEvent(pEvt);
			}
		}
	}
}





} // namespace Electra
