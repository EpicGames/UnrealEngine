// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

#include "CoreMinimal.h"

#include "ErrorDetail.h"
#include "ParameterDictionary.h"
#include "StreamTypes.h"
#include "AdaptiveStreamingPlayerMetrics.h"
#include "AdaptiveStreamingPlayerEvents.h"
#include "AdaptiveStreamingPlayerResourceRequest.h"

class FMediaSamples;
class FElectraTextureSamplePool;
class FElectraAudioSamplePool;

namespace Electra
{

/**
 * This class declares the adaptive streaming player.
**/
class IAdaptiveStreamingPlayer : public TMediaNoncopyable<IAdaptiveStreamingPlayer>
{
public:
	struct FCreateParam
	{
		TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> OutputTexturePool;
		TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> OutputAudioSamplePool;

		FGuid ExternalPlayerGUID;
		enum class EWorkerThreads
		{
			Shared,
			DedicatedWorker,
			DedicatedWorkerAndEventDispatch
		};
		EWorkerThreads WorkerThreads = EWorkerThreads::Shared;
	};

	static TSharedPtr<IAdaptiveStreamingPlayer, ESPMode::ThreadSafe> Create(const FCreateParam& InCreateParameters);

	//-------------------------------------------------------------------------
	// Various resource providers/delegates
	// These must be set prior to calling `Initialize()` and MUST NOT be changed
	// until player destruction.
	//
	virtual void SetStaticResourceProviderCallback(const TSharedPtr<IAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe>& InStaticResourceProvider) = 0;


	//-------------------------------------------------------------------------
	// Metric event listener
	//
	virtual void AddMetricsReceiver(IAdaptiveStreamingPlayerMetrics* InMetricsReceiver) = 0;
	virtual void RemoveMetricsReceiver(IAdaptiveStreamingPlayerMetrics* InMetricsReceiver) = 0;


	//-------------------------------------------------------------------------
	// Application Event or Metadata Stream (AEMS) receiver
	//   Please refer to ISO/IEC 23009-1:2019/DAM 1:2020(E)
	virtual void AddAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode) = 0;
	virtual void RemoveAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode) = 0;


	//-------------------------------------------------------------------------
	// Initializes the player. Options may be passed in to affect behaviour.
	//
	virtual void Initialize(const FParamDict& InOptions) = 0;

	/**
	 * Modifies options. Not all options are modifiable during playback.
	 */
	virtual void ModifyOptions(const FParamDict& InOptionsToSetOrChange, const FParamDict& InOptionsToClear) = 0;

	/**
	 * Returns a player option value.
	 */
	virtual FVariantValue GetMediaInfo(FName InKey) const = 0;

	/**
	 * Sets the attributes for the stream to start buffering for and playing.
	 * This must be set before calling SeekTo() or LoadManifest() to have an immediate effect.
	 * A best effort to match a stream with the given attributes will be made.
	 * If there is no exact match the player will make an educated guess.
	 * Any later call to SelectTrackByMetadata() internally overwrites these initial attributes
	 * with those of the explicitly selected track.
	 */
	virtual void SetInitialStreamAttributes(EStreamType StreamType, FStreamSelectionAttributes InitialSelection) = 0;

	/**
	 * Enables or disables frame accurate seeking.
	 * Frame accurate positioning may require internal decoding and discarding of video and audio from an
	 * earlier keyframe up to the intended time, which may make seeking significantly slower.
	 * This also affects looping which implicitly seeks back to the loop point when the end is reached.
	 * Frame accurate seeking is enabled by default.
	 * This method is intended mostly to disable frame accurate seeking on this player instance.
	 *
	 * This should be called prior to SeekTo() and should only be called once on the player instance to
	 * disable or re-enable frame accurate seeking.
	 * Calling this during playback may have undesired results.
	 */
	virtual void EnableFrameAccurateSeeking(bool bEnabled) = 0;


	//-------------------------------------------------------------------------
	// Interfacing functions
	//
	virtual TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> GetMediaSamples() = 0;


	//-------------------------------------------------------------------------
	// Manifest loading functions
	//
	//! Issue a load and parse of the manifest/main playlist file. Make initial stream selection choice by calling SetInitialStreamAttributes() beforehand.
	virtual void LoadManifest(const FString& ManifestURL) = 0;


	//-------------------------------------------------------------------------
	// Playback related functions
	//
	struct FSeekParam
	{
		FSeekParam()
		{
			Reset();
		}
		void Reset()
		{
			Time.SetToInvalid();
			StartingBitrate.Reset();
			bOptimizeForScrubbing.Reset();
		}
		// Time to seek to.
		FTimeValue Time;
		// New sequence index to associate with the newly decoded samples.
		TOptional<int32> NewSequenceIndex;
		// Maximum stream bitrate to use when seeking.
		TOptional<int32> StartingBitrate;
		// Optimize for frame scrubbing (faster display of frame at target time)?
		TOptional<bool> bOptimizeForScrubbing;
	};

	/**
	 * Seek to a new position and play from there. This includes first playstart.
	 * Playback is initially paused on first player use and must be resumed to begin.
	 * Query the seekable range (GetSeekableRange()) to get the valid time range.
	 *
	 * If the seek-to time is not set the seek will start at the beginning for
	 * on-demand presentations and on the Live edge for Live presentations.
	 *
	 * Seeks can be issued while a seek is already executing. The seek parameters
	 * control behaviour. If seeking is performed for scrubbing any new seek will
	 * be performed only when the previous seek has completed, otherwise the current
	 * seek will be canceled in favor of the new.
	 * If the new position being seeked to is within the specified distance to the
	 * last completed seek a new seek will not be performed.
	 * If new seeks are performed in rapid succession (as in scrubbing) not every
	 * new position will be seeked to. Seek commands are aggregated and the position
	 * set with the most recent call will be used as soon as any previous seek completes.
	 * As a result you will NOT receive a ReportSeekCompleted() notification for
	 * every seek requested, only for those that were executed and allowed to complete.
	 */
	virtual void SeekTo(const FSeekParam& NewPosition) = 0;

	//! Pauses playback.
	virtual void Pause() = 0;

	//! Resumes playback.
	virtual void Resume() = 0;

	//! Stops playback. Playback cannot be resumed. Final player events will be sent to registered listeners.
	virtual void Stop() = 0;


	struct FPlaybackRange
	{
		TOptional<FTimeValue>	Start;
		TOptional<FTimeValue>	End;
	};

	/**
	 * Constrains playback to the specified time range, which should be a subset of GetTimelineRange().
	 * The playback range can be specified via URL fragment parameters on the URL given to LoadManifest()
	 * if the mime type allows for it.
	 *
	 * If you set the playback range before calling LoadManifest() the URL parameter will not be used.
	 * Otherwise the URL parameters set the playback range if they are specified.
	 * You can query the playback range set by URL parameters as soon as HaveMetadata() returns true.
	 * Setting a playback range through this method overrides URL parameters.
	 *
	 * To set or change only the start or end of the playback range, set only the corresponding TOptional<>
	 * and leave the other unset.
	 * To disable either start or end set the value to FTimeValue::GetInvalid().
	 *
	 * If you only set start or end, the other value may be set by the respective URL parameter.
	 * To fully disable any playback range that may be present on the URL you should set the range to
	 * invalid values once HaveMetadata() returns true.
	 *
	 * Setting a playback range during playback will result in an immediate seek to the current
	 * playback position. Frame accurate seeking is recommended.
	 */
	virtual void SetPlaybackRange(const FPlaybackRange& InPlaybackRange) = 0;
	virtual void GetPlaybackRange(FPlaybackRange& OutPlaybackRange) = 0;


	struct FLoopParam
	{
		FLoopParam()
		{
			Reset();
		}
		void Reset()
		{
			bEnableLooping = false;
		}
		bool	bEnableLooping;
	};

	//! Puts playback into loop mode if possible. Live streams cannot be made to loop as they have infinite duration. Looping is constrained to the playback range, if one is set.
	virtual void SetLooping(const FLoopParam& InLoopParams) = 0;


	//-------------------------------------------------------------------------
	// "Trickplay" related functions
	//
	enum class EPlaybackRateType
	{
		// Smooth playback, not dropping any frames.
		Unthinned,
		// Dropping frames
		Thinned
	};
	/**
	 * Returns ranges the playback rate can be set to.
	 * There are two types, one `Unthinned`, where (ideally) no frames will be dropped and
	 * `Thinned`, where frames will be dropped to maintain the playback rate.
	 *
	 * Either way, the player will not resample audio or generate interpolated frames of video.
	 * Decoded samples will be delivered to the renderer as they are. It is also up to the
	 * renderers to consume the data at a faster or slower rate to actually realize the desired
	 * playback rate.
	 *
	 * The range of supported rates depends on the type of media and the decoder capability to
	 * decode faster than realtime. If the media allows for adaptive bitrate selection the player
	 * will choose the stream to play back accordingly.
	 * In thinned mode it will need to drop and not decode samples, so the renderer will not
	 * receive all the possible data.
	 *
	 * Live streams will only allow for rates of 0.0 (pause) and 1.0 (real time play forward).
	 */
	virtual TRangeSet<double> GetSupportedRates(EPlaybackRateType InForPlayRateType) = 0;

	struct FTrickplayParams
	{
	};
	virtual void SetPlayRate(double InDesiredPlayRate, const FTrickplayParams& InParameters) = 0;
	virtual double GetPlayRate() const = 0;


	//-------------------------------------------------------------------------
	// Error related functions
	//
	//! Returns the error that has caused playback issues.
	virtual FErrorDetail GetError() const = 0;


	//-------------------------------------------------------------------------
	// State functions
	//
	//! Returns whether or not a manifest has been loaded and assigned yet.
	virtual bool HaveMetadata() const = 0;
	//! Returns the duration of the video. Returns invalid time when there is nothing to play. Returns positive infinite for live streams.
	virtual FTimeValue GetDuration() const = 0;
	//! Returns the current play position. Returns invalid time when there is nothing to play.
	virtual FTimeValue GetPlayPosition() const = 0;
	//! Returns the seekable range.
	virtual void GetSeekableRange(FTimeRange& OutRange) const = 0;
	//! Returns the timeline range.
	virtual void GetTimelineRange(FTimeRange& OutRange) const = 0;
	//! Returns true when playback has finished.
	virtual bool HasEnded() const = 0;
	//! Returns true when data is being buffered/rebuffered, false otherwise.
	virtual bool IsBuffering() const = 0;
	//! Returns true when seeking is in progress. False if not.
	virtual bool IsSeeking() const = 0;
	//! Returns true when playing back, false if not.
	virtual bool IsPlaying() const = 0;
	//! Returns true when paused, false if not.
	virtual bool IsPaused() const = 0;

	struct FLoopState
	{
		int64	Count = 0;					//!< Number of times playback jumped back to loop. 0 on first playthrough, 1 on first loop, etc.
		bool	bIsEnabled = false;			//!< true if looping is enabled, false if not.
	};

	//! Returns the current loop state.
	virtual void GetLoopState(FLoopState& OutLoopState) const = 0;

	//! Returns track metadata of the currently active play period.
	virtual void GetTrackMetadata(TArray<FTrackMetadata>& OutTrackMetadata, EStreamType StreamType) const = 0;

	//! Returns attributes of the currently selected track.
	virtual void GetSelectedTrackAttributes(FStreamSelectionAttributes& OutAttributes, EStreamType StreamType) const = 0;

	struct FEventReplayState
	{
		FTimeValue ProposedJoinTime;
		bool bIsReplayEvent = false;
	};
	//! Returns if the media is set to be a replay (configured via URL fragment parameters)
	virtual void GetReplayEventState(FEventReplayState& OutReplayEventState) const = 0;


	//-------------------------------------------------------------------------
	// Manual stream selection functions
	//

	//! Sets the highest bitrate when selecting a candidate stream.
	virtual void SetBitrateCeiling(int32 HighestSelectableBitrate) = 0;

	//! Sets the maximum resolution to use. Set both to 0 to disable, set only one to limit widht or height only.
	//! Setting both will limit on either width or height, whichever limits first.
	virtual void SetMaxResolution(int32 MaxWidth, int32 MaxHeight) = 0;

	//! Selects a track based on given attributes (which can be constructed from one of the array members returned by GetTrackMetadata()).
	//! This selection will explicitly override the initial stream attributes set by SetInitialStreamAttributes() and be applied automatically for upcoming periods.
	virtual void SelectTrackByAttributes(EStreamType StreamType, const FStreamSelectionAttributes& Attributes) = 0;

	//! Deselect track. The stream may continue to stream to allow for immediate selection/activation but no data will be fed to the decoder.
	virtual void DeselectTrack(EStreamType StreamType) = 0;

	//! Returns true if the track stream of the specified type has beed deselected through DeselectTrack().
	virtual bool IsTrackDeselected(EStreamType StreamType) = 0;


	//-------------------------------------------------------------------------
	// Buffer related functions
	//
	struct FStreamBufferInfo
	{
		TArray<FTimeRange> TimeAvailable;		//!< Time range currently available in the buffer
		TArray<FTimeRange> TimeRequested;		//!< Time range requested for download.
		TArray<FTimeRange> TimeEnqueued;		//!< Time range already enqueued with the renderer.
		bool bIsBufferActive = false;			//!< true if the buffer is active, false if not (eg. track not selected)
	};
	virtual void QueryStreamBufferInfo(FStreamBufferInfo& OutStreamBufferInfo, EStreamType InStreamType) = 0;


	//-------------------------------------------------------------------------
	// Special functions needed for application suspend & resume.
	//
	virtual void SuspendOrResumeDecoders(bool bSuspend, const FParamDict& InOptions) = 0;


	//-------------------------------------------------------------------------
	// Debug functions
	//
	static void DebugHandle(void* pPlayer, void (*debugDrawPrintf)(void* pPlayer, const char *pFmt, ...));

protected:
	virtual ~IAdaptiveStreamingPlayer() = default;
};


} // namespace Electra
