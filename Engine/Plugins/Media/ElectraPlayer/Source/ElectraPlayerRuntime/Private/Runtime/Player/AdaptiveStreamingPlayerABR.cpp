// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/AdaptiveStreamingPlayerABR.h"

#include "Player/Manifest.h"
#include "Player/PlaybackTimeline.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/ABRRules/ABRInternal.h"
#include "Player/ABRRules/ABRVoDPlus.h"
#include "Player/ABRRules/ABRLiveStream.h"
#include "Player/ABRRules/ABRFixedStream.h"
#include "Player/CMCDHandler.h"

#include "StreamAccessUnitBuffer.h"
#include "Utilities/Utilities.h"
#include "Utilities/URLParser.h"


namespace Electra
{

	class FAdaptiveStreamSelector : public IAdaptiveStreamSelector, public IABRInfoInterface
	{
	public:
		FAdaptiveStreamSelector(IPlayerSessionServices* PlayerSessionServices, IPlayerABRLiveControl* InPlayerLiveControl);
		virtual ~FAdaptiveStreamSelector();
		void SetFormatType(EMediaFormatType FormatType) override;
		void SetBandwidthCeiling(int32 HighestManifestBitrate) override;
		void SetMaxVideoResolution(int32 MaxWidth, int32 MaxHeight) override;
		void SetCurrentPlaybackPeriod(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPlayPeriod) override;
		void SetBandwidth(int64 bitsPerSecond) override;
		void SetForcedNextBandwidth(int64 bitsPerSecond, double minBufferTimeBeforePlayback) override;
		FTimeValue GetMinBufferTimeForPlayback(EMinBufferType InBufferingType, FTimeValue InDefaultMBT) override;
		FRebufferAction GetRebufferAction() override;
		EHandlingAction PeriodicHandle() override;
		void MarkStreamAsUnavailable(const FDenylistedStream& DenylistedStream) override;
		void MarkStreamAsAvailable(const FDenylistedStream& NoLongerDenylistedStream) override;
		void PathwayChanged(const FString& InNewPathway) override;
		int64 GetLastBandwidth() override;
		int64 GetAverageBandwidth() override;
		int64 GetAverageThroughput() override;
		double GetAverageLatency() override;

		void DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...)) override;

		ESegmentAction SelectSuitableStreams(FTimeValue& OutDelay, TSharedPtrTS<const IStreamSegment> CurrentSegment) override;

		FABRDownloadProgressDecision ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override;
		void ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override;
		void ReportBufferingStart(Metrics::EBufferingReason BufferingReason) override;
		void ReportBufferingEnd(Metrics::EBufferingReason BufferingReason) override;
		void ReportOpenSource(const FString& URL) override {}
		void ReportReceivedMainPlaylist(const FString& EffectiveURL) override {}
		void ReportReceivedPlaylists() override {}
		void ReportTracksChanged() override {}
		void ReportPlaylistDownload(const Metrics::FPlaylistDownloadStats& PlaylistDownloadStats) override {}
		void ReportBandwidth(int64 EffectiveBps, int64 ThroughputBps, double LatencyInSeconds) override {}
		void ReportBufferUtilization(const Metrics::FBufferStats& BufferStats) override {}
		void ReportSegmentDownload(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) override {}
		void ReportLicenseKey(const Metrics::FLicenseKeyStats& LicenseKeyStats) override {}
		void ReportDataAvailabilityChange(const Metrics::FDataAvailabilityChange& DataAvailability) override {}
		void ReportVideoQualityChange(int32 NewBitrate, int32 PreviousBitrate, bool bIsDrasticDownswitch) override {}
		void ReportAudioQualityChange(int32 NewBitrate, int32 PreviousBitrate, bool bIsDrasticDownswitch) override {}
		void ReportDecodingFormatChange(const FStreamCodecInformation& NewDecodingFormat) override {}
		void ReportPrerollStart() override {}
		void ReportPrerollEnd() override {}
		void ReportPlaybackStart() override {}
		void ReportPlaybackPaused() override;
		void ReportPlaybackResumed() override;
		void ReportPlaybackEnded() override;
		void ReportJumpInPlayPosition(const FTimeValue& ToNewTime, const FTimeValue& FromTime, Metrics::ETimeJumpReason TimejumpReason) override {}
		void ReportPlaybackStopped() override {}
		void ReportSeekCompleted() override {}
		void ReportMediaMetadataChanged(TSharedPtrTS<FMetadataParser> Metadata) override {}
		void ReportError(const FString& ErrorReason) override {}
		void ReportLogMessage(IInfoLog::ELevel InLogLevel, const FString& LogMessage, int64 PlayerWallclockMilliseconds) override {}
		void ReportVideoFrameStats(const Metrics::FFrameStats& InFrameStats) override {}
		void ReportAudioFrameStats(const Metrics::FFrameStats& InFrameStats) override {}

	private:
		bool HaveOptionValue(const FName& InOption) override
		{ return PlayerSessionServices->HaveOptionValue(InOption); }
		const FVariantValue GetOptionValue(const FName& InOption) override
		{ return PlayerSessionServices->GetOptionValue(InOption); }
		void LogMessage(IInfoLog::ELevel Level, const FString& Message) override
		{ PlayerSessionServices->PostLog(Facility::EFacility::ABR, Level, Message); }
		TSharedPtrTS<FABRStreamInformation> GetStreamInformation(const Metrics::FSegmentDownloadStats& FromDownloadStats) override;
		const TArray<TSharedPtrTS<FABRStreamInformation>>& GetStreamInformations(EStreamType InForStreamType) override;
		FStreamCodecInformation::FResolution GetMaxStreamResolution() override
		{ return MaxStreamResolution; }
		int32 GetBandwidthCeiling() override
		{ return BandwidthCeiling; }

		void UpdateCurrentPlaybackPeriods();
		ESegmentAction SelectSuitableStreamByType(FTimeValue& OutDelay, TSharedPtrTS<const IStreamSegment> CurrentSegment, TSharedPtrTS<IManifest::IPlayPeriod> CurrentPlayPeriod, EStreamType StreamType);

		FTimeValue ABRGetPlayPosition() const override
		{ return PlayerLiveControl->ABRGetPlayPosition(); }
		FTimeRange ABRGetTimeline() const override
		{ return PlayerLiveControl->ABRGetTimeline(); }
		FTimeValue ABRGetWallclockTime() const override
		{ return PlayerLiveControl->ABRGetWallclockTime(); }
		bool ABRIsLive() const override
		{ return PlayerLiveControl->ABRIsLive(); }
		bool ABRShouldPlayOnLiveEdge() const override
		{ return PlayerLiveControl->ABRShouldPlayOnLiveEdge(); }
		TSharedPtrTS<const FLowLatencyDescriptor> ABRGetLowLatencyDescriptor() const override
		{ return PlayerLiveControl->ABRGetLowLatencyDescriptor(); }
		FTimeValue ABRGetDesiredLiveEdgeLatency() const override
		{ return PlayerLiveControl->ABRGetDesiredLiveEdgeLatency(); }
		FTimeValue ABRGetLatency() const override
		{ return PlayerLiveControl->ABRGetLatency(); }
		FTimeValue ABRGetPlaySpeed() const override
		{ return PlayerLiveControl->ABRGetPlaySpeed(); }
		void ABRGetStreamBufferStats(IPlayerABRLiveControl::FABRBufferStats& OutBufferStats, EStreamType ForStream) override
		{ PlayerLiveControl->ABRGetStreamBufferStats(OutBufferStats, ForStream); }
		FTimeRange ABRGetSupportedRenderRateScale() override
		{ return PlayerLiveControl->ABRGetSupportedRenderRateScale(); }
		void ABRSetRenderRateScale(double InRenderRateScale) override
		{ PlayerLiveControl->ABRSetRenderRateScale(InRenderRateScale); }
		double ABRGetRenderRateScale() const override
		{ return PlayerLiveControl->ABRGetRenderRateScale(); }
		void ABRTriggerClockSync(IPlayerABRLiveControl::EClockSyncType InClockSyncType) override
		{ PlayerLiveControl->ABRTriggerClockSync(InClockSyncType); }
		void ABRTriggerPlaylistRefresh() override
		{ PlayerLiveControl->ABRTriggerPlaylistRefresh(); }


		IPlayerSessionServices*								PlayerSessionServices;
		IPlayerABRLiveControl*								PlayerLiveControl;

		FCriticalSection									AccessMutex;
		TSharedPtrTS<IManifest::IPlayPeriod>				CurrentPlayPeriodVideo;
		TSharedPtrTS<IManifest::IPlayPeriod>				CurrentPlayPeriodAudio;

		TArray<TSharedPtrTS<FABRStreamInformation>>			StreamInformationVideo;
		TArray<TSharedPtrTS<FABRStreamInformation>>			StreamInformationAudio;
		FString												CurrentVideoAdaptationSetID;
		FString												CurrentAudioAdaptationSetID;

		TArray<FDenylistedStream>							DenylistedExternally;

		FStreamCodecInformation::FResolution				MaxStreamResolution;
		int32												BandwidthCeiling = 0x7fffffff;

		TUniquePtr<IABRRule>								ABRMethod;
	};




	//-----------------------------------------------------------------------------
	/**
	 * Create an instance of this class
	 *
	 * @param InPlayerSessionServices
	 *
	 * @return
	 */
	TSharedPtrTS<IAdaptiveStreamSelector> IAdaptiveStreamSelector::Create(IPlayerSessionServices* InPlayerSessionServices)
	{
		return TSharedPtrTS<IAdaptiveStreamSelector>(new FAdaptiveStreamSelector(InPlayerSessionServices, InPlayerSessionServices->GetPlayerARBControl()));
	}

	//-----------------------------------------------------------------------------
	/**
	 * CTOR
	 *
	 * @param InPlayerSessionServices
	 * @param InPlayerLiveControl
	 */
	FAdaptiveStreamSelector::FAdaptiveStreamSelector(IPlayerSessionServices* InPlayerSessionServices, IPlayerABRLiveControl* InPlayerLiveControl)
		: PlayerSessionServices(InPlayerSessionServices)
		, PlayerLiveControl(InPlayerLiveControl)
	{
		MaxStreamResolution.Width = MaxStreamResolution.Height = 32768;
	}

	//-----------------------------------------------------------------------------
	/**
	 * DTOR
	 */
	FAdaptiveStreamSelector::~FAdaptiveStreamSelector()
	{
	}

	//-----------------------------------------------------------------------------
	/**
	 * Sets the presentation format (HLS, DASH, mp4, etc...)
	 *
	 * This must be the first call that will determine the type of ABR algorithm to use.
	 *
	 * @param InFormatType
	 */
	void FAdaptiveStreamSelector::SetFormatType(EMediaFormatType InFormatType)
	{
		// Multiplexed stream with no bitrate switching alternatives?
		if (InFormatType == EMediaFormatType::ISOBMFF || InFormatType == EMediaFormatType::MKV)
		{
			ABRMethod.Reset(IABRFixedStream::Create(this, InFormatType, EABRPresentationType::OnDemand));
		}
		else
		{
			EABRPresentationType PresentationType;
			// On-demand presentation?
			if (!PlayerLiveControl->ABRIsLive())
			{
				PresentationType = EABRPresentationType::OnDemand;
				ABRMethod.Reset(IABROnDemandPlus::Create(this, InFormatType, PresentationType));
			}
			// Live / low-latency Live
			else
			{
				PresentationType = PlayerLiveControl->ABRGetLowLatencyDescriptor().IsValid() ? EABRPresentationType::LowLatency : EABRPresentationType::Live;
				ABRMethod.Reset(IABRLiveStream::Create(this, InFormatType, PresentationType));
			}
		}
	}

	void FAdaptiveStreamSelector::UpdateCurrentPlaybackPeriods()
	{
		FScopeLock lock(&AccessMutex);
		if (CurrentPlayPeriodVideo.IsValid())
		{
			TSharedPtrTS<ITimelineMediaAsset> Asset = CurrentPlayPeriodVideo->GetMediaAsset();
			if (Asset.IsValid())
			{
				CurrentVideoAdaptationSetID = CurrentPlayPeriodVideo->GetSelectedAdaptationSetID(EStreamType::Video);
				StreamInformationVideo.Empty();
				// Get video stream information
				if (!CurrentVideoAdaptationSetID.IsEmpty())
				{
					for(int32 nA=0, maxA=Asset->GetNumberOfAdaptationSets(EStreamType::Video); nA<maxA; ++nA)
					{
						TSharedPtrTS<IPlaybackAssetAdaptationSet> Adapt = Asset->GetAdaptationSetByTypeAndIndex(EStreamType::Video, nA);
						if (CurrentVideoAdaptationSetID.Equals(Adapt->GetUniqueIdentifier()))
						{
							bool bLowLatency = Adapt->IsLowLatencyEnabled();
							for(int32 i=0, iMax=Adapt->GetNumberOfRepresentations(); i<iMax; ++i)
							{
								TSharedPtrTS<IPlaybackAssetRepresentation> Repr = Adapt->GetRepresentationByIndex(i);
								TSharedPtrTS<FABRStreamInformation> si = MakeSharedTS<FABRStreamInformation>();
								si->AdaptationSetUniqueID = Adapt->GetUniqueIdentifier();
								si->RepresentationUniqueID = Repr->GetUniqueIdentifier();
								si->Bitrate = Repr->GetBitrate();
								si->QualityIndex = Repr->GetQualityIndex();
								si->bLowLatencyEnabled = bLowLatency;
								if (Repr->GetCodecInformation().IsVideoCodec())
								{
									si->Resolution = Repr->GetCodecInformation().GetResolution();
								}
								StreamInformationVideo.Push(si);
							}
							// Sort the representations by ascending bitrate
							StreamInformationVideo.StableSort([](const TSharedPtrTS<FABRStreamInformation>& a, const TSharedPtrTS<FABRStreamInformation>& b)
							{
								return a->Bitrate < b->Bitrate;
							});

							// Set the lowest and highest bitrate with the CMCD handler
							PlayerSessionServices->GetCMCDHandler()->SetPlayableBitrateRange(EStreamType::Video, StreamInformationVideo.Num() ? StreamInformationVideo[0]->Bitrate : 0, StreamInformationVideo.Num() ? StreamInformationVideo.Last()->Bitrate : 0);
							break;
						}
					}
				}
			}
		}
		else
		{
			CurrentVideoAdaptationSetID.Empty();
			StreamInformationVideo.Empty();
		}

		if (CurrentPlayPeriodAudio.IsValid())
		{
			TSharedPtrTS<ITimelineMediaAsset> Asset = CurrentPlayPeriodAudio->GetMediaAsset();
			if (Asset.IsValid())
			{
				CurrentAudioAdaptationSetID = CurrentPlayPeriodAudio->GetSelectedAdaptationSetID(EStreamType::Audio);
				StreamInformationAudio.Empty();
				// Get audio stream information
				if (!CurrentAudioAdaptationSetID.IsEmpty())
				{
					for(int32 nA=0, maxA=Asset->GetNumberOfAdaptationSets(EStreamType::Audio); nA<maxA; ++nA)
					{
						TSharedPtrTS<IPlaybackAssetAdaptationSet> Adapt = Asset->GetAdaptationSetByTypeAndIndex(EStreamType::Audio, nA);
						if (CurrentAudioAdaptationSetID.Equals(Adapt->GetUniqueIdentifier()))
						{
							bool bLowLatency = Adapt->IsLowLatencyEnabled();
							for(int32 i=0, iMax=Adapt->GetNumberOfRepresentations(); i<iMax; ++i)
							{
								TSharedPtrTS<IPlaybackAssetRepresentation> Repr = Adapt->GetRepresentationByIndex(i);
								TSharedPtrTS<FABRStreamInformation> si = MakeSharedTS<FABRStreamInformation>();
								si->AdaptationSetUniqueID = Adapt->GetUniqueIdentifier();
								si->RepresentationUniqueID = Repr->GetUniqueIdentifier();
								si->Bitrate = Repr->GetBitrate();
								si->QualityIndex = Repr->GetQualityIndex();
								si->bLowLatencyEnabled = bLowLatency;
								StreamInformationAudio.Push(si);
							}
							// Sort the representations by ascending bitrate
							StreamInformationAudio.StableSort([](const TSharedPtrTS<FABRStreamInformation>& a, const TSharedPtrTS<FABRStreamInformation>& b)
							{
								return a->Bitrate < b->Bitrate;
							});

							// Set the lowest and highest bitrate with the CMCD handler
							PlayerSessionServices->GetCMCDHandler()->SetPlayableBitrateRange(EStreamType::Audio, StreamInformationAudio.Num() ? StreamInformationAudio[0]->Bitrate : 0, StreamInformationAudio.Num() ? StreamInformationAudio.Last()->Bitrate : 0);
							break;
						}
					}
				}
			}
		}
		else
		{
			CurrentAudioAdaptationSetID.Empty();
			StreamInformationAudio.Empty();
		}

		// Notify the ABR method that there is a different set of representations now.
		if (ABRMethod.IsValid())
		{
			ABRMethod->RepresentationsChanged(EStreamType::Video, CurrentPlayPeriodVideo);
			ABRMethod->RepresentationsChanged(EStreamType::Audio, CurrentPlayPeriodAudio);
		}
	}

	//-----------------------------------------------------------------------------
	/**
	 * Sets the playback period from which to select streams now.
	 *
	 * @param InStreamType
	 * @param InCurrentPlayPeriod
	 */
	void FAdaptiveStreamSelector::SetCurrentPlaybackPeriod(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> InCurrentPlayPeriod)
	{
		FScopeLock lock(&AccessMutex);
		if (InStreamType == EStreamType::Video)
		{
			CurrentPlayPeriodVideo = InCurrentPlayPeriod;
		}
		else if (InStreamType == EStreamType::Audio)
		{
			CurrentPlayPeriodAudio = InCurrentPlayPeriod;
		}
		UpdateCurrentPlaybackPeriods();
	}

	//-----------------------------------------------------------------------------
	/**
	 * Sets the initial bandwidth, either from guessing, past history or a current measurement.
	 *
	 * @param InBitsPerSecond
	 */
	void FAdaptiveStreamSelector::SetBandwidth(int64 InBitsPerSecond)
	{
		check(ABRMethod.IsValid());
		ABRMethod->SetBandwidth(InBitsPerSecond);
	}

	//-----------------------------------------------------------------------------
	/**
	 * Sets a forced bitrate for the next segment fetches until the given duration of playable content has been received.
	 * The ABR algorithm may or may not use this.
	 *
	 * @param InBitsPerSecond
	 * @param InMinBufferTimeBeforePlayback
	 */
	void FAdaptiveStreamSelector::SetForcedNextBandwidth(int64 InBitsPerSecond, double InMinBufferTimeBeforePlayback)
	{
		check(ABRMethod.IsValid());
		ABRMethod->SetForcedNextBandwidth(InBitsPerSecond, InMinBufferTimeBeforePlayback);
	}

	//-----------------------------------------------------------------------------
	/**
	 * Sets a highest bandwidth limit for a stream type. Call with 0 to disable.
	 *
	 * @param InHighestManifestBitrate
	 */
	void FAdaptiveStreamSelector::SetBandwidthCeiling(int32 InHighestManifestBitrate)
	{
		FScopeLock lock(&AccessMutex);
		BandwidthCeiling = InHighestManifestBitrate > 0 ? InHighestManifestBitrate : 0x7fffffff;
	}

	//-----------------------------------------------------------------------------
	/**
	 * Limits video resolution.
	 *
	 * @param InMaxWidth
	 * @param InMaxHeight
	 */
	void FAdaptiveStreamSelector::SetMaxVideoResolution(int32 InMaxWidth, int32 InMaxHeight)
	{
		FScopeLock lock(&AccessMutex);
		MaxStreamResolution.Width = InMaxWidth;
		MaxStreamResolution.Height = InMaxHeight;
	}

	//-----------------------------------------------------------------------------
	/**
	 * Called to (temporarily) mark a stream as unavailable.
	 * This is used in formats like HLS when a dedicated stream playlist is in error
	 * and thus segment information for this stream is not available.
	 *
	 * @param InDenylistedStream
	 */
	void FAdaptiveStreamSelector::MarkStreamAsUnavailable(const FDenylistedStream& InDenylistedStream)
	{
		FScopeLock lock(&AccessMutex);
		DenylistedExternally.Push(InDenylistedStream);
	}

	//-----------------------------------------------------------------------------
	/**
	 * Marks a previously set to unavailable stream as being available again.
	 * Usually when the stream specific playlist (in HLS) has become available again.
	 *
	 * @param InNoLongerDenylistedStream
	 */
	void FAdaptiveStreamSelector::MarkStreamAsAvailable(const FDenylistedStream& InNoLongerDenylistedStream)
	{
		FScopeLock lock(&AccessMutex);
		DenylistedExternally.Remove(InNoLongerDenylistedStream);
	}


	//-----------------------------------------------------------------------------
	/**
	 * Called when content steering has changed to a different pathway (HLS)
	 * This _could_ mean that a whole different set of streams is suddenly available
	 * to select from, including their IDs, so we need to update our internal
	 * knowledge about what is available.
	 */
	void FAdaptiveStreamSelector::PathwayChanged(const FString& InNewPathway)
	{
		UpdateCurrentPlaybackPeriods();
	}

	//-----------------------------------------------------------------------------
	/**
	 * Returns the last measured bandwidth sample in bits per second.
	 */
	int64 FAdaptiveStreamSelector::GetLastBandwidth()
	{
		return ABRMethod.IsValid() ? ABRMethod->GetLastBandwidth() : 0;
	}

	//-----------------------------------------------------------------------------
	/**
	 * Returns the average measured bandwidth sample in bits per second.
	 */
	int64 FAdaptiveStreamSelector::GetAverageBandwidth()
	{
		return ABRMethod.IsValid() ? ABRMethod->GetAverageBandwidth() : 0;
	}

	//-----------------------------------------------------------------------------
	/**
	 * Returns the average measured throughput in bits per second.
	 */
	int64 FAdaptiveStreamSelector::GetAverageThroughput()
	{
		return ABRMethod.IsValid() ? ABRMethod->GetAverageThroughput() : 0;
	}

	//-----------------------------------------------------------------------------
	/**
	 * Returns the average measured download latency in seconds.
	 */
	double FAdaptiveStreamSelector::GetAverageLatency()
	{
		return ABRMethod.IsValid() ? ABRMethod->GetAverageLatency() : 0.0;
	}


	IAdaptiveStreamSelector::EHandlingAction FAdaptiveStreamSelector::PeriodicHandle()
	{
		return ABRMethod.IsValid() ? ABRMethod->PeriodicHandle() : IAdaptiveStreamSelector::EHandlingAction::None;
	}

	IAdaptiveStreamSelector::FRebufferAction FAdaptiveStreamSelector::GetRebufferAction()
	{
		return ABRMethod.IsValid() ? ABRMethod->GetRebufferAction() : IAdaptiveStreamSelector::FRebufferAction();
	}

	FTimeValue FAdaptiveStreamSelector::GetMinBufferTimeForPlayback(IAdaptiveStreamSelector::EMinBufferType InBufferingType, FTimeValue InDefaultMBT)
	{
		return ABRMethod.IsValid() ? ABRMethod->GetMinBufferTimeForPlayback(InBufferingType, InDefaultMBT) : FTimeValue();
	}

	void FAdaptiveStreamSelector::DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...))
	{
		if (ABRMethod.IsValid())
		{
			ABRMethod->DebugPrint(pThat, pPrintFN);
		}
	}


	void FAdaptiveStreamSelector::ReportPlaybackPaused()
	{
		if (ABRMethod.IsValid())
		{
			ABRMethod->ReportPlaybackPaused();
		}
	}

	void FAdaptiveStreamSelector::ReportPlaybackResumed()
	{
		if (ABRMethod.IsValid())
		{
			ABRMethod->ReportPlaybackResumed();
		}
	}

	void FAdaptiveStreamSelector::ReportPlaybackEnded()
	{
		if (ABRMethod.IsValid())
		{
			ABRMethod->ReportPlaybackEnded();
		}
	}

	FABRDownloadProgressDecision FAdaptiveStreamSelector::ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
	{
		check(ABRMethod.IsValid());
		return ABRMethod->ReportDownloadProgress(SegmentDownloadStats);
	}

	void FAdaptiveStreamSelector::ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
	{
		check(ABRMethod.IsValid());
		ABRMethod->ReportDownloadEnd(SegmentDownloadStats);
	}

	void FAdaptiveStreamSelector::ReportBufferingStart(Metrics::EBufferingReason BufferingReason)
	{
		check(ABRMethod.IsValid());
		ABRMethod->ReportBufferingStart(BufferingReason);
	}

	void FAdaptiveStreamSelector::ReportBufferingEnd(Metrics::EBufferingReason BufferingReason)
	{
		check(ABRMethod.IsValid());
		ABRMethod->ReportBufferingEnd(BufferingReason);
	}

	const TArray<TSharedPtrTS<FABRStreamInformation>>& FAdaptiveStreamSelector::GetStreamInformations(EStreamType InForStreamType)
	{
		return InForStreamType == EStreamType::Video ? StreamInformationVideo : StreamInformationAudio;
	}

	TSharedPtrTS<FABRStreamInformation> FAdaptiveStreamSelector::GetStreamInformation(const Metrics::FSegmentDownloadStats& FromDownloadStats)
	{
		FScopeLock lock(&AccessMutex);
		const TArray<TSharedPtrTS<FABRStreamInformation>>* StreamInfos = nullptr;
		if (FromDownloadStats.StreamType == EStreamType::Video)
		{
			StreamInfos = &StreamInformationVideo;
		}
		else if (FromDownloadStats.StreamType == EStreamType::Audio)
		{
			StreamInfos = &StreamInformationAudio;
		}
		if (StreamInfos)
		{
			for(int32 i=0, iMax=StreamInfos->Num(); i<iMax; ++i)
			{
				if (FromDownloadStats.AdaptationSetID == (*StreamInfos)[i]->AdaptationSetUniqueID &&
					FromDownloadStats.RepresentationID == (*StreamInfos)[i]->RepresentationUniqueID &&
					//FromDownloadStats.CDN              == (*StreamInfos)[i]->CDN &&
					1)
				{
					return (*StreamInfos)[i];
				}
			}
		}
		return nullptr;
	}

	//-----------------------------------------------------------------------------
	/**
	 * Chooses a stream suitable for streaming the next segment from based on network conditions.
	 *
	 * @param OutDelay
	 * @param CurrentSegment
	 *
	 * @return Action to take for the next segment, which is to load it, retry due to a failure or skip it.
	 */
	IAdaptiveStreamSelector::ESegmentAction FAdaptiveStreamSelector::SelectSuitableStreams(FTimeValue& OutDelay, TSharedPtrTS<const IStreamSegment> CurrentSegment)
	{
		FTimeValue TimeNow = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();

		// No delay to fetch this segment with.
		OutDelay.SetToZero();

		// Check if streams that were temporarily marked as bad can now be used again.
		AccessMutex.Lock();
		for(int32 i=0, iMax=StreamInformationVideo.Num(); i<iMax; ++i)
		{
			if (StreamInformationVideo[i]->Health.BecomesAvailableAgainAtUTC <= TimeNow)
			{
				StreamInformationVideo[i]->Health.BecomesAvailableAgainAtUTC.SetToZero();
			}
		}
		for(int32 i=0, iMax=StreamInformationAudio.Num(); i<iMax; ++i)
		{
			if (StreamInformationAudio[i]->Health.BecomesAvailableAgainAtUTC <= TimeNow)
			{
				StreamInformationAudio[i]->Health.BecomesAvailableAgainAtUTC.SetToZero();
			}
		}
		AccessMutex.Unlock();

		// Get the currently active play period of the finished segment, or if it is the first segment the most appropriate one.
		TSharedPtrTS<IManifest::IPlayPeriod> CurrentPlayPeriod;
		EStreamType StreamType = CurrentSegment.IsValid() ? CurrentSegment->GetType() : StreamInformationVideo.Num() ? EStreamType::Video : EStreamType::Audio;
		if (StreamType == EStreamType::Video)
		{
			CurrentPlayPeriod = CurrentPlayPeriodVideo;
		}
		else if (StreamType == EStreamType::Audio)
		{
			CurrentPlayPeriod = CurrentPlayPeriodAudio;
		}
		// No play period, nothing to do. Assume there is something coming up, so continue.
		if (!CurrentPlayPeriod.IsValid())
		{
			return ESegmentAction::FetchNext;
		}

		// Specific request?
		if (CurrentSegment.IsValid())
		{
			return SelectSuitableStreamByType(OutDelay, CurrentSegment, CurrentPlayPeriod, StreamType);
		}
		else
		{
			// Initial request. If there are no streams at all we fail.
			if (StreamInformationVideo.IsEmpty() && StreamInformationAudio.IsEmpty())
			{
				return IAdaptiveStreamSelector::ESegmentAction::Fail;
			}
			ESegmentAction ActionV = ESegmentAction::FetchNext;
			ESegmentAction ActionA = ESegmentAction::FetchNext;
			if (StreamInformationVideo.Num())
			{
				ActionV = SelectSuitableStreamByType(OutDelay, CurrentSegment, CurrentPlayPeriodVideo, EStreamType::Video);
			}
			if (StreamInformationAudio.Num())
			{
				ActionA = SelectSuitableStreamByType(OutDelay, CurrentSegment, CurrentPlayPeriodAudio, EStreamType::Audio);
			}
			if (ActionV == IAdaptiveStreamSelector::ESegmentAction::Fail || ActionA == IAdaptiveStreamSelector::ESegmentAction::Fail)
			{
				return IAdaptiveStreamSelector::ESegmentAction::Fail;
			}
			return StreamInformationVideo.Num() ? ActionV : ActionA;
		}
	}

	IAdaptiveStreamSelector::ESegmentAction FAdaptiveStreamSelector::SelectSuitableStreamByType(FTimeValue& OutDelay, TSharedPtrTS<const IStreamSegment> CurrentSegment, TSharedPtrTS<IManifest::IPlayPeriod> CurrentPlayPeriod, EStreamType StreamType)
	{
		FTimeValue TimeNow = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();

		// Get the initial list of representations that are currently possible to choose from.
		TArray<TSharedPtrTS<FABRStreamInformation>> CandidateRepresentations;
		ABRMethod->PrepareStreamCandidateList(CandidateRepresentations, StreamType, CurrentPlayPeriod, TimeNow);
		// No streams, nothing to do. Assume there is something coming up, so continue.
		if (CandidateRepresentations.Num() == 0)
		{
			return ESegmentAction::FetchNext;
		}

		// First have the ABR method evaluate this download for possible errors and decide on an action.
		IAdaptiveStreamSelector::ESegmentAction NextSegmentAction = ABRMethod->EvaluateForError(CandidateRepresentations, StreamType, OutDelay, CurrentPlayPeriod, CurrentSegment, TimeNow);
		// If there is an error action like retry, fill or fail return it right away.
		if (NextSegmentAction != IAdaptiveStreamSelector::ESegmentAction::FetchNext)
		{
			return NextSegmentAction;
		}

		// Check if the state of a stream has become unusable.
		TSharedPtrTS<ITimelineMediaAsset> Asset = CurrentPlayPeriod->GetMediaAsset();
		if (Asset.IsValid())
		{
			FString AdaptationSetId = CurrentPlayPeriod->GetSelectedAdaptationSetID(StreamType);
			if (!AdaptationSetId.IsEmpty())
			{
				for(int32 nA=0, maxA=Asset->GetNumberOfAdaptationSets(StreamType); nA<maxA; ++nA)
				{
					TSharedPtrTS<IPlaybackAssetAdaptationSet> Adapt = Asset->GetAdaptationSetByTypeAndIndex(StreamType, nA);
					if (!AdaptationSetId.Equals(Adapt->GetUniqueIdentifier()))
					{
						continue;
					}
					for(int32 i=0; i<CandidateRepresentations.Num(); ++i)
					{
						TSharedPtrTS<IPlaybackAssetRepresentation> Repr = Adapt->GetRepresentationByUniqueIdentifier(CandidateRepresentations[i]->RepresentationUniqueID);
						if (Repr.IsValid() && !Repr->CanBePlayed())
						{
							CandidateRepresentations.RemoveAt(i);
							--i;
						}
					}
					break;
				}
			}
		}

		// Evaluate which stream qualities could potentially be used and select one.
		NextSegmentAction = ABRMethod->EvaluateForQuality(CandidateRepresentations, StreamType, OutDelay, CurrentPlayPeriod, CurrentSegment, TimeNow);
		if (NextSegmentAction != IAdaptiveStreamSelector::ESegmentAction::FetchNext)
		{
			return NextSegmentAction;
		}

		// Filter out streams that are currently not healthy.
		for(int32 i=0; i<CandidateRepresentations.Num(); ++i)
		{
			if (!CandidateRepresentations[i]->Health.BecomesAvailableAgainAtUTC.IsValid() ||
				(CandidateRepresentations[i]->Health.BecomesAvailableAgainAtUTC.IsValid() && CandidateRepresentations[i]->Health.BecomesAvailableAgainAtUTC > TimeNow))
			{
				CandidateRepresentations.RemoveAt(i);
				--i;
			}
		}

		// Remove those that have been externally denylisted. This may include all of them.
		AccessMutex.Lock();
		if (DenylistedExternally.Num())
		{
			for(int32 j=0; j<CandidateRepresentations.Num(); ++j)
			{
				for(int32 i=0, iMax=DenylistedExternally.Num(); i<iMax; ++i)
				{
					if (CandidateRepresentations[j]->AdaptationSetUniqueID == DenylistedExternally[i].AdaptationSetUniqueID &&
						CandidateRepresentations[j]->RepresentationUniqueID == DenylistedExternally[i].RepresentationUniqueID)
					{
						CandidateRepresentations.RemoveAt(j);
						--j;
						break;
					}
				}
			}
		}
		AccessMutex.Unlock();

		// From whatever is left, select what is deemed the best stream.
		return ABRMethod->PerformSelection(CandidateRepresentations, StreamType, OutDelay, CurrentPlayPeriod, CurrentSegment, TimeNow);
	}


} // namespace Electra

