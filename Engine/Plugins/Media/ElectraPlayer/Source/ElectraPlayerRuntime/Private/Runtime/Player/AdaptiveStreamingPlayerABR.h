// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "StreamTypes.h"
#include "HTTP/HTTPManager.h"
#include "Player/Manifest.h"
#include "Player/PlayerSessionServices.h"
#include "Player/AdaptiveStreamingPlayerABR_State.h"
#include "Player/AdaptiveStreamingPlayerMetrics.h"

namespace Electra
{

class IPlayerABRLiveControl
{
public:
	virtual ~IPlayerABRLiveControl() = default;

	// Provides the current playback position.
	virtual FTimeValue ABRGetPlayPosition() const = 0;

	// Provides the current full media timeline, not adjusted for seeking.
	virtual FTimeRange ABRGetTimeline() const = 0;

	// Provides the current wallclock time (server real time).
	virtual FTimeValue ABRGetWallclockTime() const = 0;

	// Returns true if this is a Live presentation, false for VoD.
	virtual bool ABRIsLive() const = 0;

	// Returns true if playback should be on the Live edge. false if the user intentionally seeked away from the edge.
	virtual bool ABRShouldPlayOnLiveEdge() const = 0;

	// Provides the current low latency descriptor if low latency playback is active. Returns nullptr if not.
	virtual TSharedPtrTS<const FLowLatencyDescriptor> ABRGetLowLatencyDescriptor() const = 0;

	// Provides the desired latency to the live edge for both low-latency and normal live playback.
	virtual FTimeValue ABRGetDesiredLiveEdgeLatency() const = 0;

	// Provides the currently measured latency to the live edge.
	virtual FTimeValue ABRGetLatency() const = 0;

	// Provides the current playback speed.
	virtual FTimeValue ABRGetPlaySpeed() const = 0;

	// Returns the current stream access unit buffer stats.
	struct FABRBufferStats
	{
		FTimeValue PlayableContentDuration;
		bool bReachedEnd = false;
		bool bEndOfTrack = false;
	};
	virtual void ABRGetStreamBufferStats(FABRBufferStats& OutBufferStats, EStreamType ForStream) = 0;

	virtual FTimeRange ABRGetSupportedRenderRateScale() = 0;
	virtual void ABRSetRenderRateScale(double InRenderRateScale) = 0;
	virtual double ABRGetRenderRateScale() const = 0;

	enum class EClockSyncType
	{
		Recommended,
		Required
	};
	virtual void ABRTriggerClockSync(EClockSyncType InClockSyncType) = 0;

	virtual void ABRTriggerPlaylistRefresh() = 0;
};


class IAdaptiveStreamSelector : public IAdaptiveStreamingPlayerMetrics
{
public:
	//! Create an instance of this class
	static TSharedPtrTS<IAdaptiveStreamSelector> Create(IPlayerSessionServices* PlayerSessionServices);
	virtual ~IAdaptiveStreamSelector() = default;

	virtual void SetFormatType(EMediaFormatType FormatType) = 0;

	//! Sets the manifest from which to select streams.
	virtual void SetCurrentPlaybackPeriod(EStreamType InStreamType, TSharedPtrTS<IManifest::IPlayPeriod> CurrentPlayPeriod) = 0;

	//! Sets the initial bandwidth, either from guessing, past history or a current measurement.
	virtual void SetBandwidth(int64 bitsPerSecond) = 0;

	//! Sets a forced bitrate for the next segment fetches until the given duration of playable content has been received.
	virtual void SetForcedNextBandwidth(int64 bitsPerSecond, double minBufferTimeBeforePlayback) = 0;

	enum class EMinBufferType
	{
		Initial,
		Seeking,
		Rebuffering
	};
	virtual FTimeValue GetMinBufferTimeForPlayback(EMinBufferType InBufferingType, FTimeValue InDefaultMBT) = 0;

	enum class EHandlingAction
	{
		None,
		SeekToLive
	};

	virtual EHandlingAction PeriodicHandle() = 0;

	struct FRebufferAction
	{
		enum class EAction
		{
			Restart,
			ContinueLoading,
			GoToLive,
			ThrowError
		};
		EAction Action = EAction::Restart;
		int32 SuggestedRestartBitrate = 0;
	};

	// Call when rebuffering occurs to get the recommended action. This implies that rebuffering has actually occurred.
	// The ABR may take internal action to switch quality when this is called!
	virtual FRebufferAction GetRebufferAction() = 0;

	struct FDenylistedStream
	{
		FString		AssetUniqueID;
		FString		AdaptationSetUniqueID;
		FString		RepresentationUniqueID;
		FString		CDN;
		bool operator == (const FDenylistedStream& rhs) const
		{
			return AssetUniqueID == rhs.AssetUniqueID &&
				   AdaptationSetUniqueID == rhs.AdaptationSetUniqueID &&
				   RepresentationUniqueID == rhs.RepresentationUniqueID &&
				   CDN == rhs.CDN;
		}
	};
	virtual void MarkStreamAsUnavailable(const FDenylistedStream& DenylistedStream) = 0;
	virtual void MarkStreamAsAvailable(const FDenylistedStream& NoLongerDenylistedStream) = 0;

	virtual void PathwayChanged(const FString& InNewPathway) = 0;

	virtual FABRDownloadProgressDecision ReportDownloadProgress(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) = 0;
	virtual void ReportDownloadEnd(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) = 0;


	//! Returns the last measured bandwidth sample in bps.
	virtual int64 GetLastBandwidth() = 0;

	//! Returns the average measured bandwidth in bps.
	virtual int64 GetAverageBandwidth() = 0;

	//! Returns the average measured throughput in bps.
	virtual int64 GetAverageThroughput() = 0;

	//! Returns the average measured latency in seconds.
	virtual double GetAverageLatency() = 0;

	//! Sets a highest bandwidth limit. Call with 0 to disable.
	virtual void SetBandwidthCeiling(int32 HighestManifestBitrate) = 0;

	//! Limits video resolution.
	virtual void SetMaxVideoResolution(int32 MaxWidth, int32 MaxHeight) = 0;

	enum class ESegmentAction
	{
		FetchNext,				//!< Fetch the next segment normally.
		Retry,					//!< Retry the same segment. Another quality or CDN may have been picked out already.
		Fill,					//!< Fill the segment's duration worth with dummy data.
		Fail					//!< Abort playback
	};

	//! Selects a feasible stream from the stream set to fetch the next fragment from.
	virtual ESegmentAction SelectSuitableStreams(FTimeValue& OutDelay, TSharedPtrTS<const IStreamSegment> CurrentSegment) = 0;


	virtual void DebugPrint(void* pThat, void (*pPrintFN)(void* pThat, const char *pFmt, ...)) = 0;
};


} // namespace Electra


