// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{

struct FABRDownloadProgressDecision
{
	FABRDownloadProgressDecision()
		: Flags(EDecisionFlags::eABR_Continue)
	{
	}
	void Reset()
	{
		Flags = EDecisionFlags::eABR_Continue;
		Reason.Empty();
	}
	enum EDecisionFlags
	{
		eABR_Continue			= 0,
		eABR_AbortDownload		= 1 << 0,
		eABR_EmitPartialData	= 1 << 1,
		eABR_InsertFillerData	= 1 << 2,
	};
	EDecisionFlags		Flags;
	FString				Reason;
};

} // namespace Electra


