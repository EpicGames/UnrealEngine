// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/AdaptiveStreamingPlayerInternal.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/AdaptivePlayerMetadataKeynames.h"
#include "Decoder/SubtitleDecoder.h"
#include "Decoder/AudioDecoder.h"
#include "Utilities/Utilities.h"
#include "ParameterDictionary.h"


namespace Electra
{
namespace DecoderOptionKeys
{
static const FName SendEmptySubtitleDuringGaps(TEXT("sendEmptySubtitleDuringGaps"));
}

// AU memory
void* FAdaptiveStreamingPlayer::AUAllocate(IAccessUnitMemoryProvider::EDataType type, SIZE_T NumBytes, SIZE_T Alignment)
{
	if (Alignment)
	{
		return FMemory::Malloc(NumBytes, Alignment);
	}
	return FMemory::Malloc(NumBytes);
}

void FAdaptiveStreamingPlayer::AUDeallocate(IAccessUnitMemoryProvider::EDataType type, void* Address)
{
	FMemory::Free(Address);
}

IPlayerStreamFilter* FAdaptiveStreamingPlayer::GetStreamFilter()
{
	return this;
}

const FCodecSelectionPriorities& FAdaptiveStreamingPlayer::GetCodecSelectionPriorities(EStreamType ForStream)
{
	switch(ForStream)
	{
		case EStreamType::Video:
		{
			return CodecPrioritiesVideo;
		}
		case EStreamType::Audio:
		{
			return CodecPrioritiesAudio;
		}
		case EStreamType::Subtitle:
		{
			return CodecPrioritiesSubtitles;
		}
		default:
		{
			static FCodecSelectionPriorities Dummy;
			return Dummy;
		}
	}
}



bool FAdaptiveStreamingPlayer::CanDecodeStream(const FStreamCodecInformation& InStreamCodecInfo) const
{
	auto IsCodecPrefixExcluded = [](const FString& InCodec, const TArray<FString>& InExcludedPrefixes) -> bool
	{
		for(auto &Prefix : InExcludedPrefixes)
		{
			if (InCodec.StartsWith(Prefix, ESearchCase::IgnoreCase))
			{
				return true;
			}
		}
		return false;
	};


	if (InStreamCodecInfo.IsVideoCodec())
	{
		// Check against user specified codec prefix exclusions
		if (IsCodecPrefixExcluded(InStreamCodecInfo.GetCodecSpecifierRFC6381(), ExcludedVideoDecoderPrefixes))
		{
			return false;
		}

		// Check with (soon to be deprecated) limits
		if (!Deprecate_InternalStreamAllowedAsPerLimits(InStreamCodecInfo))
		{
			return false;
		}

		return IVideoDecoder::CanDecodeStream(InStreamCodecInfo);
	}
	else if (InStreamCodecInfo.IsAudioCodec())
	{
		// Check against user specified codec prefix exclusions
		if (IsCodecPrefixExcluded(InStreamCodecInfo.GetCodecSpecifierRFC6381(), ExcludedAudioDecoderPrefixes))
		{
			return false;
		}
		return IAudioDecoder::CanDecodeStream(InStreamCodecInfo);
	}
	else if (InStreamCodecInfo.IsSubtitleCodec())
	{
		// Check against user specified codec prefix exclusions
		if (IsCodecPrefixExcluded(InStreamCodecInfo.GetCodecSpecifierRFC6381(), ExcludedSubtitleDecoderPrefixes))
		{
			return false;
		}
	}

	return true;
}


bool FAdaptiveStreamingPlayer::CanDecodeSubtitle(const FString& MimeType, const FString& Codec) const
{
	return ISubtitleDecoder::IsSupported(MimeType, Codec);
}




bool FAdaptiveStreamingPlayer::GetHighestVideoProfileResolution(FStreamCodecInformation& OutStreamInfo, TSharedPtrTS<const FBufferSourceInfo> InBufferSource, const FTimeValue& AtTime, const FStreamCodecInformation& InForCodec)
{
	if (!InBufferSource)
	{
		return false;
	}

	// Locate the period by ID or the specified time.
	ActivePeriodCriticalSection.Lock();
	if (ActivePeriods.Num() == 0)
	{
		ActivePeriodCriticalSection.Unlock();
		return false;
	}
	TSharedPtrTS<ITimelineMediaAsset> Asset;
	const FString& PeriodID(InBufferSource->PeriodID);
	if (PeriodID.Len())
	{
		for(int32 i=0; i<ActivePeriods.Num(); ++i)
		{
			if (ActivePeriods[i].ID.Equals(PeriodID))
			{
				Asset = ActivePeriods[i].Period;
				break;
			}
		}
	}
	if (!Asset && AtTime.IsValid())
	{
		for(int32 i=0; i<ActivePeriods.Num(); ++i)
		{
			if (AtTime >= ActivePeriods[i].TimeRange.Start && AtTime < (ActivePeriods[i].TimeRange.End.IsValid() ? ActivePeriods[i].TimeRange.End : FTimeValue::GetPositiveInfinity()))
			{
				Asset = ActivePeriods[i].Period;
			}
		}
	}
	if (!Asset)
	{
		Asset = ActivePeriods[0].Period;
	}
	ActivePeriodCriticalSection.Unlock();
	if (Asset.IsValid())
	{
		TSharedPtrTS<IPlaybackAssetAdaptationSet> VideoAdaptationSet;
		if (InBufferSource->HardIndex >= 0 && InBufferSource->HardIndex < Asset->GetNumberOfAdaptationSets(EStreamType::Video))
		{
			VideoAdaptationSet = Asset->GetAdaptationSetByTypeAndIndex(EStreamType::Video, InBufferSource->HardIndex);
		}
		if (VideoAdaptationSet && VideoAdaptationSet->GetNumberOfRepresentations())
		{
			TArray<FStreamCodecInformation> VideoCodecInfos;
			for(int32 i=0, iMax=VideoAdaptationSet->GetNumberOfRepresentations(); i<iMax; ++i)
			{
				VideoCodecInfos.Push(VideoAdaptationSet->GetRepresentationByIndex(i)->GetCodecInformation());
			}
			// The representations could be a mix of profile and resolutions, although not very likely.
			// We consider profile, level and resolution separately, which may produce an amalgamation of values that
			// does not appear as such on any particular stream. We do need to know the worst profile and resolution
			// that can occur.
			OutStreamInfo = VideoCodecInfos[0];
			int32 Profile = VideoCodecInfos[0].GetProfile();
			int32 Level = VideoCodecInfos[0].GetProfileLevel();
			for(int32 i=1; i<VideoCodecInfos.Num(); ++i)
			{
				if (VideoCodecInfos[i].GetResolution().Width * VideoCodecInfos[i].GetResolution().Height > OutStreamInfo.GetResolution().Width * OutStreamInfo.GetResolution().Height)
				{
					OutStreamInfo = VideoCodecInfos[i];
				}
				if (VideoCodecInfos[i].GetProfile() > Profile)
				{
					Profile = VideoCodecInfos[i].GetProfile();
				}
				if (VideoCodecInfos[i].GetProfileLevel() > Level)
				{
					Level = VideoCodecInfos[i].GetProfileLevel();
				}
			}
			OutStreamInfo.SetProfile(Profile);
			OutStreamInfo.SetProfileLevel(Level);
			return true;
		}
	}
	return false;
}

//-----------------------------------------------------------------------------
/**
 * Creates a decoder for the specified stream type based on the first access
 * unit's format.
 *
 * @return
 */
int32 FAdaptiveStreamingPlayer::CreateDecoder(EStreamType type)
{
	if (IsTrackDeselected(type))
	{
		return 0;
	}
	if (type == EStreamType::Video)
	{
		if (VideoDecoder.Decoder == nullptr)
		{
			FAccessUnit* AccessUnit = nullptr;
			bool bOk = GetCurrentOutputStreamBuffer(type).IsValid() && GetCurrentOutputStreamBuffer(type)->PeekAndAddRef(AccessUnit);
			if (AccessUnit)
			{
				FTimeValue DecodeTime = AccessUnit->EarliestPTS.IsValid() ? AccessUnit->EarliestPTS : AccessUnit->PTS;
				check(AccessUnit->BufferSourceInfo.IsValid());
				VideoDecoder.CurrentCodecInfo.Clear();
				VideoDecoder.LastBufferSourceInfo = AccessUnit->BufferSourceInfo;
				if (AccessUnit->AUCodecData.IsValid())
				{
					VideoDecoder.CurrentCodecInfo = AccessUnit->AUCodecData->ParsedInfo;
					VideoDecoder.LastSentAUCodecData = AccessUnit->AUCodecData;
					GetMediaInfoDictionary().Set(MediaInfoVideoBitrateCurrent.Resolve(), FVariantValue((int64)VideoDecoder.CurrentCodecInfo.GetBitrate()));
					DispatchEvent(FMetricEvent::ReportCodecFormatChange(VideoDecoder.CurrentCodecInfo));
				}

				// Get the largest stream resolution of the currently selected video adaptation set.
				// This is only an initial selection as there could be other adaptation sets in upcoming periods
				// that have a larger resolution that is still within the allowed limits.
				FStreamCodecInformation HighestStream;
				bOk = GetHighestVideoProfileResolution(HighestStream, AccessUnit->BufferSourceInfo, DecodeTime, VideoDecoder.CurrentCodecInfo);
				FAccessUnit::Release(AccessUnit);
				if (!bOk)
				{
					FErrorDetail err;
					err.SetFacility(Facility::EFacility::Player);
					err.SetMessage("Stream information not found when creating video decoder");
					err.SetCode(INTERR_NO_STREAM_INFORMATION);
					PostError(err);
					return -1;
				}

				// Create the video decoder
				VideoDecoder.Decoder = IVideoDecoder::Create();
				if (VideoDecoder.Decoder)
				{
					VideoDecoder.Parent = this;
					VideoDecoder.Decoder->SetPlayerSessionServices(this);
					// Attach video decoder buffer monitor.
					VideoDecoder.Decoder->SetAUInputBufferListener(&VideoDecoder);
					VideoDecoder.Decoder->SetReadyBufferListener(&VideoDecoder);
					// Where the video decoder shall send its output to
					VideoDecoder.Decoder->SetOutputHandler(VideoOutput.OutputHandler);
					// Open the decoder after having set all listeners.
					VideoDecoder.Decoder->Open(VideoDecoder.LastSentAUCodecData, PlayerOptions.GetDictionary(), &HighestStream);
				}

				VideoDecoder.CheckIfNewDecoderMustBeSuspendedImmediately();
				if (VideoDecoder.Decoder)
				{
					// Apply the current limit.
					UpdateStreamResolutionLimit();
				}
				else
				{
					FErrorDetail err;
					err.SetFacility(Facility::EFacility::Player);
					err.SetMessage("Unsupported video codec");
					err.SetCode(INTERR_UNSUPPORTED_CODEC);
					PostError(err);
					return -1;
				}
			}
		}
		return 0;
	}
	else if (type == EStreamType::Audio)
	{
		if (AudioDecoder.Decoder == nullptr)
		{
			FAccessUnit* AccessUnit = nullptr;
			bool bOk = GetCurrentOutputStreamBuffer(type).IsValid() && GetCurrentOutputStreamBuffer(type)->PeekAndAddRef(AccessUnit);
			if (AccessUnit)
			{
				AudioDecoder.CurrentCodecInfo.Clear();
				if (AccessUnit->AUCodecData.IsValid())
				{
					AudioDecoder.CurrentCodecInfo = AccessUnit->AUCodecData->ParsedInfo;
					AudioDecoder.LastSentAUCodecData = AccessUnit->AUCodecData;
					GetMediaInfoDictionary().Set(MediaInfoAudioBitrateCurrent.Resolve(), FVariantValue((int64)AudioDecoder.CurrentCodecInfo.GetBitrate()));
					DispatchEvent(FMetricEvent::ReportCodecFormatChange(AudioDecoder.CurrentCodecInfo));
				}
				FAccessUnit::Release(AccessUnit);

				// Create the audio decoder
				AudioDecoder.Decoder = IAudioDecoder::Create();
				if (AudioDecoder.Decoder)
				{
					AudioDecoder.Parent = this;
					AudioDecoder.Decoder->SetPlayerSessionServices(this);
					// Attach buffer monitors.
					AudioDecoder.Decoder->SetAUInputBufferListener(&AudioDecoder);
					AudioDecoder.Decoder->SetReadyBufferListener(&AudioDecoder);
					// Where the audio decoder shall send its output to
					AudioDecoder.Decoder->SetOutputHandler(AudioOutput.OutputHandler);
					// Open the decoder after having set all listeners.
					AudioDecoder.Decoder->Open(AudioDecoder.LastSentAUCodecData);
				}

				AudioDecoder.CheckIfNewDecoderMustBeSuspendedImmediately();
				if (!AudioDecoder.Decoder)
				{
					FErrorDetail err;
					err.SetFacility(Facility::EFacility::Player);
					err.SetMessage("Unsupported audio codec");
					err.SetCode(INTERR_UNSUPPORTED_CODEC);
					PostError(err);
					return -1;
				}
			}
		}
		return 0;
	}
	else if (type == EStreamType::Subtitle)
	{
		if (SubtitleDecoder.Decoder == nullptr)
		{
			FAccessUnit* AccessUnit = nullptr;
			bool bOk = GetCurrentOutputStreamBuffer(type).IsValid() && GetCurrentOutputStreamBuffer(type)->PeekAndAddRef(AccessUnit);
			if (AccessUnit)
			{
				SubtitleDecoder.CurrentCodecInfo.Clear();
				if (AccessUnit->AUCodecData.IsValid())
				{
					SubtitleDecoder.CurrentCodecInfo = AccessUnit->AUCodecData->ParsedInfo;
					SubtitleDecoder.LastSentAUCodecData = AccessUnit->AUCodecData;
					DispatchEvent(FMetricEvent::ReportCodecFormatChange(SubtitleDecoder.CurrentCodecInfo));
				}
				// Create subtitle decoder for this type of AU.
				SubtitleDecoder.Decoder = ISubtitleDecoder::Create(AccessUnit);
				FAccessUnit::Release(AccessUnit);
				if (SubtitleDecoder.Decoder)
				{
					FParamDict Options;
					Options.Set(DecoderOptionKeys::SendEmptySubtitleDuringGaps, Electra::FVariantValue(true));

					SubtitleDecoder.Decoder->SetPlayerSessionServices(this);
					SubtitleDecoder.Decoder->SetMediaSamples(CurrentMediaSamples);
					SubtitleDecoder.Decoder->Open(Options);
					// Start the decoder, if decoding is currently running.
					if (DecoderState == EDecoderState::eDecoder_Running)
					{
						SubtitleDecoder.Start();
					}
				}
				else
				{
					FErrorDetail err;
					err.SetFacility(Facility::EFacility::Player);
					err.SetMessage("Unsupported subtitle codec");
					err.SetCode(INTERR_UNSUPPORTED_CODEC);
					PostError(err);
					return -1;
				}
			}
		}
		return 0;
	}
	return -1;
}


//-----------------------------------------------------------------------------
/**
 * Destroys the decoders.
 */
void FAdaptiveStreamingPlayer::DestroyDecoders()
{
/*
NOTE: We do not clear out the renderers from the decoder. On their way down the decoders should still be able
      to access the renderer without harm and dispatch their last remaining data.

	if (mVideoDecoder.Decoder)
		mVideoDecoder.Decoder->SetRenderer(nullptr);
	if (mAudioDecoder.Decoder)
		mAudioDecoder.Decoder->SetRenderer(nullptr);
*/
	AudioDecoder.Close();
	VideoDecoder.Close();
	SubtitleDecoder.Close();
}


//-----------------------------------------------------------------------------
/**
 * Check if the decoders need to change.
 */
void FAdaptiveStreamingPlayer::HandleDecoderChanges()
{
	if (VideoDecoder.bDrainingForCodecChange && VideoDecoder.bDrainingForCodecChangeDone)
	{
		VideoDecoder.Close();
		VideoDecoder.bDrainingForCodecChange = false;
		VideoDecoder.bDrainingForCodecChangeDone = false;
	}
	if (SubtitleDecoder.bRequireCodecChange)
	{
		SubtitleDecoder.Close();
		SubtitleDecoder.bRequireCodecChange = false;
	}
	CreateDecoder(EStreamType::Video);
	CreateDecoder(EStreamType::Audio);
	CreateDecoder(EStreamType::Subtitle);
}


void FAdaptiveStreamingPlayer::HandleSubtitleDecoder()
{
	// Feeds access units from the subtitle buffer to the subtitle decoder.
	// Since this is very small and sparse data the decoder does not run in a dedicated thread like
	// the video and audio decoders do.
	if (SubtitleDecoder.Decoder && !bIsTextDeselected)
	{
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> CurrentTxtOutputBuffer = GetCurrentOutputStreamBuffer(EStreamType::Subtitle);
		FAccessUnit* PeekedAU = nullptr;
		if (CurrentTxtOutputBuffer.IsValid())
		{
			CurrentTxtOutputBuffer->PeekAndAddRef(PeekedAU);
		}
		FTimeValue NextPTS = PeekedAU ? PeekedAU->PTS - SubtitleDecoder.Decoder->GetStreamedDeliveryTimeOffset() : FTimeValue::GetInvalid();
		FAccessUnit::Release(PeekedAU);
		FTimeValue CurrentPos = PlaybackState.GetPlayPosition();
		if (NextPTS.IsValid() && CurrentPos >= NextPTS)
		{
			FeedDecoder(EStreamType::Subtitle, SubtitleDecoder.Decoder, false);
		}

		if (CurrentPos.IsValid() && RenderClock->IsRunning())
		{
			ActivePeriodCriticalSection.Lock();
			FTimeValue LocalPos;
			for(int32 i=0, iMax=ActivePeriods.Num(); i<iMax; ++i)
			{
				if (CurrentPos >= ActivePeriods[i].TimeRange.Start && ((i+1 == iMax) || CurrentPos < ActivePeriods[i+1].TimeRange.Start))
				{
					LocalPos = CurrentPos - ActivePeriods[i].TimeRange.Start;
					break;
				}
			}
			ActivePeriodCriticalSection.Unlock();
			SubtitleDecoder.Decoder->UpdatePlaybackPosition(CurrentPos, LocalPos);
		}
	}
}



void FAdaptiveStreamingPlayer::VideoDecoderInputNeeded(const IAccessUnitBufferListener::FBufferStats &currentInputBufferStats)
{
	DiagnosticsCriticalSection.Lock();
	VideoBufferStats.DecoderInputBuffer = currentInputBufferStats;
	DiagnosticsCriticalSection.Unlock();
	if (!VideoDecoder.bDrainingForCodecChange)
	{
		EDecoderState decState = DecoderState;
		if (decState == EDecoderState::eDecoder_Running)
		{
			FeedDecoder(EStreamType::Video, VideoDecoder.Decoder, true);
		}
	}
}

void FAdaptiveStreamingPlayer::VideoDecoderOutputReady(const IDecoderOutputBufferListener::FDecodeReadyStats &currentReadyStats)
{
	DiagnosticsCriticalSection.Lock();
	VideoBufferStats.DecoderOutputBuffer = currentReadyStats;
	// When the output is not stalled we need to update the stall duration immediately to reset the counter.
	// Otherwise we may miss the not stalled transition when stalled -> not stalled -> stalled
	if (!currentReadyStats.bOutputStalled)
	{
		// The time here is irrelevant.
		VideoBufferStats.UpdateStalledDuration(0);
	}
	DiagnosticsCriticalSection.Unlock();
}


void FAdaptiveStreamingPlayer::AudioDecoderInputNeeded(const IAccessUnitBufferListener::FBufferStats &currentInputBufferStats)
{
	DiagnosticsCriticalSection.Lock();
	AudioBufferStats.DecoderInputBuffer = currentInputBufferStats;
	DiagnosticsCriticalSection.Unlock();
	EDecoderState decState = DecoderState;
	if (decState == EDecoderState::eDecoder_Running)
	{
		FeedDecoder(EStreamType::Audio, AudioDecoder.Decoder, true);
	}
}

void FAdaptiveStreamingPlayer::AudioDecoderOutputReady(const IDecoderOutputBufferListener::FDecodeReadyStats &currentReadyStats)
{
	DiagnosticsCriticalSection.Lock();
	AudioBufferStats.DecoderOutputBuffer = currentReadyStats;
	// When the output is not stalled we need to update the stall duration immediately to reset the counter.
	// Otherwise we may miss the not stalled transition when stalled -> not stalled -> stalled
	if (!currentReadyStats.bOutputStalled)
	{
		// The time here is irrelevant.
		AudioBufferStats.UpdateStalledDuration(0);
	}
	DiagnosticsCriticalSection.Unlock();
}



//-----------------------------------------------------------------------------
/**
 * Sends an available AU to a decoder.
 * If the current buffer level is below the underrun threshold an underrun
 * message is sent to the worker thread.
 *
 * @param Type
 * @param Decoder
 * @param bHandleUnderrun
 */
void FAdaptiveStreamingPlayer::FeedDecoder(EStreamType Type, IAccessUnitBufferInterface* Decoder, bool bHandleUnderrun)
{
	TSharedPtrTS<FMultiTrackAccessUnitBuffer> FromMultistreamBuffer = GetCurrentOutputStreamBuffer(Type);
	if (!FromMultistreamBuffer.IsValid())
	{
		return;
	}
	// If there is a seek pending we do not feed the decoders since their output will be discarded
	// by the media sample queues anyway due to non-matching sequence indices.
	// This has the side effect of quickly draining our buffers, possibly resulting in a buffer
	// underrun that will not even apply due to the upcoming seek.
	if (SeekVars.HasPendingSeek())
	{
		return;
	}

	FBufferStats* pStats = nullptr;
	FStreamCodecInformation* CurrentCodecInfo = nullptr;
	TSharedPtrTS<FAccessUnit::CodecData>* LastSentAUCodecData = nullptr;
	TSharedPtrTS<const FBufferSourceInfo>* LastBufferSourceInfo = nullptr;

	bool bCodecChangeDetected = false;
	bool bIsDeselected = false;

	switch(Type)
	{
		case EStreamType::Video:
		{
			pStats = &VideoBufferStats;
			CurrentCodecInfo = &VideoDecoder.CurrentCodecInfo;
			LastSentAUCodecData = &VideoDecoder.LastSentAUCodecData;
			LastBufferSourceInfo = &VideoDecoder.LastBufferSourceInfo;
			bIsDeselected = bIsVideoDeselected;
			break;
		}
		case EStreamType::Audio:
		{
			pStats = &AudioBufferStats;
			CurrentCodecInfo = &AudioDecoder.CurrentCodecInfo;
			LastSentAUCodecData = &AudioDecoder.LastSentAUCodecData;
			//LastBufferSourceInfo = &AudioDecoder.LastBufferSourceInfo;
			bIsDeselected = bIsAudioDeselected;
			break;
		}
		case EStreamType::Subtitle:
		{
			pStats = &TextBufferStats;
			CurrentCodecInfo = &SubtitleDecoder.CurrentCodecInfo;
			LastSentAUCodecData = &SubtitleDecoder.LastSentAUCodecData;
			//LastBufferSourceInfo = &SubtitleDecoder.LastBufferSourceInfo;
			bIsDeselected = bIsTextDeselected;
			break;
		}
		default:
		{
			checkNoEntry();
			return;
		}
	}

	// Lock the AU buffer for the duration of this function to ensure this can never clash with a Flush() call
	// since we are checking size, eod state and subsequently popping an AU, for which the buffer must stay consistent inbetween!
	// Also to ensure the active buffer doesn't get changed from one track to another.
	FMultiTrackAccessUnitBuffer::FScopedLock lock(FromMultistreamBuffer);

	// Is the buffer (the Type of elementary stream actually) active/selected?
	if (!bIsDeselected)
	{
		// Check for buffer underrun.
		if (bHandleUnderrun && RebufferCause == ERebufferCause::None && CurrentState == EPlayerState::eState_Playing && StreamState == EStreamState::eStream_Running && PipelineState == EPipelineState::ePipeline_Running)
		{
			const FTimeValue MinPushedDuration(FTimeValue::MillisecondsToHNS(500));
			bool bEODSet = FromMultistreamBuffer->IsEODFlagSet();
			bool bEOTSet = FromMultistreamBuffer->IsEndOfTrack();
			FTimeValue PushedDuration = FromMultistreamBuffer->GetPlayableDurationPushedSinceEOT();
			if (!bEODSet && !bEOTSet && FromMultistreamBuffer->Num() == 0 && PushedDuration >= MinPushedDuration)
			{
				int32 NumEnqueuedSamples = 0;
				if (Type == EStreamType::Video && VideoOutput.OutputHandler.IsValid())
				{
					TArray<IOutputHandlerBase::FEnqueuedSampleInfo> sampleInfos;
					VideoOutput.OutputHandler->GetEnqueuedSampleInfo(sampleInfos);
					NumEnqueuedSamples = sampleInfos.Num();
				}
				else if (Type == EStreamType::Audio && AudioOutput.OutputHandler.IsValid())
				{
					TArray<IOutputHandlerBase::FEnqueuedSampleInfo> sampleInfos;
					AudioOutput.OutputHandler->GetEnqueuedSampleInfo(sampleInfos);
					NumEnqueuedSamples = sampleInfos.Num();
				}
				if (NumEnqueuedSamples <= 1)
				{
					// Buffer underrun.
					RebufferCause = FromMultistreamBuffer->HasPendingTrackSwitch() ? ERebufferCause::TrackswitchUnderrun : ERebufferCause::Underrun;
					FTimeValue LastKnownPTS = FromMultistreamBuffer->GetLastPoppedPTS();
					// Only set the 'rebuffer at' time if we have a valid last known PTS. If we don't
					// then maybe this is a cascade failure from a previous rebuffer attempt for which
					// we then try that time once more.
					if (LastKnownPTS.IsValid())
					{
						RebufferDetectedAtPlayPos = LastKnownPTS;
					}
					WorkerThread.EnqueueBufferUnderrun();
				}
			}
		}

		FAccessUnit* PeekedAU = nullptr;
		FromMultistreamBuffer->PeekAndAddRef(PeekedAU);
		if (PeekedAU)
		{
			// Change in codec?
			if (CurrentCodecInfo && PeekedAU->AUCodecData.IsValid())
			{
				// Actual different codec?
				if (PeekedAU->AUCodecData->ParsedInfo.GetCodec() != CurrentCodecInfo->GetCodec())
				{
					bCodecChangeDetected = true;
				}
				// Mimetype change in subtitles?
				if (Type == EStreamType::Subtitle && !PeekedAU->AUCodecData->ParsedInfo.GetMimeType().Equals(CurrentCodecInfo->GetMimeType()))
				{
					bCodecChangeDetected = true;
				}
				// Change in period of the video stream that may have a new maximum resolution from the one before?
				if (!bCodecChangeDetected && Type == EStreamType::Video && VideoDecoder.Decoder && LastBufferSourceInfo && PeekedAU->BufferSourceInfo.IsValid() &&
					(((*LastBufferSourceInfo)->PeriodID != PeekedAU->BufferSourceInfo->PeriodID) || ((*LastBufferSourceInfo)->PeriodAdaptationSetID != PeekedAU->BufferSourceInfo->PeriodAdaptationSetID)))
				{
					// Get the new highest stream properties and check if the decoder can still handle those.
					FStreamCodecInformation HighestStream;
					if (GetHighestVideoProfileResolution(HighestStream, PeekedAU->BufferSourceInfo, PeekedAU->EarliestPTS.IsValid() ? PeekedAU->EarliestPTS : PeekedAU->PTS, PeekedAU->AUCodecData->ParsedInfo))
					{
						if (!VideoDecoder.Decoder->Reopen(PeekedAU->AUCodecData, PlayerOptions.GetDictionary(), &HighestStream))
						{
							bCodecChangeDetected = true;
						}
					}
				}
			}

			// Is there a change?
			if (bCodecChangeDetected)
			{
				if (Decoder)
				{
					// Check type of stream. We can currently change the video and subtitle codecs only.
					if (Type == EStreamType::Video)
					{
						if (VideoDecoder.Decoder && !VideoDecoder.bDrainingForCodecChange)
						{
							VideoDecoder.bDrainingForCodecChange = true;
							VideoDecoder.Decoder->DrainForCodecChange();
						}
					}
					else if (Type == EStreamType::Subtitle)
					{
						SubtitleDecoder.bRequireCodecChange = true;
					}
					else
					{
						FErrorDetail err;
						err.SetFacility(Facility::EFacility::Player);
						err.SetMessage("Codec change not supported");
						err.SetCode(INTERR_CODEC_CHANGE_NOT_SUPPORTED);
						PostError(err);
					}
				}
				FAccessUnit::Release(PeekedAU);
			}
			else
			{
				// Release peeked AU and actually pop it
				FAccessUnit* AccessUnit = nullptr;
				FromMultistreamBuffer->Pop(AccessUnit);
				check(PeekedAU == AccessUnit);
				FAccessUnit::Release(PeekedAU);
				PeekedAU = nullptr;

				if (Decoder)
				{
					// Since we are providing a new access unit now the decoder won't be at EOD any more.
					// If it was before, clear it out.
					if (pStats && pStats->DecoderInputBuffer.bEODSignaled)
					{
						pStats->DecoderInputBuffer.bEODSignaled = false;
						pStats->DecoderInputBuffer.bEODReached = false;
						Decoder->AUdataClearEOD();
					}
					Decoder->AUdataPushAU(AccessUnit);
					// Remember the buffer info for period transition checks.
					if (Type == EStreamType::Video)
					{
						VideoDecoder.LastBufferSourceInfo = AccessUnit->BufferSourceInfo;
					}
				}

				// Check for format changes.
				if (AccessUnit->AUCodecData.IsValid() && LastSentAUCodecData && LastSentAUCodecData->IsValid())
				{
					const bool bBitrateChanged = AccessUnit->AUCodecData->ParsedInfo.GetBitrate() != (*LastSentAUCodecData)->ParsedInfo.GetBitrate();
					// If there is any pertinent format change, emit an event.
					if (*LastSentAUCodecData != AccessUnit->AUCodecData)
					{
						bool bFormatChanged = bBitrateChanged || !AccessUnit->AUCodecData->ParsedInfo.Equals((*LastSentAUCodecData)->ParsedInfo);
						if (bFormatChanged)
						{
							DispatchEvent(FMetricEvent::ReportCodecFormatChange(AccessUnit->AUCodecData->ParsedInfo));
						}
						*LastSentAUCodecData = AccessUnit->AUCodecData;
					}
					// Record the currently active bitrate.
					if (bBitrateChanged)
					{
						if (Type == EStreamType::Video)
						{
							GetMediaInfoDictionary().Set(MediaInfoVideoBitrateCurrent.Resolve(), FVariantValue((int64)((*LastSentAUCodecData)->ParsedInfo.GetBitrate())));
						}
						else if (Type == EStreamType::Audio)
						{
							GetMediaInfoDictionary().Set(MediaInfoAudioBitrateCurrent.Resolve(), FVariantValue((int64)((*LastSentAUCodecData)->ParsedInfo.GetBitrate())));
						}
					}
				}

				// Check for presence of encoder latency and store it.
				// This may be updated by different streams/decoders which can't be helped. There should only be one actively
				// observed element but we have seen manifests that have the same <ProducerReferenceTime@id> in multiple AdaptationSets.
				if (AccessUnit->ProducerReferenceTime.IsValid())
				{
					PlaybackState.SetEncoderLatency(AccessUnit->DTS - AccessUnit->ProducerReferenceTime);
				}

				FAccessUnit::Release(AccessUnit);
			}
		}
	}
	// An AU is not tagged as being "the last" one. Instead the EOD is handled separately and must be dealt with
	// by the decoders accordingly.
	if (!bCodecChangeDetected && (FromMultistreamBuffer->IsEODFlagSet() || FromMultistreamBuffer->IsEndOfTrack()) && FromMultistreamBuffer->Num() == 0)
	{
		if (pStats && !pStats->DecoderInputBuffer.bEODSignaled)
		{
			if (Decoder)
			{
				Decoder->AUdataPushEOD();
			}
		}
	}
}


void FAdaptiveStreamingPlayer::ClearAllDecoderEODs()
{
	VideoDecoder.ClearEOD();
	AudioDecoder.ClearEOD();
	SubtitleDecoder.ClearEOD();
	DiagnosticsCriticalSection.Lock();
	VideoBufferStats.DecoderInputBuffer.bEODReached = false;
	VideoBufferStats.DecoderInputBuffer.bEODSignaled = false;
	VideoBufferStats.DecoderOutputBuffer.bEODreached = false;
	AudioBufferStats.DecoderInputBuffer.bEODReached = false;
	AudioBufferStats.DecoderInputBuffer.bEODSignaled = false;
	AudioBufferStats.DecoderOutputBuffer.bEODreached = false;
	TextBufferStats.DecoderInputBuffer.bEODReached = false;
	TextBufferStats.DecoderInputBuffer.bEODSignaled = false;
	TextBufferStats.DecoderOutputBuffer.bEODreached = false;
	DiagnosticsCriticalSection.Unlock();
}



bool FAdaptiveStreamingPlayer::IsSeamlessBufferSwitchPossible(EStreamType InStreamType, const TSharedPtrTS<FStreamDataBuffers>& InFromStreamBuffers)
{
	TSharedPtrTS<FMultiTrackAccessUnitBuffer> Buffer = GetStreamBuffer(InStreamType, InFromStreamBuffers);
	if (Buffer.IsValid())
	{
		//
		FAccessUnit* PeekedAU = nullptr;
		Buffer->PeekAndAddRef(PeekedAU);
		if (PeekedAU)
		{
			bool bFirstIsRenderedKeyframe = PeekedAU->bIsSyncSample && (!PeekedAU->EarliestPTS.IsValid() || (PeekedAU->EarliestPTS.IsValid() && PeekedAU->PTS >= PeekedAU->EarliestPTS));
			FAccessUnit::Release(PeekedAU);
			return bFirstIsRenderedKeyframe;
		}
	}
	return false;
}


void FAdaptiveStreamingPlayer::Deprecate_InternalInitializeDecoderLimits()
{
	// Set up video decoder resolution limits. As the media playlists are parsed the video streams will be
	// compared against these limits and those that exceed the limit will not be considered for playback.

	// Maximum allowed vertical resolution specified?
	if (PlayerOptions.HaveKey(OptionKeyMaxVerticalResolution))
	{
		PlayerConfig.H264LimitUpto30fps.MaxResolution.Height =
		PlayerConfig.H264LimitAbove30fps.MaxResolution.Height = (int32)PlayerOptions.GetValue(OptionKeyMaxVerticalResolution).GetInt64();
	}
	// A limit in vertical resolution for streams with more than 30fps?
	if (PlayerOptions.HaveKey(OptionKeyMaxVerticalResolutionAbove30fps))
	{
		PlayerConfig.H264LimitAbove30fps.MaxResolution.Height = (int32)PlayerOptions.GetValue(OptionKeyMaxVerticalResolutionAbove30fps).GetInt64();
	}
}

bool FAdaptiveStreamingPlayer::Deprecate_InternalStreamAllowedAsPerLimits(const FStreamCodecInformation& InStreamCodecInfo) const
{
	if (InStreamCodecInfo.GetCodec() == FStreamCodecInformation::ECodec::H264)
	{
		const double Rate = InStreamCodecInfo.GetFrameRate().IsValid() ? InStreamCodecInfo.GetFrameRate().GetAsDouble() : 30.0;
		const AdaptiveStreamingPlayerConfig::FConfiguration::FVideoDecoderLimit* DecoderLimit = &PlayerConfig.H264LimitUpto30fps;
		if (Rate > 31.0)
		{
			DecoderLimit = &PlayerConfig.H264LimitAbove30fps;
		}
		// Check against user configured resolution limit
		if (DecoderLimit->MaxResolution.Height && InStreamCodecInfo.GetResolution().Height > DecoderLimit->MaxResolution.Height)
		{
			return false;
		}
	}
	return true;
}



} // namespace Electra


