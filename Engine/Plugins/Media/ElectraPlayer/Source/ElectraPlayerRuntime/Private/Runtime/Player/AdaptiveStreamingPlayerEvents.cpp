// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/AdaptiveStreamingPlayerInternal.h"
#include "Player/DASH/PlayerEventDASH_Internal.h"

namespace Electra
{


//-----------------------------------------------------------------------------
/**
 * Dispatches a 'buffering' event.
 *
 * @param bBegin
 * @param reason
 */
void FAdaptiveStreamingPlayer::DispatchBufferingEvent(bool bBegin, FAdaptiveStreamingPlayer::EPlayerState Reason)
{
	if (bBegin)
	{
		DispatchEvent(FMetricEvent::ReportBufferingStart(Reason == EPlayerState::eState_Buffering ? Metrics::EBufferingReason::Initial :
														 Reason == EPlayerState::eState_Rebuffering ? Metrics::EBufferingReason::Rebuffering :
														 Reason == EPlayerState::eState_Seeking ? Metrics::EBufferingReason::Seeking : Metrics::EBufferingReason::Rebuffering));
	}
	else
	{
		DispatchEvent(FMetricEvent::ReportBufferingEnd(Reason == EPlayerState::eState_Buffering ? Metrics::EBufferingReason::Initial :
													   Reason == EPlayerState::eState_Rebuffering ? Metrics::EBufferingReason::Rebuffering :
													   Reason == EPlayerState::eState_Seeking ? Metrics::EBufferingReason::Seeking : Metrics::EBufferingReason::Rebuffering));
	}
}



void FAdaptiveStreamingPlayer::DispatchBufferUtilizationEvent(EStreamType BufferType)
{
	const FAccessUnitBufferInfo* BufferStats = nullptr;
	const FAccessUnitBuffer::FConfiguration* BufferConfig = nullptr;
	if (BufferType == EStreamType::Video)
	{
		BufferStats = &VideoBufferStats.StreamBuffer;
		BufferConfig = &PlayerConfig.StreamBufferConfigVideo;
	}
	else if (BufferType == EStreamType::Audio)
	{
		BufferStats = &AudioBufferStats.StreamBuffer;
		BufferConfig = &PlayerConfig.StreamBufferConfigAudio;
	}
	else if (BufferType == EStreamType::Subtitle)
	{
		BufferStats = &TextBufferStats.StreamBuffer;
		BufferConfig = &PlayerConfig.StreamBufferConfigText;
	}
	if (BufferStats)
	{
		Metrics::FBufferStats stats;
		stats.BufferType = BufferType;
		stats.MaxDurationInSeconds = BufferConfig ? BufferConfig->MaxDuration.GetAsSeconds() : 0.0;
		stats.DurationInUse = BufferStats->PlayableDuration.GetAsSeconds();
		stats.BytesInUse = BufferStats->CurrentMemInUse;
		DispatchEvent(FMetricEvent::ReportBufferUtilization(stats));
	}
}



/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

namespace ReservedEventSchemeNames
{
	static const TArray<FString> IgnoredByWildcard { DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_2012,
													 DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_callback_2015,
													 DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_ttfn_2016 };
};




class FAdaptiveStreamingPlayerAEMSHandler : public IAdaptiveStreamingPlayerAEMSHandler
{
public:
	FAdaptiveStreamingPlayerAEMSHandler();
	virtual ~FAdaptiveStreamingPlayerAEMSHandler();

	virtual void ExecutePendingEvents(FTimeValue CurrentPlaybackTime) override;
	virtual void FlushEverything() override;
	virtual void FlushDynamic() override;
	virtual void NewUpcomingPeriod(FString NewPeriodID, FTimeRange NewPeriodTimeRange) override;
	virtual void PlaybackStartingUp() override;
	virtual void PlaybackPeriodTransition(FString NewPeriodID, FTimeRange NewPeriodTimeRange) override;
	virtual void PlaybackReachedEnd(FTimeValue CurrentPlaybackTime) override;

	virtual void AddEvent(TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> InNewEvent, FString InPeriodID, EEventAddMode InAddMode) override;
	virtual void AddAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode, bool bIsAppReceiver) override;
	virtual void RemoveAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode) override;

private:
	struct FSubscriber
	{
		TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> Receiver;
		FString ForValue;
		IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode DispatchMode;
		bool bIsAppSubscriber = false;
		bool bReceivesAll = false;
	};

	struct FEnqueuedEvent
	{
		TArray<TWeakPtrTS<FSubscriber>> Subscribers;
		FString PeriodID;
		TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> Event;
	};

	void DiscardAllPeriodEvents(const FString& ForPeriod);
	void FireEvent(TSharedPtrTS<FEnqueuedEvent> InEvent, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode);
	void FireEventsInRange(FTimeRange InRange);

	FCriticalSection Lock;
	TMultiMap<FString, TSharedPtrTS<FSubscriber>> Subscribers;
	TArray<TSharedPtrTS<FEnqueuedEvent>> EnqueuedEvents;
	FString CurrentPeriodID;
	bool bHaveCurrentPlayPeriod = false;
	FTimeValue LastPlayPos;

	static const FString AllSchemeWildcard;
	static const FTimeValue HandlingInterval;
	static const FTimeValue PlaybackEndThreshold;
};
const FString FAdaptiveStreamingPlayerAEMSHandler::AllSchemeWildcard(TEXT("*"));
const FTimeValue FAdaptiveStreamingPlayerAEMSHandler::HandlingInterval(0.1);
const FTimeValue FAdaptiveStreamingPlayerAEMSHandler::PlaybackEndThreshold(0.5);

/***************************************************************************************************************************************************/

IAdaptiveStreamingPlayerAEMSHandler* IAdaptiveStreamingPlayerAEMSHandler::Create()
{
	return new FAdaptiveStreamingPlayerAEMSHandler;
}

/***************************************************************************************************************************************************/

FAdaptiveStreamingPlayerAEMSHandler::FAdaptiveStreamingPlayerAEMSHandler()
{
}

FAdaptiveStreamingPlayerAEMSHandler::~FAdaptiveStreamingPlayerAEMSHandler()
{
	FlushEverything();
}

void FAdaptiveStreamingPlayerAEMSHandler::ExecutePendingEvents(FTimeValue CurrentPlaybackTime)
{
	FTimeValue DeltaTime = CurrentPlaybackTime - LastPlayPos;
	if (!DeltaTime.IsValid() || DeltaTime < FTimeValue::GetZero())
	{
		DeltaTime = HandlingInterval;
	}
	// Only update so often. We do not want to iterate through all the events every time
	// even though we _think_ there aren't that many.
	if (DeltaTime >= HandlingInterval)
	{
		FTimeRange TriggerRange;
		TriggerRange.Start = LastPlayPos.IsValid() ? LastPlayPos : CurrentPlaybackTime - HandlingInterval;
		TriggerRange.End = CurrentPlaybackTime;
		LastPlayPos = CurrentPlaybackTime;
		FireEventsInRange(TriggerRange);
	}
}

void FAdaptiveStreamingPlayerAEMSHandler::FlushEverything()
{
	FScopeLock lock(&Lock);
	EnqueuedEvents.Empty();
}

void FAdaptiveStreamingPlayerAEMSHandler::FlushDynamic()
{
	FScopeLock lock(&Lock);
	for(int32 i=0; i<EnqueuedEvents.Num(); ++i)
	{
		switch(EnqueuedEvents[i]->Event->GetOrigin())
		{
			// Keep only those that came in with the playlist itself.
			case IAdaptiveStreamingPlayerAEMSEvent::EOrigin::EventStream:
				break;
			default:
				EnqueuedEvents.RemoveAt(i);
				--i;
				break;
		}
	}
}

void FAdaptiveStreamingPlayerAEMSHandler::DiscardAllPeriodEvents(const FString& ForPeriod)
{
	FScopeLock lock(&Lock);
	for(int32 i=0; i<EnqueuedEvents.Num(); ++i)
	{
		if (EnqueuedEvents[i]->PeriodID.Equals(ForPeriod))
		{
			EnqueuedEvents.RemoveAt(i);
			--i;
		}
	}
}


void FAdaptiveStreamingPlayerAEMSHandler::NewUpcomingPeriod(FString NewPeriodID, FTimeRange NewPeriodTimeRange)
{
	// No-op for now.
}

void FAdaptiveStreamingPlayerAEMSHandler::PlaybackStartingUp()
{
	CurrentPeriodID.Empty();
	bHaveCurrentPlayPeriod = false;
	FlushDynamic();
	LastPlayPos.SetToInvalid();
}

void FAdaptiveStreamingPlayerAEMSHandler::PlaybackPeriodTransition(FString NewPeriodID, FTimeRange NewPeriodTimeRange)
{
	if (bHaveCurrentPlayPeriod)
	{
		// We are transitioning out of the period we know
		DiscardAllPeriodEvents(CurrentPeriodID);
	}
	CurrentPeriodID = NewPeriodID;
	bHaveCurrentPlayPeriod = true;
}

void FAdaptiveStreamingPlayerAEMSHandler::PlaybackReachedEnd(FTimeValue CurrentPlaybackTime)
{
	FTimeRange TriggerRange;
	// Fire everything that is within a late and future threshold around the playback position.
	TriggerRange.Start = CurrentPlaybackTime - HandlingInterval;
	TriggerRange.End = CurrentPlaybackTime + PlaybackEndThreshold;
	FireEventsInRange(TriggerRange);
}

void FAdaptiveStreamingPlayerAEMSHandler::AddEvent(TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> InNewEvent, FString InPeriodID, EEventAddMode InAddMode)
{
	// Only events for which there are registered receivers need to be handled. If there is no receiver at the moment but one is added later it will
	// not receive events that were added before.
	FScopeLock lock(&Lock);

	TArray<TSharedPtrTS<FSubscriber>> Receivers;
	Subscribers.MultiFind(InNewEvent->GetSchemeIdUri(), Receivers);
	Subscribers.MultiFind(AllSchemeWildcard, Receivers);
	if (Receivers.Num())
	{
		// Find an already existing event in the queue.
		// Note: The PeriodID does not enter into this. Only event specific values.
		int32 EventIdx = EnqueuedEvents.IndexOfByPredicate([InNewEvent](const TSharedPtrTS<FEnqueuedEvent>& This) {
			return InNewEvent->GetSchemeIdUri().Equals(This->Event->GetSchemeIdUri()) &&
				   InNewEvent->GetID().Equals(This->Event->GetID()) &&
				   (InNewEvent->GetValue().IsEmpty() || InNewEvent->GetValue().Equals(This->Event->GetValue()));
		});

		TSharedPtrTS<FEnqueuedEvent> Ev;
		bool bIsNewEvent = false;
		if (EventIdx == INDEX_NONE)
		{
			Ev = MakeSharedTS<FEnqueuedEvent>();
			bIsNewEvent = true;
		}
		else if (InAddMode == EEventAddMode::UpdateIfExists)
		{
			Ev = EnqueuedEvents[EventIdx];
		}
		if (Ev.IsValid())
		{
			Ev->PeriodID = MoveTemp(InPeriodID);
			Ev->Event = InNewEvent;
			Ev->Subscribers.Empty();
			for(int32 i=0; i<Receivers.Num(); ++i)
			{
				Ev->Subscribers.Emplace(Receivers[i]);
			}
			if (bIsNewEvent)
			{
				EnqueuedEvents.Emplace(Ev);
				lock.Unlock();
				FireEvent(Ev, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnReceive);
			}
		}
	}
}

void FAdaptiveStreamingPlayerAEMSHandler::AddAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode, bool bIsAppReceiver)
{
	bool bWildcard = InForSchemeIdUri.Equals(TEXT("*"));

	if (InForSchemeIdUri.Equals(TEXT("urn:mpeg:dash:event:catchall:2020")))
	{
		InForSchemeIdUri = AllSchemeWildcard;
	}
	if (InForSchemeIdUri.Equals(AllSchemeWildcard))
	{
		InForValue.Empty();
	}

	TSharedPtrTS<FSubscriber> sub = MakeSharedTS<FSubscriber>();
	sub->Receiver = InReceiver;
	sub->ForValue = InForValue;
	sub->DispatchMode = InDispatchMode;
	sub->bIsAppSubscriber = bIsAppReceiver;
	sub->bReceivesAll = bWildcard;
	FScopeLock lock(&Lock);
	Subscribers.Add(InForSchemeIdUri, sub);
}

void FAdaptiveStreamingPlayerAEMSHandler::RemoveAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode)
{
	if (InForSchemeIdUri.Equals(TEXT("urn:mpeg:dash:event:catchall:2020")))
	{
		InForSchemeIdUri = AllSchemeWildcard;
	}
	if (InForSchemeIdUri.Equals(AllSchemeWildcard))
	{
		InForValue.Empty();
	}
	FScopeLock lock(&Lock);
	for(TMultiMap<FString, TSharedPtrTS<FSubscriber>>::TKeyIterator It = Subscribers.CreateKeyIterator(InForSchemeIdUri); It; ++It)
	{
		if (It.Value()->Receiver == InReceiver && It.Value()->ForValue == InForValue && It.Value()->DispatchMode == InDispatchMode)
		{
			It.RemoveCurrent();
			break;
		}
	}
}


void FAdaptiveStreamingPlayerAEMSHandler::FireEvent(TSharedPtrTS<FEnqueuedEvent> InEvent, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode)
{
	for(int32 nSub=0; nSub<InEvent->Subscribers.Num(); ++nSub)
	{
		TSharedPtrTS<FSubscriber> Subscriber = InEvent->Subscribers[nSub].Pin();
		if (Subscriber.IsValid())
		{
			TSharedPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> Receiver = Subscriber->Receiver.Pin();
			if (Receiver.IsValid())
			{
				if (Subscriber->DispatchMode == InDispatchMode &&
					(Subscriber->ForValue.IsEmpty() || Subscriber->ForValue.Equals(InEvent->Event->GetValue())))
				{
					// Check if the subscriber is the application and it wants to listen to everything.
					if (Subscriber->bIsAppSubscriber && Subscriber->bReceivesAll)
					{
						// Well, internal events meant for the player only we will not send to it.
						// The app can still subscribe explicitly to the event type if it needs them but
						// we do not send them on our own.
						if (ReservedEventSchemeNames::IgnoredByWildcard.Contains(InEvent->Event->GetSchemeIdUri()))
						{
							continue;
						}
					}
					Receiver->OnMediaPlayerEventReceived(InEvent->Event, InDispatchMode);
				}
			}
		}
	}
}

void FAdaptiveStreamingPlayerAEMSHandler::FireEventsInRange(FTimeRange InRange)
{
	TArray<TSharedPtrTS<FEnqueuedEvent>> EventsToFire;
	FScopeLock lock(&Lock);
	// Events that are to be fired get moved into a temporary list so we can
	// trigger them outside the lock.
	for(int32 i=0; i<EnqueuedEvents.Num(); ++i)
	{
		FTimeValue evS = EnqueuedEvents[i]->Event->GetPresentationTime();
		FTimeValue evD = EnqueuedEvents[i]->Event->GetDuration();
		FTimeValue evE = evS + (evD.IsValid() ? evD : FTimeValue::GetZero());

		// Event already expired?
		if (evE < InRange.Start)
		{
			EnqueuedEvents.RemoveAt(i);
			--i;
		}
		else if (evS >= InRange.Start && evS <= InRange.End)
		{
			EventsToFire.Emplace(EnqueuedEvents[i]);
			EnqueuedEvents.RemoveAt(i);
			--i;
		}
	}
	lock.Unlock();
	// Fire the events.
	for(int32 i=0; i<EventsToFire.Num(); ++i)
	{
		FireEvent(EventsToFire[i], IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnStart);
	}
}




} // namespace Electra

