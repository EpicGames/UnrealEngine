// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"

namespace Electra
{

class IAdaptiveStreamingPlayerAEMSEvent
{
public:
	enum class EOrigin
	{
		EventStream,
		InbandEventStream,
		TimedMetadata
	};

	virtual ~IAdaptiveStreamingPlayerAEMSEvent() = default;

	// Returns where the event originated from.
	virtual EOrigin GetOrigin() const = 0;

	// Returns the event's scheme id URI
	virtual FString GetSchemeIdUri() const = 0;

	// Returns the event's value.
	virtual FString GetValue() const = 0;

	// Returns the event's ID.
	virtual FString GetID() const = 0;

	// Returns the event's presentation time.
	virtual FTimeValue GetPresentationTime() const = 0;

	// Returns the event's duration. If the event has no defined duration the return value will be invalid.
	virtual FTimeValue GetDuration() const = 0;

	// Returns the event's message data.
	virtual const TArray<uint8>& GetMessageData() const = 0;
};



/**
 * Client side event receiver.
 *
 * As per ISO/IEC 23009-1:2019/DAM 1 event receivers subscribing to "on_receive" will receive the event the
 * instant it is received by the player. Handling of the event is the sole responsibility of the receiver.
 * Events will be received at any time and the same event can be received multiple times. Events will also
 * be received in no particular order and events may also already be outdated when received.
 * This happens with events contained inside the playlist every time the playlist is updated on the server
 * and reloaded by the client.
 */
class IAdaptiveStreamingPlayerAEMSReceiver
{
public:
	virtual ~IAdaptiveStreamingPlayerAEMSReceiver() = default;

	enum class EDispatchMode
	{
		OnReceive,
		OnStart
	};

	virtual void OnMediaPlayerEventReceived(TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> InEvent, EDispatchMode InDispatchMode) = 0;
};




class IAdaptiveStreamingPlayerAEMSHandler
{
public:
	static IAdaptiveStreamingPlayerAEMSHandler* Create();
	virtual ~IAdaptiveStreamingPlayerAEMSHandler() = default;

	// Call periodically to dispatch pending events.
	virtual void ExecutePendingEvents(FTimeValue CurrentPlaybackTime) = 0;
	// Unconditionally flushes all events from the queue.
	virtual void FlushEverything() = 0;
	// Informs that playback has moved to a new position in either direction. This may discard dynamic events from streams but keep period events from the playlist.
	virtual void FlushDynamic() = 0;
	// Informs that a new playback period is coming up. This is to truncate period events that would extend over their originating period.
	virtual void NewUpcomingPeriod(FString NewPeriodID, FTimeRange NewPeriodTimeRange) = 0;
	// Informs that playback is starting up.
	virtual void PlaybackStartingUp() = 0;
	// Informs that playback has moved forward into another period. Events of an earlier period that are outside its associated periods time range will not fire.
	virtual void PlaybackPeriodTransition(FString NewPeriodID, FTimeRange NewPeriodTimeRange) = 0;
	// Informs that playback has ended.
	virtual void PlaybackReachedEnd(FTimeValue CurrentPlaybackTime) = 0;

	enum class EEventAddMode
	{
		AddIfNotExists,
		UpdateIfExists
	};
	// Enqueues an event. Checks if the new event ID (GetID()) for the given scheme/value has already been added earlier
	// and either adds, updates or ignores the event based on the given InAddMode.
	virtual void AddEvent(TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> InNewEvent, FString InPeriodID, EEventAddMode InAddMode) = 0;


	// Adds/removes event receivers.
	virtual void AddAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode, bool bIsAppReceiver) = 0;
	virtual void RemoveAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode) = 0;
};


} // namespace Electra


