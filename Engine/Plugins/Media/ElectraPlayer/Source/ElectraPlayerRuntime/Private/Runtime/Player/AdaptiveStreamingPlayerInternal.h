// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Player/AdaptiveStreamingPlayer.h"
#include "Player/Manifest.h"
#include "Player/AdaptiveStreamingPlayerInternalConfig.h"

#include "HTTP/HTTPManager.h"
#include "HTTP/HTTPResponseCache.h"
#include "Player/PlayerSessionServices.h"
#include "SynchronizedClock.h"
#include "PlayerClock.h"

#include "Demuxer/ParserISO14496-12.h"
#include "Decoder/SubtitleDecoder.h"
#include "Decoder/AudioDecoder.h"
#include "Decoder/VideoDecoder.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/PlayerStreamReader.h"
#include "Player/PlaylistReader.h"
#include "Player/PlayerStreamFilter.h"
#include "Player/PlayerEntityCache.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/ContentSteeringHandler.h"
#include "Player/CMCDHandler.h"
#include "Player/ABRRules/ABRStatisticTypes.h"
#include "OutputHandler.h"
#include "MediaStreamMetadata.h"
#include "ElectraCDM.h"
#include "InfoLog.h"

#include "Templates/UniqueObj.h"
#include "Misc/TVariant.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Misc/Base64.h"
#include "MediaSamples.h"



#define INTERR_ALL_STREAMS_HAVE_FAILED			1
#define INTERR_UNSUPPORTED_FORMAT				2
#define INTERR_COULD_NOT_LOCATE_START_SEGMENT	3
#define INTERR_COULD_NOT_LOCATE_START_PERIOD	4
#define INTERR_UNSUPPORTED_CODEC				5
#define INTERR_CODEC_CHANGE_NOT_SUPPORTED		6
#define INTERR_NO_STREAM_INFORMATION			7
#define INTERR_GENERAL_FAILURE					8
#define INTERR_FRAGMENT_NOT_AVAILABLE			0x101
#define INTERR_FRAGMENT_READER_REQUEST			0x102
#define INTERR_CREATE_FRAGMENT_READER			0x103
#define INTERR_REBUFFER_SHALL_THROW_ERROR		0x200


// Set this to `1` to signal no data availability right at the start of a seek (and initial playback) and at the end.
// Set to `0` to only notify about mid-playback state changes.
#define NOTIFY_DATA_AVAILABILITY_AT_START_AND_END 0


namespace Electra
{
class FAdaptiveStreamingPlayer;


class FMediaRenderClock : public IMediaRenderClock
{
public:
	FMediaRenderClock()
		: bIsPaused(true)
	{
	}

	virtual ~FMediaRenderClock() = default;

	void SetCurrentTime(ERendererType ForRenderer, const FTimeValue& CurrentRenderTime) override
	{
		FClock* Clk = GetClock(ForRenderer);
		if (Clk)
		{
			FTimeValue Now(MEDIAutcTime::Current());
			FScopeLock lock(&Lock);
			Clk->RenderTime = CurrentRenderTime;
			Clk->LastSystemBaseTime = Now;
			Clk->RunningTimeOffset.SetToZero();
		}
	}

	FTimeValue GetInterpolatedRenderTime(ERendererType FromRenderer) override
	{
		FClock* Clk = GetClock(FromRenderer);
		if (Clk)
		{
			FTimeValue Now(MEDIAutcTime::Current());
			FScopeLock lock(&Lock);
			FTimeValue diff = !bIsPaused ? Now - Clk->LastSystemBaseTime : FTimeValue::GetZero();
			FTimeValue r(Clk->RenderTime);
			r += Clk->RunningTimeOffset + diff;
			return r;
		}
		return FTimeValue();
	}

	void Start()
	{
		FTimeValue Now(MEDIAutcTime::Current());
		FScopeLock lock(&Lock);
		if (bIsPaused)
		{
			for(int32 i = 0; i < FMEDIA_STATIC_ARRAY_COUNT(Clock); ++i)
			{
				Clock[i].LastSystemBaseTime = Now;
			}
			bIsPaused = false;
		}
	}

	void Stop()
	{
		FTimeValue Now(MEDIAutcTime::Current());
		FScopeLock lock(&Lock);
		if (!bIsPaused)
		{
			bIsPaused = true;
			for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(Clock); ++i)
			{
				FTimeValue diff = Now - Clock[i].LastSystemBaseTime;
				Clock[i].RunningTimeOffset += diff;
			}
		}
	}

	bool IsRunning() const
	{
		return !bIsPaused;
	}

private:
	struct FClock
	{
		FClock()
		{
			LastSystemBaseTime = MEDIAutcTime::Current();
			RunningTimeOffset.SetToZero();
		}
		FTimeValue	RenderTime;
		FTimeValue	LastSystemBaseTime;
		FTimeValue	RunningTimeOffset;
	};

	FClock* GetClock(ERendererType ForRenderer)
	{
		FClock* Clk = nullptr;
		switch(ForRenderer)
		{
			case IMediaRenderClock::ERendererType::Video:
				Clk = &Clock[0];
				break;
			case IMediaRenderClock::ERendererType::Audio:
				Clk = &Clock[1];
				break;
			case IMediaRenderClock::ERendererType::Subtitles:
				Clk = &Clock[2];
				break;
			default:
				Clk = nullptr;
				break;
		}
		return Clk;
	}

	FCriticalSection Lock;
	FClock Clock[3];
	bool bIsPaused;
};



struct FDroppedFrameStats
{
	struct FEntry
	{
		uint64 DropCount;
	};
	void Clear()
	{
		Entries.Empty();
		LastProcessedTime = -1;
		PausedDelta = -1;
		NumIntervals = 0;
		NumIntervalsWithDrops = 0;
	}
	TArray<FEntry> Entries;
	TRollingAverage<int64> AverageDropsPerInterval;
	int64 LastProcessedTime = -1;
	int64 PausedDelta = -1;
	int64 ReportingIntervalMs = -1;
	uint32 NumIntervals = 0;
	uint32 NumIntervalsWithDrops = 0;
};



/**
 * Interchange structure between the player worker thread and the public API to
 * avoid mutex locks all over the place.
 */
struct FPlaybackState
{
	FPlaybackState()
	{
		Reset();
	}
	void Reset()
	{
		FScopeLock lock(&Lock);
		SeekableRange.Reset();
		Duration.SetToInvalid();
		CurrentPlayPosition.SetToInvalid();
		EncoderLatency.SetToInvalid();
		CurrentLiveLatency.SetToInvalid();
		EndPlaybackAtTime.SetToInvalid();
		LoopState = {};
		ActivePlaybackRange.Reset();
		NewPlaybackRange.Reset();
		UnthinnedPlaybackRates.Empty();
		ThinnedPlaybackRates.Empty();
		CurrentPlaybackRate = 0.0;
		DesiredPlaybackRate = 0.0;
		bHaveMetadata = false;
		bHasEnded = false;
		bIsSeeking = false;
		bIsBuffering = false;
		bIsPlaying = false;
		bIsPaused = false;
		bPlayrangeHasChanged = false;
		bLoopStateHasChanged = false;
		bShouldPlayOnLiveEdge = false;
		for(int32 i=0; i<UE_ARRAY_COUNT(CurrentSegmentDownloadTimeRange); ++i)
		{
			CurrentSegmentDownloadTimeRange[i].Reset();
		}
	}

	mutable FCriticalSection				Lock;
	FTimeRange								SeekableRange;
	FTimeRange								TimelineRange;
	FTimeValue								Duration;
	FTimeValue								CurrentPlayPosition;
	FTimeValue								EncoderLatency;
	FTimeValue								CurrentLiveLatency;
	FTimeValue								EndPlaybackAtTime;
	IAdaptiveStreamingPlayer::FLoopState	LoopState;
	FTimeRange								ActivePlaybackRange;
	FTimeRange								NewPlaybackRange;
	TRangeSet<double>						UnthinnedPlaybackRates;
	TRangeSet<double>						ThinnedPlaybackRates;
	double									CurrentPlaybackRate;
	double									DesiredPlaybackRate;
	bool									bHaveMetadata;
	bool									bHasEnded;
	bool									bIsSeeking;
	bool									bIsBuffering;
	bool									bIsPlaying;
	bool									bIsPaused;
	bool									bPlayrangeHasChanged;
	bool									bLoopStateHasChanged;
	bool									bShouldPlayOnLiveEdge;
	TUniquePtr<IAdaptiveStreamingPlayer::FLoopParam>	PendingLoopParams;
	TArray<FTrackMetadata>					VideoTracks;
	TArray<FTrackMetadata>					AudioTracks;
	TArray<FTrackMetadata>					SubtitleTracks;
	FTimeRange								CurrentSegmentDownloadTimeRange[4];	// 0=video, 1=audio, 2=subtitiles, 3=UNSUPPORTED


	void SetSeekableRange(const FTimeRange& TimeRange)
	{
		FScopeLock lock(&Lock);
		SeekableRange = TimeRange;
	}
	void GetSeekableRange(FTimeRange& OutRange) const
	{
		FScopeLock lock(&Lock);
		OutRange = SeekableRange;
	}

	void SetTimelineRange(const FTimeRange& TimeRange)
	{
		FScopeLock lock(&Lock);
		TimelineRange = TimeRange;
	}
	void GetTimelineRange(FTimeRange& OutRange) const
	{
		FScopeLock lock(&Lock);
		OutRange = TimelineRange;
	}

	void SetDuration(const FTimeValue& InDuration)
	{
		FScopeLock lock(&Lock);
		Duration = InDuration;
	}
	FTimeValue GetDuration() const
	{
		FScopeLock lock(&Lock);
		return Duration;
	}

	void SetPlayPosition(const FTimeValue& InPosition)
	{
		FScopeLock lock(&Lock);
		CurrentPlayPosition = InPosition;
	}
	FTimeValue GetPlayPosition() const
	{
		FScopeLock lock(&Lock);
		return CurrentPlayPosition;
	}

	void SetEncoderLatency(const FTimeValue& InEncoderLatency)
	{
		FScopeLock lock(&Lock);
		EncoderLatency = InEncoderLatency;
	}
	FTimeValue GetEncoderLatency() const
	{
		FScopeLock lock(&Lock);
		return EncoderLatency;
	}

	void SetCurrentLiveLatency(const FTimeValue& InCurrentLiveLatency)
	{
		FScopeLock lock(&Lock);
		CurrentLiveLatency = InCurrentLiveLatency;
	}
	FTimeValue GetCurrentLiveLatency() const
	{
		FScopeLock lock(&Lock);
		return CurrentLiveLatency;
	}

	void SetShouldPlayOnLiveEdge(bool bInShouldPlayOnLiveEdge)
	{
		FScopeLock lock(&Lock);
		bShouldPlayOnLiveEdge = bInShouldPlayOnLiveEdge;
	}
	bool GetShouldPlayOnLiveEdge() const
	{
		FScopeLock lock(&Lock);
		return bShouldPlayOnLiveEdge;
	}

	void SetHaveMetadata(bool bInHaveMetadata)
	{
		FScopeLock lock(&Lock);
		bHaveMetadata = bInHaveMetadata;
	}
	bool GetHaveMetadata() const
	{
		FScopeLock lock(&Lock);
		return bHaveMetadata;
	}

	void SetHasEnded(bool bInHasEnded)
	{
		FScopeLock lock(&Lock);
		bHasEnded = bInHasEnded;
	}
	bool GetHasEnded() const
	{
		FScopeLock lock(&Lock);
		return bHasEnded;
	}

	void SetIsSeeking(bool bInIsSeeking)
	{
		FScopeLock lock(&Lock);
		bIsSeeking = bInIsSeeking;
	}
	bool GetIsSeeking() const
	{
		FScopeLock lock(&Lock);
		return bIsSeeking;
	}

	void SetIsBuffering(bool bInIsBuffering)
	{
		FScopeLock lock(&Lock);
		bIsBuffering = bInIsBuffering;
	}
	bool GetIsBuffering() const
	{
		FScopeLock lock(&Lock);
		return bIsBuffering;
	}

	void SetIsPlaying(bool bInIsPlaying)
	{
		FScopeLock lock(&Lock);
		bIsPlaying = bInIsPlaying;
	}
	bool GetIsPlaying() const
	{
		FScopeLock lock(&Lock);
		return bIsPlaying;
	}

	void SetIsPaused(bool bInIsPaused)
	{
		FScopeLock lock(&Lock);
		bIsPaused = bInIsPaused;
	}
	bool GetIsPaused() const
	{
		FScopeLock lock(&Lock);
		return bIsPaused;
	}

	void SetPausedAndPlaying(bool bInIsPaused, bool bInIsPlaying)
	{
		FScopeLock lock(&Lock);
		bIsPaused  = bInIsPaused;
		bIsPlaying = bInIsPlaying;
		// If asked to play, but no desired playback rate has been set yet, assume 1.0.
		// It is thought to be a user error to resume playback with no play rate.
		if (bIsPlaying && DesiredPlaybackRate == 0.0)
		{
			DesiredPlaybackRate = 1.0;
		}
		CurrentPlaybackRate = bIsPaused ? 0.0 : bIsPlaying ? DesiredPlaybackRate : 0.0;
	}

	bool SetTrackMetadata(const TArray<FTrackMetadata> &InVideoTracks, const TArray<FTrackMetadata>& InAudioTracks, const TArray<FTrackMetadata>& InSubtitleTracks)
	{
		FScopeLock lock(&Lock);

		auto ChangedTrackMetadata = [](const TArray<FTrackMetadata> &These, const TArray<FTrackMetadata>& Other) -> bool
		{
			if (These.Num() == Other.Num())
			{
				for(int32 i=0; i<These.Num(); ++i)
				{
					if (!These[i].Equals(Other[i]))
					{
						return true;
					}
				}
				return false;
			}
			return true;
		};

		bool bChanged = ChangedTrackMetadata(VideoTracks, InVideoTracks) || ChangedTrackMetadata(AudioTracks, InAudioTracks) || ChangedTrackMetadata(SubtitleTracks, InSubtitleTracks);
		VideoTracks = InVideoTracks;
		AudioTracks = InAudioTracks;
		SubtitleTracks = InSubtitleTracks;
		return bChanged;
	}
	void GetTrackMetadata(TArray<FTrackMetadata> &OutVideoTracks, TArray<FTrackMetadata>& OutAudioTracks, TArray<FTrackMetadata>& OutSubtitleTracks) const
	{
		FScopeLock lock(&Lock);
		OutVideoTracks = VideoTracks;
		OutAudioTracks = AudioTracks;
		OutSubtitleTracks = SubtitleTracks;
	}
	void HaveTrackMetadata(bool& bOutHaveVideo, bool& bOutHaveAudio, bool& bOutHaveSubtitle) const
	{
		FScopeLock lock(&Lock);
		bOutHaveVideo = VideoTracks.Num() != 0;
		bOutHaveAudio = AudioTracks.Num() != 0;
		bOutHaveSubtitle = SubtitleTracks.Num() != 0;
	}

	void SetLoopState(const IAdaptiveStreamingPlayer::FLoopState& InLoopState)
	{
		FScopeLock lock(&Lock);
		LoopState = InLoopState;
	}
	void GetLoopState(IAdaptiveStreamingPlayer::FLoopState& OutLoopState) const
	{
		FScopeLock lock(&Lock);
		OutLoopState = LoopState;
		// If there is a pending loop change request we return the requested state
		// until we have handled the request.
		if (PendingLoopParams.IsValid())
		{
			OutLoopState.bIsEnabled = PendingLoopParams->bEnableLooping;
		}
	}
	void SetLoopStateEnable(bool bEnable)
	{
		FScopeLock lock(&Lock);
		LoopState.bIsEnabled = bEnable;
	}

	void SetPlayRange(const IAdaptiveStreamingPlayer::FPlaybackRange& InNewRange)
	{
		FScopeLock lock(&Lock);
		FTimeValue NewRangeStart = InNewRange.Start.IsSet() ? InNewRange.Start.GetValue() : FTimeValue();
		FTimeValue NewRangeEnd = InNewRange.End.IsSet() ? InNewRange.End.GetValue() : FTimeValue();
		if (NewRangeStart != NewPlaybackRange.Start || NewRangeEnd != NewPlaybackRange.End)
		{
			NewPlaybackRange.Start = NewRangeStart;
			NewPlaybackRange.End = NewRangeEnd;
			bPlayrangeHasChanged = true;
		}
	}
	void SetPlayRange(const FTimeRange& InNewRange)
	{
		FScopeLock lock(&Lock);
		if (InNewRange.Start != NewPlaybackRange.Start || InNewRange.End != NewPlaybackRange.End)
		{
			NewPlaybackRange = InNewRange;
			bPlayrangeHasChanged = true;
		}
	}
	FTimeRange GetPlayRange()
	{
		FScopeLock lock(&Lock);
		return NewPlaybackRange;
	}
	void GetPlayRange(IAdaptiveStreamingPlayer::FPlaybackRange& OutRange)
	{
		OutRange.Start.Reset();
		OutRange.End.Reset();
		FScopeLock lock(&Lock);
		if (NewPlaybackRange.Start.IsValid())
		{
			OutRange.Start = NewPlaybackRange.Start;
		}
		if (NewPlaybackRange.End.IsValid())
		{
			OutRange.End = NewPlaybackRange.End;
		}
	}
	void ActivateNewPlayRange(const FTimeRange* InTimeRange)
	{
		FScopeLock lock(&Lock);
		ActivePlaybackRange = InTimeRange ? *InTimeRange : NewPlaybackRange;
		bPlayrangeHasChanged = false;
	}
	FTimeRange GetActivePlayRange()
	{
		FScopeLock lock(&Lock);
		return ActivePlaybackRange;
	}
	bool GetPlayRangeHasChanged()
	{
		FScopeLock lock(&Lock);
		return bPlayrangeHasChanged;
	}

	void SetPlaybackRates(IAdaptiveStreamingPlayer::EPlaybackRateType InForType, const TRangeSet<double>& InRates)
	{
		FScopeLock lock(&Lock);
		if (InForType == IAdaptiveStreamingPlayer::EPlaybackRateType::Unthinned)
		{
			UnthinnedPlaybackRates = InRates;
		}
		else
		{
			ThinnedPlaybackRates = InRates;
		}
	}
	TRangeSet<double> GetPlaybackRates(IAdaptiveStreamingPlayer::EPlaybackRateType InForType)
	{
		FScopeLock lock(&Lock);
		return InForType == IAdaptiveStreamingPlayer::EPlaybackRateType::Unthinned ? UnthinnedPlaybackRates : ThinnedPlaybackRates;
	}

	void SetDesiredPlayRate(double InDesiredPlayRate, const IAdaptiveStreamingPlayer::FTrickplayParams& /*InParameters*/)
	{
		FScopeLock lock(&Lock);
		DesiredPlaybackRate = InDesiredPlayRate;
	}
	double GetDesiredPlayRate() const
	{
		FScopeLock lock(&Lock);
		return DesiredPlaybackRate;
	}
	void SetCurrentPlayRate(double InRate)
	{
		FScopeLock lock(&Lock);
		CurrentPlaybackRate = InRate;
	}
	double GetCurrentPlayRate() const
	{
		FScopeLock lock(&Lock);
		return CurrentPlaybackRate;
	}


	void RequestLooping(const IAdaptiveStreamingPlayer::FLoopParam& InLoopParams)
	{
		FScopeLock lock(&Lock);
		PendingLoopParams.Reset(new IAdaptiveStreamingPlayer::FLoopParam(InLoopParams));
	}
	TUniquePtr<IAdaptiveStreamingPlayer::FLoopParam> GetPendingLoopRequest()
	{
		FScopeLock lock(&Lock);
		return MoveTemp(PendingLoopParams);
	}
	void SetLoopStateHasChanged(bool bWasChanged)
	{
		FScopeLock lock(&Lock);
		bLoopStateHasChanged = bWasChanged;
	}
	bool GetLoopStateHasChanged()
	{
		FScopeLock lock(&Lock);
		return bLoopStateHasChanged;
	}

	void SetPlaybackEndAtTime(const FTimeValue& InAtTime)
	{
		FScopeLock lock(&Lock);
		EndPlaybackAtTime = InAtTime;
	}
	FTimeValue GetPlaybackEndAtTime() const
	{
		FScopeLock lock(&Lock);
		return EndPlaybackAtTime;
	}

	FTimeRange GetCurrentDownloadRequestTimeRange(EStreamType InStreamType)
	{
		FScopeLock lock(&Lock);
		return CurrentSegmentDownloadTimeRange[StreamTypeToArrayIndex(InStreamType)];
	}
	void SetCurrentDownloadRequestTimeRange(EStreamType InStreamType, const FTimeRange& InRange)
	{
		FScopeLock lock(&Lock);
		CurrentSegmentDownloadTimeRange[StreamTypeToArrayIndex(InStreamType)] = InRange;
	}
	void ClearCurrentDownloadRequestTimeRange(EStreamType InStreamType)
	{
		FScopeLock lock(&Lock);
		CurrentSegmentDownloadTimeRange[StreamTypeToArrayIndex(InStreamType)].Reset();
	}

};






struct FMetricEvent
{
	enum class EType
	{
		OpenSource,
		ReceivedMainPlaylist,
		ReceivedPlaylists,
		TracksChanged,
		BufferingStart,
		BufferingEnd,
		Bandwidth,
		BufferUtilization,
		PlaylistDownload,
		SegmentDownload,
		DataAvailabilityChange,
		VideoQualityChange,
		AudioQualityChange,
		CodecFormatChange,
		PrerollStart,
		PrerollEnd,
		PlaybackStart,
		PlaybackPaused,
		PlaybackResumed,
		PlaybackEnded,
		PlaybackJumped,
		PlaybackStopped,
		SeekCompleted,
		MediaMetadataChanged,
		LicenseKey,
		Errored,
		LogMessage,
		VideoFrameStats
	};

	struct FParam
	{
		struct FBandwidth
		{
			int64	EffectiveBps;
			int64	ThroughputBps;
			double	Latency;
		};
		struct FQualityChange
		{
			int32	NewBitrate;
			int32	PrevBitrate;
			bool	bIsDrastic;
		};
		struct FLogMessage
		{
			FString				Message;
			IInfoLog::ELevel	Level;
			int64				AtMillis;
		};
		struct FTimeJumped
		{
			FTimeValue					ToNewTime;
			FTimeValue					FromTime;
			Metrics::ETimeJumpReason	Reason;

		};
		struct FSeekComplete
		{
			bool bWasAlreadyThere;
		};
		struct FMediaMedataDataChanged
		{
			TSharedPtrTS<FMetadataParser> NewMetadata;
		};
		FString								URL;
		Metrics::EBufferingReason			BufferingReason;
		Metrics::FBufferStats				BufferStats;
		Metrics::FPlaylistDownloadStats		PlaylistStats;
		Metrics::FSegmentDownloadStats		SegmentStats;
		Metrics::FLicenseKeyStats			LicenseKeyStats;
		Metrics::FDataAvailabilityChange	DataAvailability;
		Metrics::FFrameStats				FrameStats;
		FBandwidth							Bandwidth;
		FQualityChange						QualityChange;
		FStreamCodecInformation				CodecFormatChange;
		FTimeJumped							TimeJump;
		FSeekComplete						SeekComplete;
		FMediaMedataDataChanged				MediaMetadataChange;
		FErrorDetail						ErrorDetail;
		FLogMessage							LogMessage;
	};

	EType			Type;
	FParam			Param;
	TWeakPtr<FAdaptiveStreamingPlayer, ESPMode::ThreadSafe>	Player;
	FMediaEvent	*EventSignal = nullptr;

	static TSharedPtrTS<FMetricEvent> ReportOpenSource(const FString& URL)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::OpenSource;
		Evt->Param.URL = URL;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportReceivedMainPlaylist(const FString& EffectiveURL)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::ReceivedMainPlaylist;
		Evt->Param.URL = EffectiveURL;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportReceivedPlaylists()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::ReceivedPlaylists;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportTracksChanged()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::TracksChanged;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportBufferingStart(Metrics::EBufferingReason BufferingReason)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::BufferingStart;
		Evt->Param.BufferingReason = BufferingReason;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportBufferingEnd(Metrics::EBufferingReason BufferingReason)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::BufferingEnd;
		Evt->Param.BufferingReason = BufferingReason;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportBandwidth(int64 EffectiveBps, int64 ThroughputBps, double LatencyInSeconds)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::Bandwidth;
		Evt->Param.Bandwidth.EffectiveBps = EffectiveBps;
		Evt->Param.Bandwidth.ThroughputBps = ThroughputBps;
		Evt->Param.Bandwidth.Latency = LatencyInSeconds;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportBufferUtilization(const Metrics::FBufferStats& BufferStats)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::BufferUtilization;
		Evt->Param.BufferStats = BufferStats;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportPlaylistDownload(const Metrics::FPlaylistDownloadStats& PlaylistDownloadStats)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PlaylistDownload;
		Evt->Param.PlaylistStats = PlaylistDownloadStats;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportSegmentDownload(const Metrics::FSegmentDownloadStats& SegmentDownloadStats)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::SegmentDownload;
		Evt->Param.SegmentStats = SegmentDownloadStats;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportLicenseKey(const Metrics::FLicenseKeyStats& LicenseKeyStats)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::LicenseKey;
		Evt->Param.LicenseKeyStats = LicenseKeyStats;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportDataAvailabilityChange(const Metrics::FDataAvailabilityChange& DataAvailability)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::DataAvailabilityChange;
		Evt->Param.DataAvailability = DataAvailability;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportVideoQualityChange(int32 NewBitrate, int32 PreviousBitrate, bool bIsDrasticDownswitch)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::VideoQualityChange;
		Evt->Param.QualityChange.NewBitrate = NewBitrate;
		Evt->Param.QualityChange.PrevBitrate = PreviousBitrate;
		Evt->Param.QualityChange.bIsDrastic = bIsDrasticDownswitch;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportAudioQualityChange(int32 NewBitrate, int32 PreviousBitrate, bool bIsDrasticDownswitch)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::AudioQualityChange;
		Evt->Param.QualityChange.NewBitrate = NewBitrate;
		Evt->Param.QualityChange.PrevBitrate = PreviousBitrate;
		Evt->Param.QualityChange.bIsDrastic = bIsDrasticDownswitch;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportCodecFormatChange(const FStreamCodecInformation& NewDecodingFormat)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::CodecFormatChange;
		Evt->Param.CodecFormatChange = NewDecodingFormat;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportPrerollStart()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PrerollStart;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportPrerollEnd()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PrerollEnd;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportPlaybackStart()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PlaybackStart;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportPlaybackPaused()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PlaybackPaused;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportPlaybackResumed()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PlaybackResumed;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportPlaybackEnded()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PlaybackEnded;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportJumpInPlayPosition(const FTimeValue& ToNewTime, const FTimeValue& FromTime, Metrics::ETimeJumpReason TimejumpReason)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PlaybackJumped;
		Evt->Param.TimeJump.ToNewTime = ToNewTime;
		Evt->Param.TimeJump.FromTime = FromTime;
		Evt->Param.TimeJump.Reason = TimejumpReason;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportPlaybackStopped()
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::PlaybackStopped;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportSeekCompleted(bool bWasAlreadyThere=false)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::SeekCompleted;
		Evt->Param.SeekComplete.bWasAlreadyThere = bWasAlreadyThere;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportMediaMetadataChanged(const TSharedPtrTS<FMetadataParser>& InNextMetadata)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::MediaMetadataChanged;
		Evt->Param.MediaMetadataChange.NewMetadata = InNextMetadata;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportError(const FErrorDetail& ErrorDetail)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::Errored;
		Evt->Param.ErrorDetail = ErrorDetail;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportLogMessage(IInfoLog::ELevel inLogLevel, const FString& LogMessage, int64 PlayerWallclockMilliseconds)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::LogMessage;
		Evt->Param.LogMessage.Level = inLogLevel;
		Evt->Param.LogMessage.Message = LogMessage;
		Evt->Param.LogMessage.AtMillis = PlayerWallclockMilliseconds;
		return Evt;
	}
	static TSharedPtrTS<FMetricEvent> ReportVideoFrameStats(uint64 InNumTotalFrames, uint64 InNumTotalDroppedFrames, uint64 InNumNewDroppedFrames, const FDroppedFrameStats& InDropStats)
	{
		TSharedPtrTS<FMetricEvent> Evt = MakeSharedTS<FMetricEvent>();
		Evt->Type = EType::VideoFrameStats;
		Evt->Param.FrameStats.NumTotal = InNumTotalFrames;
		Evt->Param.FrameStats.NumDropped = InNumTotalDroppedFrames;
		Evt->Param.FrameStats.NumNewDrops = (uint32) InNumNewDroppedFrames;
		Evt->Param.FrameStats.AvgDropsPerInterval = (uint32)InDropStats.AverageDropsPerInterval.GetAverage();
		Evt->Param.FrameStats.MinDropsPerInterval = (uint32)InDropStats.AverageDropsPerInterval.GetMin();
		Evt->Param.FrameStats.MaxDropsPerInterval = (uint32)InDropStats.AverageDropsPerInterval.GetMax();
		Evt->Param.FrameStats.DropIntervalMs = (uint32)(InDropStats.ReportingIntervalMs > 0 ? InDropStats.ReportingIntervalMs : 0);
		Evt->Param.FrameStats.NumIntervals = InDropStats.NumIntervals;
		Evt->Param.FrameStats.NumIntervalsWithDrops = InDropStats.NumIntervalsWithDrops;
		return Evt;
	}

};



class FAdaptiveStreamingPlayerWorkerThread
{
public:
	static TSharedPtrTS<FAdaptiveStreamingPlayerWorkerThread> Create(bool bUseSharedWorkerThread);

	void AddPlayerInstance(FAdaptiveStreamingPlayer* Instance)
	{
		InstancesToAdd.Enqueue(Instance);
		TriggerWork();
	}
	void RemovePlayerInstance(FAdaptiveStreamingPlayer* Instance)
	{
		FInstanceToRemove That;
		FMediaEvent Done;
		That.Player = Instance;
		That.DoneSignal = &Done;
		InstancesToRemove.Enqueue(MoveTemp(That));
		TriggerWork();
		Done.Wait();
	}

	void TriggerWork()
	{
		HaveWorkSignal.Signal();
	}

	virtual ~FAdaptiveStreamingPlayerWorkerThread();
private:
	void StartWorkerThread();
	void StopWorkerThread();
	void WorkerThreadFN();

	struct FInstanceToRemove
	{
		FAdaptiveStreamingPlayer* Player = nullptr;
		FMediaEvent* DoneSignal = nullptr;
	};

	FMediaThread										WorkerThread;
	TQueue<FAdaptiveStreamingPlayer*, EQueueMode::Mpsc>	InstancesToAdd;
	TQueue<FInstanceToRemove, EQueueMode::Mpsc>			InstancesToRemove;
	TArray<FAdaptiveStreamingPlayer*>					PlayerInstances;
	FMediaEvent											HaveWorkSignal;
	bool												bTerminate = false;
	bool												bIsDedicatedWorker = false;

	static TWeakPtrTS<FAdaptiveStreamingPlayerWorkerThread>	SingletonSelf;
	static FCriticalSection									SingletonLock;
};


class FAdaptiveStreamingPlayerEventHandler
{
public:
	static TSharedPtrTS<FAdaptiveStreamingPlayerEventHandler> Create(bool bUseSharedWorkerThread);

	void DispatchEvent(TSharedPtrTS<FMetricEvent> InEvent);

	virtual ~FAdaptiveStreamingPlayerEventHandler();
private:
	void StartWorkerThread();
	void StopWorkerThread();
	void WorkerThreadFN();

	FMediaThread													WorkerThread;
	TMediaMessageQueueDynamicNoTimeout<TSharedPtrTS<FMetricEvent>>	EventQueue;
	static TWeakPtrTS<FAdaptiveStreamingPlayerEventHandler>			SingletonSelf;
	static FCriticalSection											SingletonLock;
};



class FAdaptiveStreamingPlayerPlaylistPropertyHandler
{
public:
	FAdaptiveStreamingPlayerPlaylistPropertyHandler() = default;
	~FAdaptiveStreamingPlayerPlaylistPropertyHandler() = default;
	bool ConfigureFromJSON(const FString& InJSONConfig)
	{
		if (InJSONConfig.Len())
		{
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InJSONConfig);
			return FJsonSerializer::Deserialize(Reader, Config);
		}
		return true;
	}
	bool WantProperty(FName& OutAsName, const FString& InProtocol, const FString& InProperty)
	{
		static const FString AnyProtocol(TEXT("*"));
		if (!Config.IsValid())
		{
			return false;
		}
		// Get the protocol section
		TArray<TArray<TSharedPtr<FJsonValue>>> Protocols;
		const TArray<TSharedPtr<FJsonValue>>* ProtocolObj = nullptr;
		if (Config->TryGetArrayField(InProtocol, ProtocolObj) && ProtocolObj)
		{
			Protocols.Emplace(*ProtocolObj);
		}
		ProtocolObj = nullptr;
		if (Config->TryGetArrayField(AnyProtocol, ProtocolObj) && ProtocolObj)
		{
			Protocols.Emplace(*ProtocolObj);
		}
		if (Protocols.IsEmpty())
		{
			return false;
		}
		for(auto& Protocol : Protocols)
		{
			for(auto& It : Protocol)
			{
				const TSharedPtr<FJsonObject>* Obj = nullptr;
				if (!It.IsValid() || !It->TryGetObject(Obj))
				{
					continue;
				}
				check(Obj);
				FString Key;
				if (!(*Obj)->TryGetStringField(TEXT("k"), Key))
				{
					continue;
				}
				if (Key.Equals(InProperty, ESearchCase::IgnoreCase))
				{
					FString As;
					if (!(*Obj)->TryGetStringField(TEXT("as"), As))
					{
						As = Key;
					}
					OutAsName = FName(*As);
					return true;
				}
			}
		}
		return false;
	}
private:
	TSharedPtr<FJsonObject> Config;
};


class FAdaptiveStreamingPlayer : public IAdaptiveStreamingPlayer
							   , public IPlayerSessionServices
							   , public IStreamReader::StreamReaderEventListener
							   , public IAccessUnitMemoryProvider
							   , public IPlayerStreamFilter
							   , public TSharedFromThis<FAdaptiveStreamingPlayer, ESPMode::ThreadSafe>
							   , public IPlayerABRLiveControl
{
public:
	class TDeleter
	{
	public:
		void operator()(FAdaptiveStreamingPlayer* InInstanceToDelete)
		{
			TFunction<void()> DeleteTask = [InInstanceToDelete]()
			{
				delete InInstanceToDelete;
			};
			FMediaRunnable::EnqueueAsyncTask(MoveTemp(DeleteTask));
		}
	};

	FAdaptiveStreamingPlayer(const IAdaptiveStreamingPlayer::FCreateParam& InCreateParameters);
	virtual ~FAdaptiveStreamingPlayer();

	void SetStaticResourceProviderCallback(const TSharedPtr<IAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe>& InStaticResourceProvider) override;

	void AddMetricsReceiver(IAdaptiveStreamingPlayerMetrics* InMetricsReceiver) override;
	void RemoveMetricsReceiver(IAdaptiveStreamingPlayerMetrics* InMetricsReceiver) override;

	void HandleOnce();

	// Metrics receiver accessor for event dispatcher thread.
	void LockMetricsReceivers()
	{
		MetricListenerCriticalSection.Lock();
	}
	void UnlockMetricsReceivers()
	{
		MetricListenerCriticalSection.Unlock();
	}
	const TArray<IAdaptiveStreamingPlayerMetrics*, TInlineAllocator<4>>& GetMetricsReceivers()
	{
		return MetricListeners;
	}
	void FireSyncEvent(TSharedPtrTS<FMetricEvent> Event);


	void AddAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode) override;
	void RemoveAEMSReceiver(TWeakPtrTS<IAdaptiveStreamingPlayerAEMSReceiver> InReceiver, FString InForSchemeIdUri, FString InForValue, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode) override;

	void Initialize(const FParamDict& Options) override;
	void ModifyOptions(const FParamDict& InOptionsToSetOrChange, const FParamDict& InOptionsToClear) override;
	FVariantValue GetMediaInfo(FName InKey) const override;

	void SetInitialStreamAttributes(EStreamType StreamType, FStreamSelectionAttributes InitialSelection) override;
	void EnableFrameAccurateSeeking(bool bEnabled) override;
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> GetMediaSamples() override;
	void LoadManifest(const FString& manifestURL) override;

	void SeekTo(const FSeekParam& NewPosition) override;
	void Pause() override;
	void Resume() override;
	void Stop() override;
	void SetPlaybackRange(const FPlaybackRange& InPlaybackRange) override;
	void GetPlaybackRange(FPlaybackRange& OutPlaybackRange) override;
	void SetLooping(const FLoopParam& InLoopParams) override;
	TRangeSet<double> GetSupportedRates(EPlaybackRateType InForPlayRateType) override;
	void SetPlayRate(double InDesiredPlayRate, const FTrickplayParams& InParameters) override;
	double GetPlayRate() const override;

	FErrorDetail GetError() const override;

	bool HaveMetadata() const override;
	FTimeValue GetDuration() const override;
	FTimeValue GetPlayPosition() const override;
	void GetTimelineRange(FTimeRange& OutRange) const override;
	void GetSeekableRange(FTimeRange& OutRange) const override;
	bool HasEnded() const override;
	bool IsSeeking() const override;
	bool IsBuffering() const override;
	bool IsPlaying() const override;
	bool IsPaused() const override;

	void GetLoopState(FLoopState& OutLoopState) const override;
	void GetTrackMetadata(TArray<FTrackMetadata>& OutTrackMetadata, EStreamType StreamType) const override;
	//void GetSelectedTrackMetadata(TOptional<FTrackMetadata>& OutSelectedTrackMetadata, EStreamType StreamType) const override;
	void GetSelectedTrackAttributes(FStreamSelectionAttributes& OutAttributes, EStreamType StreamType) const override;
	void GetReplayEventState(FEventReplayState& OutReplayEventState) const override;

	void SetBitrateCeiling(int32 highestSelectableBitrate) override;
	void SetMaxResolution(int32 MaxWidth, int32 MaxHeight) override;

	//void SelectTrackByMetadata(EStreamType StreamType, const FTrackMetadata& StreamMetadata) override;
	void SelectTrackByAttributes(EStreamType StreamType, const FStreamSelectionAttributes& Attributes) override;
	void DeselectTrack(EStreamType StreamType) override;
	bool IsTrackDeselected(EStreamType StreamType) override;

	void QueryStreamBufferInfo(FStreamBufferInfo& OutStreamBufferInfo, EStreamType InStreamType) override;

	void SuspendOrResumeDecoders(bool bSuspend, const FParamDict& InOptions) override;

	void DebugPrint(void* pPlayer, void (*debugDrawPrintf)(void* pPlayer, const char *pFmt, ...));
	static void DebugHandle(void* pPlayer, void (*debugDrawPrintf)(void* pPlayer, const char *pFmt, ...));

private:
	// Methods from IPlayerSessionServices
	void PostError(const FErrorDetail& Error) override;
	void PostLog(Facility::EFacility FromFacility, IInfoLog::ELevel LogLevel, const FString& Message) override;
	void SendMessageToPlayer(TSharedPtrTS<IPlayerMessage> PlayerMessage) override;
	void GetExternalGuid(FGuid& OutExternalGuid) override;
	ISynchronizedUTCTime* GetSynchronizedUTCTime() override;
	TSharedPtr<IAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe> GetStaticResourceProvider() override;
	TSharedPtrTS<IElectraHttpManager> GetHTTPManager() override;
	TSharedPtrTS<IAdaptiveStreamSelector> GetStreamSelector() override;
	IPlayerABRLiveControl* GetPlayerARBControl() override;
	IPlayerStreamFilter* GetStreamFilter() override;
	const FCodecSelectionPriorities& GetCodecSelectionPriorities(EStreamType ForStream) override;
	TSharedPtrTS<IPlaylistReader> GetManifestReader() override;
	TSharedPtrTS<FContentSteeringHandler> GetContentSteeringHandler() override;
	TSharedPtrTS<FCMCDHandler> GetCMCDHandler() override;
	TSharedPtrTS<IPlayerEntityCache> GetEntityCache() override;
	TSharedPtrTS<IHTTPResponseCache> GetHTTPResponseCache() override;
	IAdaptiveStreamingPlayerAEMSHandler* GetAEMSEventHandler() override;
	FParamDictTS& GetMediaInfoDictionary() override;
	FParamDictTS& GetMutableOptions() override;
	bool HaveOptionValue(const FName& InOption) override;
	const FVariantValue GetOptionValue(const FName& InOption) override;
	TSharedPtrTS<FDRMManager> GetDRMManager() override;
	void SetPlaybackEnd(const FTimeValue& InEndAtTime, IPlayerSessionServices::EPlayEndReason InEndingReason, TSharedPtrTS<IPlayerSessionServices::IPlayEndReason> InCustomManifestObject) override;
	ECustomPropertyResult ValidateMainPlaylistCustomProperty(const FString& InProtocol, const FString& InPlaylistURL, const TArray<FElectraHTTPStreamHeader>& InPlaylistFetchResponseHeaders, const FPlaylistProperty& InProperty) override;

	// Methods from IPlayerStreamFilter
	bool CanDecodeStream(const FStreamCodecInformation& InStreamCodecInfo) const override;
	bool CanDecodeSubtitle(const FString& MimeType, const FString& Codec) const override;


	// Methods from IPlayerABRLiveControl
	FTimeValue ABRGetPlayPosition() const override;
	FTimeRange ABRGetTimeline() const override;
	FTimeValue ABRGetWallclockTime() const override;
	bool ABRIsLive() const override;
	bool ABRShouldPlayOnLiveEdge() const override;
	TSharedPtrTS<const FLowLatencyDescriptor> ABRGetLowLatencyDescriptor() const override;
	FTimeValue ABRGetDesiredLiveEdgeLatency() const override;
	FTimeValue ABRGetLatency() const override;
	FTimeValue ABRGetPlaySpeed() const override;
	void ABRGetStreamBufferStats(IPlayerABRLiveControl::FABRBufferStats& OutBufferStats, EStreamType ForStream) override;
	FTimeRange ABRGetSupportedRenderRateScale() override;
	void ABRSetRenderRateScale(double InRenderRateScale) override;
	double ABRGetRenderRateScale() const override;
	void ABRTriggerClockSync(IPlayerABRLiveControl::EClockSyncType InClockSyncType) override;
	void ABRTriggerPlaylistRefresh() override;


	enum class EPlayerState
	{
		eState_Idle,
		eState_ParsingManifest,
		eState_PreparingStreams,
		eState_Ready,
		eState_Buffering,					//!< Initial buffering at start or after a seek (an expected buffering)
		eState_Playing,						//!<
		eState_Paused,						//!<
		eState_Rebuffering,					//!< Rebuffering due to buffer underrun. Temporary state only.
		eState_Seeking,						//!< Seeking. Temporary state only.
		eState_Error,
	};
	static const char* GetPlayerStateName(EPlayerState s)
	{
		switch(s)
		{
			case EPlayerState::eState_Idle:				return("Idle");
			case EPlayerState::eState_ParsingManifest:	return("Parsing manifest");
			case EPlayerState::eState_PreparingStreams:	return("Preparing streams");
			case EPlayerState::eState_Ready:			return("Ready");
			case EPlayerState::eState_Buffering:		return("Buffering");
			case EPlayerState::eState_Playing:			return("Playing");
			case EPlayerState::eState_Paused:			return("Paused");
			case EPlayerState::eState_Rebuffering:		return("Rebuffering");
			case EPlayerState::eState_Seeking:			return("Seeking");
			case EPlayerState::eState_Error:			return("Error");
			default:									return("undefined");
		}
	}

	enum class EDecoderState
	{
		eDecoder_Paused,
		eDecoder_Running,
	};
	static const char* GetDecoderStateName(EDecoderState s)
	{
		switch(s)
		{
			case EDecoderState::eDecoder_Paused:	return("Paused");
			case EDecoderState::eDecoder_Running:	return("Running");
			default:								return("undefined");
		}
	}

	enum class EPipelineState
	{
		ePipeline_Stopped,
		ePipeline_Prerolling,
		ePipeline_Running,
	};
	static const char* GetPipelineStateName(EPipelineState s)
	{
		switch(s)
		{
			case EPipelineState::ePipeline_Stopped:		return("Stopped");
			case EPipelineState::ePipeline_Prerolling:	return("Prerolling");
			case EPipelineState::ePipeline_Running:		return("Running");
			default:									return("undefined");
		}
	}

	enum class EStreamState
	{
		eStream_Running,
		eStream_ReachedEnd,
	};
	static const char* GetStreamStateName(EStreamState s)
	{
		switch(s)
		{
			case EStreamState::eStream_Running:		return("Running");
			case EStreamState::eStream_ReachedEnd:	return("Reached end");
			default:								return("undefined");
		}
	}

	enum class ERebufferCause
	{
		None,
		Underrun,
		TrackswitchUnderrun
	};

	struct FVideoOutput
	{
		void Close()
		{
			if (OutputHandler.IsValid())
			{
				OutputHandler->SetMediaSamples(nullptr);
				OutputHandler.Reset();
			}
		}
		void Flush(bool bHoldCurrentFrame)
		{
			if (OutputHandler.IsValid())
			{
				OutputHandler->Flush();
			}
		}
		uint64 GetTotalAddedSamples()
		{
			return OutputHandler.IsValid() ? OutputHandler->GetTotalAddedSamples() : (uint64)0;
		}
		uint64 GetTotalNumDroppedSamples()
		{
			return OutputHandler.IsValid() ? OutputHandler->GetTotalNumDroppedSamples() : (uint64)0;
		}
		uint64 GetNewlyNumDroppedSamples()
		{
			return OutputHandler.IsValid() ? OutputHandler->GetNewlyNumDroppedSamples(true) : (uint64)0;
		}
		uint64 PeekNewlyNumDroppedSamples()
		{
			return OutputHandler.IsValid() ? OutputHandler->GetNewlyNumDroppedSamples(false) : (uint64)0;
		}
		TSharedPtr<FOutputHandlerVideo, ESPMode::ThreadSafe> OutputHandler;
	};

	struct FAudioOutput
	{
		void Close()
		{
			if (OutputHandler.IsValid())
			{
				OutputHandler->SetMediaSamples(nullptr);
				OutputHandler.Reset();
			}
		}
		void Flush()
		{
			if (OutputHandler.IsValid())
			{
				OutputHandler->Flush();
			}
		}
		TSharedPtr<FOutputHandlerAudio, ESPMode::ThreadSafe> OutputHandler;
	};

	struct FVideoDecoder : public IAccessUnitBufferListener, public IDecoderOutputBufferListener
	{
		void Close()
		{
			if (Decoder)
			{
				Decoder->SetAUInputBufferListener(nullptr);
				Decoder->SetReadyBufferListener(nullptr);
			}
			delete Decoder;
			Decoder = nullptr;
			LastSentAUCodecData.Reset();
			LastBufferSourceInfo.Reset();
		}
		void Flush()
		{
			if (Decoder)
			{
				Decoder->AUdataFlushEverything();
			}
		}
		void ClearEOD()
		{
			if (Decoder)
			{
				Decoder->AUdataClearEOD();
			}
		}
		void SuspendOrResume(bool bInSuspend, const FParamDict& InOptions)
		{
			if (Decoder && ((bInSuspend && !bSuspended) || (!bInSuspend && bSuspended)))
			{
				Decoder->SuspendOrResumeDecoder(bInSuspend, InOptions);
			}
			bSuspended = bInSuspend;
		}
		void CheckIfNewDecoderMustBeSuspendedImmediately()
		{
			if (Decoder && bSuspended)
			{
				Decoder->SuspendOrResumeDecoder(true, FParamDict());
			}
		}
		virtual void DecoderInputNeeded(const IAccessUnitBufferListener::FBufferStats& currentInputBufferStats)
		{
			if (Parent)
			{
				Parent->VideoDecoderInputNeeded(currentInputBufferStats);
			}
		}

		virtual void DecoderOutputReady(const IDecoderOutputBufferListener::FDecodeReadyStats& currentReadyStats)
		{
			if (Parent)
			{
				Parent->VideoDecoderOutputReady(currentReadyStats);
			}
		}

		FStreamCodecInformation CurrentCodecInfo;
		TSharedPtrTS<FAccessUnit::CodecData> LastSentAUCodecData;
		TSharedPtrTS<const FBufferSourceInfo> LastBufferSourceInfo;
		FAdaptiveStreamingPlayer* Parent = nullptr;
		IVideoDecoder* Decoder = nullptr;
		bool bDrainingForCodecChange = false;
		bool bDrainingForCodecChangeDone = false;
		bool bSuspended = false;
	};

	struct FAudioDecoder : public IAccessUnitBufferListener, public IDecoderOutputBufferListener
	{
		void Close()
		{
			if (Decoder)
			{
				Decoder->SetAUInputBufferListener(nullptr);
				Decoder->SetReadyBufferListener(nullptr);
			}
			delete Decoder;
			Decoder = nullptr;
			LastSentAUCodecData.Reset();
		}
		void Flush()
		{
			if (Decoder)
			{
				Decoder->AUdataFlushEverything();
			}
		}
		void ClearEOD()
		{
			if (Decoder)
			{
				Decoder->AUdataClearEOD();
			}
		}
		void SuspendOrResume(bool bInSuspend, const FParamDict& InOptions)
		{
			if (Decoder && ((bInSuspend && !bSuspended) || (!bInSuspend && bSuspended)))
			{
				Decoder->SuspendOrResumeDecoder(bInSuspend, InOptions);
			}
			bSuspended = bInSuspend;
		}
		void CheckIfNewDecoderMustBeSuspendedImmediately()
		{
			if (Decoder && bSuspended)
			{
				Decoder->SuspendOrResumeDecoder(true, FParamDict());
			}
		}
		virtual void DecoderInputNeeded(const IAccessUnitBufferListener::FBufferStats& currentInputBufferStats)
		{
			if (Parent)
			{
				Parent->AudioDecoderInputNeeded(currentInputBufferStats);
			}
		}

		virtual void DecoderOutputReady(const IDecoderOutputBufferListener::FDecodeReadyStats& currentReadyStats)
		{
			if (Parent)
			{
				Parent->AudioDecoderOutputReady(currentReadyStats);
			}
		}

		FStreamCodecInformation CurrentCodecInfo;
		TSharedPtrTS<FAccessUnit::CodecData> LastSentAUCodecData;
		FAdaptiveStreamingPlayer* Parent = nullptr;
		IAudioDecoder* Decoder = nullptr;
		bool bSuspended = false;
	};

	struct FSubtitleDecoder
	{
		void Close()
		{
			if (Decoder)
			{
				Stop();
				Decoder->Close();
				delete Decoder;
				Decoder = nullptr;
				bIsRunning = false;
			}
			LastSentAUCodecData.Reset();
		}
		void Flush()
		{
			if (Decoder)
			{
				Decoder->AUdataFlushEverything();
			}
		}
		void ClearEOD()
		{
			if (Decoder)
			{
				Decoder->AUdataClearEOD();
			}
		}

		void Start()
		{
			if (Decoder)
			{
				if (!bIsRunning)
				{
					Decoder->Start();
					bIsRunning = true;
				}
			}
		}

		void Stop()
		{
			if (bIsRunning)
			{
				if (Decoder)
				{
					Decoder->Stop();
				}
				bIsRunning = false;
			}
		}

		FStreamCodecInformation CurrentCodecInfo;
		TSharedPtrTS<FAccessUnit::CodecData> LastSentAUCodecData;
		ISubtitleDecoder* Decoder = nullptr;
		bool bIsRunning = false;
		bool bRequireCodecChange = false;
	};

	class FWorkerThreadMessages
	{
	public:
		~FWorkerThreadMessages()
		{
			check(WorkMessages.IsEmpty());
		}
		void SetSharedWorkerThread(TSharedPtrTS<FAdaptiveStreamingPlayerWorkerThread> InSharedWorkerThread)
		{
			SharedWorkerThread = InSharedWorkerThread;
		}

		struct FMessage
		{
			enum class EType
			{
				// Commands
				Initialize,
				ChangeOptions,
				LoadManifest,
				Pause,
				Resume,
				Close,
				ChangeBitrate,
				LimitResolution,
				SelectTrackByMetadata,
				SelectTrackByAttributes,
				DeselectTrack,
				EndPlaybackAt,
				// Player session message
				PlayerSession,
				// Fragment reader messages
				FragmentOpen,
				FragmentClose,
				// Decoder buffer messages
				BufferUnderrun,
			};

			struct FLoadManifest
			{
				FString											URL;
				FString											MimeType;
			};
			struct FLoadBlob
			{
				TSharedPtrTS<FHTTPResourceRequest>				BlobLoadRequest;
			};
			struct FOptionChange
			{
				FParamDict										OptionsToSetOrChange;
				FParamDict										OptionsToClear;
			};
			struct FStreamReader
			{
				TSharedPtrTS<IStreamSegment>					Request;
				EStreamType										AUType = EStreamType::Unsupported;
				SIZE_T											AUSize = 0;
			};
			struct FEvent
			{
				FMediaEvent*									Event = nullptr;
			};
			struct FLoop
			{
				FLoopParam										Loop;
				//FMediaEvent*									Signal = nullptr;
			};
			struct FBitrate
			{
				int32											Value = 0;
			};
			struct FSession
			{
				TSharedPtrTS<IPlayerMessage>					PlayerMessage;
			};
			struct FResolution
			{
				int32											Width = 0;
				int32											Height = 0;
			};
			struct FInitialStreamSelect
			{
				EStreamType										StreamType = EStreamType::Unsupported;;
				FStreamSelectionAttributes						InitialSelection;
			};
			struct FMetadataTrackSelection
			{
				EStreamType										StreamType = EStreamType::Unsupported;;
				FTrackMetadata									TrackMetadata;
				FStreamSelectionAttributes						TrackAttributes;
			};
			struct FEndPlaybackAt
			{
				FTimeValue											EndAtTime;
				IPlayerSessionServices::EPlayEndReason				EndingReason = IPlayerSessionServices::EPlayEndReason::EndAll;
				TSharedPtrTS<IPlayerSessionServices::IPlayEndReason> CustomManifestObject;
			};

			EType Type;
			TVariant<FLoadManifest, FLoadBlob, FOptionChange, FStreamReader, FEvent, FLoop, FBitrate, FSession, FResolution, FInitialStreamSelect, FMetadataTrackSelection, FEndPlaybackAt> Data;
		};

		void Enqueue(FMessage::EType type, const TSharedPtrTS<IStreamSegment>& InRequest)
		{
			FMessage::FStreamReader sr { InRequest };
			FMessage Msg;
			Msg.Type = type;
			Msg.Data.Emplace<FMessage::FStreamReader>(MoveTemp(sr));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void Enqueue(FMessage::EType type, FMediaEvent* InEventSignal)
		{
			check(InEventSignal);
			FMessage::FEvent ev { InEventSignal };
			FMessage Msg;
			Msg.Type = type;
			Msg.Data.Emplace<FMessage::FEvent>(MoveTemp(ev));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void EnqueueBufferUnderrun()
		{
			FMessage Msg;
			Msg.Type = FMessage::EType::BufferUnderrun;
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendInitializeMessage()
		{
			FMessage Msg;
			Msg.Type = FMessage::EType::Initialize;
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendOptionChangeMessage(const FParamDict& InOptionsToSetOrChange, const FParamDict& InOptionsToClear)
		{
			FMessage::FOptionChange oc { InOptionsToSetOrChange, InOptionsToClear };
			FMessage Msg;
			Msg.Type = FMessage::EType::ChangeOptions;
			Msg.Data.Emplace<FMessage::FOptionChange>(MoveTemp(oc));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendLoadManifestMessage(const FString& InURL, const FString& InMimeType)
		{
			FMessage::FLoadManifest lm { InURL, InMimeType };
			FMessage Msg;
			Msg.Type = FMessage::EType::LoadManifest;
			Msg.Data.Emplace<FMessage::FLoadManifest>(MoveTemp(lm));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendPauseMessage()
		{
			FMessage Msg;
			Msg.Type = FMessage::EType::Pause;
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendResumeMessage()
		{
			FMessage Msg;
			Msg.Type = FMessage::EType::Resume;
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendCloseMessage(FMediaEvent* InEventSignal)
		{
			check(InEventSignal);
			FMessage::FEvent ev { InEventSignal };
			FMessage Msg;
			Msg.Type = FMessage::EType::Close;
			Msg.Data.Emplace<FMessage::FEvent>(MoveTemp(ev));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendBitrateMessage(EStreamType /*InType*/, int32 InValue, int32 /*InWhich*/)
		{
			FMessage::FBitrate br { InValue };
			FMessage Msg;
			Msg.Type = FMessage::EType::ChangeBitrate;
			Msg.Data.Emplace<FMessage::FBitrate>(MoveTemp(br));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendPlayerSessionMessage(const TSharedPtrTS<IPlayerMessage>& InMessage)
		{
			FMessage::FSession sess { InMessage };
			FMessage Msg;
			Msg.Type = FMessage::EType::PlayerSession;
			Msg.Data.Emplace<FMessage::FSession>(MoveTemp(sess));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendResolutionMessage(int32 InWidth, int32 InHeight)
		{
			FMessage::FResolution reso { InWidth, InHeight };
			FMessage Msg;
			Msg.Type = FMessage::EType::LimitResolution;
			Msg.Data.Emplace<FMessage::FResolution>(MoveTemp(reso));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendTrackSelectByMetadataMessage(EStreamType InStreamType, const FTrackMetadata& InTrackMetadata)
		{
			FMessage::FMetadataTrackSelection mts { InStreamType, InTrackMetadata };
			FMessage Msg;
			Msg.Type = FMessage::EType::SelectTrackByMetadata;
			Msg.Data.Emplace<FMessage::FMetadataTrackSelection>(MoveTemp(mts));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendTrackSelectByAttributeMessage(EStreamType InStreamType, const FStreamSelectionAttributes& InTrackAttributes)
		{
			FMessage::FMetadataTrackSelection mts { InStreamType };
			mts.TrackAttributes = InTrackAttributes;
			FMessage Msg;
			Msg.Type = FMessage::EType::SelectTrackByAttributes;
			Msg.Data.Emplace<FMessage::FMetadataTrackSelection>(MoveTemp(mts));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendTrackDeselectMessage(EStreamType InStreamType)
		{
			FMessage::FMetadataTrackSelection mts { InStreamType };
			FMessage Msg;
			Msg.Type = FMessage::EType::DeselectTrack;
			Msg.Data.Emplace<FMessage::FMetadataTrackSelection>(MoveTemp(mts));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}
		void SendPlaybackEndMessage(const FTimeValue& InEndAtTime, IPlayerSessionServices::EPlayEndReason InEndingReason, const TSharedPtrTS<IPlayerSessionServices::IPlayEndReason>& InCustomManifestObject)
		{
			FMessage::FEndPlaybackAt epa { InEndAtTime, InEndingReason, InCustomManifestObject };
			FMessage Msg;
			Msg.Type = FMessage::EType::EndPlaybackAt;
			Msg.Data.Emplace<FMessage::FEndPlaybackAt>(MoveTemp(epa));
			TriggerSharedWorkerThread(MoveTemp(Msg));
		}

		void TriggerSharedWorkerThread(FMessage Mesg)
		{
			WorkMessages.Enqueue(MoveTemp(Mesg));
			if (SharedWorkerThread.IsValid())
			{
				SharedWorkerThread->TriggerWork();
			}
		}

		void TriggerSharedWorkerThread()
		{
			if (SharedWorkerThread.IsValid())
			{
				SharedWorkerThread->TriggerWork();
			}
		}

		TSharedPtrTS<FAdaptiveStreamingPlayerWorkerThread>	SharedWorkerThread;
		TQueue<FMessage, EQueueMode::Mpsc>					WorkMessages;
	};

	struct FPendingStartRequest
	{
		enum class EStartType
		{
			PlayStart,
			LoopPoint,
			Seeking,
		};
		FPlayStartPosition										StartAt;
		TOptional<int32>										StartingBitrate;
		IManifest::ESearchType									SearchType = IManifest::ESearchType::Closest;
		FTimeValue												RetryAtTime;
		EStartType												StartType = EStartType::PlayStart;
	};

	struct FBufferStats
	{
		struct FStallMonitor
		{
			int64 DurationMillisec;
			int64 PreviousCheckTime;
			bool bPreviousState;
			FStallMonitor()
			{
				Clear();
			}
			void Clear()
			{
				DurationMillisec = 0;
				PreviousCheckTime = 0;
				bPreviousState = false;
			}
			void Update(int64 tNowMillisec, bool bInCurrentStallState)
			{
				if (bInCurrentStallState)
				{
					if (!bPreviousState)
					{
						PreviousCheckTime = tNowMillisec;
						DurationMillisec = 0;
					}
					else
					{
						DurationMillisec = tNowMillisec - PreviousCheckTime;
					}
				}
				else
				{
					DurationMillisec = 0;
				}
				bPreviousState = bInCurrentStallState;
			}
			int64 GetStalledDurationMillisec() const
			{
				return DurationMillisec;
			}
		};

		void Clear()
		{
			StreamBuffer.Clear();
			DecoderInputBuffer.Clear();
			DecoderOutputBuffer.Clear();
			DecoderOutputStalledMonitor.Clear();
		}
		void UpdateStalledDuration(int64 tNowMillisec)
		{
			DecoderOutputStalledMonitor.Update(tNowMillisec, DecoderOutputBuffer.bOutputStalled);
		}
		int64 GetStalledDurationMillisec() const
		{
			return DecoderOutputStalledMonitor.GetStalledDurationMillisec();
		}
		FAccessUnitBufferInfo								StreamBuffer;
		IAccessUnitBufferListener::FBufferStats				DecoderInputBuffer;
		IDecoderOutputBufferListener::FDecodeReadyStats		DecoderOutputBuffer;
		FStallMonitor										DecoderOutputStalledMonitor;
	};

	struct FPrerollVars
	{
		FPrerollVars()
		{
			Clear();
			bIsVeryFirstStart = true;
		}
		void Clear()
		{
			StartTime = -1;
			bHaveEnoughVideo = false;
			bHaveEnoughAudio = false;
			bHaveEnoughText  = false;
			bIsMidSequencePreroll = false;
		}
		int64		StartTime;
		bool		bHaveEnoughVideo;
		bool		bHaveEnoughAudio;
		bool		bHaveEnoughText;
		bool		bIsVeryFirstStart;
		bool		bIsMidSequencePreroll;
	};

	struct FPostrollVars
	{
		struct FRenderState
		{
			FRenderState()
			{
				Clear();
			}
			void Clear()
			{
				LastCheckTime   = -1;
				LastBufferCount = 0;
			}
			int64	LastCheckTime;
			int32	LastBufferCount;
		};

		FPostrollVars()
		{
			Clear();
		}

		void Clear()
		{
			Video.Clear();
			Audio.Clear();
		}

		FRenderState		Video;
		FRenderState		Audio;
	};

	struct FSeekVars
	{
		void ClearWorkVars()
		{
			bPrerollDone = false;
		}

		void Reset()
		{
			FScopeLock lock(&Lock);
			// Note: Do _not_ reset the pending request created by a user induced seek, or it may get lost if
			//       triggered while already processing a seek due to the asynchronous processing of the request.
			//PendingRequest.Reset();
			ActiveRequest.Reset();
			PlayrangeOnRequest.Reset();
			bPrerollDone = false;
		}

		void SetFinished()
		{
			ActiveRequest.Reset();
		}

		bool HasPendingSeek()
		{
			FScopeLock lock(&Lock);
			return PendingRequest.IsSet();
		}

		bool bPrerollDone = false;
		bool bIsPlayStart = true;

		TOptional<FSeekParam> ActiveRequest;

		mutable FCriticalSection Lock;
		// The pending request must be accessed under lock since it is written to by the main thread and read from the worker thread!
		TOptional<FSeekParam> PendingRequest;
		FTimeRange PlayrangeOnRequest;
	};

	struct FStreamBitrateInfo
	{
		FStreamBitrateInfo()
		{
			Clear();
		}
		void Clear()
		{
			Bitrate 	 = 0;
			QualityLevel = 0;
		}
		int32			Bitrate;
		int32			QualityLevel;
	};

	struct FPendingSegmentRequest
	{
		TSharedPtrTS<IStreamSegment>				Request;
		FTimeValue									AtTime;
		TSharedPtrTS<IManifest::IPlayPeriod>		Period;							//!< Set if transitioning between periods. This is the new period that needs to be readied.
		bool										bStartOver = false;				//!< True when switching tracks within an ongoing period
		bool										bPlayPosAutoReselect = false;	//!< True when trying to re-select a stream at the start of a new period when it was not available before.
		bool										bDidRequestNewPeriodStreams = false;
		EStreamType									StreamType = EStreamType::Unsupported;
		FPlayStartPosition							StartoverPosition;
	};

	struct FPeriodInformation
	{
		TSharedPtrTS<ITimelineMediaAsset>		Period;
		FString									ID;
		FTimeRange								TimeRange;
		int64									LoopCount = 0;

		// Currently selected buffer source per stream type in this period.
		TSharedPtrTS<FBufferSourceInfo>			BufferSourceInfoVid;
		TSharedPtrTS<FBufferSourceInfo>			BufferSourceInfoAud;
		TSharedPtrTS<FBufferSourceInfo>			BufferSourceInfoTxt;
	};


	struct FInternalStreamSelectionAttributes : public FStreamSelectionAttributes
	{
		bool IsDeselected() const
		{ return !bIsSelected; }
		void Select()
		{ bIsSelected = true; }
		void Deselect()
		{ bIsSelected = false; }
	private:
		bool bIsSelected = true;
	};



	struct FStreamDataBuffers
	{
		TSharedPtrTS<FMultiTrackAccessUnitBuffer>	VidBuffer;
		TSharedPtrTS<FMultiTrackAccessUnitBuffer>	AudBuffer;
		TSharedPtrTS<FMultiTrackAccessUnitBuffer>	TxtBuffer;
		TSharedPtrTS<FMultiTrackAccessUnitBuffer> GetBuffer(EStreamType InStreamType)
		{
			switch(InStreamType)
			{
				case EStreamType::Video: return VidBuffer;
				case EStreamType::Audio: return AudBuffer;
				case EStreamType::Subtitle: return TxtBuffer;
				default: return TSharedPtrTS<FMultiTrackAccessUnitBuffer>();
			}
		}
	};


	struct FInternalLoopState : public FLoopState
	{
		FTimeValue From;
		FTimeValue To;
	};


	struct FMediaMetadataUpdate
	{
		FMediaMetadataUpdate()
		{
			Reset();
		}
		void Reset()
		{
			NextEntries.Empty();
			// Do NOT reset the active metadata, but the time it became valid!
			ActiveSince.SetToInvalid();
		}
		void AddEntry(const FTimeValue& InValidFrom, const TSharedPtrTS<FMetadataParser>& InMetadata, bool bInTriggerInternalRefresh)
		{
			FEntry& e = NextEntries.Emplace_GetRef();
			e.ValidFrom = InValidFrom.IsValid() ? InValidFrom : FTimeValue::GetZero();
			e.Metadata = InMetadata;
			e.bTriggerInternalRefresh = bInTriggerInternalRefresh;
			NextEntries.StableSort([](const FEntry& a, const FEntry& b)
			{
				const FTimeValue& t1 = a.ValidFrom;
				const FTimeValue& t2 = b.ValidFrom;
				const int64 s1 = t1.GetSequenceIndex();
				const int64 s2 = t2.GetSequenceIndex();
				return (s1 == s2 && t1 < t2) || (s1 < s2);
			});
		}
		enum class EResult
		{
			NoChange,
			Changed,
			ChangedAndUpdate
		};
		EResult Handle(const FTimeValue& InAtTime);
		TSharedPtrTS<FMetadataParser> GetActive() const
		{
			return ActiveMetadata;
		}

		struct FEntry
		{
			FTimeValue ValidFrom;
			TSharedPtrTS<FMetadataParser> Metadata;
			bool bTriggerInternalRefresh = false;
		};
		TArray<FEntry> NextEntries;
		TSharedPtrTS<FMetadataParser> ActiveMetadata;
		FTimeValue ActiveSince;
	};

	struct FMetadataHandlingState
	{
		FMetadataHandlingState()
		{
			Reset();
		}
		void Reset()
		{
			LastSentPeriodID.Empty();
			LastHandlingTime.SetToInvalid();
		}
		FString LastSentPeriodID;
		FTimeValue LastHandlingTime;
	};

	void InternalLoadManifest(const FString& URL, const FString& MimeType);
	void InternalCancelLoadManifest();
	void InternalCloseManifestReader();
	bool SelectManifest();
	void UpdateManifest();
	void OnManifestGetMimeTypeComplete(TSharedPtrTS<FHTTPResourceRequest> InRequest);

	void VideoDecoderInputNeeded(const IAccessUnitBufferListener::FBufferStats& currentInputBufferStats);
	void VideoDecoderOutputReady(const IDecoderOutputBufferListener::FDecodeReadyStats& currentReadyStats);

	void AudioDecoderInputNeeded(const IAccessUnitBufferListener::FBufferStats& currentInputBufferStats);
	void AudioDecoderOutputReady(const IDecoderOutputBufferListener::FDecodeReadyStats& currentReadyStats);

	void StartWorkerThread();
	void StopWorkerThread();

	int32 CreateRenderers();
	void DestroyRenderers();

	int32 CreateDecoder(EStreamType type);
	void DestroyDecoders();
	bool GetHighestVideoProfileResolution(FStreamCodecInformation& OutStreamInfo, TSharedPtrTS<const FBufferSourceInfo> InBufferSource, const FTimeValue& AtTime, const FStreamCodecInformation& InForCodec);
	void UpdateStreamResolutionLimit();
	void AddUpcomingPeriod(TSharedPtrTS<IManifest::IPlayPeriod> InUpcomingPeriod);
	void RequestNewPeriodStreams(EStreamType InType, FPendingSegmentRequest& InOutCurrentRequest);
	void UpdatePeriodStreamBufferSourceInfo(TSharedPtrTS<IManifest::IPlayPeriod> InForPeriod, EStreamType InStreamType, TSharedPtrTS<FBufferSourceInfo> InBufferSourceInfo);
	TSharedPtrTS<FBufferSourceInfo> GetStreamBufferInfoAtTime(bool& bOutHavePeriods, bool& bOutFoundTime, EStreamType InStreamType, const FTimeValue& InAtTime) const;


	// AU memory / generic stream reader memory
	void* AUAllocate(IAccessUnitMemoryProvider::EDataType type, SIZE_T size, SIZE_T alignment) override;
	void AUDeallocate(IAccessUnitMemoryProvider::EDataType type, void *pAddr) override;

	// Stream reader events
	void OnFragmentOpen(TSharedPtrTS<IStreamSegment> pRequest) override;
	bool OnFragmentAccessUnitReceived(FAccessUnit* pAccessUnit) override;
	void OnFragmentReachedEOS(EStreamType InStreamType, TSharedPtr<const FBufferSourceInfo, ESPMode::ThreadSafe> InStreamSourceInfo) override;
	void OnFragmentClose(TSharedPtrTS<IStreamSegment> pRequest) override;

	void Deprecate_InternalInitializeDecoderLimits();
	bool Deprecate_InternalStreamAllowedAsPerLimits(const FStreamCodecInformation& InStreamCodecInfo) const;

	void InternalInitialize();
	void InternalHandleOnce();
	bool InternalHandleThreadMessages();
	void InternalHandleManifestReader();
	void InternalHandlePendingStartRequest(const FTimeValue& CurrentTime);
	void InternalHandlePendingFirstSegmentRequest(const FTimeValue& CurrentTime);
	void InternalHandleCompletedSegmentRequests(const FTimeValue& CurrentTime);
	void InternalHandleSegmentTrackChanges(const FTimeValue& CurrentTime);
	void InternalStartoverAtCurrentPosition();
	void InternalStartoverAtLiveEdge();

	void UpdateDiagnostics();
	void UpdateFrameStats(bool bFinal);
	void HandleNewBufferedData();
	void HandleNewOutputData();
	void HandleSessionMessage(TSharedPtrTS<IPlayerMessage> SessionMessage);
	void HandlePlayStateChanges();
	void HandleSeeking();
	void HandlePendingMediaSegmentRequests();
	void CancelPendingMediaSegmentRequests(EStreamType StreamType);
	void HandleDeselectedBuffers();
	void HandleDecoderChanges();
	void HandleSubtitleDecoder();
	void HandleMetadataChanges();
	void HandleAEMSEvents();

	void CheckForStreamEnd();

	void CheckForErrors();

	FTimeValue CalculateCurrentLiveLatency(bool bViaLatencyElement);
	double GetMinBufferTimeBeforePlayback();
	bool HaveEnoughBufferedDataToStartPlayback();
	void PrepareForPrerolling();

	void SetPlaystartOptions(FPlayStartOptions& OutOptions);
	void ClampStartRequestTime(FTimeValue& InOutTimeToClamp);
	FTimeValue ClampTimeToCurrentRange(const FTimeValue& InTime, bool bClampToStart, bool bClampToEnd);

	TSharedPtrTS<FMultiTrackAccessUnitBuffer> GetStreamBuffer(EStreamType InStreamType, const TSharedPtrTS<FStreamDataBuffers>& InFromStreamBuffers)
	{
		if (InFromStreamBuffers.IsValid())
		{
			if (InStreamType == EStreamType::Video)
			{
				return InFromStreamBuffers->VidBuffer;
			}
			else if (InStreamType == EStreamType::Audio)
			{
				return InFromStreamBuffers->AudBuffer;
			}
			else if (InStreamType == EStreamType::Subtitle)
			{
				return InFromStreamBuffers->TxtBuffer;
			}
		}
		return TSharedPtrTS<FMultiTrackAccessUnitBuffer>();
	}

	TSharedPtrTS<FMultiTrackAccessUnitBuffer> GetCurrentOutputStreamBuffer(EStreamType InStreamType)
	{
		FScopeLock lock(&DataBuffersCriticalSection);
		return GetStreamBuffer(InStreamType, ActiveDataOutputBuffers);
	}

	TSharedPtrTS<FMultiTrackAccessUnitBuffer> GetCurrentReceiveStreamBuffer(EStreamType InStreamType)
	{
		FScopeLock lock(&DataBuffersCriticalSection);
		return GetStreamBuffer(InStreamType, CurrentDataReceiveBuffers);
	}

	bool IsExpectedToStreamNow(EStreamType InType);

	bool IsSeamlessBufferSwitchPossible(EStreamType InStreamType, const TSharedPtrTS<FStreamDataBuffers>& InFromStreamBuffers);

	void GetStreamBufferUtilization(FAccessUnitBufferInfo& OutInfo, EStreamType BufferType);
	void GetStreamBufferUtilization(FAccessUnitBufferInfo& OutInfo, const FAccessUnit* InForThis);

	void FeedDecoder(EStreamType Type, IAccessUnitBufferInterface* Decoder, bool bHandleUnderrun);
	void ClearAllDecoderEODs();

	void InternalStartAt(const FSeekParam& NewPosition, const FTimeRange* InTimeRange);
	void InternalPause();
	void InternalResume();
	void InternalRebuffer();
	void InternalStop(bool bHoldCurrentFrame);
	void InternalClose();
	void InternalHandlePendingLoopRequest();
	void InternalSetPlaybackEnded();
	void InternalDeselectStream(EStreamType StreamType);

	void DispatchEvent(TSharedPtrTS<FMetricEvent> Event);
	void DispatchBufferingEvent(bool bBegin, EPlayerState Reason);
	void DispatchBufferUtilizationEvent(EStreamType BufferType);

	void UpdateDataAvailabilityState(Metrics::FDataAvailabilityChange& DataAvailabilityState, Metrics::FDataAvailabilityChange::EAvailability NewAvailability);

	void StartRendering();
	void StopRendering();

	//
	// Member variables
	//
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe>						CurrentMediaSamples;
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe>						EmptyMediaSamples;
	FGuid																ExternalPlayerGUID;

	TWeakPtr<IAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe> StaticResourceProvider;

	TSharedPtrTS<FAdaptiveStreamingPlayerEventHandler>					EventDispatcher;
	TSharedPtrTS<FAdaptiveStreamingPlayerWorkerThread>					SharedWorkerThread;
	FWorkerThreadMessages												WorkerThread;
	IAdaptiveStreamingPlayer::FCreateParam::EWorkerThreads				UseSharedWorkerThreads;

	FParamDictTS														MediaInfoDictionary;
	FParamDictTS														PlayerOptions;
	FPlaybackState														PlaybackState;
	ISynchronizedUTCTime*												SynchronizedUTCTime;
	IAdaptiveStreamingPlayerAEMSHandler*								AEMSEventHandler;

	TSharedPtrTS<FMediaRenderClock>										RenderClock;

	TSharedPtrTS<IElectraHttpManager>									HttpManager;
	TSharedPtrTS<IPlayerEntityCache>									EntityCache;
	TSharedPtrTS<IHTTPResponseCache>									HttpResponseCache;

	TSharedPtrTS<FDRMManager>											DrmManager;

	TMediaQueueDynamic<TSharedPtrTS<FErrorDetail>>						ErrorQueue;

	FString																ManifestURL;
	TSharedPtrTS<IManifest>												Manifest;
	TSharedPtrTS<IPlaylistReader>										ManifestReader;
	TSharedPtrTS<FHTTPResourceRequest>									ManifestMimeTypeRequest;
	FAdaptiveStreamingPlayerPlaylistPropertyHandler						PlaylistPropertyHandler;
	Playlist::FReplayEventParams										EventReplayParams;


	IStreamReader*														StreamReaderHandler;
	bool																bStreamingHasStarted;

	FCriticalSection													DataBuffersCriticalSection;
	TArray<TSharedPtrTS<FStreamDataBuffers>>							NextDataBuffers;
	TSharedPtrTS<FStreamDataBuffers>									CurrentDataReceiveBuffers;
	TSharedPtrTS<FStreamDataBuffers>									ActiveDataOutputBuffers;
	bool																bIsVideoDeselected;
	bool																bIsAudioDeselected;
	bool																bIsTextDeselected;

	Metrics::FDataAvailabilityChange									DataAvailabilityStateVid;
	Metrics::FDataAvailabilityChange									DataAvailabilityStateAud;
	Metrics::FDataAvailabilityChange									DataAvailabilityStateTxt;

	FStreamSelectionAttributes											StreamSelectionAttributesVid;
	FStreamSelectionAttributes											StreamSelectionAttributesAud;
	FStreamSelectionAttributes											StreamSelectionAttributesTxt;

	FInternalStreamSelectionAttributes									SelectedStreamAttributesVid;
	FInternalStreamSelectionAttributes									SelectedStreamAttributesAud;
	FInternalStreamSelectionAttributes									SelectedStreamAttributesTxt;

	TSharedPtrTS<FStreamSelectionAttributes>							PendingTrackSelectionVid;
	TSharedPtrTS<FStreamSelectionAttributes>							PendingTrackSelectionAud;
	TSharedPtrTS<FStreamSelectionAttributes>							PendingTrackSelectionTxt;

	TArray<FString>														ExcludedVideoDecoderPrefixes;
	TArray<FString>														ExcludedAudioDecoderPrefixes;
	TArray<FString>														ExcludedSubtitleDecoderPrefixes;
	FCodecSelectionPriorities											CodecPrioritiesVideo;
	FCodecSelectionPriorities											CodecPrioritiesAudio;
	FCodecSelectionPriorities											CodecPrioritiesSubtitles;

	EPlayerState														CurrentState;
	EPipelineState														PipelineState;
	EDecoderState														DecoderState;
	EStreamState														StreamState;

	FPrerollVars														PrerollVars;
	FPostrollVars														PostrollVars;
	FSeekVars															SeekVars;
	EPlayerState														LastBufferingState;
	double																RenderRateScale;
	FTimeValue															RebufferDetectedAtPlayPos;
	ERebufferCause														RebufferCause;
	bool																bIsClosing;

	TSharedPtrTS<FContentSteeringHandler>								ContentSteeringHandler;
	TSharedPtrTS<FCMCDHandler>											CMCDHandler;

	TSharedPtrTS<IAdaptiveStreamSelector>								StreamSelector;
	int32																BitrateCeiling;
	int32																VideoResolutionLimitWidth;
	int32																VideoResolutionLimitHeight;

	FStreamBitrateInfo													CurrentVideoStreamBitrate;
	FStreamBitrateInfo													CurrentAudioStreamBitrate;

	bool																bShouldBePaused;
	bool																bShouldBePlaying;

	FInternalLoopState													CurrentLoopState;
	FLoopParam															CurrentLoopParam;
	TQueue<FInternalLoopState>											NextLoopStates;

	mutable FCriticalSection											ActivePeriodCriticalSection;
	TArray<FPeriodInformation>											ActivePeriods;
	TArray<FPeriodInformation>											UpcomingPeriods;
	FMetadataHandlingState												MetadataHandlingState;
	FMediaMetadataUpdate												MediaMetadataUpdates;
	TSharedPtrTS<IManifest::IPlayPeriod>								InitialPlayPeriod;
	TSharedPtrTS<IManifest::IPlayPeriod>								CurrentPlayPeriodVideo;
	TSharedPtrTS<IManifest::IPlayPeriod>								CurrentPlayPeriodAudio;
	TSharedPtrTS<IManifest::IPlayPeriod>								CurrentPlayPeriodText;
	TSharedPtrTS<FPendingStartRequest>									PendingStartRequest;
	TSharedPtrTS<IStreamSegment>										PendingFirstSegmentRequest;
	TUniqueObj<TQueue<FPendingSegmentRequest>>							NextPendingSegmentRequests;
	TUniqueObj<TQueue<TSharedPtrTS<IStreamSegment>>>					ReadyWaitingSegmentRequests;
	TMultiMap<EStreamType, TSharedPtrTS<IStreamSegment>>				CompletedSegmentRequests;
	bool																bFirstSegmentRequestIsForLooping;

	FPlayerSequenceState												CurrentPlaybackSequenceState;

	uint32																CurrentPlaybackSequenceID[4]; // 0=video, 1=audio, 2=subtitiles, 3=UNSUPPORTED
	FTimeRange															CurrentSegmentDownloadTimeRange[4];

	FVideoOutput														VideoOutput;
	FAudioOutput														AudioOutput;
	FVideoDecoder														VideoDecoder;
	FAudioDecoder														AudioDecoder;
	FSubtitleDecoder													SubtitleDecoder;

	FCriticalSection													MetricListenerCriticalSection;
	TArray<IAdaptiveStreamingPlayerMetrics*, TInlineAllocator<4>>		MetricListeners;

	mutable FCriticalSection											DiagnosticsCriticalSection;
	FBufferStats														VideoBufferStats;
	FBufferStats														AudioBufferStats;
	FBufferStats														TextBufferStats;
	FErrorDetail														LastErrorDetail;

	FDroppedFrameStats													DroppedFrameStats;

	AdaptiveStreamingPlayerConfig::FConfiguration						PlayerConfig;

	static FAdaptiveStreamingPlayer* 									PointerToLatestPlayer;
};

} // namespace Electra

