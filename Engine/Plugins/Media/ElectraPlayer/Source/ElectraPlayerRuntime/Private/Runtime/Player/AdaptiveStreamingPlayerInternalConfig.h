// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StreamAccessUnitBuffer.h"

namespace Electra
{
	namespace AdaptiveStreamingPlayerConfig
	{
		struct FConfiguration
		{
			FConfiguration()
			{
				// Set default values to maximum permitted values.
				StreamBufferConfigVideo.MaxDuration.SetFromSeconds(6.0);

				StreamBufferConfigAudio.MaxDuration.SetFromSeconds(6.0);
				StreamBufferConfigAudioOnly.MaxDuration.SetFromSeconds(24.0);

				// Subtitle streams tend to be sparse and each AU could potentially have
				// a huge duration, so we need to ignore durations altogether.
				StreamBufferConfigText.MaxDuration.SetToPositiveInfinity();

				bHoldLastFrameDuringSeek = true;

				InitialBufferMinTimeAvailBeforePlayback = 5.0;
				SeekBufferMinTimeAvailBeforePlayback = 5.0;
				RebufferMinTimeAvailBeforePlayback = 5.0;
			}

			struct FVideoDecoderLimit
			{
				struct FTierProfileLevel
				{
					int32 Tier = 0;
					int32 Profile = 0;
					int32 Level = 0;
				};
				FStreamCodecInformation::FResolution	MaxResolution;
				FTierProfileLevel						MaxTierProfileLevel;
			};


			FAccessUnitBuffer::FConfiguration					StreamBufferConfigVideo;
			FAccessUnitBuffer::FConfiguration					StreamBufferConfigAudio;
			FAccessUnitBuffer::FConfiguration					StreamBufferConfigAudioOnly;
			FAccessUnitBuffer::FConfiguration					StreamBufferConfigText;

			bool												bHoldLastFrameDuringSeek;

			double												InitialBufferMinTimeAvailBeforePlayback;
			double												SeekBufferMinTimeAvailBeforePlayback;
			double												RebufferMinTimeAvailBeforePlayback;

		// Deprecate these soon
			FVideoDecoderLimit									H264LimitUpto30fps;
			FVideoDecoderLimit									H264LimitAbove30fps;
		};

	} // namespace AdaptiveStreamingPlayerConfig

} // namespace Electra


