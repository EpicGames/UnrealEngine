// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Misc/Paths.h"
#include "Player/AdaptiveStreamingPlayerInternal.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/HLS/PlaylistHandlerHLS.h"
#include "Player/mp4/PlaylistReaderMP4.h"
#include "Player/isobmff/PlaylistReaderISOBMFF.h"
#include "Player/DASH/PlaylistReaderDASH.h"
#include "Player/mkv/PlaylistReaderMKV.h"
#include "Player/mpegaudio/PlaylistReaderMPEGAudio.h"
#include "Utilities/Utilities.h"
#include "Utilities/StringHelpers.h"
#include "Utilities/URLParser.h"
#include "ParameterDictionary.h"
#include "MediaStreamMetadata.h"


/*********************************************************************************************************************/

#include "HAL/IConsoleManager.h"
static TAutoConsoleVariable<bool> CVarElectraUseLegacyMP4Player(
	TEXT("Electra.UseLegacyMP4Player"),
	false,
	TEXT("Use old sequentially reading mp4 player instead of new player.\n")
	TEXT(" true: use old player, false use new player"),
	ECVF_Default);

/*********************************************************************************************************************/

namespace Electra
{

namespace Playlist
{
static const FString MIMETypeMP4(TEXT("video/mp4"));
static const FString MIMETypeMP4A(TEXT("audio/mp4"));
static const FString MIMETypeQuickTime(TEXT("video/quicktime"));
static const FString MIMETypeHLS(TEXT("application/vnd.apple.mpegURL"));
static const FString MIMETypeDASH(TEXT("application/dash+xml"));
static const FString MIMETypeMKV(TEXT("video/x-matroska"));
static const FString MIMETypeMKA(TEXT("audio/x-matroska"));
static const FString MIMETypeMPEGAudio(TEXT("audio/mpeg"));
static const FString MIMETypeMPEGAudio4(TEXT("audio/aac"));



/**
 * Fix common mistakes in a file:// URL like backslashes instead of forward slashes
 * and spaces that are not escaped.
 */
FString FixLocalFileSchemeURL(const FString& InURL)
{
	FString URL(InURL);
	if (URL.StartsWith("file:", ESearchCase::IgnoreCase))
	{
		URL.ReplaceCharInline(TCHAR('\\'), TCHAR('/'));
		URL.ReplaceInline(TEXT(" "), TEXT("%20"));
	}
	return URL;
}

/**
 * Returns the MIME type of the media playlist pointed to by the given URL.
 * This only parses the URL for known extensions or scheme.
 * If the MIME type cannot be precisely determined an empty string is returned.
 *
 * @param URL    Media playlist URL
 *
 * @return MIME type or empty string if it cannot be determined
 */
FString GetMIMETypeForURL(const FString& URL)
{
	FString MimeType;
	FURL_RFC3986 UrlParser;
	if (!UrlParser.Parse(URL))
	{
		return MimeType;
	}

	//	FString PathOnly = UrlParser->GetPath();
	TArray<FString> PathComponents;
	UrlParser.GetPathComponents(PathComponents);
	if (PathComponents.Num())
	{
		FString LowerCaseExtension = FPaths::GetExtension(PathComponents.Last().ToLower());

		// Check for known extensions.
		static const FString kTextMP4(TEXT("mp4"));
		static const FString kTextMP4V(TEXT("m4v"));
		static const FString kTextMP4A(TEXT("m4a"));
		static const FString kTextMOV(TEXT("mov"));
		static const FString kTextMPD(TEXT("mpd"));
		static const FString kTextM3U8(TEXT("m3u8"));
		static const FString kTextMKV(TEXT("mkv"));
		static const FString kTextMKA(TEXT("mka"));
		static const FString kTextWEBM(TEXT("webm"));
		static const FString kTextMPA(TEXT("mpa"));
		static const FString kTextMP3(TEXT("mp3"));
		static const FString kTextAAC(TEXT("aac"));
		if (LowerCaseExtension == kTextMP4 || LowerCaseExtension == kTextMP4V)
		{
			MimeType = MIMETypeMP4;
		}
		else if (LowerCaseExtension == kTextMP4A)
		{
			MimeType = MIMETypeMP4A;
		}
		else if (LowerCaseExtension == kTextMOV)
		{
			MimeType = MIMETypeQuickTime;
		}
		else if (LowerCaseExtension == kTextMPD)
		{
			MimeType = MIMETypeDASH;
		}
		else if (LowerCaseExtension == kTextM3U8)
		{
			MimeType = MIMETypeHLS;
		}
		else if (LowerCaseExtension == kTextMKV || LowerCaseExtension == kTextWEBM)
		{
			MimeType = MIMETypeMKV;
		}
		else if (LowerCaseExtension == kTextMKA)
		{
			MimeType = MIMETypeMKA;
		}
		else if (LowerCaseExtension == kTextMPA || LowerCaseExtension == kTextMP3)
		{
			MimeType = MIMETypeMPEGAudio;
		}
		else if (LowerCaseExtension == kTextAAC)
		{
			MimeType = MIMETypeMPEGAudio4;
		}
	}

	return MimeType;
}


} // namespace Playlist







void FAdaptiveStreamingPlayer::OnManifestGetMimeTypeComplete(TSharedPtrTS<FHTTPResourceRequest> InRequest)
{
	if (!InRequest->GetWasCanceled())
	{
		FString mimeType(TEXT("application/octet-stream"));
		if (InRequest->GetError() == 0)
		{
			const HTTP::FConnectionInfo* ci = InRequest->GetConnectionInfo();
			if (ci)
			{
				// Get the content type, if it exists.
				if (ci->ContentType.Len())
				{
					mimeType = ci->ContentType;
				}
				// Use the effective URL after potential redirections.
				if (ci->EffectiveURL.Len())
				{
					ManifestURL = ci->EffectiveURL;
				}
			}
		}
		WorkerThread.SendLoadManifestMessage(ManifestURL, mimeType);
	}
}


void FAdaptiveStreamingPlayer::InternalCancelLoadManifest()
{
	if (ManifestMimeTypeRequest.IsValid())
	{
		ManifestMimeTypeRequest->Cancel();
		ManifestMimeTypeRequest.Reset();
	}
}


void FAdaptiveStreamingPlayer::InternalCloseManifestReader()
{
	if (ManifestReader.IsValid())
	{
		ManifestReader->Close();
		ManifestReader.Reset();
	}
}


//-----------------------------------------------------------------------------
/**
 * Starts asynchronous loading and parsing of a manifest.
 *
 * @param URL
 * @param MimeType
 */
void FAdaptiveStreamingPlayer::InternalLoadManifest(const FString& InURL, const FString& MimeType)
{
	// Fix potential errors in a local file URL.
	FString URL = Playlist::FixLocalFileSchemeURL(InURL);
	// Remember the original request URL since we may lose the fragment part in requests.
	ManifestURL = URL;
	ManifestMimeTypeRequest.Reset();
	if (CurrentState == EPlayerState::eState_Idle)
	{
		FString mimeType = MimeType;
		if (PlayerOptions.HaveKey(OptionKeyMimeType))
		{
			mimeType = PlayerOptions.GetValue(OptionKeyMimeType).GetFString();
		}
		else if (mimeType.IsEmpty())
		{
			mimeType = Playlist::GetMIMETypeForURL(URL);
		}
		// If there is no mime type we need to issue a HTTP HEAD request in order to get the "Content-Type" header.
		if (mimeType.IsEmpty() && (URL.StartsWith("https://", ESearchCase::CaseSensitive) || URL.StartsWith("http://", ESearchCase::CaseSensitive)))
		{
			ManifestMimeTypeRequest = MakeSharedTS<FHTTPResourceRequest>();
			ManifestMimeTypeRequest->URL(URL);
			ManifestMimeTypeRequest->Verb(TEXT("HEAD"));
			ManifestMimeTypeRequest->Callback().BindThreadSafeSP(AsShared(), &FAdaptiveStreamingPlayer::OnManifestGetMimeTypeComplete);
			ManifestMimeTypeRequest->StartGet(this);
			return;
		}
		DispatchEvent(FMetricEvent::ReportOpenSource(URL));
		if (mimeType.Len())
		{
			check(!ManifestReader.IsValid());

			CurrentState = EPlayerState::eState_ParsingManifest;
			if (mimeType == Playlist::MIMETypeHLS)
			{
				ManifestReader = IPlaylistHandlerHLS::Create(this);
				CMCDHandler->SetStreamingFormat(FCMCDHandler::EStreamingFormat::HLS);
			}
			else if (mimeType == Playlist::MIMETypeMP4 || mimeType == Playlist::MIMETypeMP4A || mimeType == Playlist::MIMETypeQuickTime)
			{
				if (CVarElectraUseLegacyMP4Player.GetValueOnAnyThread())
				{
					ManifestReader = IPlaylistReaderMP4::Create(this);
				}
				else
				{
					ManifestReader = IPlaylistReaderISOBMFF::Create(this);
				}
			}
			else if (mimeType == Playlist::MIMETypeDASH)
			{
				ManifestReader = IPlaylistReaderDASH::Create(this);
				CMCDHandler->SetStreamingFormat(FCMCDHandler::EStreamingFormat::DASH);
			}
			else if (mimeType == Playlist::MIMETypeMKV || mimeType == Playlist::MIMETypeMKA)
			{
				ManifestReader = IPlaylistReaderMKV::Create(this);
			}
			else if (mimeType == Playlist::MIMETypeMPEGAudio || mimeType == Playlist::MIMETypeMPEGAudio4)
			{
				ManifestReader = IPlaylistReaderMPEGAudio::Create(this);
			}
			else
			{
				FErrorDetail err;
				err.SetFacility(Facility::EFacility::Player);
				err.SetMessage(TEXT("Unsupported stream MIME type"));
				err.SetCode(INTERR_UNSUPPORTED_FORMAT);
				PostError(err);
			}

			if (ManifestReader.IsValid())
			{
				if (EventReplayParams.ParseFromURL(URL))
				{
					FLoopState ls;
					PlaybackState.GetLoopState(ls);
					URL = EventReplayParams.GetCleanedURL();
					EventReplayParams.SetLoopingAtStart(ls.bIsEnabled);
				}
				ManifestReader->LoadAndParse(URL, EventReplayParams);
			}
		}
		else
		{
			FErrorDetail err;
			err.SetFacility(Facility::EFacility::Player);
			err.SetMessage(TEXT("Could not determine stream MIME type"));
			err.SetCode(INTERR_UNSUPPORTED_FORMAT);
			PostError(err);
		}
	}
}


void FAdaptiveStreamingPlayer::InternalHandleManifestReader()
{
	if (ManifestReader.IsValid())
	{
		ManifestReader->HandleOnce();
	}
}



//-----------------------------------------------------------------------------
/**
 * Selects the internal presentation for playback, setting the initial metadata and
 * updating options that may appear in the playlist.
 *
 * @return
 */
bool FAdaptiveStreamingPlayer::SelectManifest()
{
	if (ManifestReader.IsValid())
	{
		check(Manifest == nullptr);
		TSharedPtrTS<IManifest> NewPresentation = ManifestReader->GetManifest();
		check(NewPresentation.IsValid());

		PlaybackState.SetSeekableRange(NewPresentation->GetSeekableTimeRange());
		PlaybackState.SetTimelineRange(NewPresentation->GetTotalTimeRange());
		PlaybackState.SetDuration(NewPresentation->GetDuration());

		// Check for playback range restriction. This is currently assumed to come from URL fragment parameters
		// like example.mp4#r=10.8,18.4
		// These do not override any user defined range values!
		FPlaybackRange PlaybackRange;
		FTimeRange RestrictedPlaybackRange;
		RestrictedPlaybackRange = NewPresentation->GetPlaybackRange(IManifest::EPlaybackRangeType::LockedPlaybackRange);
		PlaybackState.GetPlayRange(PlaybackRange);
		if (RestrictedPlaybackRange.Start.IsValid() && !PlaybackRange.Start.Get(FTimeValue()).IsValid())
		{
			PlaybackRange.Start = RestrictedPlaybackRange.Start;
		}
		if (RestrictedPlaybackRange.End.IsValid() && !PlaybackRange.End.Get(FTimeValue()).IsValid())
		{
			PlaybackRange.End = RestrictedPlaybackRange.End;
		}
		// Set the new range, but say it did not change. This is the first start and we do not need to issue a seek.
		PlaybackState.SetPlayRange(PlaybackRange);
		PlaybackState.ActivateNewPlayRange(nullptr);

		TArray<FTrackMetadata> VideoTrackMetadata;
		TArray<FTrackMetadata> AudioTrackMetadata;
		TArray<FTrackMetadata> SubtitleTrackMetadata;
		NewPresentation->GetTrackMetadata(VideoTrackMetadata, EStreamType::Video);
		NewPresentation->GetTrackMetadata(AudioTrackMetadata, EStreamType::Audio);
		NewPresentation->GetTrackMetadata(SubtitleTrackMetadata, EStreamType::Subtitle);
		PlaybackState.SetTrackMetadata(VideoTrackMetadata, AudioTrackMetadata, SubtitleTrackMetadata);
		PlaybackState.SetHaveMetadata(true);
		// Get the supported playback rates for thinned and unthinned playback.
		PlaybackState.SetPlaybackRates(EPlaybackRateType::Unthinned, NewPresentation->GetPossiblePlaybackRates(IManifest::EPlayRateType::UnthinnedRate));
		PlaybackState.SetPlaybackRates(EPlaybackRateType::Thinned, NewPresentation->GetPossiblePlaybackRates(IManifest::EPlayRateType::ThinnedRate));

		Manifest = NewPresentation;

		CurrentState = EPlayerState::eState_Ready;

		double minBufTimeMPD = Manifest->GetMinBufferTime().GetAsSeconds();
		PlayerConfig.InitialBufferMinTimeAvailBeforePlayback = Utils::Min(minBufTimeMPD, PlayerConfig.InitialBufferMinTimeAvailBeforePlayback);
		PlayerConfig.SeekBufferMinTimeAvailBeforePlayback = Utils::Min(minBufTimeMPD, PlayerConfig.SeekBufferMinTimeAvailBeforePlayback);
		PlayerConfig.RebufferMinTimeAvailBeforePlayback = Utils::Min(minBufTimeMPD, PlayerConfig.RebufferMinTimeAvailBeforePlayback);
		// If this is an audio-only stream we set the audio buffer configuration from that of audio-only.
		if (VideoTrackMetadata.IsEmpty() && !AudioTrackMetadata.IsEmpty())
		{
			PlayerConfig.StreamBufferConfigAudio = PlayerConfig.StreamBufferConfigAudioOnly;
		}


		// Let the ABR know the format as well.
		StreamSelector->SetFormatType(ManifestReader->GetMediaFormatType());

		// Live streams must not use any cache!
		if (NewPresentation->GetPresentationType() == IManifest::EType::Live && HttpResponseCache.IsValid())
		{
			HttpResponseCache->Disable();
		}

		// Update the CMCD handler
		switch(ManifestReader->GetMediaFormatType())
		{
			case EMediaFormatType::HLS:
			{
				CMCDHandler->SetStreamingFormat(FCMCDHandler::EStreamingFormat::HLS);
				CMCDHandler->SetStreamType(NewPresentation->GetPresentationType() == IManifest::EType::OnDemand ? FCMCDHandler::EStreamType::VOD : FCMCDHandler::EStreamType::Live);
				break;
			}
			case EMediaFormatType::DASH:
			{
				CMCDHandler->SetStreamingFormat(NewPresentation->GetLowLatencyDescriptor().IsValid() ? FCMCDHandler::EStreamingFormat::LowLatencyDASH : FCMCDHandler::EStreamingFormat::DASH);
				CMCDHandler->SetStreamType(NewPresentation->GetPresentationType() == IManifest::EType::OnDemand ? FCMCDHandler::EStreamType::VOD : FCMCDHandler::EStreamType::Live);
				break;
			}
			default:
			{
				CMCDHandler->SetStreamingFormat(FCMCDHandler::EStreamingFormat::Undefined);
				break;
			}
		}

		// For formats that do not reload the playlist every so often we can get rid of the manifest reader.
		if (!Manifest->ReloadsPlaylistPeriodically())
		{
			InternalCloseManifestReader();
		}
		return true;
	}
	else
	{
		// Handle other types of playlist here.
		FErrorDetail err;
		err.SetFacility(Facility::EFacility::Player);
		err.SetMessage(TEXT("Unsupported playlist/manifest type"));
		err.SetCode(INTERR_UNSUPPORTED_FORMAT);
		PostError(err);
	}
	return false;
}

void FAdaptiveStreamingPlayer::UpdateManifest()
{
	if (Manifest.IsValid())
	{
		PlaybackState.SetSeekableRange(Manifest->GetSeekableTimeRange());
		PlaybackState.SetTimelineRange(Manifest->GetTotalTimeRange());
		PlaybackState.SetDuration(Manifest->GetDuration());
	}
}

FAdaptiveStreamingPlayer::FMediaMetadataUpdate::EResult FAdaptiveStreamingPlayer::FMediaMetadataUpdate::Handle(const FTimeValue& InAtTime)
{
	TSharedPtrTS<FMetadataParser> NextMetadata;
	FTimeValue NextActiveTime;
	bool bTriggerInternalRefresh = false;
	while(NextEntries.Num())
	{
		// Make the first metadata available right away if there is none yet and the time is not valid either.
		if (!InAtTime.IsValid() && !ActiveMetadata.IsValid())
		{
			NextMetadata = NextEntries[0].Metadata;
			NextActiveTime = NextEntries[0].ValidFrom;
			bTriggerInternalRefresh = NextEntries[0].bTriggerInternalRefresh;
			break;
		}
		else
		{
			const int64 s1 = NextEntries[0].ValidFrom.GetSequenceIndex();
			const int64 s2 = InAtTime.GetSequenceIndex();
			// Sequence count of metadata less or equal to the one of the current time?
			if (s1 <= s2)
			{
				if (NextEntries[0].ValidFrom <= InAtTime)
				{
					// Is this metadata older than the one we used before? This happens when there is metadata in the
					// init segment, which is sent for every media segment so that no metadata is missed when seeking.
					// Also, when stream switches occur the metadata of the init segment from the new stream is sent
					// as well.
					if (!ActiveSince.IsValid() || NextEntries[0].ValidFrom.GetSequenceIndex() > ActiveSince.GetSequenceIndex() || NextEntries[0].ValidFrom > ActiveSince)
					{
						NextMetadata = NextEntries[0].Metadata;
						NextActiveTime = NextEntries[0].ValidFrom;
						bTriggerInternalRefresh = NextEntries[0].bTriggerInternalRefresh;
					}
					NextEntries.RemoveAt(0);
				}
				else
				{
					break;
				}
			}
			else
			{
				break;
			}
		}
	}
	if (NextMetadata.IsValid())
	{
		bool bChanged = !ActiveMetadata.IsValid() || ActiveMetadata->IsDifferentFrom(*NextMetadata);
		ActiveMetadata = MoveTemp(NextMetadata);
		ActiveSince = NextActiveTime;
		return bChanged ? bTriggerInternalRefresh ? EResult::ChangedAndUpdate : EResult::Changed : EResult::NoChange;
	}

	return EResult::NoChange;
}


IPlayerSessionServices::ECustomPropertyResult FAdaptiveStreamingPlayer::ValidateMainPlaylistCustomProperty(const FString& InProtocol, const FString& InPlaylistURL, const TArray<FElectraHTTPStreamHeader>& InPlaylistFetchResponseHeaders, const FPlaylistProperty& InProperty)
{
	if (InProperty.Tag.Len())
	{
		FName AsName;
		if (PlaylistPropertyHandler.WantProperty(AsName, InProtocol, InProperty.Tag))
		{
			/*
				If the property is asked for, we create a single JSON string describing how it was retrieved, that is, from which
				streaming protocol, the URL of the playlist and the HTTP response headers associated with the playlist fetch:

				{
					"v": "the value of the property as it appears in the playlist",
					"proto": "the streaming protocol in use",
					"url": "the url the playlist was fetched from",
					"headers": [
					            [ "responseheader1", "value1" ],
					            [ "responseheader2", "value2" ],
					            [ ... ... ],
					           ]
				}

				Note that the response headers is an array of arrays, where each entry is an array
				of two elements, [0] being the header and [1] the value. This is faster to handle than
				an object for which the key name would be variable/unknown.

				This JSON string is then set as the value for the mapped property name as the property
				configuration says, and stored in the player options from where it can be retrieved
				with the GetMediaInfo() method.
			*/

			TSharedRef<FJsonObject> RootObject = MakeShareable(new FJsonObject);
			TArray<TSharedPtr<FJsonValue>> HeaderArray;
			RootObject->SetField(TEXT("v"), MakeShareable(new FJsonValueString(InProperty.Value)));
			RootObject->SetField(TEXT("proto"), MakeShareable(new FJsonValueString(InProtocol)));
			RootObject->SetField(TEXT("url"), MakeShareable(new FJsonValueString(InPlaylistURL)));
			for(auto& Hdr : InPlaylistFetchResponseHeaders)
			{
				TArray<TSharedPtr<FJsonValue>> h;
				h.Add(MakeShareable(new FJsonValueString(Hdr.Header)));
				h.Add(MakeShareable(new FJsonValueString(Hdr.Value)));
				HeaderArray.Add(MakeShareable(new FJsonValueArray(h)));
			}
			RootObject->SetArrayField(TEXT("headers"), HeaderArray);
			FString PropertyJSON;
			TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&PropertyJSON, 0);
			if (FJsonSerializer::Serialize(RootObject, JsonWriter))
			{
				GetMediaInfoDictionary().Set(AsName, FVariantValue(PropertyJSON));
				return IPlayerSessionServices::ECustomPropertyResult::Accept;
			}
		}
	}
	return IPlayerSessionServices::ECustomPropertyResult::Default;
}


} // namespace Electra
