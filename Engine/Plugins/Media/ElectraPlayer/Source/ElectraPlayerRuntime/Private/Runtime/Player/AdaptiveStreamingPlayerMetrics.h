// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "StreamTypes.h"
#include "MediaURLType.h"
#include "InfoLog.h"
#include "Player/AdaptiveStreamingPlayerABR_State.h"
#include "Player/Playlist.h"
#include "ElectraHTTPStream.h"
#include "MediaStreamMetadata.h"

namespace Electra
{

namespace Metrics
{
	enum class ESegmentType
	{
		Undefined,
		Init,
		Media,
		Sideloaded,
		PlaylistElement
	};
	static const TCHAR * const GetSegmentTypeString(ESegmentType SegmentType)
	{
		switch(SegmentType)
		{
			case ESegmentType::Undefined:
			{
				return TEXT("Undefined");
			}
			case ESegmentType::Init:
			{
				return TEXT("Init");
			}
			case ESegmentType::Media:
			{
				return TEXT("Media");
			}
			case ESegmentType::Sideloaded:
			{
				return TEXT("Sideloaded");
			}
			case ESegmentType::PlaylistElement:
			{
				return TEXT("Playlist element");
			}
		}
		return TEXT("n/a");
	}

	enum class EBufferingReason
	{
		Initial,
		Seeking,
		Rebuffering,
	};
	static const TCHAR * const GetBufferingReasonString(EBufferingReason BufferingReason)
	{
		switch(BufferingReason)
		{
			case EBufferingReason::Initial:
				return TEXT("Initial");
			case EBufferingReason::Seeking:
				return TEXT("Seeking");
			case EBufferingReason::Rebuffering:
				return TEXT("Rebuffering");
		}
		return TEXT("n/a");
	}

	enum class ETimeJumpReason
	{
		UserSeek,
		FellOffTimeline,
		FellBehindWallclock,
		Looping
	};
	static const TCHAR * const GetTimejumpReasonString(ETimeJumpReason TimejumpReason)
	{
		switch(TimejumpReason)
		{
			case ETimeJumpReason::UserSeek:
				return TEXT("User seek");
			case ETimeJumpReason::FellOffTimeline:
				return TEXT("Fell off timeline");
			case ETimeJumpReason::FellBehindWallclock:
				return TEXT("Fell behind wallclock");
			case ETimeJumpReason::Looping:
				return TEXT("Looping");
		}
		return TEXT("n/a");
	}

	struct FBufferStats
	{
		FBufferStats()
		{
			BufferType  		 = EStreamType::Video;
			MaxDurationInSeconds = 0.0;
			DurationInUse   	 = 0.0;
			BytesInUse  		 = 0;
		}
		EStreamType		BufferType;
		double			MaxDurationInSeconds;
		double			DurationInUse;
		int64			BytesInUse;
	};

	struct FPlaylistDownloadStats
	{
		FMediaURL				Url;
		FString					FailureReason;						// Human readable failure reason. Only for display purposes.
		Playlist::EListType		ListType = Playlist::EListType::Main;
		Playlist::ELoadType		LoadType = Playlist::ELoadType::Initial;
		int32					HTTPStatusCode = 0;					// HTTP status code (0 if not connected to server yet)
		int32					RetryNumber = 0;
		bool					bWasSuccessful = false;
		bool					bDidTimeout = false;
		bool					bWasAborted = false;
	};

	struct FSegmentDownloadStats
	{
		// Chunk timing
		struct FMovieChunkInfo
		{
			int64 HeaderOffset = 0;
			int64 PayloadStartOffset = 0;
			int64 PayloadEndOffset = 0;
			int64 NumKeyframeBytes = 0;
			FTimeValue ContentDuration;
			FMovieChunkInfo() { ContentDuration = FTimeValue::GetZero(); }
		};

		// Inputs from stream request
		EStreamType StreamType;					//!< Type of stream
		ESegmentType SegmentType;				//!< Type of segment (init or media)
		FMediaURL URL;							//!< Effective URL used to download from
		FString Range;							//!< Range used to download
		FString MediaAssetID;
		FString AdaptationSetID;
		FString RepresentationID;
		double PresentationTime;				//!< Presentation time on media timeline
		double Duration;						//!< Duration of segment as specified in manifest
		int64 SteeringID;						//!< ID from the content steering handler for this request.
		int32 Bitrate;							//!< Stream bitrate as specified in manifest
		int32 QualityIndex;						//!< Quality index of this segment
		int32 HighestQualityIndex;				//!< The highest quality index that could be had.
		int32 RetryNumber;
		bool bWaitingForRemoteRetryElement;

		// Outputs from stream reader
		uint32 StatsID;							//!< ID uniquely identifying this download
		FString FailureReason;					//!< Human readable failure reason. Only for display purposes.
		double AvailibilityDelay;				//!< Time the download had to wait for the segment to enter its availability window.
		double DurationDownloaded;				//!< Duration of content successfully downloaded. May be less than Duration in case of errors.
		double DurationDelivered;				//!< Duration of content delivered to buffer. If larger than DurationDownloaded indicates dummy data was inserted into buffer.
		double TimeToFirstByte;					//!< Time in seconds until first data byte was received
		double TimeToDownload;					//!< Total time in seconds for entire download
		int64 ByteSize;							//!< Content-Length, may be -1 if unknown (either on error or chunked transfer)
		int64 NumBytesDownloaded;				//!< Number of bytes successfully downloaded.
		int32 HTTPStatusCode;					//!< HTTP status code (0 if not connected to server yet)
		bool bWasSuccessful;					//!< true if download was successful, false if not
		bool bWasAborted;						//!< true if download was aborted by ABR (not by playback!)
		bool bDidTimeout;						//!< true if a timeout occurred. Only set if timeouts are enabled. Usually the ABR will monitor and abort.
		bool bParseFailure;						//!< true if the segment could not be parsed
		bool bIsMissingSegment;					//!< true if the segment was not actually downloaded because it is missing on the timeline.
		bool bWasSkipped;						//!< true if the segment was skipped over due to internal timestamps being less than expected
		bool bWasFalloffSegment;				//!< true if the segment was no longer present on the timeline.
		bool bInsertedFillerData;
		bool bIsCachedResponse;
		TArray<IElectraHTTPStreamResponse::FTimingTrace> TimingTraces;
		TArray<FMovieChunkInfo> MovieChunkInfos;

		FSegmentDownloadStats()
		{ Reset(); }
		void Reset()
		{
			StreamType = EStreamType::Unsupported;
			SegmentType = ESegmentType::Undefined;
			URL.Empty();
			Range.Empty();
			MediaAssetID.Empty();
			AdaptationSetID.Empty();
			RepresentationID.Empty();
			PresentationTime = 0.0;
			Duration = 0.0;
			SteeringID = 0;
			Bitrate = 0;
			QualityIndex = 0;
			HighestQualityIndex = 0;
			RetryNumber = 0;
			bWaitingForRemoteRetryElement = false;
			ResetOutput();
		}
		void ResetOutput()
		{
			StatsID = 0;
			FailureReason.Empty();
			AvailibilityDelay = 0.0;
			DurationDownloaded = 0.0;
			DurationDelivered = 0.0;
			TimeToFirstByte = 0.0;
			TimeToDownload = 0.0;
			ByteSize = 0;
			NumBytesDownloaded = 0;
			HTTPStatusCode = 0 ;
			bWasSuccessful = false;
			bWasAborted = false;
			bDidTimeout = false;
			bParseFailure = false;
			bIsMissingSegment = false;
			bWasSkipped = false;
			bWasFalloffSegment = false;
			bInsertedFillerData = false;
			bIsCachedResponse = false;
			TimingTraces.Empty();
			MovieChunkInfos.Empty();
		}
	};

	struct FLicenseKeyStats
	{
		FLicenseKeyStats()
		{
			bWasSuccessful = false;
		}
		FString			URL;
		FString			FailureReason;					//!< Human readable failure reason. Only for display purposes.
		bool			bWasSuccessful;
	};

	struct FDataAvailabilityChange
	{
		enum class EAvailability
		{
			DataAvailable,
			DataNotAvailable
		};
		FDataAvailabilityChange()
			: StreamType(EStreamType::Unsupported), Availability(EAvailability::DataNotAvailable)
		{ }

		EStreamType		StreamType;						//!< Type of stream
		EAvailability	Availability;
	};

	struct FFrameStats
	{
		uint64 NumTotal = 0;
		uint64 NumDropped = 0;
		uint32 NumNewDrops = 0;
		uint32 AvgDropsPerInterval = 0;
		uint32 MinDropsPerInterval = 0;
		uint32 MaxDropsPerInterval = 0;
		uint32 DropIntervalMs = 0;
		uint32 NumIntervals = 0;
		uint32 NumIntervalsWithDrops = 0;
	};

} // namespace Metrics


/**
 *
**/
class IAdaptiveStreamingPlayerMetrics
{
public:
	virtual ~IAdaptiveStreamingPlayerMetrics() = default;

	//=================================================================================================================
	// Methods called from the media player.
	//

	/**
	 * Called when the source will be opened.
	 */
	virtual void ReportOpenSource(const FString& URL) = 0;

	/**
	 * Called when the source's main playlist has beed loaded successfully.
	 */
	virtual void ReportReceivedMainPlaylist(const FString& EffectiveURL) = 0;

	/**
	 * Called when the dependent child playlists have been loaded successfully.
	 */
	virtual void ReportReceivedPlaylists() = 0;

	/**
	 * Called when the available tracks or their properties have changed.
	 */
	virtual void ReportTracksChanged() = 0;

	/**
	 * Called at the end of every downloaded playlist.
	 */
	virtual void ReportPlaylistDownload(const Metrics::FPlaylistDownloadStats& PlaylistDownloadStats) = 0;

	/**
	 * Called when buffering of data begins.
	 */
	virtual void ReportBufferingStart(Metrics::EBufferingReason BufferingReason) = 0;

	/**
	 * Called when buffering of data ends.
	 * This does not necessarily coincide with a segment download as buffering ends as soon as
	 * sufficient data has been received to start/resume playback.
	 */
	virtual void ReportBufferingEnd(Metrics::EBufferingReason BufferingReason) = 0;

	/**
	 * Called at the end of each downloaded video segment.
	 * The order will be ReportSegmentDownload() followed by ReportBandwidth()
	 */
	virtual void ReportBandwidth(int64 EffectiveBps, int64 ThroughputBps, double LatencyInSeconds) = 0;

	/**
	 * Called before and after a segment download.
	 */
	virtual void ReportBufferUtilization(const Metrics::FBufferStats& BufferStats) = 0;

	/**
	 * Called at the end of each downloaded segment.
	 */
	virtual void ReportSegmentDownload(const Metrics::FSegmentDownloadStats& SegmentDownloadStats) = 0;

	/**
	 * Called for license key events.
	 */
	virtual void ReportLicenseKey(const Metrics::FLicenseKeyStats& LicenseKeyStats) = 0;

	/**
	 * Called when a new video stream segment is fetched at a different bitrate than before.
	 * A drastic change is one where quality _drops_ more than _one_ level.
	 */
	virtual void ReportVideoQualityChange(int32 NewBitrate, int32 PreviousBitrate, bool bIsDrasticDownswitch) = 0;

	/**
	 * Called when a new audio stream segment is fetched at a different bitrate than before.
	 * A drastic change is one where quality _drops_ more than _one_ level.
	 */
	virtual void ReportAudioQualityChange(int32 NewBitrate, int32 PreviousBitrate, bool bIsDrasticDownswitch) = 0;

	/**
	 * Called when stream data availability changes when feeding the decoder.
	 */
	virtual void ReportDataAvailabilityChange(const Metrics::FDataAvailabilityChange& DataAvailability) = 0;

	/**
	 * Called when the format of the stream being decoded changes in some way.
	 */
	virtual void ReportDecodingFormatChange(const FStreamCodecInformation& NewDecodingFormat) = 0;

	/**
	 * Called when decoders start to decode first data to pre-roll the pipeline.
	 */
	virtual void ReportPrerollStart() = 0;

	/**
	 * Called when enough initial data has been decoded to pre-roll the pipeline.
	 */
	virtual void ReportPrerollEnd() = 0;

	/**
	 * Called when playback starts for the first time. This is not called after resuming a paused playback.
	 * If playback has ended(!) and is then begun again by seeking back to an earlier point in time this
	 * callback will be triggered again. Seeking during playback will not cause this callback.
	 */
	virtual void ReportPlaybackStart() = 0;

	/**
	 * Called when the player enters pause mode, either through user request or an internal state change.
	 */
	virtual void ReportPlaybackPaused() = 0;

	/**
	 * Called when the player resumes from pause mode, either through user request or an internal state change.
	 */
	virtual void ReportPlaybackResumed() = 0;

	/**
	 * Called when playback has reached the end. See ReportPlaybackStart().
	 */
	virtual void ReportPlaybackEnded() = 0;

	/**
	 * Called when the play position jumps either because of a user induced seek or because the play position fell off the timeline
	 * or because a wallclock synchronized Live playback fell too far behind the actual time.
	 */
	virtual void ReportJumpInPlayPosition(const FTimeValue& ToNewTime, const FTimeValue& FromTime, Metrics::ETimeJumpReason TimejumpReason) = 0;

	/**
	 * Called when playback is terminally stopped.
	 */
	virtual void ReportPlaybackStopped() = 0;

	/**
	 * Called when a seek has completed such that the first new data is ready.
	 * Future data may still be buffering.
	 */
	virtual void ReportSeekCompleted() = 0;

	/**
	 * Called when the media metadata has changed.
	 */
	virtual void ReportMediaMetadataChanged(TSharedPtrTS<FMetadataParser> Metadata) = 0;

	/**
	 * Called when an error occurs. Errors always result in termination of playback.
	 */
	virtual void ReportError(const FString& ErrorReason) = 0;

	/**
	 * Called to output a log message. The log level is a player internal level.
	 */
	virtual void ReportLogMessage(IInfoLog::ELevel LogLevel, const FString& LogMessage, int64 PlayerWallclockMilliseconds) = 0;


	//=================================================================================================================
	// Methods called from the renderers.
	//

	virtual void ReportVideoFrameStats(const Metrics::FFrameStats& InFrameStats) = 0;
	virtual void ReportAudioFrameStats(const Metrics::FFrameStats& InFrameStats) = 0;
};


} // namespace Electra


