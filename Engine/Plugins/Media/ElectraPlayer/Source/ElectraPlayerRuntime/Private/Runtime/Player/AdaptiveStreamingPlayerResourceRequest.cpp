// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/PlayerSessionServices.h"

#include <Dom/JsonObject.h>
#include <Serialization/JsonReader.h>
#include <Serialization/JsonSerializer.h>
#include <Misc/Base64.h>


namespace Electra
{

FHTTPResourceRequest::FHTTPResourceRequest()
{
	Request = MakeSharedTS<IElectraHttpManager::FRequest>();
	ReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
	ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->ProgressDelegate   = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateRaw(this, &FHTTPResourceRequest::HTTPProgressCallback);
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateRaw(this, &FHTTPResourceRequest::HTTPCompletionCallback);
	Request->ProgressListener = ProgressListener;
	Request->ReceiveBuffer = ReceiveBuffer;
}

FHTTPResourceRequest::~FHTTPResourceRequest()
{
	check(!bInCallback);
	Cancel();
	if (bWasAdded && Request.IsValid())
	{
		TSharedPtrTS<IElectraHttpManager> PinnedHTTPManager = HTTPManager.Pin();
		if (PinnedHTTPManager.IsValid())
		{
			PinnedHTTPManager->RemoveRequest(Request, true);
		}
	}
	Request.Reset();
}

bool FHTTPResourceRequest::SetFromJSON(const FString& InJSONParams)
{
	if (InJSONParams.Len())
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InJSONParams);
		TSharedPtr<FJsonObject> JSONParams;
		if (FJsonSerializer::Deserialize(Reader, JSONParams))
		{
			TArray<FString> StringArray;
			FString String;
			// Verb
			if (JSONParams->TryGetStringField(TEXT("verb"), String))
			{
				Verb(String);
				// If POST see if there is base64 encoded post data to send.
				if (String.Equals(TEXT("POST")) && JSONParams->TryGetStringField(TEXT("data"), String))
				{
					TArray<uint8> Data;
					if (FBase64::Decode(String, Data))
					{
						PostData(Data);
					}
				}
			}
			else
			{
				Verb(TEXT("GET"));
			}

			// Custom user agent
			if (JSONParams->TryGetStringField(TEXT("agent"), String))
			{
				UserAgent(String);
			}

			// Accept-encoding
			if (JSONParams->TryGetStringField(TEXT("encoding"), String))
			{
				AcceptEncoding(String);
			}

			// Headers. Must always be an array of strings
			if (JSONParams->TryGetStringArrayField(TEXT("hdrs"), StringArray))
			{
				Headers(StringArray);
				StringArray.Empty();
			}

			// Connection timeout in milliseconds
			int32 TimeOutMS = 0;
			if (JSONParams->TryGetNumberField(TEXT("ctoms"), TimeOutMS))
			{
				ConnectionTimeout(FTimeValue(FTimeValue::MillisecondsToHNS(TimeOutMS)));
			}

			// No-data timeout in milliseconds
			TimeOutMS = 0;
			if (JSONParams->TryGetNumberField(TEXT("ndtoms"), TimeOutMS))
			{
				NoDataTimeout(FTimeValue(FTimeValue::MillisecondsToHNS(TimeOutMS)));
			}
		}
		else
		{
			return false;
		}
	}
	return true;
}


void FHTTPResourceRequest::StartGet(IPlayerSessionServices* InPlayerSessionServices)
{
	PlayerSessionServices = InPlayerSessionServices;
	HTTPManager = PlayerSessionServices->GetHTTPManager();
	// Is there a static resource provider that we can try?
	TSharedPtr<IAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe> StaticResourceProvider = PlayerSessionServices->GetStaticResourceProvider();
	if (StaticQueryType.IsSet() && StaticResourceProvider.IsValid())
	{
		TSharedPtr<FStaticResourceRequest, ESPMode::ThreadSafe>	StaticRequest = MakeShared<FStaticResourceRequest, ESPMode::ThreadSafe>(AsShared());
		StaticResourceProvider->ProvideStaticPlaybackDataForURL(StaticRequest);
	}
	else
	{
		bWasAdded = true;
		TSharedPtrTS<IElectraHttpManager> PinnedHTTPManager = HTTPManager.Pin();
		if (PinnedHTTPManager.IsValid())
		{
			PinnedHTTPManager->AddRequest(Request, false);
		}
	}
}

void FHTTPResourceRequest::Cancel()
{
	bWasCanceled = true;
	ProgressListener.Reset();
	ReceiveBuffer.Reset();
	if (bWasAdded && Request.IsValid())
	{
		TSharedPtrTS<IElectraHttpManager> PinnedHTTPManager = HTTPManager.Pin();
		if (PinnedHTTPManager.IsValid())
		{
			PinnedHTTPManager->RemoveRequest(Request, true);
		}
		bWasAdded = false;
	}
}


void FHTTPResourceRequest::StaticDataReady()
{
	TSharedPtrTS<IElectraHttpManager::FRequest> Req = Request;
	if (Req.IsValid())
	{
		// Was static data actually set or was there no data provided?
		if (!bStaticDataReady)
		{
			// Do the actual HTTP request now.
			bWasAdded = true;
			TSharedPtrTS<IElectraHttpManager> PinnedHTTPManager = HTTPManager.Pin();
			if (PinnedHTTPManager.IsValid())
			{
				PinnedHTTPManager->AddRequest(Req, false);
			}
		}
		else
		{
			ConnectionInfo = Req->ConnectionInfo;
			ConnectionInfo.EffectiveURL = Req->Parameters.URL;
			ConnectionInfo.bIsConnected = true;
			ConnectionInfo.bHaveResponseHeaders  = true;
			ConnectionInfo.bWasAborted = false;
			ConnectionInfo.bHasFinished = true;
			ConnectionInfo.HTTPVersionReceived = 11;
			ConnectionInfo.StatusInfo.HTTPStatus = Req->Parameters.Range.IsSet() ? 206 : 200;
			ConnectionInfo.ContentLength = ConnectionInfo.BytesReadSoFar = ReceiveBuffer.IsValid() ? ReceiveBuffer->Num() : 0;

			bInCallback = true;
			CompletedCallback.ExecuteIfBound(AsShared());
			bInCallback = false;
			bHasFinished = true;
		}
	}
}

int32 FHTTPResourceRequest::HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest)
{
	return bWasCanceled ? 1 : 0;
}

void FHTTPResourceRequest::HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest)
{
	TSharedPtrTS<FHTTPResourceRequest> Self = AsShared();
	if (Self.IsValid())
	{
		TSharedPtrTS<IElectraHttpManager::FRequest> Req = Request;
		if (Req.IsValid())
		{
			ConnectionInfo = Req->ConnectionInfo;
			if (!ConnectionInfo.bWasAborted)
			{
				Error = 0;
				if (Req->ConnectionInfo.StatusInfo.ErrorDetail.IsError())
				{
					if (ConnectionInfo.StatusInfo.ConnectionTimeoutAfterMilliseconds)
					{
						Error = 1;
					}
					else if (ConnectionInfo.StatusInfo.NoDataTimeoutAfterMilliseconds)
					{
						Error = 2;
					}
					else if (ConnectionInfo.StatusInfo.bReadError)
					{
						Error = 3;
					}
					else
					{
						Error = ConnectionInfo.StatusInfo.HTTPStatus ? ConnectionInfo.StatusInfo.HTTPStatus : 4;
					}
				}

				bInCallback = true;
				CompletedCallback.ExecuteIfBound(Self);
				bInCallback = false;
			}
		}
		bHasFinished = true;
	}
}

} // namespace Electra

