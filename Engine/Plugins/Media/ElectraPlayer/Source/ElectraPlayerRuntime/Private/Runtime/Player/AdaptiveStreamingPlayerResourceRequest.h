// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "StreamTypes.h"
#include "InfoLog.h"
#include "HTTP/HTTPManager.h"

namespace Electra
{
class IPlayerSessionServices;
class FHTTPResourceRequest;


class IAdaptiveStreamingPlayerResourceRequest : public TSharedFromThis<IAdaptiveStreamingPlayerResourceRequest, ESPMode::ThreadSafe>
{
public:
	enum class EPlaybackResourceType
	{
		Empty,
		Playlist,
		LicenseKey
	};

	virtual ~IAdaptiveStreamingPlayerResourceRequest() = default;

	//! Returns the type of the requested resource.
	virtual EPlaybackResourceType GetResourceType() const = 0;
	//! Returns the URL of the requested resource.
	virtual FString GetResourceURL() const = 0;

	//! Sets the binary resource data. If data can not be provided do not set anything (or a nullptr)
	//! For a request of `BinaryData` you need to set `TotalResourceSize` to the total size of the file so the reader
	//! will know how big the file is. This is true even for `NumBytesToRead` set to 0.
	virtual void SetPlaybackData(TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe>	PlaybackData, int64 TotalResourceSize) = 0;

	//! Signal request completion. Must be called with or without data being set.
	virtual void SignalDataReady() = 0;
};



class IAdaptiveStreamingPlayerResourceProvider
{
public:
	virtual ~IAdaptiveStreamingPlayerResourceProvider() = default;
	virtual void ProvideStaticPlaybackDataForURL(TSharedPtr<IAdaptiveStreamingPlayerResourceRequest, ESPMode::ThreadSafe> InOutRequest) = 0;
};




class IHTTPResourceRequestObject : public TSharedFromThis<IHTTPResourceRequestObject, ESPMode::ThreadSafe>
{
public:
	virtual ~IHTTPResourceRequestObject() = default;
};



class FHTTPResourceRequestCompletionSignal : public TSharedFromThis<FHTTPResourceRequestCompletionSignal, ESPMode::ThreadSafe>
{
public:
	static TSharedPtr<FHTTPResourceRequestCompletionSignal, ESPMode::ThreadSafe> Create()
	{ return MakeShared<FHTTPResourceRequestCompletionSignal, ESPMode::ThreadSafe>(); }
	virtual ~FHTTPResourceRequestCompletionSignal()
	{ Sig.Signal(); }
	virtual void CompletionCallback(TSharedPtrTS<FHTTPResourceRequest> InRequest)
	{ Sig.Signal();	}

	void Wait()
	{ Sig.Wait(); }
	void WaitAndReset()
	{ Sig.WaitAndReset(); }
	void Signal()
	{ Sig.Signal(); }
	void Reset()
	{ Sig.Reset(); }
	bool IsSignaled() const
	{ return Sig.IsSignaled(); }
	bool WaitTimeout(int64 MicroSeconds)
	{ return Sig.WaitTimeout(MicroSeconds); }
	bool WaitTimeoutAndReset(int64 MicroSeconds)
	{ return Sig.WaitTimeoutAndReset(MicroSeconds); }
private:
	FMediaEvent Sig;
};

/**
 *
 */
class FHTTPResourceRequest : public TSharedFromThis<FHTTPResourceRequest, ESPMode::ThreadSafe>
{
public:
	FHTTPResourceRequest();
	virtual ~FHTTPResourceRequest();

	DECLARE_DELEGATE_OneParam(FOnRequestCompletedCallback, TSharedPtrTS<FHTTPResourceRequest> /*Request*/);

	virtual bool SetFromJSON(const FString& InJSONParams);

	virtual FHTTPResourceRequest& URL(const FString& InURL)
	{ Request->Parameters.URL = InURL; return *this; }

	virtual FHTTPResourceRequest& Verb(const FString& InVerb)
	{ Request->Parameters.Verb = InVerb; return *this; }

	virtual FHTTPResourceRequest& UserAgent(const FString& InUserAgent)
	{ Request->Parameters.UserAgent.Set(InUserAgent); return *this; }

	virtual FHTTPResourceRequest& PostData(TArray<uint8> InPostData)
	{ Request->Parameters.PostData = MoveTemp(InPostData); return *this; }

	virtual FHTTPResourceRequest& Range(const FString& InRange)
	{ Request->Parameters.Range.Set(InRange); return *this; }

	virtual FHTTPResourceRequest& Headers(const TArray<HTTP::FHTTPHeader>& InHeaders)
	{ Request->Parameters.RequestHeaders = InHeaders; return *this; }

	virtual FHTTPResourceRequest& Headers(const TArray<FString>& InHeaders)
	{ Request->Parameters.AddFromHeaderList(InHeaders); return *this; }

	virtual FHTTPResourceRequest& AcceptEncoding(const FString& InAcceptEncoding)
	{ Request->Parameters.AcceptEncoding.Set(InAcceptEncoding); return *this; }

	virtual FHTTPResourceRequest& ConnectionTimeout(const FTimeValue& InTimeoutAfter)
	{ Request->Parameters.ConnectTimeout = InTimeoutAfter; return *this; }

	virtual FHTTPResourceRequest& NoDataTimeout(const FTimeValue& InTimeoutAfter)
	{ Request->Parameters.NoDataTimeout = InTimeoutAfter; return *this; }

	virtual FHTTPResourceRequest& AllowStaticQuery(IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType InStaticQueryType)
	{ StaticQueryType.Set(InStaticQueryType); return *this; }

	virtual FHTTPResourceRequest& Object(TSharedPtrTS<IHTTPResourceRequestObject> InUserObject)
	{ UserObject = InUserObject; return *this; }

	virtual FHTTPResourceRequest& StreamTypeAndQuality(EStreamType InStreamType, int32 InQualityIndex, int32 InMaxQualityIndex)
	{ Request->Parameters.StreamType = InStreamType; Request->Parameters.QualityIndex = InQualityIndex; Request->Parameters.MaxQualityIndex = InMaxQualityIndex; return *this; }

	virtual FHTTPResourceRequest& CompletionSignal(const TSharedPtr<FHTTPResourceRequestCompletionSignal, ESPMode::ThreadSafe>& InSig)
	{ CompletedCallback.BindThreadSafeSP(InSig.ToSharedRef(), &FHTTPResourceRequestCompletionSignal::CompletionCallback); return *this; }

	virtual FOnRequestCompletedCallback& Callback()
	{ return CompletedCallback; }

	virtual void StartGet(IPlayerSessionServices* InPlayerSessionServices);
	virtual void Cancel();


	virtual int32 GetError() const
	{ return Error; }

	virtual FString GetErrorString() const
	{
		if (Error == 0)							return FString(TEXT("No error"));
		else if (Error == 1)					return FString(TEXT("Connection timeout"));
		else if (Error == 2)					return FString(TEXT("Data timeout"));
		else if (Error == 3)					return FString(TEXT("Connection closed"));
		else if (Error == 4)					return FString(TEXT("No connection"));
		else if (Error >= 100 && Error < 600)	return FString::Printf(TEXT("HTTP status code %d"), Error);
		else if (Error == -1)					return FString(TEXT("Could not connect"));
		else									return FString::Printf(TEXT("Unknown <code %d>"), Error);
	}

	virtual bool GetHasFinished() const
	{ return bHasFinished; }

	virtual bool GetWasCanceled() const
	{ return bWasCanceled; }

	virtual FString GetURL() const
	{ return Request.IsValid() ? Request->Parameters.URL : FString(); }

	virtual FTimeValue GetConnectionTimeout() const
	{ return Request.IsValid() ? Request->Parameters.ConnectTimeout : FTimeValue(); }

	virtual FTimeValue GetNoDataTimeout() const
	{ return Request.IsValid() ? Request->Parameters.NoDataTimeout : FTimeValue(); }

	virtual IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType GetStaticQuery() const
	{ return StaticQueryType.GetWithDefault(IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Empty); }

	virtual TSharedPtrTS<IHTTPResourceRequestObject> GetObject() const
	{ return UserObject.Pin(); }

	virtual TSharedPtrTS<FWaitableBuffer> GetResponseBuffer() const
	{ return ReceiveBuffer; }

	virtual const HTTP::FConnectionInfo* GetConnectionInfo() const
	{ return &ConnectionInfo; }

	virtual TSharedPtrTS<IElectraHttpManager::FRequest> GetRequest() const
	{ return Request; }

private:

	class FStaticResourceRequest : public IAdaptiveStreamingPlayerResourceRequest
	{
	public:
		FStaticResourceRequest(TWeakPtrTS<FHTTPResourceRequest> InOwner) : Owner(InOwner) {}
		virtual ~FStaticResourceRequest() = default;

		EPlaybackResourceType GetResourceType() const override
		{
			TSharedPtrTS<FHTTPResourceRequest> p(Owner.Pin());
			return p.IsValid() ? p->GetStaticQuery() : EPlaybackResourceType::Empty;
		}

		FString GetResourceURL() const override
		{
			TSharedPtrTS<FHTTPResourceRequest> p(Owner.Pin());
			return p.IsValid() ? p->GetURL() : FString();
		}

		void SetPlaybackData(TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe>	PlaybackData, int64 TotalResourceSize) override
		{
			if (PlaybackData.IsValid())
			{
				TSharedPtrTS<FHTTPResourceRequest> p(Owner.Pin());
				if (p.IsValid())
				{
					TSharedPtrTS<FWaitableBuffer> Buf = p->GetResponseBuffer();
					if (Buf.IsValid())
					{
						Buf->Reserve(PlaybackData->Num());
						Buf->PushData(PlaybackData->GetData(), PlaybackData->Num());
						Buf->SetEOD();
					}
					p->SetStaticDataReady();
				}
			}
		}

		//! Signal request completion. Must be called with or without data being set.
		void SignalDataReady() override
		{
			TSharedPtrTS<FHTTPResourceRequest> p(Owner.Pin());
			if (p.IsValid())
			{
				p->StaticDataReady();
			}
		}
	private:
		TWeakPtrTS<FHTTPResourceRequest> Owner;
	};


	int32 HTTPProgressCallback(const IElectraHttpManager::FRequest* Request);
	void HTTPCompletionCallback(const IElectraHttpManager::FRequest* Request);
	void SetStaticDataReady()
	{ bStaticDataReady = true; }
	void StaticDataReady();
	HTTP::FConnectionInfo ConnectionInfo;
	TSharedPtrTS<IElectraHttpManager::FRequest> Request;
	TSharedPtrTS<FWaitableBuffer> ReceiveBuffer;
	TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener;
	FOnRequestCompletedCallback CompletedCallback;
	TWeakPtrTS<IHTTPResourceRequestObject> UserObject;
	TWeakPtrTS<IElectraHttpManager> HTTPManager;
	TMediaOptionalValue<IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType> StaticQueryType;
	IPlayerSessionServices* PlayerSessionServices = nullptr;
	bool bWasAdded = false;
	bool bWasCanceled = false;
	bool bHasFinished = false;
	bool bStaticDataReady = false;
	bool bInCallback = false;
	int32 Error = 0;
};

} // namespace Electra
