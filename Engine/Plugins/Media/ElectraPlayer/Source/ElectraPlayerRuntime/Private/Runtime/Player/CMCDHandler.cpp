// Copyright Epic Games, Inc. All Rights Reserved.

#include "CMCDHandler.h"
#include "Player/PlayerSessionServices.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "ElectraPlayerMisc.h"
#include "PlayerPlatform.h"

#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include "HAL/IConsoleManager.h"
static TAutoConsoleVariable<int32> CVarElectraCMCDMetrics(
	TEXT("Electra.CMCDMetrics"),
	1,
	TEXT("Controls CMCD metrics.\n")
	TEXT(" 0: disable all CMCD metrics; 1: use default settings; 2: always enable CMCD metrics"),
	ECVF_Default);


namespace Electra
{
// Standard config only enables playlist provided parameters.
static const TCHAR* const StandardCMCDConfig = TEXT("{\"cmcd\":[]}");

// Default values are: `headers` mode, `request` type, all keys enabled, version `1` and playlist parameters to be applied.
// Values that are default do not need to be included in this config.
static const TCHAR* const DefaultEnabledCMCDConfig = TEXT("{\"cmcd\":[{}]}");



FCMCDHandler::FCMCDHandler(IPlayerSessionServices* InPlayerSessionServices)
	: PlayerSessionServices(InPlayerSessionServices)
{
}

FCMCDHandler::~FCMCDHandler()
{
}

bool FCMCDHandler::IsEnabled()
{
	return bInitOk && StreamingFormat != EStreamingFormat::Undefined && !ReportElements.IsEmpty();
}

void FCMCDHandler::PeriodicHandle()
{
}

void FCMCDHandler::Initialize(const FString& InContentID, const FString& InSessionID, const FString& InConfiguration)
{
	FString Configuration(InConfiguration);

	ContentID = InContentID;
	SessionID = InSessionID;
	PlatformID = Electra::Platform::GetPlatformID();

	int32 CMCDMetrics = CVarElectraCMCDMetrics.GetValueOnAnyThread();
	// Disable CMCD entirely?
	if (CMCDMetrics == 0)
	{
		bInitOk = false;
		return;
	}

	// If the configuration is empty or cannot be parsed as a JSON then CMCD will be disabled for this session.
	if (Configuration.IsEmpty())
	{
		// Using default CMCD means to look for playlist provided parameters only.
		if (CMCDMetrics == 1)
		{
			Configuration = FString(StandardCMCDConfig);
		}
		// Always enabling CMCD without a supplied configuration means we are using an internal catch-all config.
		else if (CMCDMetrics == 2)
		{
			Configuration = FString(DefaultEnabledCMCDConfig);
		}
	}
	bInitOk = InitializeInternal(EInitFrom::GlobalConfig, Configuration);
}

bool FCMCDHandler::InitializeInternal(EInitFrom InInitFrom, const FString& InConfiguration)
{
#define V1MAPKEY(K) { TEXT(#K), EV1Keys::k_##K }
	TMap<FString, EV1Keys> AllV1Keys { V1MAPKEY(br), V1MAPKEY(bl), V1MAPKEY(bs), V1MAPKEY(d), V1MAPKEY(cid), V1MAPKEY(dl), V1MAPKEY(mtp),V1MAPKEY(nor), V1MAPKEY(nrr),
									   V1MAPKEY(ot), V1MAPKEY(pr), V1MAPKEY(rtp), V1MAPKEY(sf), V1MAPKEY(sid), V1MAPKEY(st), V1MAPKEY(su), V1MAPKEY(tb), V1MAPKEY(v) };
#undef V1MAPKEY

	TSharedPtr<FJsonObject> Config;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InConfiguration);
	if (!FJsonSerializer::Deserialize(Reader, Config))
	{
		if (InInitFrom == EInitFrom::GlobalConfig)
		{
			PlayerSessionServices->PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Warning, TEXT("Failed to parse CMCD configuration, handler will be deactivated."));
		}
		else
		{
			PlayerSessionServices->PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Warning, TEXT("Failed to parse CMCD configuration, changes will not be applied."));
		}
		return false;
	}
	// Does the configuration provide the content or session ID?
	if (InInitFrom == EInitFrom::GlobalConfig)
	{
		FString CfgContentID;
		if (Config->TryGetStringField(Key_ContentId(), CfgContentID))
		{
			ContentID = CfgContentID;
		}
		FString CfgSessionID;
		if (Config->TryGetStringField(Key_SessionId(), CfgSessionID))
		{
			SessionID = CfgSessionID;
		}
		if (ContentID.IsEmpty() || SessionID.IsEmpty())
		{
			PlayerSessionServices->PostLog(Facility::EFacility::Player, IInfoLog::ELevel::Warning, TEXT("Missing content or session ID, handler will be deactivated."));
			return false;
		}

		// See if there is a key defining the name to use for the custom platform identifier.
		if (!(Config->TryGetStringField(TEXT("platformKey"), PlatformKey)))
		{
			PlatformKey = TEXT("platform-id");
		}

		// Does the config allow or prohibit the use of parameters conveyed in the playlist?
		bool bAllowPlaylistParams = false;
		if (Config->TryGetBoolField(TEXT("applyPlaylistParams"), bAllowPlaylistParams))
		{
			bApplyPlaylistParams = bAllowPlaylistParams;
		}
		// See if there is a key we may get user data from the playlist for CMCD configuration.
		if (!Config->TryGetStringField(TEXT("importFromPlaylistKey"), PlaylistParamImportKey))
		{
			PlaylistParamImportKey = TEXT("EPIC-CMCD");
		}
	}

	// Get the `cmcd` array, if it exists. The configuration might also come in later from the playlist.
	const TArray<TSharedPtr<FJsonValue>>* CMCDArray = nullptr;
	if ((Config->TryGetArrayField(Key_CmcdArray(), CMCDArray) && CMCDArray))
	{
		auto GetVar = [](const TSharedPtr<FJsonObject>& InObj, FStringView InName, EJson InType) -> TSharedPtr<FJsonValue>
		{
			return InObj->HasField(InName) ? InObj->GetField(InName, InType) : nullptr;
		};

		// Iterate it
		TArray<FReportElement> NewReportElements;
		for(auto& It : *CMCDArray)
		{
			const TSharedPtr<FJsonObject>* Obj = nullptr;
			if (!It.IsValid() || !It->TryGetObject(Obj))
			{
				continue;
			}

			// New report element.
			FReportElement re;
			TSharedPtr<FJsonValue> JVersion = GetVar(*Obj, Key_Version(), EJson::Number);
			TSharedPtr<FJsonValue> JType = GetVar(*Obj, Key_Type(), EJson::String);
			TSharedPtr<FJsonValue> JMode = GetVar(*Obj, Key_Mode(), EJson::String);
			TSharedPtr<FJsonValue> JKeys = GetVar(*Obj, Key_Keys(), EJson::String);
			TSharedPtr<FJsonValue> JCDNs = GetVar(*Obj, Key_CDNs(), EJson::String);
			TSharedPtr<FJsonValue> JInclude  = GetVar(*Obj, Key_IncludeInRequests(), EJson::String);
			TSharedPtr<FJsonValue> JContentID = GetVar(*Obj, Key_ContentId(), EJson::String);
			TSharedPtr<FJsonValue> JSessionID = GetVar(*Obj, Key_SessionId(), EJson::String);

			re.Version = JVersion.IsValid() ? (int32)JVersion->AsNumber() : 1;
			// If not a supported version continue with the next
			if (re.Version != 1)
			{
				continue;
			}
			FString Type = JType.IsValid() ? JType->AsString() : TEXT("request");
			// With version 1 the type can only be `request`
			if (!Type.Equals(TEXT("request")))
			{
				continue;
			}
			re.Type = FReportElement::EType::Request;
			// Mode
			FString Mode = JMode.IsValid() ? JMode->AsString() : TEXT("headers");
			if (!Mode.Equals(TEXT("headers")) && !Mode.Equals(TEXT("query")))
			{
				continue;
			}
			re.Mode = Mode.Equals(TEXT("headers")) ? FReportElement::EMode::Headers : FReportElement::EMode::Query;

			// Content and session ID to use with this entry. Use global ones if not specified for this entry.
			re.ContentID = JContentID.IsValid() ? JContentID->AsString() : ContentID;
			re.SessionID = JSessionID.IsValid() ? JSessionID->AsString() : SessionID;

			// Enabled keywords. If none are given, or only the "*" wildcard, enabled all of them.
			if (!JKeys.IsValid() || JKeys->AsString().Equals(TEXT("*")))
			{
				for(auto& En : AllV1Keys)
				{
					re.EnableKey(En.Value);
				}
			}
			else
			{
				TArray<FString> EnabledKeywords;
				JKeys->AsString().ParseIntoArrayWS(EnabledKeywords, TEXT(","), true);
				for(auto& En : EnabledKeywords)
				{
					if (AllV1Keys.Contains(En))
					{
						re.EnableKey(AllV1Keys[En]);
					}
				}
			}

			// CDNs.
			//  - If absent or empty, all CDNs match.
			//  - A "." matches the default CDN
			//       - which is either indicated by a "." (the HLS default pathway)
			//       - or an empty string (no @serviceLocation in (non-DVB !!) DASH)
			// ! Note that if CDNs other than "." are listed then the playlist MUST use pathways (HLS) or @serviceLocation (DASH)!
			// ! Note that in DVB DASH and absent @serviceLocation in the MPD automatically makes it the same as the URL!
			if (JCDNs.IsValid() && !JCDNs->AsString().IsEmpty())
			{
				JCDNs->AsString().ParseIntoArrayWS(re.CDNs, TEXT(","));
			}

			// Included in requests filter.
			if (JInclude.IsValid())
			{
				TArray<FString> incReqs;
				JInclude->AsString().ParseIntoArrayWS(incReqs, TEXT(","));
				for(auto& ir : incReqs)
				{
					if (ir.Equals(IncludeInRequests_Segment()))
					{
						re.EnableRequestType(ERequestType::InitSegment);
						re.EnableRequestType(ERequestType::IndexSegment);
						re.EnableRequestType(ERequestType::Segment);
					}
					else if (ir.Equals(IncludeInRequests_FirstPlaylist()))
					{
						re.EnableRequestType(ERequestType::FirstPlaylist);
					}
					else if (ir.Equals(IncludeInRequests_Playlist()))
					{
						re.EnableRequestType(ERequestType::FirstPlaylist);
						re.EnableRequestType(ERequestType::Playlist);
						re.EnableRequestType(ERequestType::PlaylistUpdate);
					}
					else if (ir.Equals(IncludeInRequests_Steering()))
					{
						re.EnableRequestType(ERequestType::Steering);
					}
					else if (ir.Equals(IncludeInRequests_Other()))
					{
						re.EnableRequestType(ERequestType::Other);
					}
					else if (ir.Equals(IncludeInRequests_DRM()))
					{
						re.EnableRequestType(ERequestType::DRM);
					}
				}
			}
			else
			{
				// Enable all request types by default.
				re.EnableRequestType(ERequestType::InitSegment);
				re.EnableRequestType(ERequestType::IndexSegment);
				re.EnableRequestType(ERequestType::Segment);
				re.EnableRequestType(ERequestType::FirstPlaylist);
				re.EnableRequestType(ERequestType::Playlist);
				re.EnableRequestType(ERequestType::PlaylistUpdate);
				re.EnableRequestType(ERequestType::Steering);
				re.EnableRequestType(ERequestType::Other);
				re.EnableRequestType(ERequestType::DRM);
			}

			// Handle configuration items that are only permitted in the global configuration.
			if (InInitFrom == EInitFrom::GlobalConfig)
			{
				TSharedPtr<FJsonValue> JHosts = GetVar(*Obj, TEXT("hosts"), EJson::String);
				TSharedPtr<FJsonValue> JDisable = GetVar(*Obj, TEXT("disable"), EJson::String);
				// Disabled keywords.
				if (JDisable.IsValid())
				{
					TArray<FString> DisabledKeywords;
					JDisable->AsString().ParseIntoArrayWS(DisabledKeywords, TEXT(","), true);
					for(auto& Dis : DisabledKeywords)
					{
						if (AllV1Keys.Contains(Dis))
						{
							re.DisableKey(AllV1Keys[Dis]);
						}
					}
				}

				// Hosts. If empty or only the "*" wildcard we set it as empty which is a match-all later on.
				if (JHosts.IsValid() && !JHosts->AsString().Equals(TEXT("*")))
				{
					JHosts->AsString().ParseIntoArrayWS(re.Hosts, TEXT(","));
					for(int32 i=0; i<re.Hosts.Num(); ++i)
					{
						// Check if the host contains one or more wildcards in it.
						if (re.Hosts[i].Contains(TEXT("*")))
						{
							// Replace '.' with the escape backslash since we need to treat it as a literal.
							FString Host(re.Hosts[i]);
							re.Hosts.RemoveAt(i);
							--i;
							Host.ReplaceInline(TEXT("."), TEXT("\\."));
							// Replace the '*' wildcards with ".*" used with a regex.
							Host.ReplaceInline(TEXT("*"), TEXT(".*"));
							re.HostRegexPatterns.Emplace(MakeUnique<FRegexPattern>(Host));
						}
					}
				}
			}

			// Add to the report elements list.
			NewReportElements.Emplace(MoveTemp(re));
		}

		// Where to insert the new elements
		if (InInitFrom == EInitFrom::GlobalConfig)
		{
			ReportElements = MoveTemp(NewReportElements);
		}
		else
		{
			ReportElements.Insert(MoveTemp(NewReportElements), 0);
		}
	}
	return true;
}


bool FCMCDHandler::UseParametersFromPlaylist()
{
	return bApplyPlaylistParams;
}

FString FCMCDHandler::UsePlaylistParametersFromKey()
{
	return PlaylistParamImportKey;
}

void FCMCDHandler::UpdateParameters(const FString& InUpdatedConfiguration)
{
	InitializeInternal(EInitFrom::Playlist, InUpdatedConfiguration);
}


void FCMCDHandler::SetStreamingFormat(EStreamingFormat InStreamingFormat)
{
	StreamingFormat = InStreamingFormat;
}

void FCMCDHandler::SetStreamType(EStreamType InStreamType)
{
	StreamType = InStreamType;
}

void FCMCDHandler::SetPlayableBitrateRange(Electra::EStreamType InStreamType, int32 InLowestBitrate, int32 InHighestBitrate)
{
	FBandwidths bw;
	bw.Lowest = InLowestBitrate;
	bw.Highest = InHighestBitrate;
	Bandwidths.Add(InStreamType, MoveTemp(bw));
}


FString FCMCDHandler::RemoveParamsFromURL(FString InURL)
{
	// We need to remove a CMCD parameter from the MPD URL.
	FURL_RFC3986 ParsedURL;
	if (ParsedURL.Parse(InURL))
	{
		TArray<FURL_RFC3986::FQueryParam> qps;
		ParsedURL.GetQueryParams(qps, false, false);
		qps.RemoveAll([](FURL_RFC3986::FQueryParam& e) { return e.Name.Equals(TEXT("CMCD")); });
		ParsedURL.SetQueryParams(qps);
		return ParsedURL.Get();
	}
	return InURL;
}


void FCMCDHandler::SetupRequestObject(ERequestType InRequestType, EObjectType InObject /* 'ot' */, FString& InOutRequestURL, TArray<HTTP::FHTTPHeader>& InOutRequestHeaders, const FString& InCDNId, const FRequestObjectInfo& InObjectInfo)
{
	// If CMCD is not enabled, just return.
	if (!bInitOk || ReportElements.IsEmpty())
	{
		return;
	}
	// Bail out if unsupported stream format.
	if (StreamingFormat == EStreamingFormat::Undefined)
	{
		return;
	}
	// Get the host the request is to be made to. If that fails, just leave.
	FURL_RFC3986 RequestHost;
	if (!RequestHost.Parse(InOutRequestURL))
	{
		return;
	}
	FString Host(RequestHost.GetHost());
	// Handle requests first
	for(auto& Req : ReportElements)
	{
		if (Req.Type != FReportElement::EType::Request)
		{
			continue;
		}

		// Is the request type a match? If not skip to the next.
		if (!Req.IsRequestTypeEnabled(InRequestType))
		{
			continue;
		}

		// Filter for CDN? No CDNs means any CDN matches.
		if (Req.CDNs.Num())
		{
			bool bCDNMatch = false;
			for(auto& cdn : Req.CDNs)
			{
				// If the CDN filter is "." then this allows for an empty CDN ID or the "."
				// Otherwise the CDN must be a case-insensitive match.
				if ((cdn.Equals(TEXT(".")) && (InCDNId.IsEmpty() || InCDNId.Equals(TEXT(".")))) ||
					(cdn.Equals(InCDNId, ESearchCase::IgnoreCase)))
				{
					bCDNMatch = true;
					break;
				}
			}
			if (!bCDNMatch)
			{
				continue;
			}
		}

		// Match for host?
		bool bMatch = (Req.Hosts.IsEmpty() && Req.HostRegexPatterns.IsEmpty()) || Req.Hosts.Contains(Host);
		if (!bMatch && !Req.HostRegexPatterns.IsEmpty())
		{
			for(int32 i=0; i<Req.HostRegexPatterns.Num(); ++i)
			{
				FRegexMatcher HostNameMatcher(*Req.HostRegexPatterns[i], Host);
				if (HostNameMatcher.FindNext())
				{
					bMatch = true;
					break;
				}
			}
		}
		if (bMatch)
		{
			// Hostname matches, apply this report
			ApplyReport(RequestHost, Req, InRequestType, InObject, InOutRequestURL, InOutRequestHeaders, InObjectInfo);
			// Leave the loop. The first matching request wins.
			break;
		}
	}
// TBD: Extended v2 handling, check events
}

void FCMCDHandler::ApplyReport(FURL_RFC3986& InOutRequestHost, const FReportElement& InParams, ERequestType InRequestType, EObjectType InObject, FString& InOutRequestURL, TArray<HTTP::FHTTPHeader>& InOutRequestHeaders, const FRequestObjectInfo& InObjectInfo)
{
	FReqHeaders Headers;
	switch(InObject)
	{
		case EObjectType::ManifestOrPlaylist:
		{
			ApplyManifestOrPlaylist(Headers, InParams, InObjectInfo);
			break;
		}
		case EObjectType::InitSegment:
		{
			ApplyInitSegment(Headers, InParams, InObjectInfo);
			break;
		}
		case EObjectType::VideoOnly:
		case EObjectType::AudioOnly:
		case EObjectType::MuxedAudioAndVideo:
		case EObjectType::CaptionOrSubtitle:
		case EObjectType::TimedTextTrack:
		{
			ApplyMediaSegment(Headers, InObject, InParams, InObjectInfo, InOutRequestHost);
			break;
		}
		case EObjectType::Other:
		{
			ApplyGeneric(Headers, InParams, InObjectInfo);
			break;
		}
	}


	auto MakeList = [](const TArray<FString>& InList) -> FString
	{
		FString Result;
		for(int32 i=0,iMax=InList.Num(); i<iMax; ++i)
		{
			if (i)
			{
				Result.Append(TEXT(","));
			}
			Result.Append(InList[i]);
		}
		return Result;
	};

	// Add as headers or as query parameters?
	if (InParams.Mode == FReportElement::EMode::Query)
	{
		TArray<FString> AllHeaders(Headers.Request);
		AllHeaders.Append(Headers.Object);
		AllHeaders.Append(Headers.Status);
		AllHeaders.Append(Headers.Session);
		if (AllHeaders.Num())
		{
			AllHeaders.Sort();
			TArray<FURL_RFC3986::FQueryParam> qps;
			if (!FURL_RFC3986::UrlEncode(qps.Emplace_GetRef().Value, MakeList(AllHeaders), FString()))
			{
				return;
			}
			qps[0].Name = TEXT("CMCD");
			InOutRequestHost.AddOrUpdateQueryParams(qps);
			InOutRequestURL = InOutRequestHost.Get();
		}
	}
	else
	{
		HTTP::FHTTPHeader Hdr;
		if (Headers.Request.Num())
		{
			Headers.Request.Sort();
			Hdr.Value = MakeList(Headers.Request);
			Hdr.Header = TEXT("CMCD-Request");
			InOutRequestHeaders.Emplace(MoveTemp(Hdr));
		}
		if (Headers.Object.Num())
		{
			Headers.Object.Sort();
			Hdr.Value = MakeList(Headers.Object);
			Hdr.Header = TEXT("CMCD-Object");
			InOutRequestHeaders.Emplace(MoveTemp(Hdr));
		}
		if (Headers.Status.Num())
		{
			Headers.Status.Sort();
			Hdr.Value = MakeList(Headers.Status);
			Hdr.Header = TEXT("CMCD-Status");
			InOutRequestHeaders.Emplace(MoveTemp(Hdr));
		}
		if (Headers.Session.Num())
		{
			Headers.Session.Sort();
			Hdr.Value = MakeList(Headers.Session);
			Hdr.Header = TEXT("CMCD-Session");
			InOutRequestHeaders.Emplace(MoveTemp(Hdr));
		}
	}
}


void FCMCDHandler::ApplyManifestOrPlaylist(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo)
{
	// cid, ot, pr, sf, sid, st, su, v
	Add_cid(OutHeaders, InParams);
	Add_ot(OutHeaders, InParams, GetOT(EObjectType::ManifestOrPlaylist));
	double pr = PlayerSessionServices->GetPlayerARBControl()->ABRGetPlaySpeed().GetAsSeconds();
	Add_pr(OutHeaders, InParams, pr);
	Add_sf(OutHeaders, InParams);
	Add_sid(OutHeaders, InParams);
	Add_st(OutHeaders, InParams);
	Add_su(OutHeaders, InParams);
	Add_v(OutHeaders, InParams);
	Add_platform(OutHeaders, InParams);
}

void FCMCDHandler::ApplyGeneric(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo)
{
	// cid, ot, pr, sf, sid, st, su, v
	Add_cid(OutHeaders, InParams);
	Add_ot(OutHeaders, InParams, GetOT(EObjectType::Other));
	double pr = PlayerSessionServices->GetPlayerARBControl()->ABRGetPlaySpeed().GetAsSeconds();
	Add_pr(OutHeaders, InParams, pr);
	Add_sf(OutHeaders, InParams);
	Add_sid(OutHeaders, InParams);
	Add_st(OutHeaders, InParams);
	Add_su(OutHeaders, InParams);
	Add_v(OutHeaders, InParams);
	Add_platform(OutHeaders, InParams);
}

void FCMCDHandler::ApplyInitSegment(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo)
{
	// br, cid, ot, pr, sf, sid, st, su, v
	Add_br(OutHeaders, InParams, InObjectInfo);
	Add_cid(OutHeaders, InParams);
	Add_ot(OutHeaders, InParams, GetOT(EObjectType::InitSegment));
	double pr = PlayerSessionServices->GetPlayerARBControl()->ABRGetPlaySpeed().GetAsSeconds();
	Add_pr(OutHeaders, InParams, pr);
	Add_sf(OutHeaders, InParams);
	Add_sid(OutHeaders, InParams);
	Add_st(OutHeaders, InParams);
	Add_su(OutHeaders, InParams);
	Add_v(OutHeaders, InParams);
	Add_platform(OutHeaders, InParams);
}

void FCMCDHandler::ApplyMediaSegment(FReqHeaders& OutHeaders, EObjectType InObject, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo, const FURL_RFC3986& InRequestHost)
{
	// br, bl, bs, cid, d, dl, mtp, nor, nrr, ot, pr, rtp, sf, sid, st, su, tb, v
	int32 bl = -1;
	// Get the current buffer level.
	if (InObject == EObjectType::VideoOnly || InObject == EObjectType::AudioOnly || InObject == EObjectType::MuxedAudioAndVideo)
	{
		IPlayerABRLiveControl::FABRBufferStats BufferStats;
		PlayerSessionServices->GetPlayerARBControl()->ABRGetStreamBufferStats(BufferStats, InObject == EObjectType::AudioOnly ? Electra::EStreamType::Audio : Electra::EStreamType::Video);
		bl = (((int32)BufferStats.PlayableContentDuration.GetAsMilliseconds() + 50) / 100) * 100;
		Add_bl(OutHeaders, InParams, bl);
	}
	Add_br(OutHeaders, InParams, InObjectInfo);
	Add_bs(OutHeaders, InObject, InParams);
	Add_cid(OutHeaders, InParams);
	Add_d(OutHeaders, InParams, InObjectInfo);
	// Do not add `dl` for now. We do not want a CDN to use this to throttle delivery.
	int32 mtp = (int32)(((PlayerSessionServices->GetStreamSelector()->GetAverageBandwidth() + 50000) / 100000) * 100);
	Add_mtp(OutHeaders, InParams, mtp);
	Add_ot(OutHeaders, InParams, GetOT(InObject));
	double pr = PlayerSessionServices->GetPlayerARBControl()->ABRGetPlaySpeed().GetAsSeconds();
	Add_pr(OutHeaders, InParams, pr);
	Add_nornrr(OutHeaders, InRequestHost, InParams, InObjectInfo);
	// Do not add `rtp` for now. We do not want a CDN to use this to throttle delivery.
	Add_sf(OutHeaders, InParams);
	Add_sid(OutHeaders, InParams);
	Add_st(OutHeaders, InParams);
	Add_su(OutHeaders, InParams);
	Add_tb(OutHeaders, InParams, InObjectInfo);
	Add_v(OutHeaders, InParams);
	Add_platform(OutHeaders, InParams);
}

const TCHAR* FCMCDHandler::GetOT(EObjectType InObject)
{
	switch(InObject)
	{
		case EObjectType::ManifestOrPlaylist:		return TEXT("m");
		case EObjectType::AudioOnly:				return TEXT("a");
		case EObjectType::VideoOnly:				return TEXT("v");
		case EObjectType::MuxedAudioAndVideo:		return TEXT("av");
		case EObjectType::InitSegment:				return TEXT("i");
		case EObjectType::CaptionOrSubtitle:		return TEXT("c");
		case EObjectType::TimedTextTrack:			return TEXT("tt");
		case EObjectType::CryptoKeyLicenseOrCert:	return TEXT("k");
		case EObjectType::Other:					return TEXT("o");
		default:									return TEXT("");
	}
}


void FCMCDHandler::Add_br(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo)
{
	// Note: in the work-in-progress spec for v2 this must not be sent for `ot` other than `a`, `v`, `av` and `o`
	if (InParams.IsKeyEnabled(EV1Keys::k_br) && InObjectInfo.EncodedBitrate.Get(0) > 0)
	{
		OutHeaders.Object.Emplace(FString::Printf(TEXT("br=%d"), InObjectInfo.EncodedBitrate.GetValue() / 1000));
	}
}

void FCMCDHandler::Add_bl(FReqHeaders& OutHeaders, const FReportElement& InParams, int32 InBufferLength)
{
	if (InBufferLength >= 0 && InParams.IsKeyEnabled(EV1Keys::k_bl))
	{
		OutHeaders.Request.Emplace(FString::Printf(TEXT("bl=%d"), InBufferLength));
	}
}

void FCMCDHandler::Add_bs(FReqHeaders& OutHeaders, EObjectType InObject, const FReportElement& InParams)
{
	bool bWasStarved = false;
	if (InObject == EObjectType::VideoOnly || InObject == EObjectType::MuxedAudioAndVideo)
	{
		bWasStarved = Vars.bVideoStarved;
		Vars.bVideoStarved = false;
	}
	else if (InObject == EObjectType::AudioOnly)
	{
		bWasStarved = Vars.bAudioStarved;
		Vars.bAudioStarved = false;
	}
	if (bWasStarved && InParams.IsKeyEnabled(EV1Keys::k_bs))
	{
		OutHeaders.Status.Emplace(TEXT("bs"));
	}
}

void FCMCDHandler::Add_cid(FReqHeaders& OutHeaders, const FReportElement& InParams)
{
	if (InParams.IsKeyEnabled(EV1Keys::k_cid) && InParams.ContentID.Len())
	{
		OutHeaders.Session.Emplace(FString::Printf(TEXT("cid=\"%s\""), *InParams.ContentID));
	}
}

void FCMCDHandler::Add_d(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo)
{
	if (InObjectInfo.ObjectDuration.IsSet() && InParams.IsKeyEnabled(EV1Keys::k_d))
	{
		OutHeaders.Object.Emplace(FString::Printf(TEXT("d=%d"), InObjectInfo.ObjectDuration.GetValue()));
	}
}

void FCMCDHandler::Add_mtp(FReqHeaders& OutHeaders, const FReportElement& InParams, int32 InMeasuredThroughput)
{
	if (InMeasuredThroughput > 0 && InParams.IsKeyEnabled(EV1Keys::k_mtp))
	{
		OutHeaders.Request.Emplace(FString::Printf(TEXT("mtp=%d"), InMeasuredThroughput));
	}
}

void FCMCDHandler::Add_nornrr(FReqHeaders& OutHeaders, const FURL_RFC3986& InRequestHost, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo)
{
	if (InParams.Version == 1 && InObjectInfo.NextObjectRequest.Num())
	{
		// Next segment is a range within the current segment?
		if (InObjectInfo.NextObjectRequest[0].URL.IsEmpty() && !InObjectInfo.NextObjectRequest[0].Range.IsEmpty() && InParams.IsKeyEnabled(EV1Keys::k_nrr))
		{
			OutHeaders.Request.Emplace(FString::Printf(TEXT("nrr=%s"), *InObjectInfo.NextObjectRequest[0].Range));
		}
		// Next segment is a full segment?
		else if (!InObjectInfo.NextObjectRequest[0].URL.IsEmpty() && InObjectInfo.NextObjectRequest[0].Range.IsEmpty() && InParams.IsKeyEnabled(EV1Keys::k_nor))
		{
			FString Rel, EscRel;
			if (!InRequestHost.MakeRelativePath(Rel, InObjectInfo.NextObjectRequest[0].URL) ||
				!FURL_RFC3986::UrlEncode(EscRel, Rel, FString()))
			{
				return;
			}
			if (EscRel.Len())
			{
				OutHeaders.Request.Emplace(FString::Printf(TEXT("nor=\"%s\""), *EscRel));
			}
		}
		// Next segment is a part of a segment?
		else if (!InObjectInfo.NextObjectRequest[0].URL.IsEmpty() && !InObjectInfo.NextObjectRequest[0].Range.IsEmpty() && InParams.IsKeyEnabled(EV1Keys::k_nor) && InParams.IsKeyEnabled(EV1Keys::k_nrr))
		{
			FString Rel, EscRel;
			if (!InRequestHost.MakeRelativePath(Rel, InObjectInfo.NextObjectRequest[0].URL) ||
				!FURL_RFC3986::UrlEncode(EscRel, Rel, FString()))
			{
				return;
			}
			if (EscRel.Len())
			{
				OutHeaders.Request.Emplace(FString::Printf(TEXT("nor=\"%s\""), *EscRel));
			}
			OutHeaders.Request.Emplace(FString::Printf(TEXT("nrr=%s"), *InObjectInfo.NextObjectRequest[0].Range));
		}
	}
}

void FCMCDHandler::Add_ot(FReqHeaders& OutHeaders, const FReportElement& InParams, const TCHAR* InOT)
{
	if (InParams.IsKeyEnabled(EV1Keys::k_ot))
	{
		OutHeaders.Object.Emplace(FString::Printf(TEXT("ot=%s"), InOT));
	}
}

void FCMCDHandler::Add_pr(FReqHeaders& OutHeaders, const FReportElement& InParams, double InPlayRate)
{
	if (InPlayRate != 1.0 && InParams.IsKeyEnabled(EV1Keys::k_pr))
	{
		OutHeaders.Session.Emplace(FString::Printf(TEXT("pr=%s"), *FString::SanitizeFloat(InPlayRate, 0)));
	}
}

void FCMCDHandler::Add_sf(FReqHeaders& OutHeaders, const FReportElement& InParams)
{
	if (InParams.IsKeyEnabled(EV1Keys::k_sf))
	{
		switch(StreamingFormat)
		{
			case EStreamingFormat::HLS:
			{
				OutHeaders.Session.Emplace(TEXT("sf=h"));
				break;
			}
			case EStreamingFormat::DASH:
			{
				OutHeaders.Session.Emplace(TEXT("sf=d"));
				break;
			}
			case EStreamingFormat::LowLatencyDASH:
			{
				if (InParams.Version == 1)
				{
					OutHeaders.Session.Emplace(TEXT("sf=d"));
				}
				else //if (InParams.Version == 2)
				{
					OutHeaders.Session.Emplace(TEXT("sf=ld"));
				}
				break;
			}
			default:
			{
				unimplemented();
				break;
			}
		}
	}
}

void FCMCDHandler::Add_sid(FReqHeaders& OutHeaders, const FReportElement& InParams)
{
	if (InParams.IsKeyEnabled(EV1Keys::k_sid) && InParams.SessionID.Len())
	{
		OutHeaders.Session.Emplace(FString::Printf(TEXT("sid=\"%s\""), *InParams.SessionID));
	}
}

void FCMCDHandler::Add_st(FReqHeaders& OutHeaders, const FReportElement& InParams)
{
	if (InParams.IsKeyEnabled(EV1Keys::k_st))
	{
		if (StreamType == EStreamType::VOD)
		{
			OutHeaders.Session.Emplace(TEXT("st=v"));
		}
		else if (StreamType == EStreamType::Live)
		{
			OutHeaders.Session.Emplace(TEXT("st=l"));
		}
	}
}

void FCMCDHandler::Add_su(FReqHeaders& OutHeaders, const FReportElement& InParams)
{
	if (Vars.bIsStartup && InParams.IsKeyEnabled(EV1Keys::k_su))
	{
		OutHeaders.Request.Emplace(TEXT("su"));
	}
}

void FCMCDHandler::Add_tb(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo)
{
	if (InParams.IsKeyEnabled(EV1Keys::k_tb) && InObjectInfo.MediaStreamType.IsSet() && Bandwidths.Contains(InObjectInfo.MediaStreamType.GetValue()))
	{
		if (int32 bw = Bandwidths[InObjectInfo.MediaStreamType.GetValue()].Highest)
		{
			OutHeaders.Object.Emplace(FString::Printf(TEXT("tb=%d"), bw / 1000));
		}
	}
}

void FCMCDHandler::Add_v(FReqHeaders& OutHeaders, const FReportElement& InParams)
{
	if (InParams.Version > 1 && InParams.IsKeyEnabled(EV1Keys::k_v))
	{
		OutHeaders.Session.Emplace(FString::Printf(TEXT("v=%d"), InParams.Version));
	}
}

void FCMCDHandler::Add_platform(FReqHeaders& OutHeaders, const FReportElement& InParams)
{
	if (PlatformKey.Len() && PlatformID.Len())
	{
		OutHeaders.Session.Emplace(FString::Printf(TEXT("%s=\"%s\""), *PlatformKey, *PlatformID));
	}
}



void FCMCDHandler::ReportOpenSource(const FString& InURL) { }
void FCMCDHandler::ReportReceivedMainPlaylist(const FString& InEffectiveURL) { }
void FCMCDHandler::ReportReceivedPlaylists() { }
void FCMCDHandler::ReportTracksChanged() { }
void FCMCDHandler::ReportPlaylistDownload(const Metrics::FPlaylistDownloadStats& InPlaylistDownloadStats) { }
void FCMCDHandler::ReportBufferingStart(Metrics::EBufferingReason InBufferingReason)
{
	if (InBufferingReason == Metrics::EBufferingReason::Seeking)
	{
		Vars.bIsStartup = true;
	}
	else if (InBufferingReason == Metrics::EBufferingReason::Rebuffering)
	{
		Vars.bIsStartup = true;

		IPlayerABRLiveControl::FABRBufferStats bsV, bsA;
		PlayerSessionServices->GetPlayerARBControl()->ABRGetStreamBufferStats(bsV, Electra::EStreamType::Video);
		PlayerSessionServices->GetPlayerARBControl()->ABRGetStreamBufferStats(bsA, Electra::EStreamType::Audio);
		Vars.bVideoStarved = bsV.PlayableContentDuration.GetAsMilliseconds() < 100;
		Vars.bAudioStarved = bsA.PlayableContentDuration.GetAsMilliseconds() < 100;
	}
}

void FCMCDHandler::ReportBufferingEnd(Metrics::EBufferingReason InBufferingReason)
{
	// After any type of buffering this is not the startup phase any more.
	Vars.bIsStartup = false;
}

void FCMCDHandler::ReportBandwidth(int64 InEffectiveBps, int64 InThroughputBps, double InLatencyInSeconds) { }
void FCMCDHandler::ReportBufferUtilization(const Metrics::FBufferStats& InBufferStats) { }
void FCMCDHandler::ReportSegmentDownload(const Metrics::FSegmentDownloadStats& InSegmentDownloadStats) { }
void FCMCDHandler::ReportLicenseKey(const Metrics::FLicenseKeyStats& InLicenseKeyStats) { }
void FCMCDHandler::ReportVideoQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch) { }
void FCMCDHandler::ReportAudioQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch) { }
void FCMCDHandler::ReportDataAvailabilityChange(const Metrics::FDataAvailabilityChange& InDataAvailability) { }
void FCMCDHandler::ReportDecodingFormatChange(const FStreamCodecInformation& InNewDecodingFormat) { }
void FCMCDHandler::ReportPrerollStart() { }
void FCMCDHandler::ReportPrerollEnd() { }
void FCMCDHandler::ReportPlaybackStart() { }
void FCMCDHandler::ReportPlaybackPaused() { }
void FCMCDHandler::ReportPlaybackResumed() { }
void FCMCDHandler::ReportPlaybackEnded() { }
void FCMCDHandler::ReportJumpInPlayPosition(const FTimeValue& InToNewTime, const FTimeValue& InFromTime, Metrics::ETimeJumpReason InTimejumpReason) { }
void FCMCDHandler::ReportPlaybackStopped() { }
void FCMCDHandler::ReportSeekCompleted() { }
void FCMCDHandler::ReportMediaMetadataChanged(TSharedPtrTS<FMetadataParser> InMetadata) { }
void FCMCDHandler::ReportError(const FString& InErrorReason) { }
void FCMCDHandler::ReportLogMessage(IInfoLog::ELevel InLogLevel, const FString& InLogMessage, int64 InPlayerWallclockMilliseconds) { }
void FCMCDHandler::ReportVideoFrameStats(const Metrics::FFrameStats& InFrameStats) { }
void FCMCDHandler::ReportAudioFrameStats(const Metrics::FFrameStats& InFrameStats) { }

} // namespace Electra
