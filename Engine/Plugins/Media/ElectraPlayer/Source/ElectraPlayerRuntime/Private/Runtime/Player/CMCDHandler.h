// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/Regex.h"
#include "HTTP/HTTPManager.h"
#include "MediaURLType.h"
#include "ParameterDictionary.h"
#include "Utilities/URLParser.h"
#include "StreamTypes.h"
#include "Player/AdaptiveStreamingPlayerMetrics.h"
#include "MediaStreamMetadata.h"


namespace Electra
{

class IPlayerSessionServices;



/**
 * Common Media Client Data (CMCD) Handler
 *
 * This handler implements v1 of CTA-5004.
 *
 * Version 2 is still under development and changes often.
 * We recognize elements here but do not process them yet.
 *
 */
class FCMCDHandler : public IAdaptiveStreamingPlayerMetrics
{
public:
	FCMCDHandler(IPlayerSessionServices* InPlayerSessionServices);
	virtual ~FCMCDHandler();

	enum class EStreamingFormat
	{
		// CMCD v1 defines HLS, DASH and SmoothStreaming, while CMCD v2 also adds
		// low latency flavors. Upon requesting the main playlist it is not clear however if the
		// stream will be low latency or not, so the `ot=m` type can't reflect the low latency
		// version without externally provided means.
		Undefined,				// not known, used as a placeholder to omit the object type in requests
		HLS,
		DASH,
		SmoothStreaming,		// not supported
		// v2
		HESP,					// not supported
		LowLatencyDASH,
		LowLatencyLowDelayDASH,
		LowLatencyHLS,

		Other
	};

	enum class EStreamType
	{
		Undefined,
		VOD,
		Live
	};

	enum class EObjectType
	{
		Undefined,				// not known, used as a placeholder to omit the object type in requests
		ManifestOrPlaylist,		// 'm'
		AudioOnly,				// 'a'
		VideoOnly,				// 'v'
		MuxedAudioAndVideo,		// 'av'
		InitSegment,			// 'i'		(probably also to be used for the index segment)
		CaptionOrSubtitle,		// 'c'
		TimedTextTrack,			// 'tt'
		CryptoKeyLicenseOrCert,	// 'k'
		Other					// 'o'
	};

	enum class ERequestType
	{
		// filtered by `@includeInRequests`: "segment"
		InitSegment,
		IndexSegment,
		Segment,
		// filtered by `@includeInRequests`: "playlist"
		FirstPlaylist,
		Playlist,
		PlaylistUpdate,
		// filtered by `@includeInRequests`: "steering"
		Steering,
		// filtered by `@includeInRequests`: "other"
		Other,					// Callback, Chaining, Fallback, Sideload, XLink
		// filtered by `@includeInRequests`: "drm"
		DRM
	};

	static const TCHAR* IncludeInRequests_Segment() { return TEXT("segment"); }
	static const TCHAR* IncludeInRequests_FirstPlaylist() { return TEXT("firstplaylist"); }
	static const TCHAR* IncludeInRequests_Playlist() { return TEXT("playlist"); }
	static const TCHAR* IncludeInRequests_Steering() { return TEXT("steering"); }
	static const TCHAR* IncludeInRequests_Other() { return TEXT("other"); }
	static const TCHAR* IncludeInRequests_DRM() { return TEXT("drm"); }

	static const TCHAR* Key_CmcdArray() { return TEXT("cmcd"); }
	static const TCHAR* Key_ContentId() { return TEXT("contentID"); }
	static const TCHAR* Key_SessionId() { return TEXT("sessionID"); }
	static const TCHAR* Key_Type() { return TEXT("type"); }
	static const TCHAR* Key_Version() { return TEXT("v"); }
	static const TCHAR* Key_IncludeInRequests() { return TEXT("include"); }
	static const TCHAR* Key_CDNs() { return TEXT("cdns"); }
	static const TCHAR* Key_Mode() { return TEXT("mode"); }
	static const TCHAR* Key_Keys() { return TEXT("keys"); }


	/**
	 * Returns true if CMCD is enabled for this session, false if not.
	 */
	bool IsEnabled();

	/**
	 *
	 */
	void PeriodicHandle();

	/**
	 * Initialize the CMCD handler after creating it.
	 */
	void Initialize(const FString& InContentID, const FString& InSessionID, const FString& InConfiguration);


	/**
	 * Returns `true` when the initial configuration, either implicitly or by stream format defaults,
	 * allows to use parameters provided by the playlist.
	 * In that case construct a configuration structure from the parameters in the playlist and update
	 * the configuration.
	 */
	bool UseParametersFromPlaylist();

	/**
	 * If `UseParametersFromPlaylist()` returns `true` and the streaming format does not have a standardized
	 * way of conveying CMCD parameters, see if the playlist has a user data element of the name returned by
	 * this function and if so, assume it contains the CMCD configuration.
	 */
	FString UsePlaylistParametersFromKey();

	/**
	 * Update the initial configuration with this new config.
	 */
	void UpdateParameters(const FString& InUpdatedConfiguration);


	/**
	 * Set the streaming format. This should be done at least once before or after calling `Initialize()`.
	 * If the precise sub-format is only known after loading the playlist, call this method again with the
	 * final determined format.
	 */
	void SetStreamingFormat(EStreamingFormat InStreamingFormat);

	/**
	 * Set the stream type. This is not bound to change during playback (but it could, if Live turns into VoD)
	 * and it is simpler to set it once instead of providing it in every `FRequestObjectInfo` (see below).
	 */
	void SetStreamType(EStreamType InStreamType);

	/**
	 * Sets the lowest and highest bitrates of a media type this platform is allowed to play.
	 * Used for the `tb` (in v1; and later for v2 `tpb`, `lb`, `tab` and `lab`)
	 */
	void SetPlayableBitrateRange(Electra::EStreamType InStreamType, int32 InLowestBitrate, int32 InHighestBitrate);


	/*
		Request object information that can be obtained through the metrics listener and
		do not need to be provided in `FRequestObjectInfo`.

		v1
			- Buffer length (`bl`)
			- Buffer starvation (`bs`)
			- Measured throughput (`mtp`)
			- Playback rate (`pr`)
			- Startup (`su`)
			- Top encoded bitrate (`tb`)

		v2
			still under development. Fields change periodically.
			"Response" mode has been removed, so all fields related to it but still present in the document
			are not relevant unless that mode gets reinstated.

			- Playhead bitrate (`pb`)
			- Buffer starvation duration (`bsd`)
			- Live stream latency (`ltc`)
			- Backgrounded (`bg`)
			- Sequence Number (`sn`)
			- State (`sta`)
			- Timestamp (`ts`)
			- Top playable bitrate (`tpb`)
			- Lowest encoded bitrate (`lb`)
			- Top aggregated encoded bitrate (`tab`)
			- Lowest aggregated encoded bitrate (`lab`)
			- Playhead time (`pt`)
			- Player Error Code (`ec`)
			- Media Start Delay (`msd`)
			- Event (`e`)
			- Dropped Frames (`df`)
			- Non rendered (`nr`)
	*/

	struct FRequestObjectInfo
	{
		struct FNextObjectRequest
		{
			FString URL;
			FString Range;
		};

		// Internal media stream type for index and init segment where the `ot` is only `i` and not `a`,`v` or `av`.
		TOptional<Electra::EStreamType> MediaStreamType;

		// v1
		TOptional<int32> EncodedBitrate;					// `br`
		TOptional<int32> ObjectDuration;					// `d`
		TOptional<int32> Deadline;							// `dl`
		TArray<FNextObjectRequest> NextObjectRequest;		// `nor` and `nrr`
		TOptional<int32> RequestedMaximumThroughput;		// `rtp`
		/*
			set via SetPlayableBitrateRange()
		TOptional<int32> TopBitrate;						// `tb`
		*/

		// v2 (not yet supported)
		/*
		TOptional<int32> AggregateEncodedBitrate;			// `ab`
		TOptional<int32> TargetBufferLength;				// `tbl`
		*/
	};



	/**
	 * Modifies the request headers or URL of the given request to include the CMCD metrics.
	 */
	void SetupRequestObject(ERequestType InRequestType, EObjectType InObject /* 'ot' */, FString& InOutRequestURL, TArray<HTTP::FHTTPHeader>& InOutRequestHeaders, const FString& InCDNId, const FRequestObjectInfo& InObjectInfo);

	/**
	 * Removes a `CMCD` query parameter from the URL.
	 */
	FString RemoveParamsFromURL(FString InURL);

private:
	void ReportOpenSource(const FString& InURL) override;
	void ReportReceivedMainPlaylist(const FString& InEffectiveURL) override;
	void ReportReceivedPlaylists() override;
	void ReportTracksChanged() override;
	void ReportPlaylistDownload(const Metrics::FPlaylistDownloadStats& InPlaylistDownloadStats) override;
	void ReportBufferingStart(Metrics::EBufferingReason InBufferingReason) override;
	void ReportBufferingEnd(Metrics::EBufferingReason InBufferingReason) override;
	void ReportBandwidth(int64 InEffectiveBps, int64 InThroughputBps, double InLatencyInSeconds) override;
	void ReportBufferUtilization(const Metrics::FBufferStats& InBufferStats) override;
	void ReportSegmentDownload(const Metrics::FSegmentDownloadStats& InSegmentDownloadStats) override;
	void ReportLicenseKey(const Metrics::FLicenseKeyStats& InLicenseKeyStats) override;
	void ReportVideoQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch) override;
	void ReportAudioQualityChange(int32 InNewBitrate, int32 InPreviousBitrate, bool bInIsDrasticDownswitch) override;
	void ReportDataAvailabilityChange(const Metrics::FDataAvailabilityChange& InDataAvailability) override;
	void ReportDecodingFormatChange(const FStreamCodecInformation& InNewDecodingFormat) override;
	void ReportPrerollStart() override;
	void ReportPrerollEnd() override;
	void ReportPlaybackStart() override;
	void ReportPlaybackPaused() override;
	void ReportPlaybackResumed() override;
	void ReportPlaybackEnded() override;
	void ReportJumpInPlayPosition(const FTimeValue& InToNewTime, const FTimeValue& InFromTime, Metrics::ETimeJumpReason InTimejumpReason) override;
	void ReportPlaybackStopped() override;
	void ReportSeekCompleted() override;
	void ReportMediaMetadataChanged(TSharedPtrTS<FMetadataParser> InMetadata) override;
	void ReportError(const FString& InErrorReason) override;
	void ReportLogMessage(IInfoLog::ELevel InLogLevel, const FString& InLogMessage, int64 InPlayerWallclockMilliseconds) override;
	void ReportVideoFrameStats(const Metrics::FFrameStats& InFrameStats) override;
	void ReportAudioFrameStats(const Metrics::FFrameStats& InFrameStats) override;

	enum class EV1Keys : uint8
	{
		k_br,
		k_bl,
		k_bs,
		k_d,
		k_cid,
		k_dl,
		k_mtp,
		k_nor,
		k_nrr,
		k_ot,
		k_pr,
		k_rtp,
		k_sf,
		k_sid,
		k_st,
		k_su,
		k_tb,
		k_v
	};


	struct FReportElement
	{
		enum class EType
		{
			Undefined,
			Request,
			Event
		};
		enum EMode
		{
			Undefined,
			Headers,
			Query
		};
		EType Type = EType::Undefined;
		int32 Version = 0;
		TArray<FString> Hosts;
		TArray<FString> CDNs;
		TArray<TUniquePtr<FRegexPattern>> HostRegexPatterns;
		FString ContentID;
		FString SessionID;
		EMode Mode = EMode::Undefined;
		int32 EnabledKeys = 0;
		int32 EnabledRequestTypes = 0;
		void EnableKey(EV1Keys InKey)
		{
			EnabledKeys |= 1 << (int32)InKey;
		}
		void DisableKey(EV1Keys InKey)
		{
			EnabledKeys &= ~(1 << (int32)InKey);
		}
		bool IsKeyEnabled(EV1Keys InKey) const
		{
			return !!(EnabledKeys & (1 << (int32)InKey));
		}

		void EnableRequestType(ERequestType InType)
		{
			EnabledRequestTypes |= 1 << (int32)InType;
		}
		void DisableRequestType(ERequestType InType)
		{
			EnabledRequestTypes &= ~(1 << (int32)InType);
		}
		bool IsRequestTypeEnabled(ERequestType InType) const
		{
			return !!(EnabledRequestTypes & (1 << (int32)InType));
		}
	};

	struct FBandwidths
	{
		int32 Lowest = 0;
		int32 Highest = 0;
	};

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	FString ContentID;
	FString SessionID;
	FString PlatformID;
	FString PlaylistParamImportKey;
	bool bApplyPlaylistParams = true;
	bool bInitOk = false;

	EStreamingFormat StreamingFormat = EStreamingFormat::Undefined;
	EStreamType StreamType = EStreamType::Undefined;
	TArray<FReportElement> ReportElements;
	FString PlatformKey;
	TMap<Electra::EStreamType, FBandwidths> Bandwidths;

	struct FVars
	{
		bool bIsStartup = true;
		bool bVideoStarved = false;
		bool bAudioStarved = false;
	};
	FVars Vars;

	struct FReqHeaders
	{
		TArray<FString> Request;
		TArray<FString> Object;
		TArray<FString> Status;
		TArray<FString> Session;
	};

	const TCHAR* GetOT(EObjectType InObject);

	enum class EInitFrom
	{
		GlobalConfig,
		Playlist
	};
	bool InitializeInternal(EInitFrom InInitFrom, const FString& InConfiguration);

	void ApplyManifestOrPlaylist(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
	void ApplyGeneric(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
	void ApplyInitSegment(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
	void ApplyMediaSegment(FReqHeaders& OutHeaders, EObjectType InObject, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo, const FURL_RFC3986& InRequestHost);

	void ApplyReport(FURL_RFC3986& InOutRequestHost, const FReportElement& InParams, ERequestType InRequestType, EObjectType InObject, FString& InOutRequestURL, TArray<HTTP::FHTTPHeader>& InOutRequestHeaders, const FRequestObjectInfo& InObjectInfo);

	void Add_br(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
	void Add_bl(FReqHeaders& OutHeaders, const FReportElement& InParams, int32 InBufferLength);
	void Add_bs(FReqHeaders& OutHeaders, EObjectType InObject, const FReportElement& InParams);
	void Add_cid(FReqHeaders& OutHeaders, const FReportElement& InParams);
	void Add_d(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
//	void Add_dl(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
	void Add_mtp(FReqHeaders& OutHeaders, const FReportElement& InParams, int32 InMeasuredThroughput);
	void Add_nornrr(FReqHeaders& OutHeaders, const FURL_RFC3986& InRequestHost, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
	void Add_ot(FReqHeaders& OutHeaders, const FReportElement& InParams, const TCHAR* InOT);
	void Add_pr(FReqHeaders& OutHeaders, const FReportElement& InParams, double InPlayRate);
//	void Add_rtp(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
	void Add_sf(FReqHeaders& OutHeaders, const FReportElement& InParams);
	void Add_sid(FReqHeaders& OutHeaders, const FReportElement& InParams);
	void Add_st(FReqHeaders& OutHeaders, const FReportElement& InParams);
	void Add_su(FReqHeaders& OutHeaders, const FReportElement& InParams);
	void Add_tb(FReqHeaders& OutHeaders, const FReportElement& InParams, const FRequestObjectInfo& InObjectInfo);
	void Add_v(FReqHeaders& OutHeaders, const FReportElement& InParams);
	void Add_platform(FReqHeaders& OutHeaders, const FReportElement& InParams);

};

} // namespace Electra
