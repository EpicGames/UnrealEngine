// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentSteeringHandler.h"
#include "Player/PlayerSessionServices.h"
#include "Utilities/StringHelpers.h"
#include "Utilities/URLParser.h"
#include "SynchronizedClock.h"
#include "ElectraPlayerMisc.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Math/NumericLimits.h"


namespace Electra
{

namespace
{
	static const TCHAR * const _HLS_pathway = TEXT("_HLS_pathway");
	static const TCHAR * const _HLS_throughput = TEXT("_HLS_throughput");

	static const TCHAR * const _DASH_pathway = TEXT("_DASH_pathway");
	static const TCHAR * const _DASH_throughput = TEXT("_DASH_throughput");

	static const int32 kDefaultTTL = 300;	// 5 minutes

	static const int32 kCDNBandwidthExpiration = 180;	// 3 minutes of not referencing a CDN removes it from the bandwidth list.
};


FContentSteeringHandler::FContentSteeringHandler(IPlayerSessionServices* InPlayerSessionServices)
	: RandomStream((int32)FPlatformTime::Cycles())
	, PlayerSessionService(InPlayerSessionServices)
{
}


FContentSteeringHandler::~FContentSteeringHandler()
{
}


bool FContentSteeringHandler::IsValidPathway(const FString& InPathway) const
{
	for(StringHelpers::FStringIterator It(InPathway); It; ++It)
	{
		if (!((*It >= TCHAR('a') && *It <= TCHAR('z')) ||
				(*It >= TCHAR('A') && *It <= TCHAR('Z')) ||
				(*It >= TCHAR('0') && *It <= TCHAR('9')) ||
				*It == TCHAR('.') || *It == TCHAR('-') || *It == TCHAR('_')))
		{
			return false;
		}
	}
	return true;
}



bool FContentSteeringHandler::InitialSetup(FContentSteeringHandler::EStreamingProtocol InStreamProtocol, const FContentSteeringHandler::FInitialParams& InInitialParams)
{
	if (bIsSetup)
	{
		return false;
	}
	FScopeLock lock(&Lock);
	bIsSetup = true;
	StreamingProtocol = InStreamProtocol;
	InitialParams = InInitialParams;

	// Check parameters
	/*
		The default CDN (pathway / serviceLocation) is to be a single name only.
		However, the steering server response will return a list and DASH will require this to be list if
		different periods require different CDNs. Since you can't know in which period playback will start
		it stands to reason that in order for this to function as intended the initial "list" will need to
		be a list and not just a single item.
		Since it is not really against the spec since one can argue that this is just an authoring issue,
		we allow this to be a comma or whitespace seperated list.
	*/
	InitialParams.InitialDefaultCDN.ParseIntoArrayWS(CurrentCDNPriorities, TEXT(","), true);
	if (!InitialParams.bAllowAnyPathwayNames)
	{
		for(auto& It : CurrentCDNPriorities)
		{
			if (!IsValidPathway(It))
			{
				UE_LOGF(LogElectraPlayer, Error, "Invalid initial pathway name \"%ls\"", *It);
				return false;
			}
		}
	}
	if (InitialParams.CustomFirstCDNPrioritization.Len())
	{
		/*
			Our custom attribute is a string giving a comma separated list of key=value pairs where each key
			is a PATHWAY-ID / serviceLocation and the value a probably for the indicated CDN to be selected initially.
			After the key-value list a semicolon may follow if additional attributes are specific.
			For now we allow for an additional attribute `locked` to indicate that CDN switching is not permitted
			and the initial choice is to remain for the duration of the playback.
			Example: cdn-a=10,cdn-b=5;locked
			   This gives "cdn-a" a probability twice as large as "cdn-b" to be randomly selected at play start
			   and that choice is to remain in place.
		*/
		TArray<FString> OptionParts;
		InitialParams.CustomFirstCDNPrioritization.ParseIntoArray(OptionParts, TEXT(";"), true);
		if (OptionParts.IsEmpty())
		{
			UE_LOGF(LogElectraPlayer, Error, "Invalid custom initial CDN selection parameters");
			return false;
		}
		TArray<FString> CDNChoices;
		OptionParts[0].ParseIntoArray(CDNChoices, TEXT(","), true);
		if (CDNChoices.IsEmpty())
		{
			UE_LOGF(LogElectraPlayer, Error, "Invalid custom initial CDN selection parameters");
			return false;
		}
		OptionParts.RemoveAt(0);
		int32 TotalProbability = 0;
		TArray<FInitialCDNChoice> InitialCustomCDNChoices;
		for(int32 i=0; i<CDNChoices.Num(); ++i)
		{
			TArray<FString> kv;
			CDNChoices[i].ParseIntoArray(kv, TEXT("="), true);
			if (kv.Num() != 2)
			{
				UE_LOGF(LogElectraPlayer, Error, "Invalid custom initial CDN selection parameters");
				return false;
			}
			int32 prb=-1;
			LexFromString(prb, *kv[1]);
			if (prb <= 0 || !IsValidPathway(kv[0]))
			{
				UE_LOGF(LogElectraPlayer, Error, "Invalid custom initial CDN selection parameters");
				return false;
			}
			InitialCustomCDNChoices.Emplace(FInitialCDNChoice({kv[0], prb}));
			TotalProbability += prb;
		}
		// Roll the dice which CDN to select.
		FString InitiallyChosenCDN;
		int32 DiceRoll = RandomStream.RandRange(0, TotalProbability-1);
		for(int32 i=0,s=0; i<InitialCustomCDNChoices.Num(); ++i)
		{
			s += InitialCustomCDNChoices[i].Probability;
			if (DiceRoll < s)
			{
				InitiallyChosenCDN = InitialCustomCDNChoices[i].CDN;
				break;
			}
		}
		check(InitiallyChosenCDN.Len());
		// Is this choice locked?
		bIsInitiallyChosenCDNLocked = OptionParts.Contains(TEXT("locked"));

		// Update the priority list accordingly.
		CurrentCDNPriorities.Remove(InitiallyChosenCDN);
		CurrentCDNPriorities.Insert(InitiallyChosenCDN, 0);
		CurrentlyChosenDVBCDNForType[0] = CurrentlyChosenDVBCDNForType[1] = InitiallyChosenCDN;
	}
	else if (StreamingProtocol == EStreamingProtocol::DASH && !InitialParams.bUseDVBPriorities && CurrentCDNPriorities.Num())
	{
		// No custom selection attribute on a regular (non-DVB) DASH should set the first CDN priority as the one to use
		// since we are going through the DVB selection anyway. We don't want the randomization though in this case.
		CurrentlyChosenDVBCDNForType[0] = CurrentlyChosenDVBCDNForType[1] = CurrentCDNPriorities[0];
	}
	RebuildAvailableCDNList();
	bIsConfigured = true;

	// Is a new manifest required immediately?
	if (StreamingProtocol == EStreamingProtocol::DASH || StreamingProtocol == EStreamingProtocol::HLS)
	{
		bNewManifestNeeded = InitialParams.bHasContentSteering && InitialParams.bQueryBeforeStart;
		bDoFirstUpdateOnStableBuffer = InitialParams.bHasContentSteering && !InitialParams.bQueryBeforeStart;
		if (InitialParams.FirstSteeringURL.Len())
		{
			FURL_RFC3986 ru;
			if (ru.Parse(InitialParams.FirstSteeringURL))
			{
				ru.ResolveAgainst(InitialParams.RootDocumentURL);
				NextServerRequestURL = ru.Get(true, false);
			}
		}
		ProxyURL = InitialParams.ProxyURL;
	}

	return true;
}



int64 FContentSteeringHandler::GetCurrentRequestID()
{
	FScopeLock lock(&Lock);
	return SteeringRequestID;
}

bool FContentSteeringHandler::NeedToObtainNewSteeringManifestNow()
{
	FScopeLock lock(&Lock);
	return bNewManifestNeeded && !bManifestRequestIsPending;
}

FString FContentSteeringHandler::GetBaseSteeringServerRequestURL()
{
	FScopeLock lock(&Lock);
	return NextServerRequestURL;
}

FString FContentSteeringHandler::GetFinalSteeringServerRequestURL(const FString& InBaseURL)
{
	FScopeLock lock(&Lock);
	if (InBaseURL.IsEmpty() || InBaseURL.StartsWith(TEXT("data:")))
	{
		return InBaseURL;
	}
	FString Url(InBaseURL);
	// Proxy?
	if (ProxyURL.Len())
	{
		/*
			Proxy use was introduced by DASH-IF CTS 00XX 19 V0.9.0 (2022-07) and is not
			in the newer ETSI TS 103 998 V1.1.1 (2024-01) document any more. Since supporting
			this is straightforward and there might still be a use for it we support it.
		*/
		FURL_RFC3986 pu;
		if (pu.Parse(ProxyURL))
		{
			// Using a proxy requires the actual server URL to be percent encoded and provided
			// to the proxy via a "url=" query parameter.
			FString encu;
			if (FURL_RFC3986::UrlEncode(encu, Url, FString()))
			{
				pu.AddOrUpdateQueryParams(FString::Printf(TEXT("url=%s"), *encu));
				Url = pu.Get(true, false);
			}
		}
	}


	FURL_RFC3986 su;
	if (!su.Parse(Url))
	{
		return FString();
	}

	FString SteeringParams;
	// If this is the first request the query parameters to append to the request will be different.
	if (bIsFirstSteeringRequest)
	{
		// For DASH - at least according to ETSI TS 103 998 V1.1.1 (2024-01) - the first request
		// must not include any additional parameters.
		if (StreamingProtocol == EStreamingProtocol::DASH)
		{
			// ...
		}
		else if (StreamingProtocol == EStreamingProtocol::HLS)
		{
			FString pw(CurrentlySelectedHLSPathway);
			if (pw.IsEmpty() && CurrentCDNPriorities.Num())
			{
				pw = CurrentCDNPriorities[0];
			}
			SteeringParams = FString::Printf(TEXT("%s=\"%s\""), _HLS_pathway, *pw);
		}
	}
	else
	{
		if (StreamingProtocol == EStreamingProtocol::DASH)
		{
			FString pw, tp;
			for(int32 i=0; i<ReferencedCDNsSinceLastUpdate.Num(); ++i)
			{
				if (i)
				{
					pw.Append(TEXT(","));
					tp.Append(TEXT(","));
				}
				pw.Append(ReferencedCDNsSinceLastUpdate[i]);
				int64 bw = ObservedBandwidths.Contains(ReferencedCDNsSinceLastUpdate[i]) ? ObservedBandwidths[ReferencedCDNsSinceLastUpdate[i]].Bandwidth.GetSMA() : 0;
				tp.Append(FString::Printf(TEXT("%lld"), (long long int)bw));
			}
			if (pw.Len())
			{
				SteeringParams = FString::Printf(TEXT("%s=\"%s\"&%s=%s"), _DASH_pathway, *pw, _DASH_throughput, *tp);
			}
		}
		else if (StreamingProtocol == EStreamingProtocol::HLS)
		{
			int64 bw = ObservedBandwidths.Contains(CurrentlySelectedHLSPathway) ? ObservedBandwidths[CurrentlySelectedHLSPathway].Bandwidth.GetSMA() : 0;
			if (bw > 0)
			{
				SteeringParams = FString::Printf(TEXT("%s=\"%s\"&%s=%lld"), _HLS_pathway, *CurrentlySelectedHLSPathway, _HLS_throughput, (long long int)bw);
			}
			else
			{
				SteeringParams = FString::Printf(TEXT("%s=\"%s\""), _HLS_pathway, *CurrentlySelectedHLSPathway);
			}
		}

		if (SteeringParams.Len())
		{
			FString esc;
			FURL_RFC3986::UrlEncode(esc, SteeringParams, FURL_RFC3986::GetUrlEncodeSubDelimsChars());
			SteeringParams = MoveTemp(esc);
		}
	}
	su.AddOrUpdateQueryParams(SteeringParams);
	Url = su.Get(true, false);
	return Url;
}


void FContentSteeringHandler::SetSteeringServerRequestIsPending()
{
	FScopeLock lock(&Lock);
	bManifestRequestIsPending = true;
	bDoFirstUpdateOnStableBuffer = false;
	bNewManifestNeeded = false;
}

void FContentSteeringHandler::UpdateWithSteeringServerResponse(const FString& InResponse, int32 InHTTPStatusCode, const TArray<HTTP::FHTTPHeader>& InResponseHeaders)
{
	FScopeLock lock(&Lock);
	bManifestRequestIsPending = false;
	// Presumably successful?
	if (!InResponse.IsEmpty() && InHTTPStatusCode == 200)
	{
		TSharedPtr<FJsonObject> SteerParams;
		bool bStopAllUpdates = false;
		bool bIsValid = FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(InResponse), SteerParams);
		if (bIsValid)
		{
			int32 VERSION = 0;
			int32 TTL = kDefaultTTL;
			if (!SteerParams->TryGetNumberField(TEXT("VERSION"), VERSION))
			{
				UE_LOGF(LogElectraPlayer, Error, "Steering manifest is missing mandatory VERSION field");
				bIsValid = false;
			}
			if (VERSION == 1)
			{
				if (!SteerParams->TryGetNumberField(TEXT("TTL"), TTL))
				{
					UE_LOGF(LogElectraPlayer, Error, "Steering manifest is missing mandatory TTL field");
					// The earlier DASH spec had TTL as OD(300) instead of this being mandatory.
					// We emit the warning, but allow it being absent using the 300s default value set above.
					if (StreamingProtocol != EStreamingProtocol::DASH)
					{
						bIsValid = false;
					}
				}
				if (TTL > 0)
				{
					LastTTL = TTL;
					TimeForNextUpdate = PlayerSessionService->GetSynchronizedUTCTime()->GetTime() + FTimeValue((double)TTL, 0);
				}
				else
				{
					UE_LOGF(LogElectraPlayer, Error, "Steering manifest has bad TTL field of %d seconds", TTL);
					bIsValid = false;
				}

				FString RELOAD_URI;
				SteerParams->TryGetStringField(TEXT("RELOAD-URI"), RELOAD_URI);

				// Update the reload URL if one is given.
				if (!RELOAD_URI.IsEmpty())
				{
					// When not using a proxy we update the actual request URL.
					if (ProxyURL.IsEmpty())
					{
						FURL_RFC3986 ru;
						if (ru.Parse(RELOAD_URI))
						{
							ru.ResolveAgainst(NextServerRequestURL);
							NextServerRequestURL = ru.Get(true, false);
						}
					}
					// Otherwise the reload URL is supposedly another proxy URL
					else
					{
						ProxyURL = RELOAD_URI;
					}
				}

				const TArray<TSharedPtr<FJsonValue>>* PATHWAY_PRIORITIES = nullptr;
				// Try getting the pathway priority array.
				// In the earlies DASH spec this had a different field name, so if the proper one is absent try the legacy name.
				if ((SteerParams->TryGetArrayField(TEXT("PATHWAY-PRIORITY"), PATHWAY_PRIORITIES)) ||
					(StreamingProtocol == EStreamingProtocol::DASH && SteerParams->TryGetArrayField(TEXT("SERVICE-LOCATION-PRIORITY"), PATHWAY_PRIORITIES)))
				{
					TArray<FString> NewPriorities;
					for(auto& pwpri : *PATHWAY_PRIORITIES)
					{
						FString pw;
						if (pwpri->TryGetString(pw))
						{
							NewPriorities.Emplace(MoveTemp(pw));
						}
					}
					// If there are no pathways given we keep using the current ones.
					if (NewPriorities.Num())
					{
						CurrentCDNPriorities = MoveTemp(NewPriorities);
					}
				}
				// In HLS the PATHWAY-PRIORITY is mandatory while it's optional in DASH.
				else if (StreamingProtocol == EStreamingProtocol::HLS)
				{
					UE_LOGF(LogElectraPlayer, Error, "Steering manifest is missing mandatory PATHWAY-PRIORITY field");
					bIsValid = false;
				}

				const TArray<TSharedPtr<FJsonValue>>* PATHWAY_CLONES = nullptr;
				// Try getting the pathway clones array.
				if (SteerParams->TryGetArrayField(TEXT("PATHWAY-CLONES"), PATHWAY_CLONES))
				{
					for(auto& pwclnv : *PATHWAY_CLONES)
					{
						const TSharedPtr<FJsonObject>* pwcln;
						if (!pwclnv->TryGetObject(pwcln))
						{
							UE_LOGF(LogElectraPlayer, Error, "Steering manifest PATHWAY-CLONES array element is not an object");
							bIsValid = false;
							continue;
						}

						FPathwayCloneEntry CloneEntry;

						if (!(*pwcln)->TryGetStringField(TEXT("BASE-ID"), CloneEntry.BaseId))
						{
							UE_LOGF(LogElectraPlayer, Error, "Steering manifest is missing mandatory BASE-ID field in PATHWAY-CLONES");
							bIsValid = false;
							continue;
						}
						if (!(*pwcln)->TryGetStringField(TEXT("ID"), CloneEntry.Id))
						{
							UE_LOGF(LogElectraPlayer, Error, "Steering manifest is missing mandatory ID field in PATHWAY-CLONES");
							bIsValid = false;
							continue;
						}
						const TSharedPtr<FJsonObject>* URI_REPLACEMENT;
						if (!(*pwcln)->TryGetObjectField(TEXT("URI-REPLACEMENT"), URI_REPLACEMENT))
						{
							UE_LOGF(LogElectraPlayer, Error, "Steering manifest is missing mandatory URI-REPLACEMENT field in PATHWAY-CLONES");
							bIsValid = false;
							continue;
						}
						// Get the optional HOST element
						(*URI_REPLACEMENT)->TryGetStringField(TEXT("HOST"), CloneEntry.Host);
						// Get PARAMS, if any
						const TSharedPtr<FJsonObject>* PARAMS;
						if ((*URI_REPLACEMENT)->TryGetObjectField(TEXT("PARAMS"), PARAMS))
						{
							for(auto& qp : (*PARAMS)->Values)
							{
								FString v;
								if (qp.Value->TryGetString(v))
								{
									FURL_RFC3986::FQueryParam& qpv(CloneEntry.Params.Emplace_GetRef());
									qpv.Name = qp.Key;
									qpv.Value = v;
								}
							}
						}
						// PER-VARIANT-URIS
						const TSharedPtr<FJsonObject>* PER_VARIANT_URIS;
						if ((*URI_REPLACEMENT)->TryGetObjectField(TEXT("PER-VARIANT-URIS"), PER_VARIANT_URIS))
						{
							for(auto& pvu : (*PER_VARIANT_URIS)->Values)
							{
								FString v;
								if (pvu.Value->TryGetString(v))
								{
									CloneEntry.PerVariantURIs.Emplace(pvu.Key, v);
								}
							}
						}
						// PER-RENDITION-URIS
						const TSharedPtr<FJsonObject>* PER_RENDITION_URIS;
						if ((*URI_REPLACEMENT)->TryGetObjectField(TEXT("PER-RENDITION-URIS"), PER_RENDITION_URIS))
						{
							for(auto& pvu : (*PER_RENDITION_URIS)->Values)
							{
								FString v;
								if (pvu.Value->TryGetString(v))
								{
									CloneEntry.PerRenditionURIs.Emplace(pvu.Key, v);
								}
							}
						}
						// Remove any clone already in the table for the updated one.
						CurrentCloneEntries.RemoveAll([that=CloneEntry.Id](const FPathwayCloneEntry& InClone){return InClone.Id.Equals(that);});
						// Add the clone entry to the current clone list unless the clone was already created earlier.
						if (!AlreadyClonePathways.Contains(CloneEntry.Id))
						{
							CurrentCloneEntries.Emplace(MoveTemp(CloneEntry));
						}
					}
				}

				RebuildAvailableCDNList();
			}
			else
			{
				if (VERSION > 1)
				{
					UE_LOGF(LogElectraPlayer, Error, "Steering manifest VERSION %d is not yet understood", VERSION);
				}
				bIsValid = false;
				bStopAllUpdates = false;
			}
		}

		if (bStopAllUpdates)
		{
			TimeForNextUpdate.SetToPositiveInfinity();
		}
		bIsFirstSteeringRequest = false;
		++SteeringRequestID;
		bNeedUpdateOfReferencedList = true;
	}
	// Gone?
	else if (InHTTPStatusCode == 410)
	{
		TimeForNextUpdate.SetToPositiveInfinity();
	}
	// Too many requests?
	else if (InHTTPStatusCode == 429)
	{
		int32 TryAgainAfterSeconds = -1;
		// Is there a "Retry-After" header telling us when to try again?
		for(auto& hdr : InResponseHeaders)
		{
			// Check all headers in cast there is more than one, giving the result in different formats.
			if (hdr.Header.Equals(TEXT("Retry-After"), ESearchCase::IgnoreCase))
			{
				// There are two possible formats. A simple positive integer giving seconds
				// after which to try again OR a "Date"-style string specifying a fixed date.
				int32 Delay = -1;
				LexFromString(Delay, *hdr.Value);
				if (Delay > 0)
				{
					TryAgainAfterSeconds = Delay;
					break;
				}
			}
		}
		if (TryAgainAfterSeconds > 0)
		{
			TimeForNextUpdate = PlayerSessionService->GetSynchronizedUTCTime()->GetTime() + FTimeValue((double)TryAgainAfterSeconds, 0);
		}
	}
	else
	{
		// Any other kind of error lets us stay with the current selection and issue another request
		// after the previous TTL interval.
		int32 TTL = LastTTL > 0 ? LastTTL : kDefaultTTL;
		TimeForNextUpdate = PlayerSessionService->GetSynchronizedUTCTime()->GetTime() + FTimeValue((double)TTL, 0);
	}
}


void FContentSteeringHandler::RebuildAvailableCDNList()
{
	AvailableCDNs = CurrentCDNPriorities;
	CDNPriorityDisplay.Empty();

	for(int32 i=0; i<AvailableCDNs.Num(); ++i)
	{
		if (PenalizedCDNList.ContainsByPredicate([cdn=AvailableCDNs[i]](const FPenalizedCDN& pen) { return cdn.Equals(pen.CDN); }))
		{
			AvailableCDNs.RemoveAt(i);
			--i;
			continue;
		}
		// Add the CDN to the list for (debug) display.
		if (i)
		{
			CDNPriorityDisplay.Append(TEXT(","));
		}
		CDNPriorityDisplay.Append(AvailableCDNs[i]);
	}
}


void FContentSteeringHandler::ReachedStableBuffer()
{
	FScopeLock lock(&Lock);
	if (!bIsSetup || !bIsConfigured)
	{
		return;
	}

	// If the first request was not made on startup we need to perform an update
	// now unless there is already one pending.
	if (bDoFirstUpdateOnStableBuffer)
	{
		bDoFirstUpdateOnStableBuffer = false;
		if (!bNewManifestNeeded && !bManifestRequestIsPending)
		{
			bNewManifestNeeded = true;
		}
	}
}

void FContentSteeringHandler::PeriodicHandle()
{
	FScopeLock lock(&Lock);
	if (!bIsSetup || !bIsConfigured)
	{
		return;
	}

	FTimeValue Now = PlayerSessionService->GetSynchronizedUTCTime()->GetTime();

	// If real content steering is in use see if it needs to be refreshed.
	if (InitialParams.bHasContentSteering)
	{
		if (TimeForNextUpdate.IsValid() && Now > TimeForNextUpdate)
		{
			TimeForNextUpdate.SetToInvalid();
			bNewManifestNeeded = true;
		}
	}

// TODO: clear penalized cdns
	//RebuildAvailableCDNList();


	// Expire bandwidth measurements that have no relevance any more
	TArray<FString> OutdatedCDNBWs;
	for(auto& cdnbwIt : ObservedBandwidths)
	{
		if (!ReferencedCDNsSinceLastUpdate.Contains(cdnbwIt.Key) && Now > cdnbwIt.Value.ExpiresAt)
		{
			OutdatedCDNBWs.Emplace(cdnbwIt.Key);
		}
	}
	for(auto& odcdn : OutdatedCDNBWs)
	{
		ObservedBandwidths.Remove(odcdn);
	}
}


void FContentSteeringHandler::FinishedDownloadRequestOn(const Metrics::FSegmentDownloadStats& InDownloadStats, const FContentSteeringHandler::FStreamParams& InStreamParams)
{
	FScopeLock lock(&Lock);
	// When a download has completed the steering server request will not be the first one any more.
	bIsFirstSteeringRequest = false;

	// If the download was started before the most recent steering update we do not track it.
	if ((StreamingProtocol == EStreamingProtocol::DASH && InDownloadStats.SteeringID < SteeringRequestID) || InDownloadStats.URL.CDN.IsEmpty())
	{
		return;
	}

	// Clear the list of recently referenced CDNs now if a steering update has occurred.
	// We do this here and not when receiving the update so that the current list remains
	// valid in case steering updates are performed while the player is paused to avoid
	// the list being emptied then.
	if (bNeedUpdateOfReferencedList)
	{
		bNeedUpdateOfReferencedList = false;
		ReferencedCDNsSinceLastUpdate.Empty();
	}

	// Add to the list of CDNs that have been referenced since the last steering update.
	ReferencedCDNsSinceLastUpdate.AddUnique(InDownloadStats.URL.CDN);

	// Bandwidth observation only needs to be made for actual content steering
	if (InitialParams.bHasContentSteering)
	{
		bool bUseBW = false;
		if ((InDownloadStats.StreamType == EStreamType::Video && InStreamParams.bActiveVideo) ||
			(InDownloadStats.StreamType == EStreamType::Audio && !InStreamParams.bActiveVideo && InStreamParams.bActiveAudio))
		{
			if (InDownloadStats.bWasSuccessful && !InDownloadStats.bIsCachedResponse && InDownloadStats.SegmentType == Metrics::ESegmentType::Media)
			{
				bUseBW = true;
			}
		}

		if (bUseBW)
		{
			const double ttdl = InDownloadStats.TimeToDownload - InDownloadStats.TimeToFirstByte;
			if (ttdl > 0.0)
			{
				if (!ObservedBandwidths.Contains(InDownloadStats.URL.CDN))
				{
					FCDNThroughput& cdnThru = ObservedBandwidths.Add(InDownloadStats.URL.CDN);
					cdnThru.StreamType = InDownloadStats.StreamType;
					cdnThru.CDN = InDownloadStats.URL.CDN;
					//cdnThru.Bandwidth.Resize(5);
				}
				FCDNThroughput& cdnThru = ObservedBandwidths[InDownloadStats.URL.CDN];
				cdnThru.ExpiresAt = PlayerSessionService->GetSynchronizedUTCTime()->GetTime() + FTimeValue((double)kCDNBandwidthExpiration);
				int64 DlBps = (int64)(InDownloadStats.NumBytesDownloaded * 8 / ttdl);
				int64 tpSoFar = cdnThru.Bandwidth.GetSMA();
				// If we have an average don't let the new value be excessively larger to avoid adding spikes due to odd transfers.
				const int32 kPrvThrs = 3;
				DlBps = tpSoFar > 100000 && DlBps > tpSoFar * kPrvThrs ? tpSoFar * kPrvThrs : DlBps;
				cdnThru.Bandwidth.AddValue(DlBps);
			}
		}
	}

	// Build a string for display of the CDNs that were accessed since the last steering update.
	CDNAccessDisplay.Empty();
	for(int32 i=0; i<ReferencedCDNsSinceLastUpdate.Num(); ++i)
	{
		if (i)
		{
			CDNAccessDisplay.Append(TEXT(","));
		}
		CDNAccessDisplay.Append(ReferencedCDNsSinceLastUpdate[i]);
		if (ObservedBandwidths.Contains(ReferencedCDNsSinceLastUpdate[i]))
		{
			CDNAccessDisplay.Append(FString::Printf(TEXT(" (%" INT64_FMT " Kbps)"), ObservedBandwidths[ReferencedCDNsSinceLastUpdate[i]].Bandwidth.GetSMA() / 1000));
		}
	}
}


FString FContentSteeringHandler::GetCurrentCDNListForDisplay()
{
	FScopeLock lock(&Lock);
	return CDNPriorityDisplay;
}

FString FContentSteeringHandler::GetPenalizedCDNListForDisplay()
{
	FScopeLock lock(&Lock);
// Penalizing a CDN is not implemented yet, so nothing to return here.
	return FString();
}

FString FContentSteeringHandler::GetRecentlyAccessedCDNListForDisplay()
{
	FScopeLock lock(&Lock);
	return CDNAccessDisplay;
}


void FContentSteeringHandler::PenalizeCDN(const FString& InCDN, int32 InDVBPriority, int32 InForSeconds)
{
	FScopeLock lock(&Lock);
	check(!"not implemented yet");
	// Check if the CDN choice is locked in which case we must not switch away.
}

FContentSteeringHandler::FSelectedCandidateURL FContentSteeringHandler::SelectBestCandidateFrom(FString& OutMessage, ESelectFor InForType, const TArray<FContentSteeringHandler::FCandidateURL>& InFromCandidates)
{
	if (InFromCandidates.IsEmpty())
	{
		OutMessage = FString::Printf(TEXT("No candidates provided"));
		return FSelectedCandidateURL();
	}
	switch(StreamingProtocol)
	{
		case EStreamingProtocol::HLS:
		{
			return SelectBestHLSCandidateFrom(OutMessage, InForType, InFromCandidates);
		}
		case EStreamingProtocol::DASH:
		{
			return !InitialParams.bUseDVBPriorities ? SelectBestDASHCandidateFrom(OutMessage, InForType, InFromCandidates) : SelectBestDVBDASHCandidateFrom(OutMessage, InForType, InFromCandidates);
		}
		default:
		{
			return FSelectedCandidateURL(InFromCandidates[0], SteeringRequestID, false);
		}
	}
}

void FContentSteeringHandler::SetCurrentlyActivePathway(const FString& InCurrentPathway)
{
	FScopeLock lock(&Lock);
	CurrentlySelectedHLSPathway = InCurrentPathway;
}

TArray<FContentSteeringHandler::FPathwayCloneEntry> FContentSteeringHandler::GetCurrentCloneEntries()
{
	FScopeLock lock(&Lock);
	return CurrentCloneEntries;
}

void FContentSteeringHandler::CreatedClone(const FString& InClonedPathwayId)
{
	FScopeLock lock(&Lock);
	AlreadyClonePathways.AddUnique(InClonedPathwayId);
	CurrentCloneEntries.RemoveAll([&](const FPathwayCloneEntry& ce){return InClonedPathwayId.Equals(ce.Id);});
}


void FContentSteeringHandler::AddDynamicDASHClonesToCandidateList(TArray<FSelectedCandidateURL>& InOutFromCandidates)
{
	// Go over the list of clones. If the clone does not appear in the list of candidates we have to create it.
	// Otherwise we let the explicitly given candidate win assuming it is more current due to an MPD update
	// than the clone information we have.
	for(int32 nC=0; nC<CurrentCloneEntries.Num(); ++nC)
	{
		if (InOutFromCandidates.ContainsByPredicate([cloneId=CurrentCloneEntries[nC].Id](const FSelectedCandidateURL& InCand){return cloneId.Equals(InCand.MediaURL.CDN);}))
		{
			continue;
		}
		// Clone if possible. For this we need to have the reference in the candidate list.
		// Note: A clone may reference another clone but only if that had already been resolved. This is specified
		//       in the HLS and DASH specification. Likewise, if the base to clone from is not known the clone is ignored.
		for(auto& Base : InOutFromCandidates)
		{
			if (CurrentCloneEntries[nC].BaseId.Equals(Base.MediaURL.CDN))
			{
				FURL_RFC3986 baseURL;
				if (baseURL.Parse(Base.MediaURL.URL))
				{
					// New Host?
					if (CurrentCloneEntries[nC].Host.Len())
					{
						baseURL.SetHost(CurrentCloneEntries[nC].Host);
					}
					// New/changed query parameters?
					if (CurrentCloneEntries[nC].Params.Num())
					{
						baseURL.AddOrUpdateQueryParams(CurrentCloneEntries[nC].Params);
					}
					// Add the clone to the end of the list so it itself becomes eligible for cloning.
					FSelectedCandidateURL& cu = InOutFromCandidates.Emplace_GetRef(Base);
					cu.MediaURL.CDN = CurrentCloneEntries[nC].Id;
					cu.MediaURL.URL = baseURL.Get();
					cu.SteeringID = SteeringRequestID;
					cu.bWasDynamicallyCloned = true;
				}
				break;
			}
		}
	}
}

FContentSteeringHandler::FSelectedCandidateURL FContentSteeringHandler::SelectBestDASHCandidateFrom(FString& OutMessage, ESelectFor InForType, const TArray<FCandidateURL>& InFromCandidates)
{
	FScopeLock lock(&Lock);
	// If this is a regular DASH MPD without content steering and without DVB DASH baseURL properties
	// we can still treat it as DVB DASH due to the necessary attributes being defaulted to usable values.
	if (!InitialParams.bHasContentSteering)
	{
		return SelectBestDVBDASHCandidateFrom(OutMessage, InForType, InFromCandidates);
	}

	// Create a copy of the input so we can modify it by adding dynamically created clones.
	TArray<FSelectedCandidateURL> Candidates;
	for(auto& cand : InFromCandidates)
	{
		Candidates.Emplace(FSelectedCandidateURL(cand, SteeringRequestID, false));
	}
	AddDynamicDASHClonesToCandidateList(Candidates);

	// Go over each available (non-penalized) CDN in the priority list and see if there is a matching candidate.
	// If so, return it since we are done.
	for(auto& av : AvailableCDNs)
	{
		for(auto& cand : Candidates)
		{
			if (av.Equals(cand.MediaURL.CDN))
			{
				SteeringRequestIDWhenNoCandidatesMatched.Reset();
				return cand;
			}
		}
	}

	// No candidate is in the current priority list.

	// Check if we had this problem before and still have it with an updated steering manifest.
	// In that case - according to ETSI TS 103 998 V1.1.1 (2024-01) Section 7 14) c) - we are to fail.
	if (SteeringRequestIDWhenNoCandidatesMatched.IsSet() && SteeringRequestID > SteeringRequestIDWhenNoCandidatesMatched.GetValue())
	{
		OutMessage = FString::Printf(TEXT("Still no candidate representation found on any priority pathway after steering manifest update."));
		return FSelectedCandidateURL();
	}
	// Take note of the fact that we had to resort to using a CDN that was not on the priority list.
	SteeringRequestIDWhenNoCandidatesMatched = SteeringRequestID;

	// Go over the candidates and pick one that is not on the penalty list.
	for(auto& cand : Candidates)
	{
		if (!PenalizedCDNList.ContainsByPredicate([candCDN=cand.MediaURL.CDN](const FPenalizedCDN& InPenCDN){return InPenCDN.CDN.Equals(candCDN);}))
		{
			return cand;
		}
	}
	// No candidate is viable. Fail now.
	OutMessage = FString::Printf(TEXT("All candidate representations are penalized or not on the priority list."));
	return FSelectedCandidateURL();
}

FContentSteeringHandler::FSelectedCandidateURL FContentSteeringHandler::SelectBestDVBDASHCandidateFrom(FString& OutMessage, ESelectFor InForType, const TArray<FCandidateURL>& InFromCandidates)
{
	FScopeLock lock(&Lock);

	// Copy the list so we can modify it.
	TArray<FCandidateURL> Candidates(InFromCandidates);
	// Remove all candidates that are on the penalty list.
	for(auto& pencdn : PenalizedCDNList)
	{
		Candidates.RemoveAll([cdn=pencdn.CDN,pri=pencdn.DVBpriority](const FCandidateURL& InCandidate){ return InCandidate.MediaURL.DVBpriority == pri || InCandidate.MediaURL.CDN.Equals(cdn); });
	}
	if (Candidates.IsEmpty())
	{
		OutMessage = FString::Printf(TEXT("All candidate representations are penalized."));
		return FSelectedCandidateURL();
	}
	// Find the lowest priority value of the remaining candidates.
	int64 LowestPri = TNumericLimits<int64>::Max();
	for(auto& cand : Candidates)
	{
		LowestPri = cand.MediaURL.DVBpriority < LowestPri ? cand.MediaURL.DVBpriority : LowestPri;
	}
	// Remove the candidates having a larger value
	Candidates.RemoveAll([LowestPri](const FCandidateURL& InCandidate){ return InCandidate.MediaURL.DVBpriority > LowestPri; });

	// See if we have the last chosen CDN still among the candidates.
	const int32 TypIdx = InForType == ESelectFor::Playlist ? 0 : 1;
	if (!CurrentlyChosenDVBCDNForType[TypIdx].IsEmpty())
	{
		if (const FCandidateURL* Best = Candidates.FindByPredicate([chosen=CurrentlyChosenDVBCDNForType[TypIdx]](const FCandidateURL& InCandidate){ return InCandidate.MediaURL.CDN.Equals(chosen);}))
		{
			return FSelectedCandidateURL(*Best, SteeringRequestID, false);
		}
		// Not found, clear previous choice.
		CurrentlyChosenDVBCDNForType[TypIdx].Empty();
	}

	// Get the total probability of the candidates
	int32 TotalProbability = 0;
	for(auto& cand : Candidates)
	{
		TotalProbability += cand.MediaURL.DVBweight;
	}
	// Roll the dice which CDN to select.
	int32 DiceRoll = RandomStream.RandRange(0, TotalProbability-1);
	for(int32 i=0,s=0; i<Candidates.Num(); ++i)
	{
		s += Candidates[i].MediaURL.DVBweight;
		if (DiceRoll < s)
		{
			CurrentlyChosenDVBCDNForType[TypIdx] = Candidates[i].MediaURL.CDN;
			return FSelectedCandidateURL(Candidates[i], SteeringRequestID, false);
		}
	}
	OutMessage = FString::Printf(TEXT("Failed to roll the dice."));
	return FSelectedCandidateURL();
}

FContentSteeringHandler::FSelectedCandidateURL FContentSteeringHandler::SelectBestHLSCandidateFrom(FString& OutMessage, ESelectFor InForType, const TArray<FCandidateURL>& InFromCandidates)
{
	FScopeLock lock(&Lock);

	// Go over each available (non-penalized) CDN in the priority list and see if there is a matching candidate.
	// If so, return it since we are done.
	for(auto& av : AvailableCDNs)
	{
		for(auto& cand : InFromCandidates)
		{
			if (av.Equals(cand.MediaURL.CDN))
			{
				return FSelectedCandidateURL(cand, SteeringRequestID, false);
			}
		}
	}

	// No candidate is in the current priority list.
	// Go over the candidates and pick one that is not on the penalty list.
	for(auto& cand : InFromCandidates)
	{
		if (!PenalizedCDNList.ContainsByPredicate([candCDN=cand.MediaURL.CDN](const FPenalizedCDN& InPenCDN){return InPenCDN.CDN.Equals(candCDN);}))
		{
			return FSelectedCandidateURL(cand, SteeringRequestID, false);
		}
	}
	// No candidate is viable. Fail now.
	OutMessage = FString::Printf(TEXT("All candidate representations are penalized or not on the priority list."));
	return FSelectedCandidateURL();
}


} // namespace Electra
