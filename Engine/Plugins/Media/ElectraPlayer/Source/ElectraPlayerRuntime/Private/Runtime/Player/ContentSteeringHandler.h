// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "PlayerCore.h"
#include "HTTP/HTTPManager.h"
#include "MediaURLType.h"
#include "ParameterDictionary.h"
#include "Utilities/URLParser.h"
#include "StreamTypes.h"
#include "Player/AdaptiveStreamingPlayerMetrics.h"
#include "Player/ABRRules/ABRStatisticTypes.h"
#include "Math/RandomStream.h"


namespace Electra
{

class IPlayerSessionServices;

class FContentSteeringHandler
{
public:
	FContentSteeringHandler(IPlayerSessionServices* InPlayerSessionServices);
	virtual ~FContentSteeringHandler();

	enum class EStreamingProtocol
	{
		HLS,
		DASH,
		Other
	};
	struct FInitialParams
	{
		// The URL of the main playlist in case the first steering manifest URL is relative.
		FString RootDocumentURL;
		// First URL to request a steering manifest from
		FString FirstSteeringURL;
		// The DASH @defaultServiceLocation or HLS PATHWAY-ID attribute provided in the first playlist.
		FString InitialDefaultCDN;
		// Used with DASH content steering only, the DASH-IF proposed @proxyServerURL attribute.
		FString ProxyURL;
		// A custom attribute with priority values to randomize the first CDN to use.
		FString CustomFirstCDNPrioritization;
		// Whether or not the steering server must be contacted before requesting anything else.
		// This is not relevant to the operation of this handler and purely informational.
		bool bQueryBeforeStart = false;
		// True if actual content steering is conveyed in the playlist. False if we use custom prioritization.
		bool bHasContentSteering = false;
		// For use with DASH only, if the MPD is using a DVB profile and CDN selection should be made
		// according to the <BaseURL>@priority/@weight attributes.
		// If content steering is explicitly enabled (`FirstSteeringURL` is not empty) the DVB method will not be used.
		bool bUseDVBPriorities = false;
		// True to not validate the pathway names so synthesized ones can be used.
		bool bAllowAnyPathwayNames = false;
	};

	/**
	 * Perform initial setup according to parameters provided in the main playlist.
	 * Can be done just once.
	 * Returns true if successful, false if already configured.
	 */
	bool InitialSetup(EStreamingProtocol InStreamProtocol, const FInitialParams& InInitialParams);

	/**
	 * Returns whether or not content steering was initialized to handle DVB DASH or not.
	 */
	bool IsDVBDash() const
	{ return InitialParams.bUseDVBPriorities; }

	/**
	 * Returns the current internal request ID to associate the next steered HTTP request with.
	 * This helps track of actively used CDNs when reporting back where a completed request was made on.
	 */
	int64 GetCurrentRequestID();

	/**
	 * To be called on every steered completed HTTP request.
	 * This tracks the CDN being used and the throughput observed.
	 */
	struct FStreamParams
	{
		bool bActiveVideo = false;
		bool bActiveAudio = false;
		bool bActiveSubtitles = false;
	};
	void FinishedDownloadRequestOn(const Metrics::FSegmentDownloadStats& InDownloadStats, const FStreamParams& InStreamParams);

	/**
	 * Call this periodically to check if a new steering manifest needs to be retrieved.
	 * Returns true if one is needed, false if not.
	 */
	bool NeedToObtainNewSteeringManifestNow();

	/**
	 * Returns the URL to perform the next steering manifest request against.
	 * This returns the base URL only without any query parameters applied.
	 * You may append query parameters of your own on this URL before calling
	 * GetFinalSteeringServerRequestURL() to obtain the final URL to perform
	 * the request against.
	 */
	FString GetBaseSteeringServerRequestURL();

	/**
	 * Call this to prepare the base URL you got from GetBaseSteeringServerRequestURL()
	 * to which you may have appended your own query parameters with the query parameters
	 * necessary to make the steering server manifest request.
	 */
	FString GetFinalSteeringServerRequestURL(const FString& InBaseURL);

	/**
	 * Call this to set a flag that you are performing a steering manifest retrieval.
	 * This will temporarily cause `NeedToObtainNewSteeringManifestNow()` to return false
	 * to avoid making repeated requests.
	 * This internal flag is cleared on the next `UpdateWithSteeringServerResponse()` call.
	 */
	void SetSteeringServerRequestIsPending();

	/**
	 * Call this to provide the steering server response.
	 */
	void UpdateWithSteeringServerResponse(const FString& InResponse, int32 InHTTPStatusCode, const TArray<HTTP::FHTTPHeader>& InResponseHeaders);

	/**
	 * To be called when the player has reached a stable buffer for the first time.
	 * This is used when the steering manifest is not queried before start in which case it is to
	 * be queried when the player buffers have become stable.
	 */
	void ReachedStableBuffer();

	/**
	 * Call this every so often to handle internal state.
	 */
	void PeriodicHandle();

	void PenalizeCDN(const FString& InCDN, int32 InDVBPriority, int32 InForSeconds);

	/**
	 * Input candidate for selection.
	 */
	struct FCandidateURL
	{
		FMediaURL MediaURL;
		FParamDict AdditionalParams;
	};

	/**
	 * Output candidate from selection.
	 */
	struct FSelectedCandidateURL : FCandidateURL
	{
		FSelectedCandidateURL() = default;
		FSelectedCandidateURL(const FCandidateURL& InCandidate, int64 InSteeringID, bool bInWasCloned)
			: FCandidateURL(InCandidate), SteeringID(InSteeringID), bWasDynamicallyCloned(bInWasCloned)
		{ }
		// ID of the steering manifest at the time of selection.
		int64 SteeringID = 0;
		// Indicates if this candidate has been dynamically cloned.
		bool bWasDynamicallyCloned = false;
	};
	enum class ESelectFor
	{
		Playlist,
		Segment
	};
	FSelectedCandidateURL SelectBestCandidateFrom(FString& OutMessage, ESelectFor InForType, const TArray<FCandidateURL>& InFromCandidates);

	/**
	 * Used by HLS to select the chosen pathway.
	 */
	void SetCurrentlyActivePathway(const FString& InCurrentPathway);


	/**
	 * Structure to create a clone from an existing pathway.
	 */
	struct FPathwayCloneEntry
	{
		FString BaseId;
		FString Id;
		FString Host;									// new hostname
		TArray<FURL_RFC3986::FQueryParam> Params;		// new query parameters
		TMap<FString, FString> PerVariantURIs;
		TMap<FString, FString> PerRenditionURIs;
	};
	// Returns the current list of to-be-cloned pathways.
	TArray<FPathwayCloneEntry> GetCurrentCloneEntries();
	/**
	 * Notifies that a clone has been created. This removes the entry from the clone list and
	 * prevents it from being added to the list again in future steering manifest updates
	 * since we assume a clone can be created only once.
	 */
	void CreatedClone(const FString& InClonedPathwayId);


	/**
	 * Returns the current list of prioritized CDN pathways to be used.
	 * Penalized CDNs are not included in this list.
	 */
	FString GetCurrentCDNListForDisplay();
	FString GetPenalizedCDNListForDisplay();
	FString GetRecentlyAccessedCDNListForDisplay();

private:
	struct FInitialCDNChoice
	{
		FString CDN;
		int32 Probability = 0;
	};

	struct FPenalizedCDN
	{
		FString CDN;
		FTimeValue Until;
		int32 DVBpriority = 0;
	};

	struct FCDNThroughput
	{
		TSimpleMovingAverage<int64> Bandwidth;
		FString CDN;
		FTimeValue ExpiresAt;
		EStreamType StreamType = EStreamType::Unsupported;
	};

	void RebuildAvailableCDNList();

	bool IsValidPathway(const FString& InPathway) const;

	FSelectedCandidateURL SelectBestHLSCandidateFrom(FString& OutMessage, ESelectFor InForType, const TArray<FCandidateURL>& InFromCandidates);
	FSelectedCandidateURL SelectBestDASHCandidateFrom(FString& OutMessage, ESelectFor InForType, const TArray<FCandidateURL>& InFromCandidates);
	FSelectedCandidateURL SelectBestDVBDASHCandidateFrom(FString& OutMessage, ESelectFor InForType, const TArray<FCandidateURL>& InFromCandidates);

	void AddDynamicDASHClonesToCandidateList(TArray<FSelectedCandidateURL>& InOutFromCandidates);

	FCriticalSection Lock;
	FRandomStream RandomStream;
	IPlayerSessionServices* PlayerSessionService = nullptr;
	bool bIsSetup = false;
	bool bIsConfigured = false;
	EStreamingProtocol StreamingProtocol = EStreamingProtocol::Other;
	FInitialParams InitialParams;
	TArray<FString> CurrentCDNPriorities;
	FString CurrentlySelectedHLSPathway;
	FString CurrentlyChosenDVBCDNForType[2];
	bool bIsInitiallyChosenCDNLocked = false;

	TArray<FString> AvailableCDNs;

	FTimeValue TimeForNextUpdate;
	bool bNewManifestNeeded = false;
	bool bManifestRequestIsPending = false;
	FString NextServerRequestURL;
	FString ProxyURL;
	bool bIsFirstSteeringRequest = true;
	bool bDoFirstUpdateOnStableBuffer = false;
	int64 SteeringRequestID = 0;
	int32 LastTTL = 0;
	TArray<FPathwayCloneEntry> CurrentCloneEntries;
	TArray<FString> AlreadyClonePathways;
	TOptional<int64> SteeringRequestIDWhenNoCandidatesMatched;

	TArray<FPenalizedCDN> PenalizedCDNList;

	TArray<FString> ReferencedCDNsSinceLastUpdate;
	TMap<FString, FCDNThroughput> ObservedBandwidths;
	bool bNeedUpdateOfReferencedList = false;

	FString CDNPriorityDisplay;
	FString CDNAccessDisplay;
};

} // namespace Electra
