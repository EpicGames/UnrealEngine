// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ChunkedArray.h"
#include "OptionalValue.h"
#include "Runtime/ErrorDetail.h"
#include "PlayerTime.h"

namespace Electra
{
struct FMPDLoadRequestDASH;
class FManifestParserDASH;
class FDashMPD_RootEntities;
class FDashMPD_OtherType;
class FDashMPD_BaseURLType;
class FDashMPD_DescriptorType;

/**
 * DASH MPD helper class to build an element from the XML document.
 */
class IDashMPDXMLElementBuilder
{
public:
	virtual ~IDashMPDXMLElementBuilder() = default;
	virtual bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) = 0;
	virtual bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) = 0;
	virtual bool ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName) = 0;

	virtual const FString& GetName() const = 0;

	void SetXMLLineNumber(int32 XMLLineNumber)
	{
		InXMLLineNumber = XMLLineNumber;
	}
	int32 GetXMLLineNumber() const
	{
		return InXMLLineNumber;
	}
	FErrorDetail AddNamespace(const TCHAR* Name, const TCHAR* URI);
protected:
	int32 InXMLLineNumber = 0;
	TMap<FString, FString>	Namespaces;
};




/**
 * DASH MPD element base class.
 *
 * Every MPD element derives from this class.
 */
class IDashMPDElement : public TSharedFromThis<IDashMPDElement, ESPMode::ThreadSafe>, public IDashMPDXMLElementBuilder
{
public:
	enum class EType
	{
		MPD,
		Period,
		AdaptationSet,
		Representation,
		URLType,
		BaseURL,
		Location,
		ProgramInformation,
		Descriptor,
		Event,
		EventStream,
		SegmentBase,
		MultipleSegmentBase,
		SegmentList,
		SegmentTemplate,
		SegmentTimeline,
		SegmentURL,
		ContentComponent,
		SubRepresentation,
		Subset,
		URLQueryInfo,
		InitializationSet,
		Preselection,
		FailoverContent,
		Switching,
		RandomAccess,
		Label,
		ProducerReferenceTime,
		LeapSecondInformation,
		ContentPopularityRate,
		ContentPopularityRate_PR,
		UIntVWithID,
		UIntPairsWithID,
		OperatingBandwidth,
		OperatingQuality,
		PlaybackRate,
		Latency,
		ServiceDescription,
		Resync,
		ContentSteering,
		ClientDataReporting,
		CMCDParameters,
		Other
	};

	struct FXmlAttribute
	{
		const FString& GetName() const 	{ return Name; }
		const FString& GetValue() const	{ return Value; }

		FString	Name;
		FString Value;
	};

	struct FXLink
	{
		bool IsSet() const
		{ return Href.Len() > 0; }

		void Clear()
		{
			Href.Empty();
			Actuate.Empty();
			LoadRequest.Reset();
		}

		const FString& GetHref() const		{ return Href; }
		const FString& GetActuate() const	{ return Actuate; }

		FString Href;
		FString Actuate;

		TSharedPtrTS<FMPDLoadRequestDASH> LoadRequest;
		int64 LastResolveID = 0;
		bool bWasRemoteEntity = false;
	};

	/**
	 * Used by a few elements to either be a boolean (default false) or an optional integer
	 */
	struct FBoolUInt64
	{
		bool bBoolean = false;
		TMediaOptionalValue<uint64>	Number;
	};

	IDashMPDElement(const TCHAR* InName, const TCHAR* InData) : Name(InName), Data(InData)
	{ }
	virtual ~IDashMPDElement() = default;
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	bool ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName) override;
	virtual EType GetElementType() const = 0;

	const FString& GetName() const override
	{ return Name; }

	TSharedPtrTS<IDashMPDElement> GetParentElement() const						{ return ParentElement.Pin(); }
	const FString& GetData() const												{ return Data; }
	const TArray<FXmlAttribute>& GetOtherAttributes() const						{ return OtherAttributes; }
	const TArray<TSharedPtrTS<FDashMPD_OtherType>>& GetOtherChildren() const	{ return OtherChildren; }

	virtual FString GetDocumentURL() const;

	virtual const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetEssentialProperties() const;
	virtual const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetSupplementalProperties() const;

	virtual const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& GetBaseURLs() const;

	virtual FXLink& GetXLink();

	/**
	 * This is to be called to replace the given child element with one or more remote elements.
	 */
	virtual int32 ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID);


	/**
	 * Sets the parent element of this element which is used in traversing the hierarchy upwards
	 * in finding attributes of a parent element.
	 */
	void SetParentElement(const TSharedPtrTS<IDashMPDElement>& InParentElement)
	{
		ParentElement = InParentElement;
	}
protected:
	bool NoChildElementAllowed(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber);
	bool HandleXLink(FXLink& OutXLink, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue);

	FString Name;
	// Weak pointer to the parent element.
	TWeakPtrTS<IDashMPDElement> ParentElement;
	// Element data string
	FString Data;
	// An array containing attributes not explicitly parsed into element members.
	TArray<FXmlAttribute> OtherAttributes;
	// An array of children that were not explicitly parsed.
	TArray<TSharedPtrTS<FDashMPD_OtherType>> OtherChildren;
};


/**
 * An MPD element we do not care about but need to parse over.
 */
class FDashMPD_OtherType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_OtherType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	EType GetElementType() const override
	{ return EType::Other; }
};


/**
 * 5.6.2 - BaseURLType
 */
class FDashMPD_BaseURLType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_BaseURLType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_BaseURLType() = default;
	EType GetElementType() const override
	{ return EType::BaseURL; }

	// Convenience method to get the URL.
	const FString& GetURL() const											{ return Data; }

	const FString& GetServiceLocation() const								{ return ServiceLocation; }
	const FString& GetByteRange() const										{ return ByteRange; }
	const FTimeValue& GetAvailabilityTimeOffset() const						{ return AvailabilityTimeOffset; }
	const TMediaOptionalValue<bool>& GetAvailabilityTimeComplete() const	{ return bAvailabilityTimeComplete; }
	const FTimeValue& GetTimeShiftBufferDepth() const						{ return TimeShiftBufferDepth; }
	const TMediaOptionalValue<int64> GetDVBPriority() const					{ return DVBPriority; }
	const TMediaOptionalValue<int64> GetDVBWeight() const					{ return DVBWeight; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	FString ServiceLocation;
	FString ByteRange;
	FTimeValue AvailabilityTimeOffset = FTimeValue::GetZero();
	TMediaOptionalValue<bool> bAvailabilityTimeComplete;
	FTimeValue TimeShiftBufferDepth;
	TMediaOptionalValue<int64> DVBPriority;
	TMediaOptionalValue<int64> DVBWeight;
};


/**
 * 5.3.9.2 (Table 17) - URLType
 */
class FDashMPD_URLType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_URLType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_URLType() = default;
	EType GetElementType() const override
	{ return EType::URLType; }

	const FString& GetSourceURL() const	{ return SourceURL; }
	const FString& GetRange() const		{ return Range; }

private:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	FString SourceURL;
	FString Range;
};


/**
 * 5.3.9.2 (Table 18) - SegmentURLType
 */
class FDashMPD_SegmentURLType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_SegmentURLType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_SegmentURLType() = default;
	EType GetElementType() const override
	{ return EType::SegmentURL; }

	const FString& GetMedia() const			{ return Media; }
	const FString& GetMediaRange() const	{ return MediaRange; }
	const FString& GetIndex() const			{ return Index; }
	const FString& GetIndexRange() const	{ return IndexRange; }

private:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	FString Media;
	FString MediaRange;
	FString Index;
	FString IndexRange;
};


/**
 * Annex I.2 - UrlQueryInfoType
 * Annex I.3 - ExtendedUrlInfoType
 *
 * For simplicity both these types are merged into a single type.
 */
class FDashMPD_UrlQueryInfoType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	enum class EExtendedUrlInfoType
	{
		ExtUrlQueryInfo,
		ExtHttpHeaderInfo
	};

	FDashMPD_UrlQueryInfoType(const TCHAR* Name, const TCHAR* Data, EExtendedUrlInfoType InExtendedUrlInfoType=EExtendedUrlInfoType::ExtUrlQueryInfo) : Super(Name, Data), ExtendedUrlInfoType(InExtendedUrlInfoType)
	{
		// Add default value. It will be removed when the @includeInRequests attribute is encountered.
		// This is valid even for the base UrlQueryInfo which _only_ applies to segments.
		IncludeInRequests.Add(TEXT("segment"));
	}
	virtual ~FDashMPD_UrlQueryInfoType() = default;
	EType GetElementType() const override
	{ return EType::URLQueryInfo; }

	const FString& GetQueryTemplate() const					{ return QueryTemplate; }
	bool GetUseMPDUrlQuery() const							{ return bUseMPDUrlQuery; }
	const FString& GetQueryString() const					{ return QueryString; }
	FXLink& GetXLink() override								{ return XLink; }

	EExtendedUrlInfoType GetExtendedUrlInfoType() const		{ return ExtendedUrlInfoType; }
	const TArray<FString>& GetIncludeInRequests() const		{ return IncludeInRequests; }
	const TArray<FString>& GetHeaderParamSources() const	{ return HeaderParamSources; }
	bool GetSameOriginOnly() const							{ return bSameOriginOnly; }

protected:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	FString QueryTemplate;
	bool bUseMPDUrlQuery = false;
	FString QueryString;
	FXLink XLink;

	EExtendedUrlInfoType ExtendedUrlInfoType;
	TArray<FString> IncludeInRequests;
	TArray<FString> HeaderParamSources;
	bool bSameOriginOnly = false;
};


/**
 * 5.8.2 - DescriptorType
 */
class FDashMPD_DescriptorType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_DescriptorType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_DescriptorType() = default;
	EType GetElementType() const override
	{ return EType::Descriptor; }

	const FString& GetSchemeIdUri() const	{ return SchemeIdUri; }
	const FString& GetValue() const			{ return Value; }
	const FString& GetID() const			{ return ID; }

	const TArray<TSharedPtrTS<IDashMPDElement>>& GetWellKnownDescriptors() const
	{ return WellKnownDescriptors; }

	// Non-DASH attributes and other XML elements as JSON. Used for <ContentProtection> elements
	// where these elements need to be passed to a CDM system.
	const FString& GetCustomElementAndAttributeJSON() const
	{ return CustomElementAndAttributeJSON; }
	void SetCustomElementAndAttributeJSON(const FString& InCustomElementAndAttributeJSON)
	{ CustomElementAndAttributeJSON = InCustomElementAndAttributeJSON; }

	void SetSchemeIdUri(const FString& InSchemeIdUri) { SchemeIdUri = InSchemeIdUri; }
	void SetValue(const FString& InValue) { Value = InValue; }
	void SetID(const FString& InID) { ID = InID; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	int32 ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID) override;

	TArray<TSharedPtrTS<IDashMPDElement>> WellKnownDescriptors;

	FString SchemeIdUri;
	FString Value;
	FString ID;

	FString CustomElementAndAttributeJSON;
};


/**
 * 5.7.2 - ProgramInformationType
 */
class FDashMPD_ProgramInformationType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_ProgramInformationType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_ProgramInformationType() = default;
	EType GetElementType() const override
	{ return EType::ProgramInformation; }

	TSharedPtrTS<FDashMPD_OtherType> GetTitle() const		{ return Title; }
	TSharedPtrTS<FDashMPD_OtherType> GetSource() const		{ return Source; }
	TSharedPtrTS<FDashMPD_OtherType> GetCopyright() const	{ return Copyright; }
	const FString& GetLanguage() const						{ return Language; }
	const FString& GetMoreInformationURL() const			{ return MoreInformationURL; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	TSharedPtrTS<FDashMPD_OtherType> Title;
	TSharedPtrTS<FDashMPD_OtherType> Source;
	TSharedPtrTS<FDashMPD_OtherType> Copyright;
	FString Language;
	FString MoreInformationURL;
};


/**
 * 5.10.2.2 (Table 35) - EventType
 */
class FDashMPD_EventType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_EventType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_EventType() = default;
	EType GetElementType() const override
	{ return EType::Event; }

	uint64 GetPresentationTime() const							{ return PresentationTime; }
	const TMediaOptionalValue<uint64>& GetDuration() const		{ return Duration; }
	const TMediaOptionalValue<uint64>& GetID() const			{ return ID; }
	const FString& GetContentEncoding() const					{ return ContentEncoding; }
	const FString& GetMessageData() const						{ return MessageData; }

protected:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	// Attributes
	uint64 PresentationTime = 0;
	TMediaOptionalValue<uint64>	Duration;
	TMediaOptionalValue<uint64>	ID;
	FString ContentEncoding;
	FString MessageData;
};


/**
 * 5.10.2.2 (Table 34) - EventStreamType
 */
class FDashMPD_EventStreamType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_EventStreamType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_EventStreamType() = default;
	EType GetElementType() const override
	{ return EType::EventStream; }

	const TArray<TSharedPtrTS<FDashMPD_EventType>>& GetEvents() const	{ return Events; }

	const FString& GetSchemeIdUri() const									{ return SchemeIdUri; }
	const FString& GetValue() const											{ return Value; }
	const TMediaOptionalValue<uint32>& GetTimescale() const					{ return Timescale; }
	const TMediaOptionalValue<uint64>& GetPresentationTimeOffset() const	{ return PresentationTimeOffset; }
	FXLink& GetXLink() override												{ return XLink; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	TArray<TSharedPtrTS<FDashMPD_EventType>> Events;

	// Attributes
	FString SchemeIdUri;
	FString Value;
	TMediaOptionalValue<uint32> Timescale;
	TMediaOptionalValue<uint64> PresentationTimeOffset;
	FXLink XLink;
};


/**
 * 5.3.4.2 - ContentComponentType
 */
class FDashMPD_ContentComponentType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_ContentComponentType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_ContentComponentType() = default;
	EType GetElementType() const override
	{ return EType::ContentComponent; }

	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetAccessibilities() const	{ return Accessibilities; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetRoles() const			{ return Roles; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetRatings() const			{ return Ratings; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetViewpoints() const		{ return Viewpoints; }

	const FString& GetID_AsStr() const		{ return ID_AsStr; }
	const uint64& GetID() const				{ return ID; }
	const FString& GetLanguage() const		{ return Language; }
	const FString& GetContentType() const  	{ return ContentType; }
	const FTimeFraction& GetPAR() const		{ return PAR; }
	const FString& GetTag() const			{ return Tag; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Accessibilities;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Roles;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Ratings;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Viewpoints;

	// Attributes
	FString ID_AsStr;
	uint64 ID = 0;
	FString Language;
	FString ContentType;
	FTimeFraction PAR;
	FString Tag;
};


/**
 * 5.3.9.6.2 - SegmentTimelineType
 */
class FDashMPD_SegmentTimelineType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	/**
	 * An <S> element within the SegmentTimelineType
	 */
	struct FSElement
	{
		FSElement()
		{
			HaveT = 0;
			HaveN = 0;
			HaveD = 0;
			HaveR = 0;
			HaveK = 0;
		}
		uint64 T = 0;
		uint64 N = 0;
		uint64 D = 0;
		uint64 K = 0;
		int32 R = 0;
		uint8 HaveT : 1;
		uint8 HaveN : 1;
		uint8 HaveD : 1;
		uint8 HaveR : 1;
		uint8 HaveK : 1;
	};

	FDashMPD_SegmentTimelineType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_SegmentTimelineType() = default;
	EType GetElementType() const override
	{ return EType::SegmentTimeline; }

	const TChunkedArray<FSElement, 20480>& GetS_Elements() const	{ return S_Elements; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	bool ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName) override;

private:
	// At 40 bytes per 'S' entry we get 512 per chunk.
	TChunkedArray<FSElement, 20480>	S_Elements;
};


/**
 * 5.3.9.7.2 - FailoverContentType
 */
class FDashMPD_FailoverContentType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	/**
	 * An <FCS> element within the FailoverContentType
	 */
	struct FFCSElement
	{
		FFCSElement()
		{
			HaveT = 0;
			HaveD = 0;
		}
		uint64 T = 0;
		uint64 D = 0;
		uint8 HaveT : 1;
		uint8 HaveD : 1;
	};

	FDashMPD_FailoverContentType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_FailoverContentType() = default;
	EType GetElementType() const override
	{ return EType::FailoverContent; }

	const TChunkedArray<FFCSElement, 3072>& GetFCS_Elements() const	{ return FCS_Elements; }

	bool GetValid() const	{ return bValid; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	bool ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName) override;

private:
	// At 24 bytes per 'FCS' entry we get 128 per chunk.
	TChunkedArray<FFCSElement, 3072>	FCS_Elements;

	// Attributes
	bool bValid = true;
};


/**
 * 5.3.9.2.3 - SegmentBaseType
 */
class FDashMPD_SegmentBaseType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_SegmentBaseType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_SegmentBaseType() = default;
	EType GetElementType() const override
	{ return EType::SegmentBase; }

	TSharedPtrTS<FDashMPD_URLType> GetInitialization() const				{ return Initialization; }
	TSharedPtrTS<FDashMPD_URLType> GetRepresentationIndex() const			{ return RepresentationIndex; }
	TSharedPtrTS<FDashMPD_FailoverContentType> GetFailoverContent() const	{ return FailoverContent; }

	const TMediaOptionalValue<uint32>& GetTimescale() const					{ return Timescale; }
	const TMediaOptionalValue<uint64>& GetPresentationTimeOffset() const	{ return PresentationTimeOffset; }
	const TMediaOptionalValue<int32>& GetEptDelta() const					{ return EptDelta; }

	const TMediaOptionalValue<uint64>& GetPresentationDuration() const		{ return PresentationDuration; }
	const FTimeValue& GetTimeShiftBufferDepth() const						{ return TimeShiftBufferDepth; }
	const FString& GetIndexRange() const									{ return IndexRange; }
	bool GetIndexRangeExact() const											{ return bIndexRangeExact; }
	const FTimeValue& GetAvailabilityTimeOffset() const						{ return AvailabilityTimeOffset; }
	const TMediaOptionalValue<bool>& GetAvailabilityTimeComplete() const	{ return bAvailabilityTimeComplete; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	TSharedPtrTS<FDashMPD_URLType> Initialization;
	TSharedPtrTS<FDashMPD_URLType> RepresentationIndex;
	TSharedPtrTS<FDashMPD_FailoverContentType> FailoverContent;

	// Attributes
	TMediaOptionalValue<uint32> Timescale;
	TMediaOptionalValue<uint64> PresentationTimeOffset;
	TMediaOptionalValue<int32> EptDelta;
	TMediaOptionalValue<uint64>	PresentationDuration;
	FTimeValue TimeShiftBufferDepth;
	FString IndexRange;
	bool bIndexRangeExact = false;
	FTimeValue AvailabilityTimeOffset = FTimeValue::GetZero();
	TMediaOptionalValue<bool> bAvailabilityTimeComplete;
};


/**
 * 5.3.9.2.2 (Table 16) - MultipleSegmentBaseType
 */
class FDashMPD_MultipleSegmentBaseType : public FDashMPD_SegmentBaseType
{
	typedef FDashMPD_SegmentBaseType Super;
public:
	FDashMPD_MultipleSegmentBaseType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_MultipleSegmentBaseType() = default;
	EType GetElementType() const override
	{ return EType::MultipleSegmentBase; }

	TSharedPtrTS<FDashMPD_SegmentTimelineType> GetSegmentTimeline() const	{ return SegmentTimeline; }
	TSharedPtrTS<FDashMPD_URLType> GetBitstreamSwitching() const			{ return BitstreamSwitching; }

	const TMediaOptionalValue<uint32>& GetDuration() const		{ return Duration; }
	const TMediaOptionalValue<uint32>& GetStartNumber() const	{ return StartNumber; }
	const TMediaOptionalValue<uint32>& GetEndNumber() const		{ return EndNumber; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	TSharedPtrTS<FDashMPD_SegmentTimelineType> SegmentTimeline;
	TSharedPtrTS<FDashMPD_URLType> BitstreamSwitching;

	// Attributes
	TMediaOptionalValue<uint32> Duration;
	TMediaOptionalValue<uint32> StartNumber;
	TMediaOptionalValue<uint32> EndNumber;
};


/**
 * 5.3.9.3.2 - SegmentListType
 */
class FDashMPD_SegmentListType : public FDashMPD_MultipleSegmentBaseType
{
	typedef FDashMPD_MultipleSegmentBaseType Super;
public:
	FDashMPD_SegmentListType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_SegmentListType() = default;
	EType GetElementType() const override
	{ return EType::SegmentList; }

	const TArray<TSharedPtrTS<FDashMPD_SegmentURLType>>& GetSegmentURLs() const	{ return SegmentURLs; }

	FXLink& GetXLink() override	{ return XLink; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	TArray<TSharedPtrTS<FDashMPD_SegmentURLType>> SegmentURLs;

	// Attributes
	FXLink XLink;
};


/**
 * 5.3.9.4.2 - SegmentTemplateType
 */
class FDashMPD_SegmentTemplateType : public FDashMPD_MultipleSegmentBaseType
{
	typedef FDashMPD_MultipleSegmentBaseType Super;
public:
	FDashMPD_SegmentTemplateType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_SegmentTemplateType() = default;
	EType GetElementType() const override
	{ return EType::SegmentTemplate; }

	// The following accessors are suffixed 'Template' to differentiate the method from the inherited methods
	// that do not return a template string but an URLType.
	const FString& GetMediaTemplate() const					{ return Media; }
	const FString& GetIndexTemplate() const					{ return Index; }
	const FString& GetInitializationTemplate() const		{ return Initialization; }
	const FString& GetBitstreamSwitchingTemplate() const	{ return BitstreamSwitching; }

protected:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	// Attributes
	FString Media;
	FString Index;
	FString Initialization;
	FString BitstreamSwitching;
};


/**
 * 5.3.8.2 - SubsetType
 */
class FDashMPD_SubsetType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_SubsetType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_SubsetType() = default;
	EType GetElementType() const override
	{ return EType::Subset; }

	const TArray<uint64>& GetContains() const	{ return Contains; }
	const FString& GetID() const				{ return ID; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	TArray<uint64> Contains;
	FString ID;
};


/**
 * 5.3.3.4 - SwitchingType
 */
class FDashMPD_SwitchingType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_SwitchingType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_SwitchingType() = default;
	EType GetElementType() const override
	{ return EType::Switching; }

	uint64 GetInterval() const		{ return Interval; }
	const FString& GetType() const	{ return Type; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	uint64 Interval = 0;
	FString Type;			// "media" or "bitstream"
};


/**
 * 5.3.5.5 - RandomAccessType
 */
class FDashMPD_RandomAccessType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_RandomAccessType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_RandomAccessType() = default;
	EType GetElementType() const override
	{ return EType::RandomAccess; }

	uint64 GetInterval() const								{ return Interval; }
	const FString& GetType() const							{ return Type; }
	const FTimeValue& GetMinBufferTime() const				{ return MinBufferTime; }
	const TMediaOptionalValue<uint64>& GetBandwidth() const	{ return Bandwidth; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	uint64 Interval = 0;
	FString Type = TEXT("closed");			// "closed", "open" or "gradual"
	FTimeValue MinBufferTime;
	TMediaOptionalValue<uint64>	Bandwidth;
};


/**
 * 5.3.10.2 - LabelType
 */
class FDashMPD_LabelType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_LabelType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_LabelType() = default;
	EType GetElementType() const override
	{ return EType::Label; }

	uint64 GetID() const				{ return ID; }
	const FString& GetLanguage() const	{ return Language; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	uint64 ID = 0;
	FString Language;
};


/**
 * 5.12.2 - ProducerReferenceTimeType
 */
class FDashMPD_ProducerReferenceTimeType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_ProducerReferenceTimeType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_ProducerReferenceTimeType() = default;
	EType GetElementType() const override
	{ return EType::ProducerReferenceTime; }

	TSharedPtrTS<FDashMPD_DescriptorType> GetUTCTiming() const	{ return UTCTiming; }

	uint32 GetID() const						{ return (uint32)ID; }
	bool GetInband() const						{ return bInband; }
	const FString& GetType() const				{ return Type; }
	const FString& GetApplicationScheme() const	{ return ApplicationScheme; }
	const FString& GetWallclockTime() const		{ return WallclockTime; }
	uint64 GetPresentationTime() const			{ return PresentationTime; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	TSharedPtrTS<FDashMPD_DescriptorType> UTCTiming;

	// Attributes
	uint64 ID = 0;
	bool bInband = false;
	FString Type = TEXT("encoder");	// one of "encoder", "captured" or "application"
	FString ApplicationScheme;
	FString WallclockTime;
	uint64 PresentationTime = 0;
};


/**
 * 5.13.1 - LeapSecondInformationType
 */
class FDashMPD_LeapSecondInformationType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_LeapSecondInformationType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_LeapSecondInformationType() = default;
	EType GetElementType() const override
	{ return EType::LeapSecondInformation; }

	int64 GetAvailabilityStartLeapOffset() const									{ return AvailabilityStartLeapOffset; }
	const TMediaOptionalValue<int64>& GetNextAvailabilityStartLeapOffset() const	{ return NextAvailabilityStartLeapOffset; }
	const FTimeValue& GetNextLeapChangeTime() const									{ return NextLeapChangeTime; }

private:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	int64 AvailabilityStartLeapOffset = 0;
	TMediaOptionalValue<int64> NextAvailabilityStartLeapOffset;
	FTimeValue NextLeapChangeTime;
};


/**
 * 5.3.1.3 - UIntVWithIDType
 */
class FDashMPD_UIntVWithIDType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_UIntVWithIDType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_UIntVWithIDType() = default;
	EType GetElementType() const override
	{ return EType::UIntVWithID; }

	const TArray<uint32>& GetElements() const	{ return Elements; }

	uint64 GetID() const						{ return ID; }
	const TArray<FString>& GetProfiles() const	{ return Profiles; }
	const FString& GetContentType() const		{ return ContentType; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	bool ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName) override;

	TArray<uint32> Elements;

	// Attributes
	uint64 ID = 0;
	TArray<FString> Profiles;
	FString ContentType;
};


/**
 * K.4.2.1 - UIntPairsWithIDType
 */
class FDashMPD_UIntPairsWithIDType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_UIntPairsWithIDType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_UIntPairsWithIDType() = default;
	EType GetElementType() const override
	{ return EType::UIntPairsWithID; }

	const TArray<uint32>& GetElements() const	{ return Elements; }

	const FString& GetType() const	{ return Type; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	bool ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName) override;

	TArray<uint32> Elements;

	// Attributes
	FString Type;
};


/**
 * K.4.2.1 - OperatingBandwidthType
 */
class FDashMPD_OperatingBandwidthType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_OperatingBandwidthType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_OperatingBandwidthType() = default;
	EType GetElementType() const override
	{ return EType::OperatingBandwidth; }

	const FString& GetMediaType() const						{ return MediaType; }
	const TMediaOptionalValue<uint64>& GetMin() const		{ return Min; }
	const TMediaOptionalValue<uint64>& GetMax() const		{ return Max; }
	const TMediaOptionalValue<uint64>& GetTarget() const	{ return Target; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	FString MediaType;		// one of "video", "audio", "any", "all"
	TMediaOptionalValue<uint64> Min;
	TMediaOptionalValue<uint64> Max;
	TMediaOptionalValue<uint64> Target;
};


/**
 * K.4.2.1 - OperatingQualityType
 */
class FDashMPD_OperatingQualityType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_OperatingQualityType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_OperatingQualityType() = default;
	EType GetElementType() const override
	{ return EType::OperatingQuality; }

	const FString& GetMediaType() const							{ return MediaType; }
	const TMediaOptionalValue<uint64>& GetMin() const			{ return Min; }
	const TMediaOptionalValue<uint64>& GetMax() const			{ return Max; }
	const TMediaOptionalValue<uint64>& GetTarget() const		{ return Target; }
	const FString& GetType() const								{ return Type; }
	const TMediaOptionalValue<uint64>& GetMaxDifference() const	{ return MaxDifference; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	FString MediaType;		// one of "video", "audio", "any"
	TMediaOptionalValue<uint64> Min;
	TMediaOptionalValue<uint64> Max;
	TMediaOptionalValue<uint64> Target;
	FString Type;
	TMediaOptionalValue<uint64> MaxDifference;
};


/**
 * K.4.2.1 - PlaybackRateType
 */
class FDashMPD_PlaybackRateType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_PlaybackRateType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_PlaybackRateType() = default;
	EType GetElementType() const override
	{ return EType::PlaybackRate; }

	const FTimeValue& GetMin() const	{ return Min; }
	const FTimeValue& GetMax() const	{ return Max; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	FTimeValue Min;
	FTimeValue Max;
};


/**
 * K.4.2.1 - LatencyType
 */
class FDashMPD_LatencyType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_LatencyType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_LatencyType() = default;
	EType GetElementType() const override
	{ return EType::Latency; }

	const TArray<TSharedPtrTS<FDashMPD_UIntPairsWithIDType>>& GetQualityLatencies() const	{ return QualityLatencies; }

	const TMediaOptionalValue<uint64>& GetReferenceID() const	{ return ReferenceID; }
	const TMediaOptionalValue<uint64>& GetTarget() const		{ return Target; }
	const TMediaOptionalValue<uint64>& GetMax() const			{ return Max; }
	const TMediaOptionalValue<uint64>& GetMin() const			{ return Min; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	TArray<TSharedPtrTS<FDashMPD_UIntPairsWithIDType>> QualityLatencies;

	// Attributes
	TMediaOptionalValue<uint64> ReferenceID;
	TMediaOptionalValue<uint64> Target;
	TMediaOptionalValue<uint64> Max;
	TMediaOptionalValue<uint64> Min;
};


/**
 * 5.4.12 - ContentPopularityRateType
 */
class FDashMPD_ContentPopularityRateType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	/**
	 * A <PR> element
	 */
	class FPRElement : public IDashMPDElement
	{
		typedef IDashMPDElement Super;
	public:
		FPRElement(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
		{ }
		virtual ~FPRElement() = default;
		EType GetElementType() const override
		{ return EType::ContentPopularityRate_PR; }

		uint64 GetPopularityRate() const						{ return PopularityRate; }
		const TMediaOptionalValue<uint64>& GetStart() const		{ return Start; }
		int32 GetR() const										{ return R; }

	protected:
		bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	private:
		// Attributes
		uint64 PopularityRate = 0;
		TMediaOptionalValue<uint64>	Start;
		int32 R = 0;
	};

	FDashMPD_ContentPopularityRateType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_ContentPopularityRateType() = default;
	EType GetElementType() const override
	{ return EType::ContentPopularityRate; }

	const TArray<TSharedPtrTS<FPRElement>>& GetPR_Elements() const	{ return PR_Elements; }

	const FString& GetSource() const				{ return Source; }
	const FString& GetSource_Description() const	{ return Source_Description; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	TArray<TSharedPtrTS<FPRElement>>	PR_Elements;

	// Attributes
	FString Source; // one of "content", "statistics" or "other"
	FString Source_Description;
};



/**
 * DASH-IF CTS 00XX V0.9.0 / ETSI TS 103 998 V1.1.1 - ContentSteeringType
 */
class FDashMPD_ContentSteeringType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_ContentSteeringType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_ContentSteeringType() = default;
	EType GetElementType() const override
	{ return EType::ContentSteering; }

	const FString& GetContentSteeringServer() const		{ return GetData(); }
	const FString& GetProxyServerURL() const			{ return ProxyServerURL; }
	const FString& GetDefaultServiceLocation() const	{ return DefaultServiceLocation; }
	bool QueryBeforeStart() const						{ return bQueryBeforeStart; }
	bool ClientRequirement() const						{ return bClientRequirement; }

private:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	FString ProxyServerURL;
	FString DefaultServiceLocation;
	bool bQueryBeforeStart = false;
	bool bClientRequirement = false;
};



/**
 * DASH-IF proposed CMCD reporting type
 */
class FDashMPD_CMCDParametersType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_CMCDParametersType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{
		Mode = TEXT("query");
		// Add default value. It will be removed when the @includeInRequests attribute is encountered.
		// This is valid even for the base UrlQueryInfo which _only_ applies to segments.
		IncludeInRequests.Add(TEXT("segment"));
	}
	virtual ~FDashMPD_CMCDParametersType() = default;
	EType GetElementType() const override
	{ return EType::CMCDParameters; }

	int32 GetVersion() const							{ return Version; }
	const TArray<FString>& GetIncludeInRequests() const	{ return IncludeInRequests; }
	const TArray<FString>& GetKeys() const				{ return Keys; }
	const FString& GetMode() const						{ return Mode; }
	const FString& GetContentID() const					{ return ContentID; }
	const FString& GetSessionID() const					{ return SessionID; }
	const FString& GetSchemeIdUri() const				{ return SchemeIdUri; }
private:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	// Attributes
	TArray<FString> Keys;
	TArray<FString> IncludeInRequests;
	FString Mode;
	FString SchemeIdUri;
	FString ContentID;
	FString SessionID;
	int32 Version = 1;
};

class FDashMPD_ClientDataReportingType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_ClientDataReportingType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_ClientDataReportingType() = default;
	EType GetElementType() const override
	{ return EType::ClientDataReporting; }

	const TArray<FString>& GetServiceLocations() const	{ return ServiceLocations; }
	const TArray<FString>& GetAdaptationSets() const	{ return AdaptationSets; }

	const TArray<TSharedPtrTS<FDashMPD_CMCDParametersType>>& GetCMCDParameters_Elements() const	{ return CMCDParameters_Elements; }
private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	TArray<TSharedPtrTS<FDashMPD_CMCDParametersType>> CMCDParameters_Elements;
	// Attributes
	TArray<FString> ServiceLocations;
	TArray<FString> AdaptationSets;
};



/**
 * K.4.2.1 - ServiceDescriptionType
 */
class FDashMPD_ServiceDescriptionType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_ServiceDescriptionType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_ServiceDescriptionType() = default;
	EType GetElementType() const override
	{ return EType::ServiceDescription; }

	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetScopes() const							{ return Scopes; }
	const TArray<TSharedPtrTS<FDashMPD_LatencyType>>& GetLatencies() const							{ return Latencies; }
	const TArray<TSharedPtrTS<FDashMPD_PlaybackRateType>>& GetPlaybackRates() const					{ return PlaybackRates; }
	const TArray<TSharedPtrTS<FDashMPD_OperatingQualityType>>& GetOperatingQualities() const		{ return OperatingQualities; }
	const TArray<TSharedPtrTS<FDashMPD_OperatingBandwidthType>>& GetOperatingBandwidths() const		{ return OperatingBandwidths; }
	const TArray<TSharedPtrTS<FDashMPD_ContentSteeringType>>& GetContentSteerings() const			{ return ContentSteerings; }
	const TArray<TSharedPtrTS<FDashMPD_ClientDataReportingType>>& GetClientDataReportings() const	{ return ClientDataReportings; }

	uint32 GetID() const { return (uint32) ID; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

private:
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Scopes;
	TArray<TSharedPtrTS<FDashMPD_LatencyType>> Latencies;
	TArray<TSharedPtrTS<FDashMPD_PlaybackRateType>> PlaybackRates;
	TArray<TSharedPtrTS<FDashMPD_OperatingQualityType>> OperatingQualities;
	TArray<TSharedPtrTS<FDashMPD_OperatingBandwidthType>> OperatingBandwidths;
	TArray<TSharedPtrTS<FDashMPD_ContentSteeringType>> ContentSteerings;
	TArray<TSharedPtrTS<FDashMPD_ClientDataReportingType>> ClientDataReportings;

	// Attributes
	uint64 ID = 0;
};


/**
 * 5.3.12 - ResyncType
 */
class FDashMPD_ResyncType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_ResyncType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_ResyncType() = default;
	EType GetElementType() const override
	{ return EType::Resync; }

	int32 GetType() const							{ return Type; }
	TMediaOptionalValue<uint32> GetdT() const		{ return dT; }
	TMediaOptionalValue<double> GetdIMax() const	{ return dIMax; }
	double GetdIMin() const							{ return dIMin; }
	bool GetMarker() const							{ return bMarker; }
	bool GetRangeAccess() const						{ return bRangeAccess; }
	const FString& GetIndex() const					{ return Index; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	int32 Type = 0;						// SAP type
	TMediaOptionalValue<uint32> dT;
	TMediaOptionalValue<double> dIMax;
	double dIMin = 0.0;
	bool bMarker = false;
	bool bRangeAccess = false;
	FString Index;
};


/**
 * 5.3.7.2 - RepresentationBaseType
 */
class FDashMPD_RepresentationBaseType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_RepresentationBaseType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_RepresentationBaseType() = default;

	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetFramePackings() const						{ return FramePackings; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetAudioChannelConfigurations() const			{ return AudioChannelConfigurations; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetContentProtections() const					{ return ContentProtections; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetEssentialProperties() const	override		{ return EssentialProperties; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetSupplementalProperties() const override		{ return SupplementalProperties; }
	const TArray<TSharedPtrTS<FDashMPD_EventStreamType>>& GetInbandEventStreams() const					{ return InbandEventStreams; }
	const TArray<TSharedPtrTS<FDashMPD_SwitchingType>>& GetSwitchings() const							{ return Switchings; }
	const TArray<TSharedPtrTS<FDashMPD_RandomAccessType>>& GetRandomAccesses() const					{ return RandomAccesses; }
	const TArray<TSharedPtrTS<FDashMPD_LabelType>>& GetGroupLabels() const								{ return GroupLabels; }
	const TArray<TSharedPtrTS<FDashMPD_LabelType>>& GetLabels() const									{ return Labels; }
	const TArray<TSharedPtrTS<FDashMPD_ProducerReferenceTimeType>>& GetProducerReferenceTimes() const	{ return ProducerReferenceTimes; }
	const TArray<TSharedPtrTS<FDashMPD_ContentPopularityRateType>>& GetContentPopularityRates() const	{ return ContentPopularityRates; }
	const TArray<TSharedPtrTS<FDashMPD_ResyncType>>& GetResyncs() const									{ return Resyncs; }

	const TArray<FString>& GetProfiles() const						{ return Profiles; }
	const TMediaOptionalValue<uint64>& GetWidth() const				{ return Width; }
	const TMediaOptionalValue<uint64>& GetHeight() const			{ return Height; }
	const FTimeFraction& GetSAR() const								{ return SAR; }
	const FTimeFraction& GetFrameRate() const						{ return FrameRate; }
	const TArray<uint64>& GetAudioSamplingRate() const				{ return AudioSamplingRate; }
	const FString& GetMimeType() const								{ return MimeType; }
	const TArray<FString>& GetSegmentProfiles() const				{ return SegmentProfiles; }
	const TArray<FString>& GetCodecs() const						{ return Codecs; }
	const FTimeValue& GetMaximumSAPPeriod() const					{ return MaximumSAPPeriod; }
	int64 GetStartWithSAP() const									{ return StartWithSAP; }
	const FTimeValue& GetMaxPlayoutRate() const						{ return MaxPlayoutRate; }
	const TMediaOptionalValue<bool>& GetCodingDependency() const	{ return bCodingDependency; }
	const FString& GetScanType() const								{ return ScanType; }
	uint64 GetSelectionPriority() const								{ return SelectionPriority; }
	const FString& GetTag() const									{ return Tag; }

protected:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	int32 ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID) override;

private:
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> FramePackings;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> AudioChannelConfigurations;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> ContentProtections;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> EssentialProperties;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> SupplementalProperties;
	TArray<TSharedPtrTS<FDashMPD_EventStreamType>> InbandEventStreams;
	TArray<TSharedPtrTS<FDashMPD_SwitchingType>> Switchings;
	TArray<TSharedPtrTS<FDashMPD_RandomAccessType>> RandomAccesses;
	TArray<TSharedPtrTS<FDashMPD_LabelType>> GroupLabels;
	TArray<TSharedPtrTS<FDashMPD_LabelType>> Labels;
	TArray<TSharedPtrTS<FDashMPD_ProducerReferenceTimeType>> ProducerReferenceTimes;
	TArray<TSharedPtrTS<FDashMPD_ContentPopularityRateType>> ContentPopularityRates;
	TArray<TSharedPtrTS<FDashMPD_ResyncType>> Resyncs;

	// Attributes
	TArray<FString> Profiles;
	TMediaOptionalValue<uint64>	Width;
	TMediaOptionalValue<uint64>	Height;
	FTimeFraction SAR;
	FTimeFraction FrameRate;
	TArray<uint64>	AudioSamplingRate;
	FString MimeType;
	TArray<FString> SegmentProfiles;
	TArray<FString> Codecs;
	FTimeValue MaximumSAPPeriod;
	int64 StartWithSAP = 0;
	FTimeValue MaxPlayoutRate;
	TMediaOptionalValue<bool> bCodingDependency;
	FString ScanType;
	uint64 SelectionPriority = 1;
	FString Tag;
};


/**
 * 5.3.6.2 - SubRepresentationType
 */
class FDashMPD_SubRepresentationType : public FDashMPD_RepresentationBaseType
{
	typedef FDashMPD_RepresentationBaseType Super;
public:
	FDashMPD_SubRepresentationType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_SubRepresentationType() = default;
	EType GetElementType() const override
	{ return EType::SubRepresentation; }

private:
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	// Attributes
	TMediaOptionalValue<uint64>	Level;
	TArray<uint64> DependencyLevels;
	TMediaOptionalValue<uint64>	Bandwidth;
	TArray<FString> ContentComponents;
};


/**
 * 5.3.12.2 - InitializationSetType
 */
class FDashMPD_InitializationSetType : public FDashMPD_RepresentationBaseType
{
	typedef FDashMPD_RepresentationBaseType Super;
public:
	FDashMPD_InitializationSetType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_InitializationSetType() = default;
	EType GetElementType() const override
	{ return EType::InitializationSet; }

	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetAccessibilities() const	{ return Accessibilities; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetRoles() const			{ return Roles; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetRatings() const			{ return Ratings; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetViewpoints() const		{ return Viewpoints; }

	// Attributes
	uint64 GetID() const									{ return ID; }
	bool GetInAllPeriods() const							{ return bInAllPeriods; }
	const FString& GetContentType() const					{ return ContentType; }
	const FTimeFraction& GetPAR() const						{ return PAR; }
	const TMediaOptionalValue<uint64>& GetMaxWidth() const	{ return MaxWidth; }
	const TMediaOptionalValue<uint64>& GetMaxHeight() const	{ return MaxHeight; }
	const FTimeFraction& GetMaxFrameRate() const			{ return MaxFrameRate; }
	const FString& GetInitialization() const				{ return Initialization; }
	FXLink& GetXLink() override								{ return XLink; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Accessibilities;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Roles;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Ratings;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Viewpoints;

	// Attributes
	uint64 ID = 0;
	bool bInAllPeriods = true;
	FString ContentType;
	FTimeFraction PAR;
	TMediaOptionalValue<uint64>	MaxWidth;
	TMediaOptionalValue<uint64>	MaxHeight;
	FTimeFraction MaxFrameRate;
	FString Initialization;
	FXLink XLink;
};


/**
 * 5.3.11.3 - PreselectionType
 */
class FDashMPD_PreselectionType : public FDashMPD_RepresentationBaseType
{
	typedef FDashMPD_RepresentationBaseType Super;
public:
	FDashMPD_PreselectionType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_PreselectionType() = default;
	EType GetElementType() const override
	{ return EType::Preselection; }

	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetAccessibilities() const	{ return Accessibilities; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetRoles() const			{ return Roles; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetRatings() const			{ return Ratings; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetViewpoints() const		{ return Viewpoints; }

	const FString& GetI() const									{ return ID; }
	const TArray<FString>& GetPreselectionComponents() const	{ return PreselectionComponents; }
	const FString& GetLanguage() const							{ return Language; }
	const FString& GetOrder() const								{ return Order; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;

	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Accessibilities;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Roles;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Ratings;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Viewpoints;

	// Attributes
	FString ID = TEXT("1");
	TArray<FString> PreselectionComponents;
	FString Language;
	FString Order = TEXT("undefined");
};


/**
 * 5.3.5.2 - RepresentationType
 */
class FDashMPD_RepresentationType : public FDashMPD_RepresentationBaseType
{
	typedef FDashMPD_RepresentationBaseType Super;
public:
	FDashMPD_RepresentationType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_RepresentationType() = default;
	EType GetElementType() const override
	{ return EType::Representation; }

	const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& GetBaseURLs() const override				{ return BaseURLs; }
	const TArray<TSharedPtrTS<FDashMPD_SubRepresentationType>>& GetSubRepresentations() const	{ return SubRepresentations; }
	TSharedPtrTS<FDashMPD_SegmentBaseType> GetSegmentBase() const								{ return SegmentBase; }
	TSharedPtrTS<FDashMPD_SegmentListType> GetSegmentList() const								{ return SegmentList; }
	TSharedPtrTS<FDashMPD_SegmentTemplateType> GetSegmentTemplate() const						{ return SegmentTemplate; }

	// Attributes
	const FString& GetID() const									{ return ID; }
	uint64 GetBandwidth() const										{ return Bandwidth; }
	const TMediaOptionalValue<uint64>& GetQualityRanking() const	{ return QualityRanking; }
	const TArray<FString>& GetDependencyIDs() const					{ return DependencyIDs; }
	const TArray<FString>& GetAssociationIDs() const				{ return AssociationIDs; }
	const TArray<FString>& GetAssociationTypes() const				{ return AssociationTypes; }
	const TArray<FString>& GetMediaStreamStructureIDs() const		{ return MediaStreamStructureIDs; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	int32 ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID) override;

	TArray<TSharedPtrTS<FDashMPD_BaseURLType>> BaseURLs;
	TArray<TSharedPtrTS<FDashMPD_SubRepresentationType>> SubRepresentations;
	TSharedPtrTS<FDashMPD_SegmentBaseType> SegmentBase;
	TSharedPtrTS<FDashMPD_SegmentListType> SegmentList;
	TSharedPtrTS<FDashMPD_SegmentTemplateType> SegmentTemplate;

	// Attributes
	FString ID;
	uint64 Bandwidth = 0;
	TMediaOptionalValue<uint64>	QualityRanking;
	TArray<FString> DependencyIDs;
	TArray<FString> AssociationIDs;
	TArray<FString> AssociationTypes;
	TArray<FString> MediaStreamStructureIDs;
};


/**
 * 5.3.3.2 - AdaptationSetType
 */
class FDashMPD_AdaptationSetType : public FDashMPD_RepresentationBaseType
{
	typedef FDashMPD_RepresentationBaseType Super;
public:
	FDashMPD_AdaptationSetType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_AdaptationSetType() = default;
	EType GetElementType() const override
	{ return EType::AdaptationSet; }


	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetAccessibilities() const			{ return Accessibilities; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetRoles() const					{ return Roles; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetRatings() const					{ return Ratings; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetViewpoints() const				{ return Viewpoints; }
	const TArray<TSharedPtrTS<FDashMPD_ContentComponentType>>& GetContentComponents() const	{ return ContentComponents; }
	const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& GetBaseURLs() const override			{ return BaseURLs; }
	TSharedPtrTS<FDashMPD_SegmentBaseType> GetSegmentBase() const							{ return SegmentBase; }
	TSharedPtrTS<FDashMPD_SegmentListType> GetSegmentList() const							{ return SegmentList; }
	TSharedPtrTS<FDashMPD_SegmentTemplateType> GetSegmentTemplate() const					{ return SegmentTemplate; }
	const TArray<TSharedPtrTS<FDashMPD_RepresentationType>>& GetRepresentations() const		{ return Representations; }

	const FString& GetID_AsStr() const								{ return ID_AsStr; }
	uint64 GetID() const											{ return ID; }
	const TMediaOptionalValue<uint64>& GetGroup() const				{ return Group; }
	const FString& GetLanguage() const								{ return Language; }
	const FString& GetContentType() const							{ return ContentType; }
	const FTimeFraction& GetPAR() const								{ return PAR; }
	const TMediaOptionalValue<uint64>& GetMinBandwidth() const		{ return MinBandwidth; }
	const TMediaOptionalValue<uint64>& GetMaxBandwidth() const		{ return MaxBandwidth; }
	const TMediaOptionalValue<uint64>& GetMinWidth() const			{ return MinWidth; }
	const TMediaOptionalValue<uint64>& GetMaxWidth() const			{ return MaxWidth; }
	const TMediaOptionalValue<uint64>& GetMinHeight() const			{ return MinHeight; }
	const TMediaOptionalValue<uint64>& GetMaxHeight() const			{ return MaxHeight; }
	const FTimeFraction& GetMinFrameRate() const					{ return MinFrameRate; }
	const FTimeFraction& GetMaxFrameRate() const					{ return MaxFrameRate; }
	const FBoolUInt64& GetSegmentAlignment() const					{ return bSegmentAlignment; }
	const FBoolUInt64& GetSubsegmentAlignment() const				{ return bSubsegmentAlignment; }
	int64 GetSubsegmentStartsWithSAP() const						{ return SubsegmentStartsWithSAP; }
	const TMediaOptionalValue<bool>& GetBitstreamSwitching() const	{ return bBitstreamSwitching; }
	const TArray<uint64>& GetInitializationSetRefs() const			{ return InitializationSetRefs; }
	FXLink& GetXLink() override										{ return XLink; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	int32 ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID) override;

	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Accessibilities;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Roles;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Ratings;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Viewpoints;
	TArray<TSharedPtrTS<FDashMPD_ContentComponentType>> ContentComponents;
	TArray<TSharedPtrTS<FDashMPD_BaseURLType>> BaseURLs;
	TSharedPtrTS<FDashMPD_SegmentBaseType> SegmentBase;
	TSharedPtrTS<FDashMPD_SegmentListType> SegmentList;
	TSharedPtrTS<FDashMPD_SegmentTemplateType> SegmentTemplate;
	TArray<TSharedPtrTS<FDashMPD_RepresentationType>> Representations;

	// Attributes
	FString ID_AsStr;
	uint64 ID = 0;
	TMediaOptionalValue<uint64>	Group;
	FString Language;
	FString ContentType;		// One of: "text", "image", "audio", "video", "application" and "font"
	FTimeFraction PAR;
	TMediaOptionalValue<uint64>	MinBandwidth;
	TMediaOptionalValue<uint64>	MaxBandwidth;
	TMediaOptionalValue<uint64>	MinWidth;
	TMediaOptionalValue<uint64>	MaxWidth;
	TMediaOptionalValue<uint64>	MinHeight;
	TMediaOptionalValue<uint64>	MaxHeight;
	FTimeFraction MinFrameRate;
	FTimeFraction MaxFrameRate;
	FBoolUInt64 bSegmentAlignment;
	FBoolUInt64 bSubsegmentAlignment;
	int64 SubsegmentStartsWithSAP = 0;
	TMediaOptionalValue<bool> bBitstreamSwitching;
	TArray<uint64> InitializationSetRefs;
	FXLink XLink;
};


/**
 * 5.3.2.2 - PeriodType
 */
class FDashMPD_PeriodType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_PeriodType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_PeriodType() = default;
	EType GetElementType() const override
	{ return EType::Period; }

	const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& GetBaseURLs() const override						{ return BaseURLs; }
	TSharedPtrTS<FDashMPD_SegmentBaseType> GetSegmentBase() const										{ return SegmentBase; }
	TSharedPtrTS<FDashMPD_SegmentListType> GetSegmentList() const										{ return SegmentList; }
	TSharedPtrTS<FDashMPD_SegmentTemplateType> GetSegmentTemplate() const								{ return SegmentTemplate; }
	TSharedPtrTS<FDashMPD_DescriptorType> GetAssetIdentifier() const									{ return AssetIdentifier; }
	const TArray<TSharedPtrTS<FDashMPD_EventStreamType>>& GetEventStreams() const						{ return EventStreams; }
	const TArray<TSharedPtrTS<FDashMPD_AdaptationSetType>>& GetAdaptationSets() const					{ return AdaptationSets; }
	const TArray<TSharedPtrTS<FDashMPD_SubsetType>>& GetSubsets() const									{ return Subsets; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetSupplementalProperties() const override		{ return SupplementalProperties; }
	const TArray<TSharedPtrTS<FDashMPD_ServiceDescriptionType>>& GetServiceDescriptions() const			{ return ServiceDescriptions; }
	const TArray<TSharedPtrTS<FDashMPD_AdaptationSetType>>& GetEmptyAdaptationSets() const				{ return EmptyAdaptationSets; }
	const TArray<TSharedPtrTS<FDashMPD_LabelType>>& GetGroupLabels() const								{ return GroupLabels; }
	const TArray<TSharedPtrTS<FDashMPD_PreselectionType>>& GetPreselections() const						{ return Preselections; }

	const FString& GetID() const											{ return ID; }
	const FTimeValue& GetStart() const										{ return Start; }
	const FTimeValue& GetDuration() const									{ return Duration; }
	const TMediaOptionalValue<bool>& GetBitstreamSwitching() const			{ return bBitstreamSwitching; }
	FXLink& GetXLink() override												{ return XLink; }

	// Methods to manipulate the presentation type.
	void SetDuration(const FTimeValue& NewDuration)
	{
		Duration = NewDuration;
	}

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	int32 ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID) override;

	TArray<TSharedPtrTS<FDashMPD_BaseURLType>> BaseURLs;

	TSharedPtrTS<FDashMPD_SegmentBaseType> SegmentBase;
	TSharedPtrTS<FDashMPD_SegmentListType> SegmentList;
	TSharedPtrTS<FDashMPD_SegmentTemplateType> SegmentTemplate;

	TSharedPtrTS<FDashMPD_DescriptorType> AssetIdentifier;
	TArray<TSharedPtrTS<FDashMPD_EventStreamType>> EventStreams;
	TArray<TSharedPtrTS<FDashMPD_AdaptationSetType>> AdaptationSets;
	TArray<TSharedPtrTS<FDashMPD_SubsetType>> Subsets;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> SupplementalProperties;

	TArray<TSharedPtrTS<FDashMPD_ServiceDescriptionType>> ServiceDescriptions;
	TArray<TSharedPtrTS<FDashMPD_AdaptationSetType>> EmptyAdaptationSets;
	TArray<TSharedPtrTS<FDashMPD_LabelType>> GroupLabels;
	TArray<TSharedPtrTS<FDashMPD_PreselectionType>> Preselections;

	// Attributes
	FString ID;
	FTimeValue Start;
	FTimeValue Duration;
	TMediaOptionalValue<bool> bBitstreamSwitching;
	FXLink XLink;
};


/**
 * 5.3.1.2 - MPDType
 */
class FDashMPD_MPDType : public IDashMPDElement
{
	typedef IDashMPDElement Super;
public:
	FDashMPD_MPDType(const TCHAR* Name, const TCHAR* Data) : Super(Name, Data)
	{ }
	virtual ~FDashMPD_MPDType() = default;
	EType GetElementType() const override
	{ return EType::MPD; }

	const TArray<TSharedPtrTS<FDashMPD_ProgramInformationType>>& GetProgramInformations() const			{ return ProgramInformations; }
	const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& GetBaseURLs() const override						{ return BaseURLs; }
	const TArray<TSharedPtrTS<FDashMPD_OtherType>>& GetLocations() const								{ return Locations; }
	const TArray<TSharedPtrTS<FDashMPD_PeriodType>>& GetPeriods() const									{ return Periods; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetEssentialProperties() const override		{ return EssentialProperties; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetSupplementalProperties() const override		{ return SupplementalProperties; }
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& GetUTCTimings() const							{ return UTCTimings; }
	const TArray<TSharedPtrTS<FDashMPD_ServiceDescriptionType>>& GetServiceDescriptions() const			{ return ServiceDescriptions; }
	const TArray<TSharedPtrTS<FDashMPD_InitializationSetType>>& GetInitializationSets() const			{ return InitializationSets; }
	const TArray<TSharedPtrTS<FDashMPD_UIntVWithIDType>>& GetInitializationGroups() const				{ return InitializationGroups; }
	const TArray<TSharedPtrTS<FDashMPD_UIntVWithIDType>>& GetInitializationPresentations() const		{ return InitializationPresentations; }
	TSharedPtrTS<FDashMPD_LeapSecondInformationType> GetLeapSecondInformation() const					{ return LeapSecondInformation; }
	TSharedPtrTS<FDashMPD_ContentSteeringType> GetContentSteering() const								{ return ContentSteering; }

	const FString& GetID() const							{ return ID; }
	const TArray<FString>& GetProfiles()					{ return Profiles; }
	const FString& GetType() const							{ return Type; }
	const FTimeValue& GetAvailabilityStartTime() const		{ return AvailabilityStartTime; }
	const FTimeValue& GetAvailabilityEndTime() const		{ return AvailabilityEndTime; }
	const FTimeValue& GetPublishTime() const				{ return PublishTime; }
	const FTimeValue& GetMediaPresentationDuration() const	{ return MediaPresentationDuration; }
	const FTimeValue& GetMinimumUpdatePeriod() const		{ return MinimumUpdatePeriod; }
	const FTimeValue& GetMinBufferTime() const				{ return MinBufferTime; }
	const FTimeValue& GetTimeShiftBufferDepth() const		{ return TimeShiftBufferDepth; }
	const FTimeValue& GetSuggestedPresentationDelay() const	{ return SuggestedPresentationDelay; }
	const FTimeValue& GetMaxSegmentDuration() const			{ return MaxSegmentDuration; }
	const FTimeValue& GetMaxSubsegmentDuration() const		{ return MaxSubsegmentDuration; }

	FString GetDocumentURL() const override					{ return DocumentURL; }
	void SetDocumentURL(const FString& InDocumentURL)		{ DocumentURL = InDocumentURL; }
	const FTimeValue& GetFetchTime() const					{ FScopeLock Lock(&UpdateLock); return FetchTime; }
	void SetFetchTime(const FTimeValue& InFetchTime)		{ FScopeLock Lock(&UpdateLock); FetchTime = InFetchTime; }
	FString GetETag() const									{ return ETag; }
	void SetETag(const FString& InETag)						{ ETag = InETag; }
	void RemoveUTCTimingElement(TSharedPtrTS<FDashMPD_DescriptorType> TimingElement)
	{
		FScopeLock lock(&UpdateLock);
		UTCTimings.Remove(TimingElement);
	}
	void AddUTCTimingElement(const FString& InData, const FString& InScheme, const FString& InID, const FString& InValue)
	{
		FScopeLock lock(&UpdateLock);
		auto Element = UTCTimings.Emplace_GetRef(MakeSharedTS<FDashMPD_DescriptorType>(TEXT("UTCTiming"), *InData));
		Element->SetSchemeIdUri(InScheme);
		Element->SetID(InID);
		Element->SetValue(InValue);
	}

	void SetDVBDash(bool bInDVBDash)						{ bIsDVBDash = bInDVBDash; }
	bool IsDVBDash() const									{ return bIsDVBDash; }

	// Methods to manipulate the presentation type.
	void LockAccess()											{ UpdateLock.Lock(); }
	void UnlockAccess()											{ UpdateLock.Unlock(); }
	void SetType(const FString& InType)							{ Type = InType; }
	void SetPublishTime(const FTimeValue& InPUBT)				{ PublishTime = InPUBT; }
	void SetAvailabilityStartTime(const FTimeValue& InAST)		{ AvailabilityStartTime = InAST; }
	void SetAvailabilityEndTime(const FTimeValue& InAET)		{ AvailabilityEndTime = InAET; }
	void SetMediaPresentationDuration(const FTimeValue& InPD)	{ MediaPresentationDuration = InPD; }
	void SetMinimumUpdatePeriod(const FTimeValue& InMUP)		{ MinimumUpdatePeriod = InMUP; }
	void SetTimeShiftBufferDepth(const FTimeValue& InTSB)		{ TimeShiftBufferDepth = InTSB; }
	void SetSuggestedPresentationDelay(const FTimeValue& InSPD)	{ SuggestedPresentationDelay = InSPD; }

private:
	bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	int32 ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID) override;

	// The MPD is not expected to get modified after being built except for potential XLink changes
	// and extending the validity if an MPD update returned a 304 on a conditional GET.
	// For these occurrences we need to lock access to the document on occasion.
	mutable FCriticalSection UpdateLock;

	// The URL from which this MPD was loaded. Provided for convenience.
	FString DocumentURL;

	// Optional ETag for conditional GET update requests.
	FString ETag;

	// Set if the MPD is specifying a DVB-DASH profile.
	bool bIsDVBDash = false;

	// Time the MPD load request was started.
	FTimeValue FetchTime;

	// Program information.
	TArray<TSharedPtrTS<FDashMPD_ProgramInformationType>> ProgramInformations;

	// Base URLs.
	TArray<TSharedPtrTS<FDashMPD_BaseURLType>> BaseURLs;

	// Locations where the MPD can be found.
	TArray<TSharedPtrTS<FDashMPD_OtherType>> Locations;

	// Periods in order as they appear in the MPD.
	TArray<TSharedPtrTS<FDashMPD_PeriodType>> Periods;

	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> EssentialProperties;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> SupplementalProperties;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>> UTCTimings;

	TArray<TSharedPtrTS<FDashMPD_ServiceDescriptionType>> ServiceDescriptions;
	TArray<TSharedPtrTS<FDashMPD_InitializationSetType>> InitializationSets;
	TArray<TSharedPtrTS<FDashMPD_UIntVWithIDType>> InitializationGroups;
	TArray<TSharedPtrTS<FDashMPD_UIntVWithIDType>> InitializationPresentations;
	TSharedPtrTS<FDashMPD_LeapSecondInformationType> LeapSecondInformation;
	TSharedPtrTS<FDashMPD_ContentSteeringType> ContentSteering;


	// Attributes of this MPD element.
	FString ID;
	TArray<FString> Profiles;
	FString Type = TEXT("static");
	FTimeValue AvailabilityStartTime;
	FTimeValue AvailabilityEndTime;
	FTimeValue PublishTime;
	FTimeValue MediaPresentationDuration;
	FTimeValue MinimumUpdatePeriod;
	FTimeValue MinBufferTime;
	FTimeValue TimeShiftBufferDepth;
	FTimeValue SuggestedPresentationDelay;
	FTimeValue MaxSegmentDuration;
	FTimeValue MaxSubsegmentDuration;
};


/**
 * This is merely a container into which the XML parser adds root elements as it finds them.
 * We need this when resolving remote xlink entities that bring in elements not within a root <MPD> element.
 */
class FDashMPD_RootEntities
{
public:
	TArray<TSharedPtrTS<FDashMPD_MPDType>> MPDs;
	TArray<TSharedPtrTS<FDashMPD_PeriodType>> Periods;
	TArray<TSharedPtrTS<FDashMPD_AdaptationSetType>> AdaptationSets;
	TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>> UrlQueries;
	TArray<TSharedPtrTS<FDashMPD_InitializationSetType>> InitializationSets;
	TArray<TSharedPtrTS<FDashMPD_EventStreamType>> EventStreams;
	TArray<TSharedPtrTS<FDashMPD_SegmentListType>> SegmentLists;
};


}
