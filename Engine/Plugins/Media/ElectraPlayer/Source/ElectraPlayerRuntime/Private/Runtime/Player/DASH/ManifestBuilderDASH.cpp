// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "ElectraPlayerPrivate.h"
#include "Player/Manifest.h"
#include "Player/PlaybackTimeline.h"

#include "Player/PlayerSessionServices.h"
#include "Player/PlayerStreamFilter.h"
#include "Player/PlayerEntityCache.h"
#include "Player/DASH/OptionKeynamesDASH.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/DASH/PlayerEventDASH.h"
#include "Player/DASH/PlayerEventDASH_Internal.h"

#include "ManifestBuilderDASH.h"
#include "ManifestParserDASH.h"
#include "MPDElementsDASH.h"

#include "Stats/Stats.h"
#include "Utilities/StringHelpers.h"
#include "SynchronizedClock.h"
#include "StreamTypes.h"
#include "ErrorDetail.h"

#include "Utilities/URLParser.h"
#include "Utilities/Utilities.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/ISO639-Map.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"

#include "Player/DRM/DRMManager.h"

#define ERRCODE_DASH_MPD_BUILDER_INTERNAL							1
#define ERRCODE_DASH_MPD_BUILDER_UNSUPPORTED_PROFILE				100
#define ERRCODE_DASH_MPD_BUILDER_UNSUPPORTED_ESSENTIAL_PROPERTY		101
#define ERRCODE_DASH_MPD_BUILDER_MISSING_REQUIRED_ATTRIBUTE			102
#define ERRCODE_DASH_MPD_BUILDER_UNREQUIRED_ATTRIBUTE				103
#define ERRCODE_DASH_MPD_BUILDER_XLINK_NOT_SUPPORTED_ON_ELEMENT		104
#define ERRCODE_DASH_MPD_BUILDER_URL_FAILED_TO_RESOLVE				105
#define ERRCODE_DASH_MPD_BUILDER_MISSING_REQUIRED_ELEMENT			106
#define ERRCODE_DASH_MPD_BUILDER_AMBIGUOUS_URLS						107

#define ERRCODE_DASH_MPD_BUILDER_BAD_PERIOD_START					200
#define ERRCODE_DASH_MPD_BUILDER_EARLY_PERIODS_MUST_BE_LAST			201
#define ERRCODE_DASH_MPD_BUILDER_MEDIAPRESENTATIONDURATION_NEEDED	202
#define ERRCODE_DASH_MPD_BUILDER_BAD_PERIOD_DURATION				203

#define ERRCODE_DASH_MPD_BUILDER_REJECTED_BY_APPLICATION			300


DECLARE_CYCLE_STAT(TEXT("FManifestDASHInternal::Build"), STAT_ElectraPlayer_FManifestDASHInternal_Build, STATGROUP_ElectraPlayer);
DECLARE_CYCLE_STAT(TEXT("FManifestDASHInternal::XLink"), STAT_ElectraPlayer_FManifestDASHInternal_XLink, STATGROUP_ElectraPlayer);


namespace Electra
{

namespace
{
	const TCHAR* const XLinkActuateOnLoad = TEXT("onLoad");
	const TCHAR* const XLinkActuateOnRequest = TEXT("onRequest");
	const TCHAR* const XLinkResolveToZero = TEXT("urn:mpeg:dash:resolve-to-zero:2013");

	const TCHAR* const SchemeHTTP = TEXT("http://");
	const TCHAR* const SchemeHTTPS = TEXT("https://");
	const TCHAR* const SchemeDATA = TEXT("data:");

	const TCHAR* const SupportedProfiles[] =
	{
		TEXT("urn:mpeg:dash:profile:isoff-on-demand:2011"),
		TEXT("urn:mpeg:dash:profile:isoff-live:2011"),
//		TEXT("urn:mpeg:dash:profile:isoff-ext-on-demand:2014"),			// ??
//		TEXT("urn:mpeg:dash:profile:isoff-ext-live:2014"),				// full xlink required
//		TEXT("urn:mpeg:dash:profile:isoff-common:2014"),				// may contain both ext-on-demand and ext-live additions
//		TEXT("urn:mpeg:dash:profile:isoff-broadcast:2015"),				// RandomAccess and Switching support required
		TEXT("urn:mpeg:dash:profile:full:2011"),						// NOTE: Full profile is not supported, but some sources that are a lesser profile are incorrectly saying they require full, so we hope for the best.
		// Possibly add DASH-IF-IOP profiles?
//		TEXT(""),
		// DVB-DASH profiles
		TEXT("urn:dvb:dash:profile:dvb-dash:2014"),
		TEXT("urn:dvb:dash:profile:dvb-dash:isoff-ext-on-demand:2014"),
		TEXT("urn:dvb:dash:profile:dvb-dash:isoff-ext-live:2014"),
		// Low latency Live
		TEXT("http://www.dashif.org/guidelines/low-latency-live-v5"),
		// WebM profiles
		TEXT("urn:webm:dash:profile:webm-on-demand:2012"),
		TEXT("urn:mpeg:dash:profile:webm-on-demand:2012"),
	};

	const TCHAR* const DVBDashProfilePrefix = TEXT("urn:dvb:dash:profile:");

	const TCHAR* const ScanTypeInterlace = TEXT("interlace");

	const TCHAR* const AudioChannelConfigurationLegacy = TEXT("urn:mpeg:dash:23003:3:audio_channel_configuration:2011");
	const TCHAR* const AudioChannelConfiguration = TEXT("urn:mpeg:mpegB:cicp:ChannelConfiguration");
	const TCHAR* const AudioChannelConfigurationDolby = TEXT("tag:dolby.com,2014:dash:audio_channel_configuration:2011");

	const TCHAR* const DASHRole = TEXT("urn:mpeg:dash:role:2011");

	const TCHAR* const EssentialDashReservedPropertyPrefix = TEXT("urn:mpeg:dash:");

	const TCHAR* const UrlParam2014 = TEXT("urn:mpeg:dash:urlparam:2014");
	const TCHAR* const UrlParam2016 = TEXT("urn:mpeg:dash:urlparam:2016");

	const TCHAR* const SupportedEssentialProperties[] =
	{
		UrlParam2014,
		UrlParam2016
	};

    const TCHAR* const Scheme_urn_mpeg_dash_mp4protection_2011 = TEXT("urn:mpeg:dash:mp4protection:2011");

	// Recognized subtitle mime types
    const TCHAR* const SubtitleMimeType_SideloadedTTML = TEXT("application/ttml+xml");
    const TCHAR* const SubtitleMimeType_SideloadedVTT = TEXT("text/vtt");
    const TCHAR* const SubtitleMimeType_Streamed = TEXT("application/mp4");

	// Thumbnail mime types
    const TCHAR* const ThumbnailMimeType_Jpeg = TEXT("image/jpeg");
    const TCHAR* const ThumbnailMimeType_Png= TEXT("image/png");
}


namespace DASHAttributeHelpers
{
	const IDashMPDElement::FXmlAttribute* GetAttribute(const TSharedPtrTS<FDashMPD_DescriptorType>& InDescriptor, const TCHAR* InAttribute, const TCHAR* InOptionalNamespace)
	{
		const TArray<IDashMPDElement::FXmlAttribute>& Attributes = InDescriptor->GetOtherAttributes();
		FString NameWithNS = InOptionalNamespace ? FString::Printf(TEXT("%s:%s"), InOptionalNamespace, InAttribute) : FString(InAttribute);
		for(int32 i=0; i<Attributes.Num(); ++i)
		{
			if (Attributes[i].GetName().Equals(NameWithNS) || Attributes[i].GetName().Equals(InAttribute))
			{
				return &Attributes[i];
			}
			// If not found in the desired namespace and it doesn't exist in the default namespace let's see if there is a match
			// in a some other namespace. This sounds counter-intuitive but namespaces can be chosen arbitrarily.
			// Since there are usually no attribute conflicts that warrant a namespace let's see if the attribute exists in some
			// unexpected namespace.
			NameWithNS = FString::Printf(TEXT(":%s"), InAttribute);
			if (Attributes[i].GetName().EndsWith(NameWithNS, ESearchCase::CaseSensitive))
			{
				return &Attributes[i];
			}
		}
		return nullptr;
	}
};


namespace DASHUrlHelpers
{
	ELECTRA_IMPL_DEFAULT_ERROR_METHODS(DASHMPDBuilder);

	bool IsAbsoluteURL(const FString& URL)
	{
		// For simplicities sake we check if the last element starts with 'http://' or 'https://' to determine whether it is
		// an absolute or relative URL. We also allow for data URLs
		return URL.StartsWith(SchemeHTTPS) || URL.StartsWith(SchemeHTTP) || URL.StartsWith(SchemeDATA);
	}

	bool AreBaseURLsValid(const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& InBaseURLs)
	{
		int32 NumRelativeURLs = 0;
		int32 NumAbsoluteURLs = 0;
		for(auto& BUrl : InBaseURLs)
		{
			if (DASHUrlHelpers::IsAbsoluteURL(BUrl->GetURL()))
			{
				++NumAbsoluteURLs;
			}
			else
			{
				++NumRelativeURLs;
			}
		}
		//return NumRelativeURLs && NumAbsoluteURLs ? false : NumRelativeURLs <= 1;
		return NumRelativeURLs <= 1;
	}

	bool AreURLQueryInfosValid(const TSharedPtrTS<const IDashMPDElement>& InElement)
	{
		if (!InElement.IsValid())
		{
			return true;
		}
		// Annex I.2 limits the 2014 param to at most one per hierarchy level.
		// Note that for the 2016 version there is no such limit as different instances may be used to affect different types of requests.
		int32 nEss2014 = InElement->GetEssentialProperties().FilterByPredicate([](const TSharedPtrTS<FDashMPD_DescriptorType>& d) {return d->GetSchemeIdUri().Equals(UrlParam2014);}).Num();
		int32 nSupp2014 = InElement->GetSupplementalProperties().FilterByPredicate([](const TSharedPtrTS<FDashMPD_DescriptorType>& d) {return d->GetSchemeIdUri().Equals(UrlParam2014);}).Num();
		return nEss2014 + nSupp2014 <= 1;
	}


	// Get all the <BaseURL>'s for this element throughout the MPD hierarchy, expanding relative URLs against their
	// parent URLs up the the MPD document URL if necessary.
	bool GetAllCandidateBaseURLs(TArray<FDASHMediaURL>& OutCandidateURLs, TSharedPtrTS<const IDashMPDElement> StartingAt, const FDASHMediaURL& InMPDUrl, bool bIsDVBDash)
	{
		// Collect all <BaseURL> elements in hierarchy order, starting at the current element going all the way to
		// the top of the MPD.
		TArray<TArray<TSharedPtrTS<FDashMPD_BaseURLType>>> HierarchicalBaseURLs;
		while(StartingAt.IsValid())
		{
			const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& ElementBaseURLs = StartingAt->GetBaseURLs();
			if (ElementBaseURLs.Num())
			{
				// Insert at the front, so the final list will be in hierarchy order.
				HierarchicalBaseURLs.Insert(ElementBaseURLs, 0);
			}
			StartingAt = StartingAt->GetParentElement();
		}
		// Start building the candidate URLs
		struct FParsedDASHMediaURL : FDASHMediaURL
		{
			FURL_RFC3986 Parsed;
			void operator = (const FDASHMediaURL& InBase) { FDASHMediaURL::operator=(InBase); }
			bool operator == (const FDASHMediaURL& InOther) const { return URL.Equals(InOther.URL) && CDN.Equals(InOther.CDN); }
		};
		FParsedDASHMediaURL RootDocument;
		RootDocument = InMPDUrl;
		if (!RootDocument.Parsed.Parse(RootDocument.URL))
		{
			UE_LOGF(LogElectraPlayer, Error, "Could not parse the MPD URL when building element URL");
			return false;
		}
		TArray<FParsedDASHMediaURL> Base, Work;
		Base.Emplace(RootDocument);
		for(int32 nL=0; nL<HierarchicalBaseURLs.Num(); ++nL)
		{
			for(int32 nI=0; nI<Base.Num(); ++nI)
			{
				// Resolve each entry in `Base` against the <BaseURL> elements on the next hierarchy level.
				for(int32 j=0; j<HierarchicalBaseURLs[nL].Num(); ++j)
				{
					auto bu = HierarchicalBaseURLs[nL][j];
					FURL_RFC3986 nextUrl;
					if (!nextUrl.Parse(bu->GetURL()))
					{
						UE_LOGF(LogElectraPlayer, Error, "Could not parse a <BaseURL> element while resolving URL");
						return false;
					}
					FParsedDASHMediaURL& nxt(Work.Emplace_GetRef());
					nxt.Parsed = Base[nI].Parsed;
					nxt.Parsed.ResolveWith(nextUrl);
					nxt.URL = nxt.Parsed.Get();
					if (nextUrl.IsAbsolute())
					{
						nxt.CDN = bu->GetServiceLocation();
						// An empty serviceLocation on an absolute URL uses the URL as the serviceLocation according to DVB-DASH.
						if (nxt.CDN.IsEmpty() && bIsDVBDash)
						{
							nxt.CDN = nxt.URL;
						}
						if (bu->GetDVBPriority().IsSet() && bu->GetDVBPriority().Value() < 0)
						{
							UE_LOGF(LogElectraPlayer, Error, "Invalid @priority found on <BaseURL> element while resolving URL");
							return false;
						}
						nxt.DVBpriority = bu->GetDVBPriority().GetWithDefault(1);
						if (bu->GetDVBWeight().IsSet() && bu->GetDVBWeight().Value() < 0)
						{
							UE_LOGF(LogElectraPlayer, Error, "Invalid @weight found on <BaseURL> element while resolving URL");
							return false;
						}
						nxt.DVBweight = bu->GetDVBWeight().GetWithDefault(1);
					}
					else
					{
						nxt.CDN = Base[nI].CDN;
						nxt.DVBpriority = Base[nI].DVBpriority;
						nxt.DVBweight = Base[nI].DVBweight;
					}
					nxt.ATO = Base[nI].ATO + (bu->GetAvailabilityTimeOffset().IsValid() ? bu->GetAvailabilityTimeOffset() : FTimeValue::GetZero());
					nxt.ByteRange = bu->GetByteRange().Len() ? bu->GetByteRange() : Base[nI].ByteRange;
					if (!Base[nI].ATOComplete.IsSet())
					{
						nxt.ATOComplete = bu->GetAvailabilityTimeComplete();
					}
					else if (!Base[nI].ATOComplete.IsSet() && bu->GetAvailabilityTimeComplete().IsSet() && Base[nI].ATOComplete.Value() != bu->GetAvailabilityTimeComplete().Value())
					{
						UE_LOGF(LogElectraPlayer, Warning, "Inconsistent @availabilityTimeComplete found on hierarchical <BaseURL> elements while resolving URL");
						// Use the child element's attribute here.
						nxt.ATOComplete = bu->GetAvailabilityTimeComplete();
					}
				}
			}
			// The working list now becomes the new base list.
			Base = MoveTemp(Work);
		}

		// Add unique (by URL and CDN) results to the return list.
		// If on any hierarchy level there is a relative AND an absolute URL we may end up with the same
		// URL multiple times.
		for(auto& It : Base)
		{
			if (!OutCandidateURLs.Contains(It))
			{
				OutCandidateURLs.Add(It);
			}
		}
		return true;
	}

	/**
	 * Collects BaseURL elements beginning at the specified element up the MPD hierarchy.
	 * At most one BaseURL per level is added to the output, with only the first BaseURL element being used.
	 * Results are added in hierarchy order, eg. MPD, Period, Adaptation, Representation.
	 */
	void GetAllHierarchyBaseURLs(IPlayerSessionServices* InPlayerSessionServices, TArray<TSharedPtrTS<const FDashMPD_BaseURLType>>& OutBaseURLs, TSharedPtrTS<const IDashMPDElement> StartingAt)
	{
		while(StartingAt.IsValid())
		{
			const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& ElementBaseURLs = StartingAt->GetBaseURLs();
			if (ElementBaseURLs.Num())
			{
				OutBaseURLs.Insert(ElementBaseURLs[0], 0);
			}
			StartingAt = StartingAt->GetParentElement();
		}
	}

	/**
	 * Collects UrlQueryInfo and/or ExtUrlQueryInfo/ExtHttpHeaderInfo elements from EssentialProperty and SupplementalProperty elements
	 * beginning at the specified element up the MPD hierarchy.
	 * At most one element matching the requested type will be returned per hierarchy level in accordance with the specification (see I.2.1).
	 * Results are added in hierarchy order, eg. MPD, Period, Adaptation, Representation
	 */
	void GetAllHierarchyUrlQueries(TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>>& OutUrlQueries, TSharedPtrTS<const IDashMPDElement> StartingAt, EUrlQueryRequestType ForRequestType, bool bInclude2014)
	{
		while(StartingAt.IsValid())
		{
			TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>> UrlQueries;
			// Get the essential and supplemental properties into a single list for processing.
			TArray<TSharedPtrTS<FDashMPD_DescriptorType>> Desc;
			Desc = StartingAt->GetEssentialProperties().FilterByPredicate([bInclude2014](const TSharedPtrTS<FDashMPD_DescriptorType>& d)
				{ return d->GetSchemeIdUri().Equals(UrlParam2016) || (bInclude2014 && d->GetSchemeIdUri().Equals(UrlParam2014)); });
			Desc.Append(StartingAt->GetSupplementalProperties().FilterByPredicate([bInclude2014](const TSharedPtrTS<FDashMPD_DescriptorType>& d)
				{ return d->GetSchemeIdUri().Equals(UrlParam2016) || (bInclude2014 && d->GetSchemeIdUri().Equals(UrlParam2014)); }));
			for(int32 i=0; i<Desc.Num(); ++i)
			{
				const TArray<TSharedPtrTS<IDashMPDElement>>& WellKnown = Desc[i]->GetWellKnownDescriptors();
				for(int32 j=0; j<WellKnown.Num(); ++j)
				{
					if (WellKnown[j]->GetElementType() == IDashMPDElement::EType::URLQueryInfo)
					{
						TSharedPtrTS<FDashMPD_UrlQueryInfoType> uq = StaticCastSharedPtr<FDashMPD_UrlQueryInfoType>(WellKnown[j]);
						const TArray<FString>& IncIn = uq->GetIncludeInRequests();
						for(int32 k=0; k<IncIn.Num(); ++k)
						{
							// NOTE: The '*' wildcard is not yet officially supported.
							//       This is an extension proposed in ETSI TS 103 998 V1.1.1 (2024-01)
							if (IncIn[k].Equals(TEXT("*")) ||
								(ForRequestType == EUrlQueryRequestType::Segment  && IncIn[k].Equals(TEXT("segment"))) ||
								(ForRequestType == EUrlQueryRequestType::Xlink    && IncIn[k].Equals(TEXT("xlink"))) ||
								(ForRequestType == EUrlQueryRequestType::Mpd      && IncIn[k].Equals(TEXT("mpd"))) ||
								(ForRequestType == EUrlQueryRequestType::Callback && IncIn[k].Equals(TEXT("callback"))) ||
								(ForRequestType == EUrlQueryRequestType::Chaining && IncIn[k].Equals(TEXT("chaining"))) ||
								(ForRequestType == EUrlQueryRequestType::Fallback && IncIn[k].Equals(TEXT("fallback"))) ||
								(ForRequestType == EUrlQueryRequestType::Steering && IncIn[k].Equals(TEXT("steering"))))
							{
								UrlQueries.Emplace(MoveTemp(uq));
							}
						}
					}
				}
			}
			OutUrlQueries.Insert(UrlQueries, 0);
			StartingAt = StartingAt->GetParentElement();
		}
	}

	/**
	 * Applies the URL query elements to the specified URL in hierarchy order.
	 */
	FErrorDetail ApplyUrlQueries(IPlayerSessionServices* PlayerSessionServices, const FString& InMPDUrl, FString& InOutURL, FString& OutRequestHeader, const TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>>& UrlQueries)
	{
		FErrorDetail Error;
		// Short circuit zero queries.
		if (UrlQueries.Num() == 0)
		{
			return Error;
		}

		// Short circuit data URLs. Since the URL is plain text data of the content we must not append any query parameters
		if (InOutURL.StartsWith(SchemeDATA))
		{
			return Error;
		}

		bool bAllowedToUse = true;
		bool bMpdUrlParsed = false;
		bool bInUrlParsed = false;
		FURL_RFC3986 MpdUrl, InUrl;

		FString TotalFinalQueryString;
		FString TotalFinalHttpRequestHeader;
		int32 NumBadQueryTemplates = 0;

		// The URL queries are in correct hierarchy order.
		for(int32 nUrlQuery=0; nUrlQuery<UrlQueries.Num(); ++nUrlQuery)
		{
			const TSharedPtrTS<FDashMPD_UrlQueryInfoType>& uq = UrlQueries[nUrlQuery];

			// FIXME: Remote elements need to be dereferenced but we do not support xlink on them at the moment.
			if (uq->GetXLink().IsSet())
			{
				Error = CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("xlink is not supported on UrlQueryInfo / ExtUrlQueryInfo elements!")), ERRCODE_DASH_MPD_BUILDER_XLINK_NOT_SUPPORTED_ON_ELEMENT);
				return Error;
			}

			// The standard does not say what to do when @queryTemplate is empty. Since everything is selected when
			// it is set to "$querypart$" we have to assume that if it is empty nothing is to be selected in which
			// case we skip over this item.
			FString queryTemplate = uq->GetQueryTemplate();
			if (queryTemplate.IsEmpty())
			{
				continue;
			}

			TArray<FURL_RFC3986::FQueryParam> QueryParamList;
			// Using @headerParamSource?
			if (uq->GetHeaderParamSources().Num() == 0)
			{
				// No. In this case @useMPDUrlQuery and/or @queryString may be set.
				// If the MPD URL is to be used add its query parameters to the list.
				if (uq->GetUseMPDUrlQuery())
				{
					if (!bMpdUrlParsed)
					{
						bMpdUrlParsed = true;
						MpdUrl.Parse(InMPDUrl);
					}
					MpdUrl.GetQueryParams(QueryParamList, false);
				}
				// Then, if there is an explicit query string, break it down and add it to the list.
				if (!uq->GetQueryString().IsEmpty())
				{
					FURL_RFC3986::GetQueryParams(QueryParamList, uq->GetQueryString(), false);
				}

				// Filter out any urn: schemes. We do not support any of them.
				for(int32 i=0; i<QueryParamList.Num(); ++i)
				{
					if (QueryParamList[i].Value.StartsWith(TEXT("$urn:")))
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Ignoring unsupported scheme \"%s\" in UrlQueryInfo."), *QueryParamList[i].Value));
						QueryParamList.RemoveAt(i);
						--i;
					}
				}

				// Same origin only?
				// Note: The standard states that if the parameters are instantiated the the MPD (@queryString) or the MPD URL (@useMPDUrlQuery)
				//       the origin is the MPD URL!
				if (uq->GetSameOriginOnly())
				{
					if (!bMpdUrlParsed)
					{
						bMpdUrlParsed = true;
						MpdUrl.Parse(InMPDUrl);
					}
					if (!bInUrlParsed)
					{
						bInUrlParsed = true;
						InUrl.Parse(InOutURL);
					}
					if (!InUrl.HasSameOriginAs(MpdUrl))
					{
						// Not the same origin. Skip over this element.
						continue;
					}
				}
			}
			else
			{
				TSharedPtrTS<IPlayerEntityCache> EntityCache = PlayerSessionServices ? PlayerSessionServices->GetEntityCache() : nullptr;
				if (!EntityCache.IsValid())
				{
					continue;
				}
				/*
					When @headerParamSource is used then neither @useMPDUrlQuery and @queryString must be set.
					We do not validate this, we merely ignore them.
					NOTE: There is a conflict in the standard. In I.3.2 Table I.3 states for @headerParamSource:
							"If this attribute is present then: (a) @queryTemplate attribute shall be present and shall contain the $header:<header-name>$ identifier,
							and (b) neither @useMPDUrlQuery nor @queryString attribute shall be present."
						  with (b) being the noteworthy part.
						  Yet in I.3.4.2 is stated "3) Otherwise the value of initialQueryString is given by @queryString."
						  which is contradicting as there is no @queryString to be used.
						  We go with the table and clause (b).
				*/
				for(int32 nHdrSrc=0; nHdrSrc<uq->GetHeaderParamSources().Num(); ++nHdrSrc)
				{
					FString ResponseHeaderURL;
					TArray<HTTP::FHTTPHeader> ResponseHeaders;
					if (uq->GetHeaderParamSources()[nHdrSrc].Equals(TEXT("segment")))
					{
						EntityCache->GetRecentResponseHeaders(ResponseHeaderURL, ResponseHeaders, IPlayerEntityCache::EEntityType::Segment);
					}
					else if (uq->GetHeaderParamSources()[nHdrSrc].Equals(TEXT("xlink")))
					{
						EntityCache->GetRecentResponseHeaders(ResponseHeaderURL, ResponseHeaders, IPlayerEntityCache::EEntityType::XLink);
					}
					else if (uq->GetHeaderParamSources()[nHdrSrc].Equals(TEXT("mpd")))
					{
						EntityCache->GetRecentResponseHeaders(ResponseHeaderURL, ResponseHeaders, IPlayerEntityCache::EEntityType::Document);
					}
					else if (uq->GetHeaderParamSources()[nHdrSrc].Equals(TEXT("callback")))
					{
						EntityCache->GetRecentResponseHeaders(ResponseHeaderURL, ResponseHeaders, IPlayerEntityCache::EEntityType::Callback);
					}
					else
					{
						// Unsupported source. Skip over it.
						continue;
					}
					// Is there something available?
					if (!ResponseHeaderURL.IsEmpty() && ResponseHeaders.Num())
					{
						// Same origin only?
						if (uq->GetSameOriginOnly())
						{
							FURL_RFC3986 HeaderUrl;
							HeaderUrl.Parse(ResponseHeaderURL);
							if (!bInUrlParsed)
							{
								bInUrlParsed = true;
								InUrl.Parse(InOutURL);
							}
							if (!InUrl.HasSameOriginAs(HeaderUrl))
							{
								// Not the same origin. Skip over this element.
								continue;
							}
						}
						// Add all response headers to the parameter list.
						for(int32 i=0; i<ResponseHeaders.Num(); ++i)
						{
							// Only add if it isn't there yet, otherwise update it.
							bool bUpdated = false;
							for(int32 j=0; j<QueryParamList.Num(); ++j)
							{
								if (QueryParamList[j].Name.Equals(*ResponseHeaders[i].Header))
								{
									QueryParamList[j].Value = ResponseHeaders[i].Value;
									bUpdated = true;
									break;
								}
							}
							if (!bUpdated)
							{
								QueryParamList.Emplace(FURL_RFC3986::FQueryParam({MoveTemp(ResponseHeaders[i].Header), MoveTemp(ResponseHeaders[i].Value)}));
							}
						}
					}
				}
			}

			// Process the elements as described by the query template.
			FString FinalQueryString;
			while(!queryTemplate.IsEmpty())
			{
				int32 tokenPos = INDEX_NONE;
				if (!queryTemplate.FindChar(TCHAR('$'), tokenPos))
				{
					// The queryTemplate needs to bring all required '&' with it. We do not have to add anything besides what is in the template.
					FinalQueryString.Append(queryTemplate);
					break;
				}
				else
				{
					// Append everything up to the first token.
					if (tokenPos)
					{
						FinalQueryString.Append(queryTemplate.Mid(0, tokenPos));
					}
					// Need to find another token.
					int32 token2Pos = queryTemplate.Find(TEXT("$"), ESearchCase::CaseSensitive, ESearchDir::FromStart, tokenPos+1);
					if (token2Pos != INDEX_NONE)
					{
						FString token(queryTemplate.Mid(tokenPos+1, token2Pos-tokenPos-1));
						queryTemplate.RightChopInline(token2Pos+1, EAllowShrinking::No);
						// An empty token results from "$$" used to insert a single '$'.
						if (token.IsEmpty())
						{
							FinalQueryString.AppendChar(TCHAR('$'));
						}
						// $querypart$ ?
						else if (token.Equals(TEXT("querypart")))
						{
							FString queryPart;
							for(int32 i=0; i<QueryParamList.Num(); ++i)
							{
								if (i)
								{
									queryPart.AppendChar(TCHAR('&'));
								}
								queryPart.Append(QueryParamList[i].Name);
								queryPart.AppendChar(TCHAR('='));
								queryPart.Append(QueryParamList[i].Value);
							}
							FinalQueryString.Append(queryPart);
						}
						// $query:<param>$ ?
						else if (token.StartsWith(TEXT("query:"), ESearchCase::CaseSensitive))
						{
							token.RightChopInline(6, EAllowShrinking::No);
							int32 Index = QueryParamList.IndexOfByPredicate([token](const FURL_RFC3986::FQueryParam& qp){ return qp.Name.Equals(token); });
							if (Index != INDEX_NONE)
							{
								// Emit a warning if there is no '=' to append the value to.
								if (FinalQueryString.IsEmpty() || (FinalQueryString.Len() && FinalQueryString[FinalQueryString.Len()-1] != TCHAR('=')))
								{
									LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("UrlQueryInfo does not provide a '=' to append the value of \"%s\" to."), *token));
								}

								FinalQueryString.Append(QueryParamList[Index].Value);
							}
						}
						// $header:<header-name>$ ?
						else if (token.StartsWith(TEXT("header:"), ESearchCase::CaseSensitive))
						{
							token.RightChopInline(7, EAllowShrinking::No);
							int32 Index = QueryParamList.IndexOfByPredicate([token](const FURL_RFC3986::FQueryParam& qp){ return qp.Name.Equals(token); });
							if (Index != INDEX_NONE)
							{
								// Emit a warning if there is no '=' to append the value to.
								if (FinalQueryString.IsEmpty() || (FinalQueryString.Len() && FinalQueryString[FinalQueryString.Len()-1] != TCHAR('=')))
								{
									LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("UrlQueryInfo does not provide a '=' to append the value of \"%s\" to."), *token));
								}

								FinalQueryString.Append(QueryParamList[Index].Value);
							}
						}
						else
						{
							// Unknown. Skip over and continue.
						}
					}
					else
					{
						// Bad query template string. Ignore this UrlQueryInfo.
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("UrlQueryInfo has a malformed @queryTemplate string.")));
						FinalQueryString.Empty();
						++NumBadQueryTemplates;
						break;
					}
				}
			}

			// In case parameter replacement failed we could have sequences of double ampersands (&&) or a last/only & char.
			while(FinalQueryString.ReplaceInline(TEXT("&&"), TEXT("&")))
			{ }
			if (FinalQueryString.Len() && FinalQueryString[FinalQueryString.Len()-1] == TCHAR('&'))
			{
				FinalQueryString.LeftChopInline(1);
			}

			// Where does the output go? URL query param or HTTP request header?
			if (uq->GetExtendedUrlInfoType() == FDashMPD_UrlQueryInfoType::EExtendedUrlInfoType::ExtUrlQueryInfo)
			{
				// Do we need to add a '&' before appending the queryPart to the finalQueryString?
				if (FinalQueryString.Len() && TotalFinalQueryString.Len() && TotalFinalQueryString[0] != TCHAR('&'))
				{
					TotalFinalQueryString.AppendChar(TCHAR('&'));
				}
				TotalFinalQueryString.Append(FinalQueryString);
			}
			else if (uq->GetExtendedUrlInfoType() == FDashMPD_UrlQueryInfoType::EExtendedUrlInfoType::ExtHttpHeaderInfo)
			{
				FString FinalHeaderString = FinalQueryString.Replace(TEXT("&"), TEXT(", "), ESearchCase::CaseSensitive);
				if (FinalHeaderString.Len() && TotalFinalHttpRequestHeader.Len() > 1 && TotalFinalHttpRequestHeader[0] != TCHAR(',') && TotalFinalHttpRequestHeader[1] != TCHAR(' '))
				{
					TotalFinalHttpRequestHeader.Append(TEXT(", "));
				}
				TotalFinalHttpRequestHeader.Append(FinalHeaderString);
			}
			else
			{
				// Unsupported. Ignore.
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Unsupported type of UrlQueryInfo")));
			}
		}

		// If there was a bad @queryTemplate we fail if all have failed. If even one succeeded we take it.
		if (NumBadQueryTemplates == UrlQueries.Num())
		{
			// This is not an actual error. Instead we pretend as if we do not understand the UrlQuery element.
			return Error;
		}

		if (!bInUrlParsed)
		{
			bInUrlParsed = true;
			InUrl.Parse(InOutURL);
		}
		InUrl.AddOrUpdateQueryParams(TotalFinalQueryString);
		InOutURL = InUrl.Get();
		OutRequestHeader = TotalFinalHttpRequestHeader;

		return Error;
	}


	/**
	 * Resolves an element URLs against its parent BaseURL elements in the hierarchy.
	 * Returns true if an absolute URL could be generated, false if not.
	 * Since the MPD URL had to be an absolute URL this cannot actually fail.
	 */
	bool BuildAbsoluteElementURL(FString& OutURL, FTimeValue& ATO, TMediaOptionalValue<bool>& bATOComplete, const FString& DocumentURL, const TArray<TSharedPtrTS<const FDashMPD_BaseURLType>>& BaseURLs, const FString& InElementURL)
	{
		FTimeValue SumOfUrlATOs(FTimeValue::GetZero());
		ATO.SetToZero();
		bATOComplete.Reset();

		// Short circuit data: URLs
		if (InElementURL.StartsWith(TEXT("data:")))
		{
			OutURL = InElementURL;
			return true;
		}

		FURL_RFC3986 UrlParser;

		// If there is no element URL and BaseURLs this is probably for an MPD update in which case we return the original document URL.
		if (InElementURL.IsEmpty() && BaseURLs.Num() == 0)
		{
			OutURL = DocumentURL;
			return true;
		}

		// Starting with the document URL apply the <BaseURL> elements down the path to the element.
		// If at any level an absolute URL is specified it replaces the URL as a whole, otherwise
		// relative URLs are resolved against the parent.
		if (!UrlParser.Parse(DocumentURL))
		{
			return false;
		}
		for(int32 nBase=0; nBase<BaseURLs.Num(); ++nBase)
		{
			UrlParser.ResolveWith(BaseURLs[nBase]->GetURL());
			SumOfUrlATOs += BaseURLs[nBase]->GetAvailabilityTimeOffset();
			if (!bATOComplete.IsSet())
			{
				bATOComplete = BaseURLs[nBase]->GetAvailabilityTimeComplete();
			}
			else if (BaseURLs[nBase]->GetAvailabilityTimeComplete().IsSet() && BaseURLs[nBase]->GetAvailabilityTimeComplete().Value() != bATOComplete.Value())
			{
				// Inconsistent availabilityTimeComplete across the hierarchy.
				// TBD
			}
		}
		UrlParser.ResolveWith(InElementURL);
		OutURL = UrlParser.Get();
		ATO = SumOfUrlATOs;
		return UrlParser.IsAbsolute();
	}

	FString ApplyAnnexEByteRange(FString InURL, FString InRange, FString InByteRangeParam)
	{
		if (InRange.IsEmpty() || InByteRangeParam.IsEmpty())
		{
			return MoveTemp(InURL);
		}
		int32 DashPos = INDEX_NONE;
		if (!InRange.FindChar(TCHAR('-'), DashPos))
		{
			return MoveTemp(InURL);
		}

		int32 QueryPos = INDEX_NONE;
		InURL.FindChar(TCHAR('?'), QueryPos);
		// Process the elements as described by the byte range.
		FString NewURL;
		while(!InByteRangeParam.IsEmpty())
		{
			int32 tokenPos = INDEX_NONE;
			if (!InByteRangeParam.FindChar(TCHAR('$'), tokenPos))
			{
				NewURL.Append(InByteRangeParam);
				break;
			}
			else
			{
				// Append everything up to the first token.
				if (tokenPos)
				{
					NewURL.Append(InByteRangeParam.Mid(0, tokenPos));
				}
				// Need to find another token.
				int32 token2Pos = InByteRangeParam.Find(TEXT("$"), ESearchCase::CaseSensitive, ESearchDir::FromStart, tokenPos+1);
				if (token2Pos != INDEX_NONE)
				{
					FString token(InByteRangeParam.Mid(tokenPos+1, token2Pos-tokenPos-1));
					InByteRangeParam.RightChopInline(token2Pos+1, EAllowShrinking::No);
					// An empty token results from "$$" used to insert a single '$'.
					if (token.IsEmpty())
					{
						NewURL.AppendChar(TCHAR('$'));
					}
					// $base$ ?
					else if (token.Equals(TEXT("base")))
					{
						NewURL.Append(InURL.Mid(0, QueryPos != INDEX_NONE ? QueryPos : MAX_int32));
					}
					// $query$ ?
					else if (token.Equals(TEXT("query")))
					{
						FString query = InURL.Mid(QueryPos != INDEX_NONE ? QueryPos + 1 : MAX_int32);
						if (query.IsEmpty())
						{
							// Remove preceeding separator character (which we assume it to be meant to be the ampersand)
							if (NewURL.Len() && NewURL[NewURL.Len() - 1] == TCHAR('&'))
							{
								NewURL.LeftChopInline(1, EAllowShrinking::No);
							}
							// If the next char in the template is a separator character it is to be removed.
							if (InByteRangeParam.Len() && InByteRangeParam[0] == TCHAR('&'))
							{
								InByteRangeParam.RightChopInline(1, EAllowShrinking::No);
							}
						}
						else
						{
							NewURL.Append(query);
						}
					}
					// $first$ ?
					else if (token.Equals(TEXT("first")))
					{
						NewURL.Append(InRange.Mid(0, DashPos));
					}
					// $last$ ?
					else if (token.Equals(TEXT("last")))
					{
						NewURL.Append(InRange.Mid(DashPos + 1));
					}
					else
					{
						// Unknown. Skip over and continue.
					}
				}
				else
				{
					// Bad template string. Ignore this.
					return InURL;
				}
			}
		}
		return NewURL;
	}

	FString ApplyAnnexEByteRange(FString InURL, FString InRange, const TArray<TSharedPtrTS<const FDashMPD_BaseURLType>>& BaseURLs)
	{
		// Short circuit missing parameters.
		if (InRange.IsEmpty() || BaseURLs.Num() == 0)
		{
			return MoveTemp(InURL);
		}
		// Find the first <BaseURL> element having a byteRange attribute.
		FString byteRange;
		for(int32 i=0; i<BaseURLs.Num(); ++i)
		{
			byteRange = BaseURLs[i]->GetByteRange();
			if (!byteRange.IsEmpty())
			{
				break;
			}
		}
		return ApplyAnnexEByteRange(InURL, InRange, byteRange);
	}


} // namespace DASHUrlHelpers



class FManifestBuilderDASH : public IManifestBuilderDASH
{
public:
	FManifestBuilderDASH(IPlayerSessionServices* InPlayerSessionServices);
	virtual ~FManifestBuilderDASH() = default;

	virtual FErrorDetail BuildFromMPD(TSharedPtrTS<FManifestDASHInternal>& OutMPD, TCHAR* InOutMPDXML, const FString& EffectiveURL, const FString& ETag, const TArray<FElectraHTTPStreamHeader>& InResponseHeaders) override;

private:
	ELECTRA_IMPL_DEFAULT_ERROR_METHODS(DASHMPDBuilder);

	IPlayerSessionServices* PlayerSessionServices = nullptr;
};


/*********************************************************************************************************************/

IManifestBuilderDASH* IManifestBuilderDASH::Create(IPlayerSessionServices* InPlayerSessionServices)
{
	return new FManifestBuilderDASH(InPlayerSessionServices);
}

/*********************************************************************************************************************/

FManifestBuilderDASH::FManifestBuilderDASH(IPlayerSessionServices* InPlayerSessionServices)
	: PlayerSessionServices(InPlayerSessionServices)
{
}

FErrorDetail FManifestBuilderDASH::BuildFromMPD(TSharedPtrTS<FManifestDASHInternal>& OutMPD, TCHAR* InOutMPDXML, const FString& EffectiveURL, const FString& ETag, const TArray<FElectraHTTPStreamHeader>& InResponseHeaders)
{
	TSharedPtrTS<FDashMPD_MPDType> NewMPD;
	TArray<TWeakPtrTS<IDashMPDElement>> XLinkElements;
	FDashMPD_RootEntities RootEntities;

	FErrorDetail Error = IManifestParserDASH::BuildFromMPD(RootEntities, XLinkElements, InOutMPDXML, TEXT("MPD"), PlayerSessionServices);
	if (Error.IsSet())
	{
		return Error;
	}
	if (RootEntities.MPDs.Num() == 0)
	{
		return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("No root <MPD> element found")), ERRCODE_DASH_MPD_BUILDER_MISSING_REQUIRED_ELEMENT);
	}
	NewMPD = RootEntities.MPDs[0];
	NewMPD->SetDocumentURL(EffectiveURL);
	NewMPD->SetETag(ETag);

	// Check if this MPD uses a profile we can handle. (If our array is empty we claim to handle anything)
	bool bHasUsableProfile = UE_ARRAY_COUNT(SupportedProfiles) == 0;
	const TArray<FString>& MPDProfiles = NewMPD->GetProfiles();
	if (MPDProfiles.Num() == 0)
	{
		return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("MPD@profiles is required")), ERRCODE_DASH_MPD_BUILDER_MISSING_REQUIRED_ATTRIBUTE);
	}
	bool bUsesDVBDashProfile = false;
	for(int32 nSupportedProfile=0; !bHasUsableProfile && nSupportedProfile<UE_ARRAY_COUNT(SupportedProfiles); ++nSupportedProfile)
	{
		for(int32 nProfile=0; nProfile<MPDProfiles.Num(); ++nProfile)
		{
			if (MPDProfiles[nProfile].Equals(SupportedProfiles[nSupportedProfile]))
			{
				bHasUsableProfile = true;
			}
			if (MPDProfiles[nProfile].StartsWith(DVBDashProfilePrefix))
			{
				bUsesDVBDashProfile = true;
			}
		}
	}
	if (!bHasUsableProfile)
	{
		return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("MPD is of no supported profile")), ERRCODE_DASH_MPD_BUILDER_UNSUPPORTED_PROFILE);
	}
	NewMPD->SetDVBDash(bUsesDVBDashProfile);

	// Check if this MPD requires essential properties we do not understand.
	FString DashProtocol(TEXT("dash"));
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& EssentialProperties = NewMPD->GetEssentialProperties();
	TArray<bool> IsSupported;
	IsSupported.SetNum(EssentialProperties.Num());
	// Not supported by default.
	for(int32 i=0; i<IsSupported.Num(); ++i)
	{
		IsSupported[i] = false;
	}
	for(int32 nEssProp=0; nEssProp<EssentialProperties.Num(); ++nEssProp)
	{
		// If supported there is no need to check again.
		if (IsSupported[nEssProp])
		{
			continue;
		}
		// Check against a fixed list of properties we support internally.
		for(int32 nSupportedEssProp=0; nSupportedEssProp<UE_ARRAY_COUNT(SupportedEssentialProperties); ++nSupportedEssProp)
		{
			if (EssentialProperties[nEssProp]->GetSchemeIdUri().Equals(SupportedEssentialProperties[nSupportedEssProp]))
			{
				IsSupported[nEssProp] = true;
				break;
			}
		}
		// If still not supported and not a DASH prefix check with the external registration
		if (!IsSupported[nEssProp] && !EssentialProperties[nEssProp]->GetSchemeIdUri().StartsWith(EssentialDashReservedPropertyPrefix))
		{
			IPlayerSessionServices::FPlaylistProperty Prop;
			Prop.Tag = EssentialProperties[nEssProp]->GetSchemeIdUri();
			Prop.Value = EssentialProperties[nEssProp]->GetValue();
			IPlayerSessionServices::ECustomPropertyResult Result = PlayerSessionServices->ValidateMainPlaylistCustomProperty(DashProtocol, EffectiveURL, InResponseHeaders, Prop);
			// Essential properties need to be explicitly accepted. `Default` means the external registry doesn't know or care, which means no one knows how to handle it.
			IsSupported[nEssProp] = Result == IPlayerSessionServices::ECustomPropertyResult::Accept;
		}

		// If supported propagate this to all elements sharing the same ID
		if (IsSupported[nEssProp] && EssentialProperties[nEssProp]->GetID().Len())
		{
			for(int32 j=0; j<EssentialProperties.Num(); ++j)
			{
				if (j!=nEssProp && EssentialProperties[nEssProp]->GetID().Equals(EssentialProperties[j]->GetID()))
				{
					IsSupported[j] = true;
				}
			}
		}
	}
	// Call external registry with an end-of-properties call.
	if (PlayerSessionServices->ValidateMainPlaylistCustomProperty(DashProtocol, EffectiveURL, InResponseHeaders, IPlayerSessionServices::FPlaylistProperty()) == IPlayerSessionServices::ECustomPropertyResult::Reject)
	{
		return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("MPD has been rejected by application")), ERRCODE_DASH_MPD_BUILDER_REJECTED_BY_APPLICATION);
	}
	// If any one essential property is not supported we cannot commence playback.
	for(int32 nEssProp=0; nEssProp<EssentialProperties.Num(); ++nEssProp)
	{
		if (!IsSupported[nEssProp])
		{
			return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("MPD requires unsupported EssentialProperty \"%s\" for playback"), *EssentialProperties[nEssProp]->GetSchemeIdUri()), ERRCODE_DASH_MPD_BUILDER_UNSUPPORTED_ESSENTIAL_PROPERTY);
		}
	}

	// For the purpose of loading an MPD from scratch we need to look only at xlink:actuate onLoad elements.
	// These MAY reference other onRequest elements (like UrlQueryInfo) in the process.
	for(int32 nRemoteElement=0; nRemoteElement<XLinkElements.Num(); ++nRemoteElement)
	{
		TSharedPtrTS<IDashMPDElement> XElem = XLinkElements[nRemoteElement].Pin();
		if (XElem.IsValid() && !XElem->GetXLink().GetActuate().Equals(XLinkActuateOnLoad))
		{
			XLinkElements.RemoveAt(nRemoteElement);
			--nRemoteElement;
		}
	}

	TSharedPtrTS<FManifestDASHInternal> NewManifest = MakeSharedTS<FManifestDASHInternal>();
	Error = NewManifest->Build(PlayerSessionServices, NewMPD, MoveTemp(XLinkElements));
	if (Error.IsOK() || Error.IsTryAgain())
	{
		OutMPD = MoveTemp(NewManifest);
	}
	return Error;
}



FErrorDetail FManifestDASHInternal::PrepareRemoteElementLoadRequest(TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, TWeakPtrTS<IDashMPDElement> InElementWithXLink, int64 RequestID)
{
	FErrorDetail Error;
	TSharedPtrTS<IDashMPDElement> XElem = InElementWithXLink.Pin();
	if (XElem.IsValid())
	{
		IDashMPDElement::FXLink& xl = XElem->GetXLink();

		// There should not be a pending request when we get here.
		check(!xl.LoadRequest.IsValid());
		if (xl.LoadRequest.IsValid())
		{
			// If there is one we assume it is in flight and return without raising an error.
			return Error;
		}

		FMPDLoadRequestDASH::ELoadType LoadReqType = FMPDLoadRequestDASH::ELoadType::MPD;
		switch(XElem->GetElementType())
		{
			case IDashMPDElement::EType::Period:				LoadReqType = FMPDLoadRequestDASH::ELoadType::XLink_Period;				break;
			case IDashMPDElement::EType::AdaptationSet:			LoadReqType = FMPDLoadRequestDASH::ELoadType::XLink_AdaptationSet;		break;
			case IDashMPDElement::EType::URLQueryInfo:			LoadReqType = FMPDLoadRequestDASH::ELoadType::XLink_URLQuery;			break;
			case IDashMPDElement::EType::EventStream:			LoadReqType = FMPDLoadRequestDASH::ELoadType::XLink_EventStream;		break;
			case IDashMPDElement::EType::SegmentList:			LoadReqType = FMPDLoadRequestDASH::ELoadType::XLink_SegmentList;		break;
			case IDashMPDElement::EType::InitializationSet:		LoadReqType = FMPDLoadRequestDASH::ELoadType::XLink_InitializationSet;	break;
			default:
				return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("xlink on %s element is not supported"), *XElem->GetName()), ERRCODE_DASH_MPD_BUILDER_XLINK_NOT_SUPPORTED_ON_ELEMENT);
		}

		FString URL;
		FString RequestHeader;
		FTimeValue UrlATO;
		TMediaOptionalValue<bool> bATOComplete;
		// Get the xlink:href
		FString XlinkHRef = xl.GetHref();
		if (!XlinkHRef.Equals(XLinkResolveToZero))
		{
			if (DASHUrlHelpers::IsAbsoluteURL(XlinkHRef))
			{
				URL = MoveTemp(XlinkHRef);
			}
			else
			{
				TArray<TSharedPtrTS<const FDashMPD_BaseURLType>> OutBaseURLs;
				DASHUrlHelpers::GetAllHierarchyBaseURLs(PlayerSessionServices, OutBaseURLs, XElem->GetParentElement());
				if (!DASHUrlHelpers::BuildAbsoluteElementURL(URL, UrlATO, bATOComplete, MPDRoot->GetDocumentURL(), OutBaseURLs, XlinkHRef))
				{
					// Not resolving to an absolute URL is very unlikely as we had to load the MPD itself from somewhere.
					return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("xlink:href did not resolve to an absolute URL")), ERRCODE_DASH_MPD_BUILDER_URL_FAILED_TO_RESOLVE);
				}
			}

			// The URL query might need to be changed. Look for the UrlQuery properties.
			TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>> UrlQueries;
			DASHUrlHelpers::GetAllHierarchyUrlQueries(UrlQueries, XElem->GetParentElement(), DASHUrlHelpers::EUrlQueryRequestType::Xlink, false);
			// FIXME: pass xlink requests along to allow UrlQuery to do xlink as well (recursively).
			Error = DASHUrlHelpers::ApplyUrlQueries(PlayerSessionServices, MPDRoot->GetDocumentURL(), URL, RequestHeader, UrlQueries);
			if (Error.IsSet())
			{
				return Error;
			}
		}
		else
		{
			URL = XlinkHRef;
		}
		// Create the request.
		TSharedPtrTS<FMPDLoadRequestDASH> LoadReq = MakeSharedTS<FMPDLoadRequestDASH>();
		LoadReq->LoadType = LoadReqType;
		LoadReq->Url.URL = URL;
		if (RequestHeader.Len())
		{
			LoadReq->Headers.Emplace(HTTP::FHTTPHeader({DASH::HTTPHeaderOptionName, RequestHeader}));
		}
		LoadReq->XLinkElement = XElem;

		// Put the request on the xlink element so we know it has been requested and the result is pending.
		xl.LoadRequest = LoadReq;
		xl.LastResolveID = RequestID;

		// Add the request to the list to return for execution.
		OutRemoteElementLoadRequests.Emplace(LoadReq);
	}
	return Error;
}


void FManifestDASHInternal::InjectEpicTimingSources()
{
	const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& UTCTimings = MPDRoot->GetUTCTimings();

	const TArray<FURL_RFC3986::FQueryParam>& frgs(ReplayEventParams.GetUrlFragmentParams());
	for(int32 i=0,iMax=frgs.Num(); i<iMax; ++i)
	{
		if (frgs[i].Name.Equals(CustomOptions::Custom_EpicUTCUrl))
		{
			MPDRoot->AddUTCTimingElement(FString(), DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_httpxsdate2014, FString(), frgs[i].Value);
			bDidInjectUTCTimingElements = true;
		}
		else if (frgs[i].Name.Equals(CustomOptions::Custom_EpicUTCNow))
		{
			MPDRoot->AddUTCTimingElement(FString(), DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_direct2014, FString(), frgs[i].Value);
			bDidInjectUTCTimingElements = true;
		}
	}
}

void FManifestDASHInternal::TransformIntoEpicEvent()
{
	// For this to work the presentation must be 'static'
	if (PresentationType == EPresentationType::Static)
	{
		FString Arg;
		bool bStaticStart = false;
		bool bDynamicStart = false;

		const TArray<FURL_RFC3986::FQueryParam>& frgs(ReplayEventParams.GetUrlFragmentParams());
		for(int32 i=0,iMax=frgs.Num(); i<iMax; ++i)
		{
			bStaticStart = frgs[i].Name.Equals(CustomOptions::Custom_EpicStaticStart);
			bDynamicStart = frgs[i].Name.Equals(CustomOptions::Custom_EpicDynamicStart);
			if (bStaticStart || bDynamicStart)
			{
				Arg = frgs[i].Value;
				break;
			}
		}

		// Any indication that this is a replay event?
		if (!bStaticStart && !bDynamicStart)
		{
			return;
		}

		TArray<FString> Params;
		if (!Arg.IsEmpty())
		{
			const TCHAR* const Delimiter = TEXT(",");
			Arg.ParseIntoArray(Params, Delimiter, true);
		}
		if (Params.Num())
		{
			EpicEventType = bStaticStart ? EEpicEventType::Static : EEpicEventType::Dynamic;
			// Get the event start time. This is either a Posix time in seconds since the Epoch (1/1/1970) or the special
			// word 'now' optionally followed by a value to be added or subtracted from now.
			FTimeValue Start;
			FString Time = Params[0];
			if (Time.StartsWith(TEXT("now")))
			{
				Time.RightChopInline(3);
				Start = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
			}
			if (!Time.IsEmpty())
			{
				FTimeValue Posix = FTimeValue().SetFromTimeFraction(FTimeFraction().SetFromFloatString(Time));
				if (Posix.IsValid())
				{
					if (Start.IsValid())
					{
						Start += Posix;
					}
					else
					{
						Start = Posix;
					}
				}
			}
			if (Start.IsValid())
			{
				MPDRoot->SetAvailabilityStartTime(Start);
				if (EpicEventType == EEpicEventType::Dynamic)
				{
					MPDRoot->SetPublishTime(Start);
					MPDRoot->SetType(TEXT("dynamic"));
					PresentationType = EPresentationType::Dynamic;
					if (Params.Num() > 1)
					{
						FTimeValue spd = FTimeValue().SetFromTimeFraction(FTimeFraction().SetFromFloatString(Params[1]));
						MPDRoot->SetSuggestedPresentationDelay(spd);
					}
					// In case there is a @suggestedPresentationDelay present (even if should not) we use it, otherwise we use the @minBufferTime
					else if (!MPDRoot->GetSuggestedPresentationDelay().IsValid())
					{
						MPDRoot->SetSuggestedPresentationDelay(MPDRoot->GetMinBufferTime());
					}
				}
				// Adjust period start times with the fake AST
				for(auto &Period : GetPeriods())
				{
					Period->StartAST = Period->Start + Start;
					Period->EndAST = Period->End + Start;
				}
			}
		}
	}
}

void FManifestDASHInternal::PrepareEpicEventForLooping(int32 InNumLoopsToAdd)
{
	if (OriginalPresentationDuration.IsValid())
	{
		FTimeValue ast = GetAnchorTime();
		FTimeValue AddDur = OriginalPresentationDuration * InNumLoopsToAdd;
		FTimeValue Start = ast + AddDur;
		MPDRoot->SetAvailabilityStartTime(Start);
		if (EpicEventType == EEpicEventType::Dynamic)
		{
			MPDRoot->SetPublishTime(Start);
		}
		for(auto &Period : GetPeriods())
		{
			Period->StartAST += AddDur;
			Period->EndAST += AddDur;
		}
	}
}


FErrorDetail FManifestDASHInternal::Build(IPlayerSessionServices* InPlayerSessionServices, TSharedPtr<FDashMPD_MPDType, ESPMode::ThreadSafe> InMPDRoot, TArray<TWeakPtrTS<IDashMPDElement>> InXLinkElements)
{
	PlayerSessionServices = InPlayerSessionServices;
	RemoteElementsToResolve = MoveTemp(InXLinkElements);

	FErrorDetail Error;

	MPDRoot = InMPDRoot;

	// What type of presentation is this?
	// We do not validate the MPD@type here. Anything not 'static' is handled as 'dynamic'.
	PresentationType = MPDRoot->GetType().Equals(TEXT("static")) ? EPresentationType::Static : EPresentationType::Dynamic;

	// Dynamic presentations require both MPD@availabilityStartTime and MPD@publishTime
	if (PresentationType == EPresentationType::Dynamic)
	{
		if (!MPDRoot->GetAvailabilityStartTime().IsValid() || !MPDRoot->GetPublishTime().IsValid())
		{
			return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Dynamic presentations require both MPD@availabilityStartTime and MPD@publishTime to be valid")), ERRCODE_DASH_MPD_BUILDER_MISSING_REQUIRED_ATTRIBUTE);
		}
	}
	else
	{
		if (MPDRoot->GetMinimumUpdatePeriod().IsValid())
		{
			return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Static presentations must not have MPD@minimumUpdatePeriod")), ERRCODE_DASH_MPD_BUILDER_UNREQUIRED_ATTRIBUTE);
		}
	}

	// MPD@minBufferTime is required.
	if (!MPDRoot->GetMinBufferTime().IsValid())
	{
		return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("MPD@minBufferTime is required")), ERRCODE_DASH_MPD_BUILDER_MISSING_REQUIRED_ATTRIBUTE);
	}

	// At least one Period is required.
	if (MPDRoot->GetPeriods().Num() == 0)
	{
		return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("At least one Period is required")), ERRCODE_DASH_MPD_BUILDER_MISSING_REQUIRED_ATTRIBUTE);
	}

	// Go over the list of remote elements we need to process.
	// Initially this list includes onLoad elements only but may get expanded if these elements reference additional onRequest elements.
	// As long as this list is not empty we are not done with the initial MPD setup.
	for(int32 nRemoteElement=0; nRemoteElement<RemoteElementsToResolve.Num(); ++nRemoteElement)
	{
		Error = PrepareRemoteElementLoadRequest(PendingRemoteElementLoadRequests, RemoteElementsToResolve[nRemoteElement], 1);
		if (Error.IsSet())
		{
			return Error;
		}
	}
	if (PendingRemoteElementLoadRequests.Num())
	{
		return Error.SetTryAgain();
	}
	return BuildAfterInitialRemoteElementDownload();
}

FErrorDetail FManifestDASHInternal::BuildAfterInitialRemoteElementDownload()
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_FManifestDASHInternal_Build);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, FManifestDASHInternal_Build);

	FErrorDetail Error;

	FTimeValue AST = GetAnchorTime();
	SegmentFetchDelay.SetToZero();

	// If on the MPD level there are multiple <BaseURL> elements then they need to be all absolute.
	// There cannot be a mix between absolute and relative URLs and there can also not be multiple relative URLs either.
	if (!DASHUrlHelpers::AreBaseURLsValid(MPDRoot->GetBaseURLs()))
	{
		return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Inconsistent relative <BaseURL> elements found on MPD level")), ERRCODE_DASH_MPD_BUILDER_AMBIGUOUS_URLS);
	}
	// There may be at most one UrlQueryInfo as per Annex I.2 on each hierarchy level.
	if (!DASHUrlHelpers::AreURLQueryInfosValid(MPDRoot))
	{
		return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Multiple Annex I.2 URL query elements found on MPD level")), ERRCODE_DASH_MPD_BUILDER_AMBIGUOUS_URLS);
	}

	bool bWarnedPresentationDuration = false;
	// Go over the periods one at a time as XLINK attributes could bring in additional periods.
	FTimeValue CombinedPeriodDuration(FTimeValue::GetZero());
	const TArray<TSharedPtrTS<FDashMPD_PeriodType>>& MPDperiods = MPDRoot->GetPeriods();
	for(int32 nPeriod=0,nPeriodMax=MPDperiods.Num(); nPeriod<nPeriodMax; ++nPeriod)
	{
		TSharedPtrTS<FPeriod> p = MakeSharedTS<FPeriod>();

		// Keep a weak reference to the original period.
		p->Period = MPDperiods[nPeriod];

		p->ID = MPDperiods[nPeriod]->GetID();
		// We need an ID to track this period, even if there is no Period@id in the MPD.
		// In this case we make up an artificial name.
		if (p->ID.IsEmpty())
		{
			p->ID = FString::Printf(TEXT("$period.%d$"), nPeriod);
		}

		// As with the MPD <BaseURL> elements above the same is true on Period level.
		if (!DASHUrlHelpers::AreBaseURLsValid(MPDperiods[nPeriod]->GetBaseURLs()))
		{
			return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Inconsistent relative <BaseURL> elements found in period \"%s\""), *p->ID), ERRCODE_DASH_MPD_BUILDER_AMBIGUOUS_URLS);
		}
		// And also the UrlQueryInfo
		if (!DASHUrlHelpers::AreURLQueryInfosValid(MPDperiods[nPeriod]))
		{
			return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Multiple Annex I.2 URL query elements found in period \"%s\""), *p->ID), ERRCODE_DASH_MPD_BUILDER_AMBIGUOUS_URLS);
		}

		p->Start = MPDperiods[nPeriod]->GetStart();
		p->Duration = MPDperiods[nPeriod]->GetDuration();
		// If the period start is not specified this could be an early available period.
		// See 5.3.2.1
		if (!p->Start.IsValid())
		{
			if (nPeriod == 0 && PresentationType == EPresentationType::Static)
			{
				p->Start.SetToZero();
			}
			else if (nPeriod && MPDperiods[nPeriod-1]->GetDuration().IsValid())
			{
				p->Start = Periods.Last()->Start + MPDperiods[nPeriod-1]->GetDuration();
			}
			else if (PresentationType == EPresentationType::Dynamic)
			{
				if (nPeriod == 0)
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("The first period \"%s\" of this dynamic presentation has no start. Is this intended?"), *p->ID));
				}
				p->bIsEarlyPeriod = true;
			}
			else
			{
				return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Period@start cannot be derived for period \"%s\""), *p->ID), ERRCODE_DASH_MPD_BUILDER_BAD_PERIOD_START);
			}
		}
		// Calculate a period end time for convenience. This is more accessible than dealing with start+duration.
		if (!p->bIsEarlyPeriod)
		{
			// Check if a regular period follows an early available period, which is a violation of 5.3.2.1
			if (Periods.Num() && Periods.Last()->bIsEarlyPeriod)
			{
				return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Early available period \"%s\" must not be followed by a regular period (\"%s\")"), *Periods.Last()->ID, *p->ID), ERRCODE_DASH_MPD_BUILDER_EARLY_PERIODS_MUST_BE_LAST);
			}

			// If there are previous periods then these are now clearly followed by another period, namely this one.
			if (Periods.Num())
			{
				Periods.Last()->bHasFollowingPeriod = true;
			}

			// If the previous period did not have a duration to calculate the end time with we set
			// its end time to the start time of this period.
			if (Periods.Num() && !Periods.Last()->End.IsValid())
			{
				Periods.Last()->End = p->Start;
			}

			if (MPDperiods[nPeriod]->GetDuration().IsValid())
			{
				p->End = p->Start + MPDperiods[nPeriod]->GetDuration();
			}
			else if (nPeriod < nPeriodMax-1)
			{
				if (MPDperiods[nPeriod + 1]->GetStart().IsValid())
				{
					p->End = MPDperiods[nPeriod + 1]->GetStart();
					if (!p->Duration.IsValid())
					{
						p->Duration = p->End - p->Start;
					}
				}
			}
			else if (nPeriod == nPeriodMax-1)
			{
				if (MPDRoot->GetMediaPresentationDuration().IsValid())
				{
					FTimeValue firstPeriodStart = Periods.Num() ? Periods[0]->Start : p->Start;
					p->End = firstPeriodStart + MPDRoot->GetMediaPresentationDuration();
					if (!p->Duration.IsValid())
					{
						p->Duration = p->End - p->Start;
					}
				}
				else if (PresentationType == EPresentationType::Static)
				{
					return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Static presentations with the last Period@duration missing need to have MPD@mediaPresentationDuration set!")), ERRCODE_DASH_MPD_BUILDER_MEDIAPRESENTATIONDURATION_NEEDED);
				}
				else if (PresentationType == EPresentationType::Dynamic && !MPDRoot->GetMinimumUpdatePeriod().IsValid())
				{
					return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Dynamic presentations with the last Period@duration missing and no MPD@minimumUpdateTime need to have MPD@mediaPresentationDuration set!")), ERRCODE_DASH_MPD_BUILDER_MEDIAPRESENTATIONDURATION_NEEDED);
				}
			}

			// Check for potential period overlap or gaps.
			if (Periods.Num())
			{
				// Check for period ordering. Start times must be increasing.
				if (p->Start.IsValid() && Periods.Last()->Start.IsValid() && p->Start < Periods.Last()->Start)
				{
					return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Period@start times must be increasing for periods \"%s\" and \"%s\""), *p->ID, *Periods.Last()->ID), ERRCODE_DASH_MPD_BUILDER_MEDIAPRESENTATIONDURATION_NEEDED);
				}

				FTimeValue diff = p->Start - Periods.Last()->End;
				// Does this period cut into the preceeding one?
				if (diff.IsValid() && diff < FTimeValue::GetZero())
				{
					// Set the end time of the preceeding period to the start of this one.
					Periods.Last()->End = p->Start;
				}
				// Is there a gap?
				else if (diff.IsValid() && diff > FTimeValue::GetZero())
				{
					// This is not desireable. There will be no content in the preceeding period to cover the gap.
					// Depending on the duration of the gap this could be problem.
					if (diff.GetAsSeconds() > 0.5)
					{
						// Log a warning!
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("There is a gap of %.f seconds between periods \"%s\" and \"%s\"!"), diff.GetAsSeconds(), *Periods.Last()->ID, *p->ID));
					}
				}
			}
		}

		// Set the time range including AST
		p->StartAST = p->Start + AST;
		p->EndAST = p->End + AST;

		if (p->Start.IsValid() && p->End.IsValid())
		{
			FTimeValue periodDur = p->End - p->Start;
			if (periodDur >= FTimeValue::GetZero())
			{
				if (p->Duration.IsValid() && periodDur > p->Duration)
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Period \"%s\" has %.f seconds shorter duration than next period start indicates!"), *p->ID, (periodDur - p->Duration).GetAsSeconds()));
				}
				CombinedPeriodDuration += periodDur;
				// Do a check if the media presentation duration cuts into the periods or is longer than the last period.
				if (MPDRoot->GetMediaPresentationDuration().IsValid())
				{
					FTimeValue diff = MPDRoot->GetMediaPresentationDuration() - CombinedPeriodDuration;
					if (diff < FTimeValue::GetZero() && diff.GetAsSeconds() < -0.5 && !bWarnedPresentationDuration)
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("MPD@mediaPresentationDuration cuts into Period \"%s\" by %.f seconds!"), *p->ID, -diff.GetAsSeconds()));
						bWarnedPresentationDuration = true;
					}
					else if (nPeriod == nPeriodMax-1 && diff > FTimeValue::GetZero() && diff.GetAsSeconds() > 0.5 && !bWarnedPresentationDuration)
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("MPD@mediaPresentationDuration extends past the last period's end by %.f seconds. There is no content to play for that time!"), diff.GetAsSeconds()));
						bWarnedPresentationDuration = true;
					}
				}
			}
			else
			{
				return CreateErrorAndLog(PlayerSessionServices, FString::Printf(TEXT("Duration of Period \"%s\" is calculated as negative!"), *p->ID), ERRCODE_DASH_MPD_BUILDER_BAD_PERIOD_DURATION);
			}
		}

		// Add period to the list.
		Periods.Emplace(MoveTemp(p));
	}


	// Check for supported <ServiceDescription> elements on MPD level. Those on Period level we ignore for now.
	TArray<TSharedPtrTS<FDashMPD_ServiceDescriptionType>> ServiceDescs = MPDRoot->GetServiceDescriptions().FilterByPredicate([](const TSharedPtrTS<FDashMPD_ServiceDescriptionType>& InDesc)
	{
		// No <Scope> means this <ServiceDescription> applies to all clients, not specific ones.
		// If we need to look for specific scopes we can do this here.
		return InDesc->GetScopes().Num() == 0;
	});
	// Are there any indicating low-latency playback?
	TArray<TSharedPtrTS<FDashMPD_ServiceDescriptionType>> LowLatencyDescs = ServiceDescs.FilterByPredicate([](const TSharedPtrTS<FDashMPD_ServiceDescriptionType>& InDesc)
	{
		return InDesc->GetLatencies().Num() != 0;
	});
	if (LowLatencyDescs.Num())
	{
		// If there is more than one we just use the first.
		if (LowLatencyDescs.Num() > 1)
		{
			LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("MPD has more than one applicable low-latency <ServiceDescription> element. Using the first one.")));
		}
		TSharedPtrTS<FDashMPD_ServiceDescriptionType> llDesc = LowLatencyDescs[0];
		if (llDesc->GetLatencies().Num() > 1)
		{
			LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Selected MPD <ServiceDescription> element has more than one <Latency> element. Using the first one.")));
		}
		if (llDesc->GetPlaybackRates().Num() > 1)
		{
			LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Selected MPD <ServiceDescription> element has more than one <PlaybackRate> element. Using the first one.")));
		}

		TSharedPtrTS<FDashMPD_LatencyType> llType = llDesc->GetLatencies()[0];
		TSharedPtrTS<FDashMPD_PlaybackRateType> llRate = llDesc->GetPlaybackRates().Num() ? llDesc->GetPlaybackRates()[0] : nullptr;

		LowLatencyDescriptor = MakeSharedTS<FLowLatencyDescriptor>();
		LowLatencyDescriptor->Latency.ReferenceID = llType->GetReferenceID().IsSet() ? (int64)llType->GetReferenceID().Value() : -1;
		LowLatencyDescriptor->Latency.Target = llType->GetTarget().IsSet() ? FTimeValue((int64) llType->GetTarget().Value(), (uint32)1000) : FTimeValue();
		LowLatencyDescriptor->Latency.Min = llType->GetMin().IsSet() ? FTimeValue((int64) llType->GetMin().Value(), (uint32)1000) : FTimeValue();
		LowLatencyDescriptor->Latency.Max = llType->GetMax().IsSet() ? FTimeValue((int64) llType->GetMax().Value(), (uint32)1000) : FTimeValue();
		LowLatencyDescriptor->PlayRate.Min = llRate.IsValid() ? llRate->GetMin() : FTimeValue();
		LowLatencyDescriptor->PlayRate.Max = llRate.IsValid() ? llRate->GetMax() : FTimeValue();
	}


	// As per ISO/IEC 23009-1:2019/DAM 1 "Change 2: Event Stream and Timed Metadata Processing" Section A.11.11 Detailed processing
	// all events from MPD EventStreams are to be collected right now.
	for(int32 i=0; i<Periods.Num(); ++i)
	{
		SendEventsFromAllPeriodEventStreams(Periods[i]);
	}

	return Error;
}


void FManifestDASHInternal::PreparePeriodAdaptationSets(TSharedPtrTS<FPeriod> Period, bool bRequestXlink)
{
	check(Period.IsValid());
	TSharedPtrTS<FDashMPD_PeriodType> MPDPeriod = Period->Period.Pin();
	if (MPDPeriod.IsValid())
	{
		// If already prepared just return.
		if (Period->GetHasBeenPrepared() && !bRequestXlink)
		{
			return;
		}

		check(MPDRoot.IsValid());
		const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& UTCTimings = MPDRoot->GetUTCTimings();

		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_FManifestDASHInternal_Build);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, FManifestDASHInternal_Build);

		IPlayerStreamFilter* StreamFilter = PlayerSessionServices->GetStreamFilter();

		auto CheckEssentialProperties = [](const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& InEssentialProperties) -> bool
		{
			for(int32 nEssProp=0; nEssProp<InEssentialProperties.Num(); ++nEssProp)
			{
				// Check against a fixed list of properties we support internally.
				for(int32 nSupportedEssProp=0; nSupportedEssProp<UE_ARRAY_COUNT(SupportedEssentialProperties); ++nSupportedEssProp)
				{
					if (!InEssentialProperties[nEssProp]->GetSchemeIdUri().Equals(SupportedEssentialProperties[nSupportedEssProp]))
					{
						return false;
					}
				}
			}
			return true;
		};

		Period->AdaptationSets.Empty();
		const TArray<TSharedPtrTS<FDashMPD_AdaptationSetType>>& MPDAdaptationSets = MPDPeriod->GetAdaptationSets();
		int32 nAdapt=0, nAdaptMax=MPDAdaptationSets.Num();
		for(; nAdapt<nAdaptMax; ++nAdapt)
		{
			const TSharedPtrTS<FDashMPD_AdaptationSetType>& MPDAdaptationSet = MPDAdaptationSets[nAdapt];

			// Check the <BaseURL> elements and the UrlQueryInfos in the adaptation set.
			if (!DASHUrlHelpers::AreBaseURLsValid(MPDAdaptationSet->GetBaseURLs()))
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Inconsistent relative <BaseURL> elements found in AdaptationSet, ignoring this AdaptationSet.")));
				continue;
			}
			// And also the UrlQueryInfo
			if (!DASHUrlHelpers::AreURLQueryInfosValid(MPDAdaptationSet))
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Multiple Annex I.2 URL query elements found in AdaptationSet, ignoring this AdaptationSet.")));
				continue;
			}
			// Check if there are any unsupported EssentialProperties. If so, this AdaptationSet cannot be used.
			if (!CheckEssentialProperties(MPDAdaptationSet->GetEssentialProperties()))
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Unsupported <EssentialProperty> found in AdaptationSet, ignoring this AdaptationSet.")));
				continue;
			}

			TSharedPtrTS<FAdaptationSet> AdaptationSet = MakeSharedTS<FAdaptationSet>();
			// Store the index of this AdaptationSet in the array with the adaptation set.
			// AdaptationSets do not need to have an ID and we will filter out some sets without removing
			// them from the enclosing period (on purpose!!!).
			// Keeping the index helps locate the adaptation set in the enclosing period's adaptation set array.
			// Even with MPD updates the adaptation sets are not permitted to change so in theory the index will
			// be valid at all times.
			AdaptationSet->UniqueSequentialSetIndex = nAdapt;
			AdaptationSet->AdaptationSet = MPDAdaptationSet;
			AdaptationSet->PAR = MPDAdaptationSet->GetPAR();
			BCP47::ParseRFC5646Tag(AdaptationSet->LanguageTag, MPDAdaptationSet->GetLanguage());
			AdaptationSet->SelectionPriority = (int32) MPDAdaptationSet->GetSelectionPriority();
			// Set up labels. For now we ignore GroupLabel elements.
			for(auto& itLabel : MPDAdaptationSet->GetLabels())
			{
				FLabelElement& Label = AdaptationSet->Labels.Emplace_GetRef();
				Label.ID = itLabel->GetID();
				Label.Language = ISO639::RFC5646To639_1(itLabel->GetLanguage());
				Label.LabelText = itLabel->GetData();
			}

			// Content components are not supported.
			if (MPDAdaptationSet->GetContentComponents().Num())
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("ContentComponent in AdaptationSet is not supported, ignoring this AdaptationSet.")));
				continue;
			}
			// Frame packing of any kind is not supported.
			else if (MPDAdaptationSet->GetFramePackings().Num())
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("FramePacking in AdaptationSet is not supported, ignoring this AdaptationSet.")));
				continue;
			}
			// Interlace video is not supported.
			else if (MPDAdaptationSet->GetScanType().Equals(ScanTypeInterlace))
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Interlace video in AdaptationSet is not supported, ignoring this AdaptationSet.")));
				continue;
			}
			// Ratings are not supported (as per DASH-IF-IOP)
			else if (MPDAdaptationSet->GetRatings().Num())
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Rating in AdaptationSet is not supported, ignoring this AdaptationSet.")));
				continue;
			}
			// We do not support Viewpoints.
			else if (MPDAdaptationSet->GetViewpoints().Num())
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Viewpoint in AdaptationSet is not supported, ignoring this AdaptationSet.")));
				continue;
			}

			// Check Roles for unsupported ones.
			bool bBadRole = false;
			for(int32 nRoles=0; nRoles<MPDAdaptationSet->GetRoles().Num(); ++nRoles)
			{
				if (MPDAdaptationSet->GetRoles()[nRoles]->GetSchemeIdUri().Equals(TEXT("urn:mpeg:dash:stereoid:2011")))
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Unsupported AdaptationSet Role found (\"%s\"), ignoring this AdaptationSet."), *MPDAdaptationSet->GetRoles()[nRoles]->GetSchemeIdUri()));
					bBadRole = true;
				}
				else if (MPDAdaptationSet->GetRoles()[nRoles]->GetSchemeIdUri().Equals(DASHRole))
				{
					// Check for role values we understand.
					FString Role = MPDAdaptationSet->GetRoles()[nRoles]->GetValue();
					if (Role.Equals(TEXT("main")) || Role.Equals(TEXT("alternate")) || Role.Equals(TEXT("supplementary")) || Role.Equals(TEXT("commentary")) || Role.Equals(TEXT("dub")) ||
						Role.Equals(TEXT("emergency")) || Role.Equals(TEXT("caption")) || Role.Equals(TEXT("subtitle")) || Role.Equals(TEXT("sign")) || Role.Equals(TEXT("description")))
					{
						AdaptationSet->Roles.Emplace(Role);
					}
					else
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Unsupported Role type \"%s\" found."), *Role));
					}
				}
			}
			if (bBadRole)
			{
				continue;
			}
			// By default the absence of a Role makes an AdaptationSet take on the "main" role.
			if (AdaptationSet->Roles.Num() == 0)
			{
				AdaptationSet->Roles.Emplace(FString(TEXT("main")));
			}

			// Check Accessibility
			for(int32 nAcc=0; nAcc<MPDAdaptationSet->GetAccessibilities().Num(); ++nAcc)
			{
				if (MPDAdaptationSet->GetAccessibilities()[nAcc]->GetSchemeIdUri().Equals(DASHRole))
				{
					FString Accessibility = MPDAdaptationSet->GetAccessibilities()[nAcc]->GetValue();
					if (Accessibility.Equals(TEXT("sign")) || Accessibility.Equals(TEXT("caption")) || Accessibility.Equals(TEXT("description")) || Accessibility.Equals(TEXT("enhanced-audio-intelligibility")))
					{
						AdaptationSet->Accessibilities.Emplace(Accessibility);
					}
					else
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Unsupported Accessibility type \"%s\" found."), *Accessibility));
					}
				}
				else if (MPDAdaptationSet->GetAccessibilities()[nAcc]->GetSchemeIdUri().Equals(TEXT("urn:scte:dash:cc:cea-608:2015")))
				{
					// We do not parse this out here. We prepend a "608:" prefix and take the value verbatim for now.
					AdaptationSet->Accessibilities.Emplace(FString(TEXT("608:"))+MPDAdaptationSet->GetAccessibilities()[nAcc]->GetValue());
				}
				else if (MPDAdaptationSet->GetAccessibilities()[nAcc]->GetSchemeIdUri().Equals(TEXT("urn:scte:dash:cc:cea-708:2015")))
				{
					// We do not parse this out here. We prepend a "708:" prefix and take the value verbatim for now.
					AdaptationSet->Accessibilities.Emplace(FString(TEXT("708:"))+MPDAdaptationSet->GetAccessibilities()[nAcc]->GetValue());
				}
			}

			// We do not rely on the AdaptationSet@contentType, AdaptationSet@mimeType and/or AdaptationSet@codecs as these are often not set.
			// First we go over the representations in the set and hope they have the @codecs attribute set.
			const TArray<TSharedPtrTS<FDashMPD_RepresentationType>>& MPDRepresentations = MPDAdaptationSet->GetRepresentations();
			struct FQualityEntry
			{
				int32 Bitrate = 0;
				int32 ListedIndex = 0;
				TSharedPtrTS<FRepresentation> Rep;
			};
			TArray<FQualityEntry> RepresentationQualityIndexList;
			for(int32 nRepr=0, nReprMax=MPDRepresentations.Num(); nRepr<nReprMax; ++nRepr)
			{
				const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation = MPDRepresentations[nRepr];

				// Check the <BaseURL> elements and the UrlQueryInfos
				if (!DASHUrlHelpers::AreBaseURLsValid(MPDRepresentation->GetBaseURLs()))
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Inconsistent relative <BaseURL> elements found in Representation, ignoring this Representation.")));
					continue;
				}
				// And also the UrlQueryInfo
				if (!DASHUrlHelpers::AreURLQueryInfosValid(MPDRepresentation))
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Multiple Annex I.2 URL query elements found in Representation, ignoring this Representation.")));
					continue;
				}
				// Check if there are any unsupported EssentialProperties. If so, this Representation cannot be used.
				if (!CheckEssentialProperties(MPDRepresentation->GetEssentialProperties()))
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Unsupported <EssentialProperty> found in Representation, ignoring this Representation.")));
					continue;
				}

				// Subrepresentations and dependent representations are not supported.
				if (MPDRepresentation->GetSubRepresentations().Num())
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("SubRepresentations are not supported, ignoring this Representation.")));
					continue;
				}
				else if (MPDRepresentation->GetDependencyIDs().Num())
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Representation dependencies are not supported, ignoring this Representation.")));
					continue;
				}
				else if (MPDRepresentation->GetMediaStreamStructureIDs().Num())
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Media stream structures are not supported, ignoring this Representation.")));
					continue;
				}
				// Frame packing of any kind is not supported.
				else if (MPDRepresentation->GetFramePackings().Num())
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("FramePacking in Representation is not supported, ignoring this Representation.")));
					continue;
				}
				// Interlace is not supported.
				else if (MPDRepresentation->GetScanType().Equals(ScanTypeInterlace))
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Interlace video in Representation is not supported, ignoring this Representation.")));
					continue;
				}

				// Encryption on representation level is not supported. As per DASH-IF-IOP v4.3 encryption must be specified on adaptation set level.
				if (MPDRepresentation->GetContentProtections().Num())
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("ContentProtection in Representation is not supported. Must be on enclosing AdaptationSet! Ignoring this Representation.")));
					continue;
				}


				// Is there a @codecs list on the representation itself?
				TArray<FString> MPDCodecs = MPDRepresentation->GetCodecs();
				if (MPDCodecs.Num() == 0)
				{
					// If not then there needs to be one on the AdaptationSet. However, this will specify the highest profile and level
					// necessary to decode any and all representations and is thus potentially too restrictive.
					MPDCodecs = MPDAdaptationSet->GetCodecs();
				}
				// There *should* be only one codec since we are not considering multiplexed streams (ContentComponent / SubRepresentation).
				if (MPDCodecs.Num() > 1)
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("More than one codec found for Representation, using only first codec.")));
				}


				TSharedPtrTS<FRepresentation> Representation = MakeSharedTS<FRepresentation>();
				Representation->Representation = MPDRepresentation;

				// Before checking for supported codecs we need to check if this is probably a subtitle representation.
				// These can come in several flavors and several TTML profiles for which checking just the codec is not
				// likely to cover all cases.
				FString MimeType = MPDRepresentation->GetMimeType();
				if (MimeType.IsEmpty())
				{
					MimeType = MPDAdaptationSet->GetMimeType();
				}
				// Mime type is mandatory on representation or adaptation set level. So if there is none we may ignore this representation as it's an authoring error.
				if (MimeType.IsEmpty())
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Could not mandatory Representation@mimeType or the enclosing AdaptationSet. Ignoring this Representation.")));
					continue;
				}
				// Sadly we cannot rely on either @contentType or a Role to be set.
				if (MimeType.Equals(SubtitleMimeType_Streamed))
				{
					// application/mp4 needs to have @codecs set.
				}
				else if (MimeType.Equals(SubtitleMimeType_SideloadedTTML))
				{
					if (MPDCodecs.Num() == 0)
					{
						MPDCodecs.Emplace(TEXT("stpp"));
					}
					Representation->bIsSideloadedSubtitle = true;
					Representation->StreamContainerType = FRepresentation::EStreamContainerType::ISO14496_12;
				}
				else if (MimeType.Equals(SubtitleMimeType_SideloadedVTT))
				{
					if (MPDCodecs.Num() == 0)
					{
						MPDCodecs.Emplace(TEXT("wvtt"));
					}
					Representation->bIsSideloadedSubtitle = true;
					Representation->StreamContainerType = FRepresentation::EStreamContainerType::ISO14496_12;
				}

				// Check for thumbnails
				if (MimeType.Equals(ThumbnailMimeType_Jpeg) || MimeType.Equals(ThumbnailMimeType_Png))
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Thumbnail representations are not supported, ignoring this Representation.")));
					continue;
				}

				if (MPDCodecs.Num() == 0)
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Neither @codecs found on Representation or AdaptationSet level, ignoring this Representation.")));
					continue;
				}
				if (!Representation->CodecInfo.ParseFromRFC6381(MPDCodecs[0]))
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Could not parse Representation@codecs \"%s\", possibly unsupported codec. Ignoring this Representation."), *MPDCodecs[0]));
					continue;
				}

				Representation->ID = MPDRepresentation->GetID();
				Representation->Bitrate = MPDRepresentation->GetBandwidth();
				Representation->StreamMimeType = MimeType;
				if (MimeType.Contains(TEXT("mp4"), ESearchCase::IgnoreCase))
				{
					Representation->StreamContainerType = FRepresentation::EStreamContainerType::ISO14496_12;
				}
				else if (MimeType.Contains(TEXT("webm"), ESearchCase::IgnoreCase) || MimeType.Contains(TEXT("matroska"), ESearchCase::IgnoreCase))
				{
					Representation->StreamContainerType = FRepresentation::EStreamContainerType::Matroska;
				}

				Representation->CodecInfo.SetBitrate(Representation->Bitrate);

				// Propagate the language code from the AdaptationSet into the codec info
				Representation->CodecInfo.SetStreamLanguageTag(AdaptationSet->LanguageTag);

				// Set up codec details based on available attributes of the Representation or its enclosing AdaptationSet.
				if (Representation->CodecInfo.IsVideoCodec())
				{
					// Resolution on Representation (expected)?
					if (MPDRepresentation->GetWidth().IsSet() && MPDRepresentation->GetHeight().IsSet())
					{
						Representation->CodecInfo.SetResolution(FStreamCodecInformation::FResolution(MPDRepresentation->GetWidth().Value(), MPDRepresentation->GetHeight().Value()));
					}
					// Resolution inherited from AdaptationSet? (possible if resolution is the same for every Representation)
					else if (MPDAdaptationSet->GetWidth().IsSet() && MPDAdaptationSet->GetHeight().IsSet())
					{
						Representation->CodecInfo.SetResolution(FStreamCodecInformation::FResolution(MPDAdaptationSet->GetWidth().Value(), MPDAdaptationSet->GetHeight().Value()));
					}

					// Framerate?
					if (MPDRepresentation->GetFrameRate().IsValid())
					{
						Representation->CodecInfo.SetFrameRate(MPDRepresentation->GetFrameRate());
					}
					else if (MPDAdaptationSet->GetFrameRate().IsValid())
					{
						Representation->CodecInfo.SetFrameRate(MPDAdaptationSet->GetFrameRate());
					}

					// Aspect ratio?
					if (MPDRepresentation->GetSAR().IsValid())
					{
						Representation->CodecInfo.SetAspectRatio(FStreamCodecInformation::FAspectRatio(MPDRepresentation->GetSAR().GetNumerator(), MPDRepresentation->GetSAR().GetDenominator()));
					}
					else if (MPDAdaptationSet->GetSAR().IsValid())
					{
						Representation->CodecInfo.SetAspectRatio(FStreamCodecInformation::FAspectRatio(MPDAdaptationSet->GetSAR().GetNumerator(), MPDAdaptationSet->GetSAR().GetDenominator()));
					}

					// Can be used?
					if (StreamFilter && !StreamFilter->CanDecodeStream(Representation->CodecInfo))
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Video representation \"%s\" in Period \"%s\" rejected by application."), *MPDRepresentation->GetID(), *Period->GetID()));
						continue;
					}

					// Update the video codec in the adaptation set with that of the highest bandwidth.
					if (MPDRepresentation->GetBandwidth() > AdaptationSet->MaxBandwidth)
					{
						AdaptationSet->MaxBandwidth = (int32) MPDRepresentation->GetBandwidth();
						AdaptationSet->Codec = Representation->CodecInfo;
					}
				}
				else if (Representation->CodecInfo.IsAudioCodec())
				{
					// Audio sample rate tends to be a single value, but could be a range, in which case we use the lower bound.
					if (MPDRepresentation->GetAudioSamplingRate().Num())
					{
						Representation->CodecInfo.SetSamplingRate((int32) MPDRepresentation->GetAudioSamplingRate()[0]);
					}
					else if (MPDAdaptationSet->GetAudioSamplingRate().Num())
					{
						Representation->CodecInfo.SetSamplingRate((int32) MPDAdaptationSet->GetAudioSamplingRate()[0]);
					}

					// Get the audio channel configurations from both Representation and AdaptationSet.
					TArray<TSharedPtrTS<FDashMPD_DescriptorType>> AudioChannelConfigurations(MPDRepresentation->GetAudioChannelConfigurations());
					AudioChannelConfigurations.Append(MPDAdaptationSet->GetAudioChannelConfigurations());
					// It is also possible for audio descriptors to have ended in the Essential- or SupplementalProperty. Append those to the list as well.
					AudioChannelConfigurations.Append(MPDRepresentation->GetSupplementalProperties());
					AudioChannelConfigurations.Append(MPDRepresentation->GetEssentialProperties());
					AudioChannelConfigurations.Append(MPDAdaptationSet->GetSupplementalProperties());
					AudioChannelConfigurations.Append(MPDAdaptationSet->GetEssentialProperties());
					for(int32 nACC=0, nACCMax=AudioChannelConfigurations.Num(); nACC<nACCMax; ++nACC)
					{
						if (AudioChannelConfigurations[nACC]->GetSchemeIdUri().Equals(AudioChannelConfigurationLegacy) ||	// "urn:mpeg:dash:23003:3:audio_channel_configuration:2011"
							AudioChannelConfigurations[nACC]->GetSchemeIdUri().Equals(AudioChannelConfiguration))			// "urn:mpeg:mpegB:cicp:ChannelConfiguration"
						{
							// Value = channel config as per 23001-8:2013 table 8
							uint32 v = 0;
							LexFromString(v, *AudioChannelConfigurations[nACC]->GetValue());
							Representation->CodecInfo.SetChannelConfiguration(v);
							Representation->CodecInfo.SetNumberOfChannels(ElectraDecodersUtil::MPEG::AACUtils::GetNumberOfChannelsFromChannelConfiguration(v));
							break;
						}
						else if (AudioChannelConfigurations[nACC]->GetSchemeIdUri().Equals(AudioChannelConfigurationDolby))	// "tag:dolby.com,2014:dash:audio_channel_configuration:2011"
						{
							// Ignored for now.
							continue;
						}
						else
						{
							/*
								Other audio related descriptors could be:
								   urn:mpeg:mpegB:cicp:OutputChannelPosition
								   urn:mpeg:mpegB:cicp:ProgramLoudness
								   urn:mpeg:mpegB:cicp:AnchorLoudness
								   tag:dolby.com,2014:dash:DolbyDigitalPlusExtensionType:2014
								   tag:dolby.com,2014:dash:complexityIndexTypeA:2014"
								and more.

								There is also information in the Accessibility descriptors like
								   urn:tva:metadata:cs:AudioPurposeCS:2007
							*/
						}
					}

					// Can be used?
					if (StreamFilter && !StreamFilter->CanDecodeStream(Representation->CodecInfo))
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Audio representation \"%s\" in Period \"%s\" rejected by application."), *MPDRepresentation->GetID(), *Period->GetID()));
						continue;
					}

					// Update the audio codec in the adaptation set with that of the highest bandwidth.
					if (MPDRepresentation->GetBandwidth() > AdaptationSet->MaxBandwidth)
					{
						AdaptationSet->MaxBandwidth = (int32) MPDRepresentation->GetBandwidth();
						AdaptationSet->Codec = Representation->CodecInfo;
					}
				}
				else if (Representation->CodecInfo.IsSubtitleCodec())
				{
					// There is a possibility that the MPD uses "video/mp4" instead of "application/mp4".
					if (MimeType.Equals(TEXT("video/mp4")))
					{
						MimeType = SubtitleMimeType_Streamed;
					}
					// Override the mime type. This is needed for sideloaded subtitles.
					Representation->CodecInfo.SetMimeType(MimeType);

					// Can be used?
					// For sideloaded subtitles we only need the mime type while for regular subtitles we need the codec.
					FString CodecMimeType, CodecName;
					if (Representation->IsSideloadedSubtitle())
					{
						CodecMimeType = Representation->GetCodecInformation().GetMimeType();
					}
					else
					{
						CodecName = Representation->GetCodecInformation().GetCodecSpecifierRFC6381();
					}
					if (StreamFilter && !StreamFilter->CanDecodeSubtitle(CodecMimeType, CodecName))
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Subtitle representation \"%s\" in Period \"%s\" cannot be decoded, ignoring."), *MPDRepresentation->GetID(), *Period->GetID()));
						continue;
					}

					if (Representation->bIsSideloadedSubtitle || MPDRepresentation->GetBandwidth() > AdaptationSet->MaxBandwidth)
					{
						AdaptationSet->MaxBandwidth = (int32) MPDRepresentation->GetBandwidth();
						AdaptationSet->Codec = Representation->CodecInfo;
					}
				}
				else
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Unknown type of representation \"%s\" in Period \"%s\", ignoring."), *MPDRepresentation->GetID(), *Period->GetID()));
					continue;
				}


				/*
					Check if anywhere in the hierarchy (Representation->AdaptationSet->Period) there is a <SegmentTemplate> and if there is whether
					it has @availabilityTimeComplete set to false, indicating this segment is available via chunked transfer encoding (CTE) and thus
					potentially usable for low-latency streaming.
				*/
				TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>> SegmentTemplate({MPDRepresentation->GetSegmentTemplate(), MPDAdaptationSet->GetSegmentTemplate(), MPDPeriod->GetSegmentTemplate()});
				SegmentTemplate.Remove(nullptr);
				if (SegmentTemplate.Num())
				{
					TMediaOptionalValue<bool> bTimeComplete;
					TArray<TSharedPtrTS<const FDashMPD_BaseURLType>> BaseURLElements;
					DASHUrlHelpers::GetAllHierarchyBaseURLs(PlayerSessionServices, BaseURLElements, MPDRepresentation);
					for(auto &segTemp : SegmentTemplate)
					{
						if (!bTimeComplete.IsSet())
						{
							bTimeComplete = segTemp->GetAvailabilityTimeComplete();
						}
						else if (segTemp->GetAvailabilityTimeComplete().IsSet() && segTemp->GetAvailabilityTimeComplete().Value() != bTimeComplete.Value())
						{
							// Inconsistent. Emit warning and assume 'true' so segment is not available for low-latency.
							if (!Representation->bWarnedAboutInconsistentAvailabilityTimeComplete)
							{
								Representation->bWarnedAboutInconsistentAvailabilityTimeComplete = true;
								LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Inconsistent @availabilityTimeComplete value across <SegmentTemplate> hierarchy for representation \"%s\" in Period \"%s\", ignoring."), *MPDRepresentation->GetID(), *Period->GetID()));
							}
							bTimeComplete.Reset();
							break;
						}
					}
					for(auto &baseUrl : BaseURLElements)
					{
						if (!bTimeComplete.IsSet())
						{
							bTimeComplete = baseUrl->GetAvailabilityTimeComplete();
						}
						else if (baseUrl->GetAvailabilityTimeComplete().IsSet() && baseUrl->GetAvailabilityTimeComplete().Value() != bTimeComplete.Value())
						{
							// Inconsistent. Emit warning and assume 'true' so segment is not available for low-latency.
							if (!Representation->bWarnedAboutInconsistentAvailabilityTimeComplete)
							{
								Representation->bWarnedAboutInconsistentAvailabilityTimeComplete = true;
								LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Inconsistent @availabilityTimeComplete value across <SegmentTemplate> / <BaseURL> hierarchy for representation \"%s\" in Period \"%s\", ignoring."), *MPDRepresentation->GetID(), *Period->GetID()));
							}
							bTimeComplete.Reset();
							break;
						}
					}
					// Set as potentially low-latency usable?
					if (bTimeComplete.IsSet() && bTimeComplete.Value() == false)
					{
						Representation->bAvailableAsLowLatency.Set(true);
					}
				}


				// Any producer reference times?
				TArray<TSharedPtrTS<FDashMPD_ProducerReferenceTimeType>> ProducerReferenceTimes = MPDRepresentation->GetProducerReferenceTimes();
				if (ProducerReferenceTimes.Num() == 0)
				{
					ProducerReferenceTimes = MPDAdaptationSet->GetProducerReferenceTimes();
				}
				for(auto &ProdRef : ProducerReferenceTimes)
				{
					if (ProdRef->GetType().Equals(TEXT("encoder")) || ProdRef->GetType().Equals(TEXT("captured")))
					{
						FProducerReferenceTimeInfo prti;
						prti.ID = ProdRef->GetID();
						prti.bInband = ProdRef->GetInband();
						prti.Type = ProdRef->GetType().Equals(TEXT("encoder")) ? FProducerReferenceTimeInfo::EType::Encoder : FProducerReferenceTimeInfo::EType::Captured;
						prti.PresentationTime = ProdRef->GetPresentationTime();

						// As per 5.12.2, if a timing element is specified then it must also be specified in the MPD.
						FTimeValue WallclockTime;
						if (ProdRef->GetUTCTiming().IsValid())
						{
							bool bFound = false;
							for(auto &MPDUTCTiming : UTCTimings)
							{
								// Compare only the scheme, not the value. The scheme is needed to interpret the format the wallClockTime is
								// specified in correctly and nothing else.
								if (MPDUTCTiming->GetSchemeIdUri().Equals(ProdRef->GetUTCTiming()->GetSchemeIdUri()))
								{
									bFound = true;
									if ((ProdRef->GetUTCTiming()->GetSchemeIdUri().Equals(DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_httpiso2014)) ||
										(ProdRef->GetUTCTiming()->GetSchemeIdUri().Equals(DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_direct2014)))
									{
										ISO8601::ParseDateTime(WallclockTime, ProdRef->GetWallclockTime());
									}
									else if (ProdRef->GetUTCTiming()->GetSchemeIdUri().Equals(DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_httpxsdate2014))
									{
										if (!ISO8601::ParseDateTime(WallclockTime, ProdRef->GetWallclockTime()))
										{
											UnixEpoch::ParseFloatString(WallclockTime, ProdRef->GetWallclockTime());
										}
									}
									else if (ProdRef->GetUTCTiming()->GetSchemeIdUri().Equals(DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_httphead2014))
									{
										RFC7231::ParseDateTime(WallclockTime, ProdRef->GetWallclockTime());
									}
									else
									{
										// Unsupported type. No need to emit a warning since the MPD's root <UTCTiming> element will have warned already.
										// Just ignore it.
									}
									break;
								}
							}
							if (!bFound)
							{
								LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation <ProducerReferenceTime> references a <UTCTiming> element not present in the MPD! Ignoring this element!")));
							}
						}
						prti.WallclockTime = WallclockTime;
						if (WallclockTime.IsValid())
						{
							// Add the parsed element to the map for fast access when handling <Latency> in the <ServiceDescription> element.
							ProducerReferenceTimeElements.Add(prti.ID, MakeSharedTS<FProducerReferenceTimeInfo>(prti));
							// Assign it to the representation.
							Representation->ProducerReferenceTimeInfos.Emplace(MoveTemp(prti));
						}
					}
					else
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Representation uses <ProducerReferenceTime> of unsupported type, ignoring.")));
					}
				}

				// For all intents and purposes we consider this Representation as usable now.
				Representation->bIsUsable = true;
				AdaptationSet->Representations.Emplace(Representation);
				// Add this representation to the bandwidth-to-index list.
				FQualityEntry& qe = RepresentationQualityIndexList.Emplace_GetRef();
				qe.Bitrate = MPDRepresentation->GetBandwidth();
				qe.ListedIndex = nRepr;
				qe.Rep = Representation;
			}


			// If the adaptation set contains usable Representations we mark the AdaptationSet as usable as well.
			if (AdaptationSet->Representations.Num())
			{
				// Encryption?
				if (MPDAdaptationSet->GetContentProtections().Num())
				{
					if (!CanUseEncryptedAdaptation(AdaptationSet))
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("ContentProtection is not supported, ignoring this AdaptationSet.")));
						continue;
					}
				}

				// Sort by ascending bitrate, keeping the given order for same rates.
				RepresentationQualityIndexList.StableSort([](const FQualityEntry& a, const FQualityEntry& b)
				{
					return a.Bitrate == b.Bitrate ? a.ListedIndex < b.ListedIndex : a.Bitrate < b.Bitrate;
				});
				int32 CurrentQualityIndex = -1;
				int32 CurrentQualityBitrate = -1;
				for(auto& qlIt : RepresentationQualityIndexList)
				{
					if (CurrentQualityBitrate != qlIt.Bitrate)
					{
						CurrentQualityBitrate = qlIt.Bitrate;
						++CurrentQualityIndex;
					}
					qlIt.Rep->QualityIndex = CurrentQualityIndex;
				}

				AdaptationSet->bIsUsable = true;

				TMediaOptionalValue<bool> lowLatencyUsable;
				bool bFirst = true;
				for(auto &Repr : AdaptationSet->Representations)
				{
					if (bFirst)
					{
						bFirst = false;
						lowLatencyUsable = Repr->bAvailableAsLowLatency;
					}
					if ((lowLatencyUsable.IsSet() && Repr->bAvailableAsLowLatency.IsSet() && lowLatencyUsable.Value() != Repr->bAvailableAsLowLatency.Value()) ||
						(lowLatencyUsable.IsSet() != Repr->bAvailableAsLowLatency.IsSet()))
					{
						// Inconsistent.
						if (!AdaptationSet->bWarnedAboutInconsistentAvailabilityTimeComplete)
						{
							AdaptationSet->bWarnedAboutInconsistentAvailabilityTimeComplete = true;
							LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Inconsistent @availabilityTimeComplete value across the Representations in AdaptationSet \"%s\" in Period \"%s\", ignoring."), *MPDAdaptationSet->GetID_AsStr(), *Period->GetID()));
						}
					}
				}
				// If all representations are flagged as usable for low-latency then tag the adaptation set as well.
				if (!AdaptationSet->bWarnedAboutInconsistentAvailabilityTimeComplete && lowLatencyUsable.GetWithDefault(false))
				{
					AdaptationSet->bAvailableAsLowLatency.Set(true);
				}

				Period->AdaptationSets.Emplace(AdaptationSet);
			}
		}

		// Go over the AdaptationSets that are now remaining and check if they are set to switch between.
		for(auto &AdaptationSet : Period->GetAdaptationSets())
		{
			TArray<TSharedPtrTS<FDashMPD_DescriptorType>> SwitchedSets;
			TSharedPtrTS<FDashMPD_AdaptationSetType> MPDAdaptationSet = AdaptationSet->AdaptationSet.Pin();
			SwitchedSets = MPDAdaptationSet->GetSupplementalProperties().FilterByPredicate([](const TSharedPtrTS<FDashMPD_DescriptorType>& d)
				{ return d->GetSchemeIdUri().Equals(TEXT("urn:mpeg:dash:adaptation-set-switching:2016")); });
			if (SwitchedSets.Num())
			{
				const TCHAR* const CommaDelimiter = TEXT(",");
				for(auto &SwitchDesc : SwitchedSets)
				{
					// Get the IDs of the switched-to adaptation sets from the comma separated @value of the descriptor.
					TArray<FString> SwitchedToIDs;
					SwitchDesc->GetValue().ParseIntoArray(SwitchedToIDs, CommaDelimiter, true);
					for(auto &SwitchID : SwitchedToIDs)
					{
						// The ID is the ID of the AdaptationSet in the MPD, not the one of the FAdaptationSet!
						// Locate the FAdaptationSet that wraps the MPD's AdaptationSet with the given ID.
						// This may be NULL when that AdaptationSet was not usable or flat out does not exist.
						TSharedPtrTS<FAdaptationSet> SwitchedAS = Period->GetAdaptationSetByMPDID(SwitchID.TrimStartAndEnd());
						if (SwitchedAS.IsValid())
						{
							// Check that this is not the same adaptation set we are currently handling!
							if (SwitchedAS == AdaptationSet)
							{
								LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("AdaptationSet references self in adaptation-set-switching property!")));
								continue;
							}
							// Cross reference the switch-to and switching-from sets.
							AdaptationSet->SwitchToSetIDs.AddUnique(SwitchedAS->GetUniqueIdentifier());
							SwitchedAS->SwitchedFromSetIDs.AddUnique(AdaptationSet->GetUniqueIdentifier());
						}
					}
				}
			}
		}

		// Build special switching AdaptationSets that aggregate all representations from the referenced sets.
		TArray<TSharedPtrTS<FAdaptationSet>> SwitchingAdaptationSets;
		for(auto &AdaptationSet : Period->GetAdaptationSets())
		{
			TArray<FString> SwitchGroupIDs;
			struct FSwitchGroupBuilder
			{
				static void AddSet(TArray<FString>& SwitchGroupIDs, const TSharedPtrTS<const FPeriod>& P, const TSharedPtrTS<FAdaptationSet>& AS)
				{
					if (AS.IsValid() && !SwitchGroupIDs.Contains(AS->GetUniqueIdentifier()))
					{
						SwitchGroupIDs.Add(AS->GetUniqueIdentifier());
						for(auto &N : AS->GetSwitchToSetIDs())
						{
							AddSet(SwitchGroupIDs, P, P->GetAdaptationSetByUniqueID(N));
						}
						for(auto &N : AS->GetSwitchedFromSetIDs())
						{
							AddSet(SwitchGroupIDs, P, P->GetAdaptationSetByUniqueID(N));
						}
					}
				}
			};
			// If this set is referencing others or is itself being referenced
			if (!AdaptationSet->bIsInSwitchGroup && (AdaptationSet->GetSwitchToSetIDs().Num() || AdaptationSet->GetSwitchedFromSetIDs().Num()))
			{
				FSwitchGroupBuilder::AddSet(SwitchGroupIDs, Period, AdaptationSet);

				TSharedPtrTS<FAdaptationSet> SwitchSet = MakeSharedTS<FAdaptationSet>();
				SwitchSet->UniqueSequentialSetIndex = nAdapt++;
				SwitchSet->bIsSwitchGroup = true;
				SwitchSet->SwitchToSetIDs = MoveTemp(SwitchGroupIDs);
				SwitchSet->bIsUsable = true;
				SwitchSet->bIsEnabled = true;
				for(int32 i=0; i<SwitchSet->SwitchToSetIDs.Num(); ++i)
				{
					TSharedPtrTS<FAdaptationSet> SwitchedAS = Period->GetAdaptationSetByUniqueID(SwitchSet->SwitchToSetIDs[i]);
					SwitchedAS->bIsInSwitchGroup = true;
					if (i == 0)
					{
						SwitchSet->Roles = SwitchedAS->Roles;
						SwitchSet->Accessibilities = SwitchedAS->Accessibilities;
						SwitchSet->PAR = SwitchedAS->PAR;
						SwitchSet->LanguageTag = SwitchedAS->LanguageTag;
					}
					if (SwitchedAS->MaxBandwidth > SwitchSet->MaxBandwidth)
					{
						SwitchSet->MaxBandwidth = SwitchedAS->MaxBandwidth;
						SwitchSet->Codec = SwitchedAS->Codec;
					}

					SwitchSet->Representations.Append(SwitchedAS->Representations);
				}
				// Note: We do not aggregate encryption information here. These get accessed through the original adaptation sets.
				SwitchingAdaptationSets.Emplace(MoveTemp(SwitchSet));
			}
		}
		Period->AdaptationSets.Append(MoveTemp(SwitchingAdaptationSets));

		Period->SetHasBeenPrepared(true);
	}
}


bool FManifestDASHInternal::CanUseEncryptedAdaptation(const TSharedPtrTS<FAdaptationSet>& InAdaptationSet)
{
	TSharedPtrTS<FDRMManager> DRMManager = PlayerSessionServices->GetDRMManager();
	if (DRMManager.IsValid())
	{
		const TSharedPtrTS<FDashMPD_AdaptationSetType> MPDAdaptationSet = InAdaptationSet->AdaptationSet.Pin();
		const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& ContentProtections = MPDAdaptationSet->GetContentProtections();
		FString Mime = InAdaptationSet->GetCodec().GetMimeTypeWithCodecAndFeatures();
		// See if there is a DASH scheme saying that common encryption is in use.
		for(int32 i=0; i<ContentProtections.Num(); ++i)
		{
			if (ContentProtections[i]->GetSchemeIdUri().Equals(Scheme_urn_mpeg_dash_mp4protection_2011, ESearchCase::IgnoreCase))
			{
				InAdaptationSet->CommonEncryptionScheme = ContentProtections[i]->GetValue();
				const IDashMPDElement::FXmlAttribute* default_KID = DASHAttributeHelpers::GetAttribute(ContentProtections[i], TEXT("default_KID"), TEXT("cenc"));
				InAdaptationSet->DefaultKID = default_KID ? default_KID->GetValue() : FString();
				break;
			}
		}
		for(int32 i=0; i<ContentProtections.Num(); ++i)
		{
			TSharedPtr<ElectraCDM::IMediaCDMCapabilities, ESPMode::ThreadSafe> DRMCapabilities;
			// Skip over the common encryption scheme which we handled in the first pass already.
			if (ContentProtections[i]->GetSchemeIdUri().Equals(Scheme_urn_mpeg_dash_mp4protection_2011, ESearchCase::IgnoreCase))
			{
				continue;
			}

			// Get the scheme specific attributes and child elements as a JSON. If this does not exist yet
			// create it and store with the descriptor for later use.
			FString SchemeSpecificData = ContentProtections[i]->GetCustomElementAndAttributeJSON();
			if (SchemeSpecificData.IsEmpty())
			{
				FString AttrPrefix, TextProp;
				bool bNoNamespaces = false;
				DRMManager->GetCDMCustomJSONPrefixes(ContentProtections[i]->GetSchemeIdUri(), ContentProtections[i]->GetValue(), AttrPrefix, TextProp, bNoNamespaces);
				IManifestParserDASH::BuildJSONFromCustomElement(SchemeSpecificData, ContentProtections[i], false, bNoNamespaces, false, true, *AttrPrefix, *TextProp);
				ContentProtections[i]->SetCustomElementAndAttributeJSON(SchemeSpecificData);
			}

			DRMCapabilities = DRMManager->GetCDMCapabilitiesForScheme(ContentProtections[i]->GetSchemeIdUri(), ContentProtections[i]->GetValue(), SchemeSpecificData);
			if (DRMCapabilities.IsValid())
			{
				ElectraCDM::IMediaCDMCapabilities::ESupportResult Result;
				Result = DRMCapabilities->SupportsCipher(InAdaptationSet->CommonEncryptionScheme);
				if (Result != ElectraCDM::IMediaCDMCapabilities::ESupportResult::Supported)
				{
					continue;
				}
				Result = DRMCapabilities->SupportsType(Mime);
				if (Result != ElectraCDM::IMediaCDMCapabilities::ESupportResult::Supported)
				{
					continue;
				}
				Result = DRMCapabilities->RequiresSecureDecoder(Mime);
				if (Result == ElectraCDM::IMediaCDMCapabilities::ESupportResult::SecureDecoderRequired)
				{
					//LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Use of secure decoders is not supported.")));
					continue;
				}

				FAdaptationSet::FContentProtection Prot;
				Prot.Descriptor = ContentProtections[i];
				Prot.DefaultKID = InAdaptationSet->GetDefaultKID();
				Prot.CommonScheme = InAdaptationSet->GetCommonEncryptionScheme();
				InAdaptationSet->PossibleContentProtections.Emplace(MoveTemp(Prot));
			}
		}
	}
	return InAdaptationSet->PossibleContentProtections.Num() > 0;
}



void FManifestDASHInternal::SendEventsFromAllPeriodEventStreams(TSharedPtrTS<FPeriod> InPeriod)
{
	check(InPeriod.IsValid());
	TSharedPtrTS<FDashMPD_PeriodType> MPDPeriod = InPeriod->Period.Pin();
	if (MPDPeriod.IsValid())
	{
		FTimeValue PeriodTimeOffset = GetAnchorTime() + InPeriod->GetStart();
		// Iterate of all the period event streams. We expect them to have been xlink resolved already.
		const TArray<TSharedPtrTS<FDashMPD_EventStreamType>>& MPDEventStreams = MPDPeriod->GetEventStreams();
		for(int32 nEvS=0; nEvS<MPDEventStreams.Num(); ++nEvS)
		{
			const TSharedPtrTS<FDashMPD_EventStreamType>& EvS = MPDEventStreams[nEvS];
			uint32 Timescale = EvS->GetTimescale().GetWithDefault(1);
			int64 PTO = (int64) EvS->GetPresentationTimeOffset().GetWithDefault(0);

			const TArray<TSharedPtrTS<FDashMPD_EventType>>& Events = EvS->GetEvents();
			for(int32 i=0; i<Events.Num(); ++i)
			{
				const TSharedPtrTS<FDashMPD_EventType>& Event = Events[i];
				TSharedPtrTS<DASH::FPlayerEvent> NewEvent = MakeSharedTS<DASH::FPlayerEvent>();
				NewEvent->SetOrigin(IAdaptiveStreamingPlayerAEMSEvent::EOrigin::EventStream);
				NewEvent->SetSchemeIdUri(EvS->GetSchemeIdUri());
				NewEvent->SetValue(EvS->GetValue());
				if (Event->GetID().IsSet())
				{
					NewEvent->SetID(LexToString(Event->GetID().Value()));
				}
				FTimeValue PTS((int64)Event->GetPresentationTime() - PTO, Timescale);
				PTS += PeriodTimeOffset;
				NewEvent->SetPresentationTime(PTS);
				if (Event->GetDuration().IsSet())
				{
					NewEvent->SetDuration(FTimeValue((int64)Event->GetDuration().Value(), Timescale));
				}
				FString Data = Event->GetData();
				if (Data.IsEmpty())
				{
					Data = Event->GetMessageData();
				}
				// If the data is still empty then there could have been an entire XML element tree which we have parsed.
				// We could reconstruct the XML from it here and pass that along, but for now we do not do this.
				/*
					if (Data.IsEmpty())
					{
					}
				*/
				NewEvent->SetMessageData(Data, Event->GetContentEncoding().Equals(TEXT("base64")));
				NewEvent->SetPeriodID(InPeriod->GetUniqueIdentifier());
				// Add the event to the handler.
				PlayerSessionServices->GetAEMSEventHandler()->AddEvent(NewEvent, InPeriod->GetID(), IAdaptiveStreamingPlayerAEMSHandler::EEventAddMode::UpdateIfExists);
			}
		}
	}
}



/**
 * Resolves an xlink request made by the initial load of the MPD.
 */
FErrorDetail FManifestDASHInternal::ResolveInitialRemoteElementRequest(TSharedPtrTS<FMPDLoadRequestDASH> RequestResponse, FString XMLResponse, bool bSuccess)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_FManifestDASHInternal_XLink);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, FManifestDASHInternal_XLink);

	// Because this is intended solely for initial MPD entities we do not need to worry about anyone accessing our internal structures
	// while we update them. The player proper has not been informed yet that the initial manifest is ready for use.
	FErrorDetail Error;

	// Remove the request from the pending list. If it is not in there this is not a problem.
	PendingRemoteElementLoadRequests.Remove(RequestResponse);
	// Likewise for the pending element list.
	RemoteElementsToResolve.Remove(RequestResponse->XLinkElement);

	TSharedPtrTS<IDashMPDElement> XLinkElement = RequestResponse->XLinkElement.Pin();
	if (XLinkElement.IsValid())
	{
		int64 LastResolveID = XLinkElement->GetXLink().LastResolveID;
		int64 NewResolveID = LastResolveID;
		FDashMPD_RootEntities RootEntities;
		TArray<TWeakPtrTS<IDashMPDElement>> NewXLinkElements;

		// Was this a dummy request to handle a resolve-to-zero in the same way as a real remote entity?
		if (XLinkElement->GetXLink().GetHref().Equals(XLinkResolveToZero))
		{
			int32 Result = ReplaceElementWithRemoteEntities(XLinkElement, RootEntities, LastResolveID, NewResolveID);
			// Result doesn't really matter as long as the element has been removed.
			if (Result < 0)
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("xlink element to resolve-to-zero was not found.")));
			}
			// Ideally the reference count of the element should now be one. If it is not the element is currently in use elsewhere,
			// which is not a problem as long as it is being released soon.
			if (!XLinkElement.IsUnique())
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("xlink element resolved-to-zero is presently being referenced.")));
			}
		}
		else if (bSuccess && !XMLResponse.IsEmpty())
		{
			// Parse the response. We expect the same root elements as is the element itself. Anything else results in an error.
			Error = IManifestParserDASH::BuildFromMPD(RootEntities, NewXLinkElements, XMLResponse.GetCharArray().GetData(), *XLinkElement->GetName(), PlayerSessionServices);
			if (Error.IsOK())
			{
				// There may not be any new xlink:actuate onLoad in the remote entities (see 5.5.3).
				bool bCircularXLink = false;
				for(int32 i=0; i<NewXLinkElements.Num(); ++i)
				{
					TSharedPtrTS<IDashMPDElement> xl = NewXLinkElements[i].Pin();
					if (xl.IsValid() && xl->GetXLink().GetActuate().Equals(XLinkActuateOnLoad))
					{
						bCircularXLink = true;
						break;
					}
				}
				// SegmentList can only have one occurrence anywhere it is a possible element.
				if (RootEntities.SegmentLists.Num() > 1)
				{
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Remote xlink element <SegmentList> cannot contain more than one instance. Ignoring all but the first.")));
				}
				if (!bCircularXLink)
				{
					int32 Result = ReplaceElementWithRemoteEntities(XLinkElement, RootEntities, LastResolveID, NewResolveID);
					// Result doesn't really matter as long as the element has been removed.
					if (Result == 0)
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("xlink element to be updated was not found. May have been removed through an MPD update.")));
					}
					else if (Result < 0)
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("xlink element could not be updated (%d)"), Result));
					}
					// Ideally the reference count of the element should now be one. If it is not the element is currently in use elsewhere,
					// which is not a problem as long as it is being released soon.
					if (!XLinkElement.IsUnique())
					{
						LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("xlink element being update is presently being referenced somewhere.")));
					}
				}
				else
				{
					XLinkElement->GetXLink().Clear();
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Got circular reference on remote entity (an xlink:actuate=onLoad). Invalidating xlink")));
				}
			}
			else
			{
				// An error means the original element stays in place and the xlink is removed (see 5.5.3).
				Error.Clear();
				XLinkElement->GetXLink().Clear();
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Got inappropriate target for remote entity. Invalidating xlink")));
			}
		}
		else
		{
			// Unsuccessful or an empty response. Leave the original element intact. (see 5.5.3) and remove the xlink.
			XLinkElement->GetXLink().Clear();
			if (bSuccess)
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Got empty remote entity. Invalidating xlink")));
			}
			else
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Failed to fetch remote entity. Invalidating xlink")));
			}
		}
	}
	// All done now?
	if (RemoteElementsToResolve.Num())
	{
		// No.
		Error.SetTryAgain();
	}
	return Error;
}


int32 FManifestDASHInternal::ReplaceElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID)
{
	if (Element.IsValid())
	{
		TSharedPtrTS<IDashMPDElement> Parent = Element->GetParentElement();
		if (Parent.IsValid())
		{
			return Parent->ReplaceChildElementWithRemoteEntities(Element, NewRootEntities, OldResolveID, NewResolveID);
		}
	}
	return 0;
}


FTimeValue FManifestDASHInternal::GetAnchorTime() const
{
	return MPDRoot->GetAvailabilityStartTime().IsValid() ? MPDRoot->GetAvailabilityStartTime() : FTimeValue::GetZero();
}

FTimeRange FManifestDASHInternal::GetTotalTimeRange() const
{
	if (GetPresentationType() == FManifestDASHInternal::EPresentationType::Static)
	{
		FTimeValue Anchor = GetAnchorTime();
		TotalTimeRange.Start = Anchor + GetPeriods()[0]->GetStart();
		TotalTimeRange.End = TotalTimeRange.Start + GetDuration();
	}
	else
	{
		bool bIsUpdating = AreUpdatesExpected();
		FTimeValue ast = GetAnchorTime();
		FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		FTimeValue LastEnd = GetLastPeriodEndTime(true);
		TotalTimeRange.End = bIsUpdating ? Now : LastEnd;
		if (TotalTimeRange.End.IsValid())
		{
			if (LastEnd.IsValid() && TotalTimeRange.End > LastEnd)
			{
				TotalTimeRange.End = LastEnd;
			}

			FTimeValue tsb = MPDRoot->GetTimeShiftBufferDepth();
			if (tsb.IsValid())
			{
				TotalTimeRange.Start = TotalTimeRange.End - tsb;
			}
			else
			{
				TotalTimeRange.Start = ast;
			}
			FTimeValue PST = ast + GetPeriods()[0]->GetStart();
			if (TotalTimeRange.Start < PST)
			{
				TotalTimeRange.Start = PST;
			}
			if (TotalTimeRange.End < TotalTimeRange.Start)
			{
				TotalTimeRange.End = TotalTimeRange.Start;
			}
		}
		else
		{
			TotalTimeRange.Start = TotalTimeRange.End = ast;
		}
	}
	return TotalTimeRange;
}

FTimeValue FManifestDASHInternal::GetDesiredLiveLatency() const
{
	return CalculateDistanceToLiveEdge();
}


FTimeValue FManifestDASHInternal::CalculateDistanceToLiveEdge() const
{
	if (CalculatedLiveDistance.IsValid())
	{
		return CalculatedLiveDistance;
	}

	// Check if there is a user provided value. If there is it takes precedence over everything else.
	FTimeValue Distance = PlayerSessionServices->GetOptionValue(OptionKeyLiveSeekableEndOffset).SafeGetTimeValue(FTimeValue());

	// If there is a low latency descriptor we use the target latency from it.
	if (LowLatencyDescriptor.IsValid())
	{
		Distance = LowLatencyDescriptor->Latency.Target;
	}

	// If not set use the MPD@suggestedPresentationDelay
	if (!Distance.IsValid())
	{
		Distance = MPDRoot->GetSuggestedPresentationDelay();
#if 0
		/*
			If the @suggestedPresentationDelay is set equal to or shorter than the @minBufferTime there is a
			conflict. Since we have to buffer @minBufferTime worth of content, which could be arriving in real-time only
			then how could we maintain the @suggestedPresentationDelay?
		*/
		if (Distance < MPDRoot->GetMinBufferTime() && !bWarnedAboutTooSmallSuggestedPresentationDelay)
		{
			bWarnedAboutTooSmallSuggestedPresentationDelay = true;
			LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("MPD@suggestedPresentationDelay of %.3fs is less than MPD@minBufferTime of %.3fs"), Distance.GetAsSeconds(), MPDRoot->GetMinBufferTime().GetAsSeconds()));
		}
#endif
	}
	// If not set see if there is an MPD@maxSegmentDuration and use that.
	if (!Distance.IsValid())
	{
		// Keep a distance of 3 segment durations.
		// The DASH-IF-IOP recommendation is 2 to 4 SDURATION. See section 4.3.3.2.2.
		Distance = MPDRoot->GetMaxSegmentDuration() * 3;
		// And no less than 4 seconds.
		if (Distance.GetAsMilliseconds(4000) < 4000)
		{
			Distance.SetFromMilliseconds(4000);
		}
	}
	// If still not valid use MPD@minBufferTime. That one is mandatory to be present.
	if (!Distance.IsValid())
	{
		// Also stay 3 durations away. This could be a really short or a really large value now.
		// We make no adjustments here.
		Distance = MPDRoot->GetMinBufferTime() * 3;
	}
	// We do clamp this to the MPD@timeShiftBufferDepth however to ensure it doesn't get bigger than that.
	if (Distance > MPDRoot->GetTimeShiftBufferDepth())
	{
		Distance = MPDRoot->GetTimeShiftBufferDepth();
	}
	check(Distance.IsValid());
	CalculatedLiveDistance = Distance;
	return Distance;
}

bool FManifestDASHInternal::UsesAST() const
{
	return MPDRoot->GetAvailabilityStartTime().IsValid();
}

bool FManifestDASHInternal::IsStaticType() const
{
	return GetPresentationType() == FManifestDASHInternal::EPresentationType::Static;
}

bool FManifestDASHInternal::HasInjectedTimingSources() const
{
	return bDidInjectUTCTimingElements;
}

FTimeValue FManifestDASHInternal::GetMinimumUpdatePeriod() const
{
	if (GetPresentationType() == FManifestDASHInternal::EPresentationType::Dynamic && MPDRoot.IsValid())
	{
		return MPDRoot->GetMinimumUpdatePeriod();
	}
	return FTimeValue::GetInvalid();
}

bool FManifestDASHInternal::AreUpdatesExpected() const
{
	// Returns true if the MPD is subject to updates.
	if (GetPresentationType() == FManifestDASHInternal::EPresentationType::Static)
	{
		return false;
	}
	return MPDRoot.IsValid() && MPDRoot->GetMinimumUpdatePeriod().IsValid();
}


FTimeValue FManifestDASHInternal::GetAvailabilityEndTime() const
{
	return GetMPDRoot()->GetAvailabilityEndTime();
}

FTimeValue FManifestDASHInternal::GetTimeshiftBufferDepth() const
{
	FTimeValue TSB = GetMPDRoot()->GetTimeShiftBufferDepth();
	if (!TSB.IsValid())
	{
		// Section 5.3.1.2
		// "When not present, the value is infinite. This value of the attribute is undefined if the type attribute is equal to 'static'."
		TSB = GetPresentationType() == FManifestDASHInternal::EPresentationType::Static ? FTimeValue::GetZero() : FTimeValue::GetPositiveInfinity();
	}
	return TSB;
}


FTimeValue FManifestDASHInternal::GetLastPeriodEndTime(bool bForTimeline) const
{
	// As per Annex A.3.2
	FTimeValue ast = GetAnchorTime();
	FTimeValue FetchTime = MPDRoot->GetFetchTime();
	FTimeValue MediaPresentationDuration = FTimeValue::GetPositiveInfinity();
	if (MPDRoot->GetMediaPresentationDuration().IsValid())
	{
		MediaPresentationDuration = MPDRoot->GetMediaPresentationDuration();
	}
	else if (Periods.Last()->GetDuration().IsValid())
	{
		MediaPresentationDuration = Periods.Last()->GetStart() + Periods.Last()->GetDuration() - Periods[0]->GetStart();
	}
	else if (Periods.Last()->GetEnd().IsValid())
	{
		MediaPresentationDuration = Periods.Last()->GetEnd() - Periods[0]->GetStart();
	}

	FTimeValue End = ast + MediaPresentationDuration;
	// If MUP is zero then it is expected that InbandEventStream is used to signal when to update the MPD.
	// Since the MPD will not update through MUP in this case we need to return that the period goes up
	// to mediaPresentationDuration, in this case infinity.
	if (bForTimeline && MPDRoot->GetMinimumUpdatePeriod().IsValid() && MPDRoot->GetMinimumUpdatePeriod() > FTimeValue::GetZero())
	{
		FTimeValue CheckTime = FetchTime + MPDRoot->GetMinimumUpdatePeriod();
		return CheckTime < End ? CheckTime : End;
	}
	else
	{
		return End;
	}
}


FTimeValue FManifestDASHInternal::GetMPDValidityEndTime() const
{
	// Here we return the time until which the MPD is valid.
	FTimeValue ast = GetAnchorTime();
	FTimeValue FetchTime = MPDRoot->GetFetchTime();
	if (MPDRoot->GetMinimumUpdatePeriod().IsValid())
	{
		return FetchTime + MPDRoot->GetMinimumUpdatePeriod();
	}
	return FetchTime;
}


FTimeRange FManifestDASHInternal::GetSeekableTimeRange() const
{
	const FTimeValue FixedSeekEndDistance(0.0);
	if (GetPresentationType() == FManifestDASHInternal::EPresentationType::Static)
	{
		FTimeValue Anchor = GetAnchorTime();
		SeekableTimeRange.Start = Anchor + GetPeriods()[0]->GetStart();
		// FIXME: 10 seconds is an arbitrary value. We do not know the actual segment duration of the very last segment
		//        so the only recourse right now is to only allow seeking up to some sensible point before the end.
		SeekableTimeRange.End = Anchor + GetPeriods().Last()->GetEnd() - FixedSeekEndDistance;
		if (SeekableTimeRange.End < SeekableTimeRange.Start)
		{
			SeekableTimeRange.End = SeekableTimeRange.Start;
		}
	}
	else
	{
		// We can only rely on the MPD up to the time it was fetched. While it could be documenting
		// future segments already (a pre-existing presentation made available over time) we would not
		// be able to fetch them anyway on account of their availability window not being valid yet.
		// Typically we would want to play some distance away from the bleeding Live edge.
		bool bIsUpdating = AreUpdatesExpected() || EpicEventType == EEpicEventType::Dynamic;
		FTimeValue Distance = bIsUpdating ? CalculateDistanceToLiveEdge() : FixedSeekEndDistance;
		FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		FTimeValue LastEnd = GetLastPeriodEndTime(true);
		FTimeValue CurrentEnd = Now;
		if (LastEnd.IsValid() && CurrentEnd > LastEnd)
		{
			CurrentEnd = LastEnd;
		}
		FTimeValue ast = GetAnchorTime();
		if (CurrentEnd.IsValid())
		{
			SeekableTimeRange.End = CurrentEnd - Distance;

			FTimeValue tsb = MPDRoot->GetTimeShiftBufferDepth();
			if (tsb.IsValid())
			{
				SeekableTimeRange.Start = CurrentEnd - tsb;
			}
			else
			{
				SeekableTimeRange.Start = ast;
			}
			/*
				The start must be covered by a Period. See: https://dashif-documents.azurewebsites.net/Guidelines-TimingModel/master/Guidelines-TimingModel.html#timing-timeshift
					"Clients SHALL NOT allow seeking into regions of the time shift buffer that are not covered by periods,
					regardless of whether such regions are before or after the periods described by the MPD."
			*/
			FTimeValue PST = ast + GetPeriods()[0]->GetStart();
			if (SeekableTimeRange.Start < PST)
			{
				SeekableTimeRange.Start = PST;
			}
			if (SeekableTimeRange.End < SeekableTimeRange.Start)
			{
				SeekableTimeRange.End = SeekableTimeRange.Start;
			}
			else if (SeekableTimeRange.End > LastEnd - Distance)
			{
				SeekableTimeRange.End = LastEnd - Distance;
			}
		}
		else
		{
			SeekableTimeRange.Start = SeekableTimeRange.End = ast;
		}
	}
	return SeekableTimeRange;
}

FTimeValue FManifestDASHInternal::GetDuration() const
{
	// An MPD@mediaPresentationDuration is authoritative in both static and dynamic presentations.
	if (MPDRoot->GetMediaPresentationDuration().IsValid())
	{
		return MPDRoot->GetMediaPresentationDuration();
	}
	else if (GetPresentationType() == FManifestDASHInternal::EPresentationType::Static)
	{
		if (MPDRoot->GetMediaPresentationDuration().IsValid())
		{
			return MPDRoot->GetMediaPresentationDuration();
		}
		else
		{
			// In a static period there cannot be any early periods so we can just use the difference between first and last.
			return Periods.Last()->GetEnd() - Periods[0]->GetStart();
		}
	}
	else
	{
		if (!MPDRoot->GetMinimumUpdatePeriod().IsValid() && (Periods.Last()->GetDuration().IsValid() || Periods.Last()->GetEnd().IsValid()))
		{
			return (Periods.Last()->GetEnd().IsValid() ? Periods.Last()->GetEnd() : Periods.Last()->GetStart() + Periods.Last()->GetDuration()) - Periods[0]->GetStart();
		}
		return FTimeValue::GetPositiveInfinity();
	}
}


void FManifestDASHInternal::EndPresentationAt(const FTimeValue& EndsAt, const FString& InPeriod)
{
	TSharedPtrTS<FPeriod> Period;
	if (InPeriod.IsEmpty() && Periods.Num())
	{
		Period = Periods.Last();
	}
	else
	{
		Period = GetPeriodByUniqueID(InPeriod);
	}
	if (Period.IsValid())
	{
		Period->EndPresentationAt(EndsAt - GetAnchorTime());
	}
	// Updates no longer expected.
	MPDRoot->SetMinimumUpdatePeriod(FTimeValue::GetInvalid());
	if (Periods.Num())
	{
		FTimeValue NewDuration = EndsAt - (GetAnchorTime() + Periods[0]->GetStart());
		MPDRoot->SetMediaPresentationDuration(NewDuration);
	}
}



void FManifestDASHInternal::PrepareDefaultStartTime()
{
	FTimeRange PlaybackRange = GetPlayTimesFromURI(IManifest::EPlaybackRangeType::TemporaryPlaystartRange);
	DefaultStartTime = PlaybackRange.Start;
	DefaultEndTime = PlaybackRange.End;
}


FTimeRange FManifestDASHInternal::GetPlayTimesFromURI(IManifest::EPlaybackRangeType InRangeType) const
{
	FTimeRange FromTo;

	// We are interested in the 't', 'r' and 'period' fragment values here.
	FString Time, PeriodID;
	for(int32 i=0,iMax=URLFragmentComponents.Num(); i<iMax; ++i)
	{
		if (InRangeType == IManifest::EPlaybackRangeType::TemporaryPlaystartRange)
		{
			if (URLFragmentComponents[i].Name.Equals(TEXT("t")))
			{
				Time = URLFragmentComponents[i].Value;
			}
			else if (URLFragmentComponents[i].Name.Equals(TEXT("period")))
			{
				PeriodID = URLFragmentComponents[i].Value;
			}
		}
		else if (InRangeType == IManifest::EPlaybackRangeType::LockedPlaybackRange)
		{
			if (URLFragmentComponents[i].Name.Equals(TEXT("r")))
			{
				Time = URLFragmentComponents[i].Value;
			}
		}
	}
	if (Time.IsEmpty() && PeriodID.IsEmpty())
	{
		return FromTo;
	}

	FTimeRange AvailableTimeRange = GetTotalTimeRange();
	// Is the time specified as a POSIX time?
	TArray<FString> TimeRange;
	const TCHAR* const TimeDelimiter = TEXT(",");
	if (Time.StartsWith(TEXT("posix")))
	{
		Time.RightChopInline(6);
		Time.ParseIntoArray(TimeRange, TimeDelimiter, false);
		FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		if (TimeRange.Num() && !TimeRange[0].IsEmpty())
		{
			// Is the start time the special time 'now'?
			if (TimeRange[0].Equals(TEXT("now")))
			{
				// A static event will not use an updated wallclock NOW, so if 'now' is used we do an init
				// with the current time.
				if (EpicEventType == EEpicEventType::Static)
				{
					FromTo.Start = Now;
				}
				else
				{
					// 'now' is dynamic. The time will continue to flow between here where we set the value and
					// the moment playback will begin with buffered data.
					// We do not lock 'now' with the current time but leave it unset. This results in the start
					// time to be taken from the seekable range's end value which is updating dynamically.
					FromTo.Start.SetToInvalid();
				}
			}
			else
			{
				FTimeValue s;
				if (UnixEpoch::ParseFloatString(s, *TimeRange[0]))
				{
					FromTo.Start = s;
				}
			}
		}
		if (TimeRange.Num() > 1 && !TimeRange[1].IsEmpty())
		{
			if (TimeRange[1].Equals(TEXT("now")))
			{
				FromTo.End = Now;
			}
			else
			{
				FTimeValue e;
				if (UnixEpoch::ParseFloatString(e, *TimeRange[1]))
				{
					FromTo.End= e;
				}
			}
		}
	}
	else
	{
		FTimeValue PeriodStart;
		// If there is no period specified then the period is the one with the earliest start time
		if (PeriodID.IsEmpty())
		{
			PeriodStart = Periods[0]->GetStart();
		}
		else
		{
			// Look for the named period.
			for(int32 i=0; i<Periods.Num(); ++i)
			{
				if (Periods[i]->GetID().Equals(PeriodID))
				{
					PeriodStart = Periods[i]->GetStart();
					break;
				}
			}
			// If the named period wasn't found use the first one.
			if (!FromTo.Start.IsValid())
			{
				PeriodStart = Periods[0]->GetStart();
			}
		}

		PeriodStart += GetAnchorTime();
		FromTo.Start = PeriodStart;
		// If there is no t specified we are done, otherwise we need to parse it.
		if (!Time.IsEmpty())
		{
			FTimeValue Offset;
			// We need to parse out the 't' and add it to the period.
			Time.ParseIntoArray(TimeRange, TimeDelimiter, false);
			if (TimeRange.Num() && !TimeRange[0].IsEmpty())
			{
				if (RFC2326::ParseNPTTime(Offset, TimeRange[0]))
				{
					FromTo.Start = PeriodStart + Offset;
				}
			}
			if (TimeRange.Num() > 1 && !TimeRange[1].IsEmpty())
			{
				if (RFC2326::ParseNPTTime(Offset, TimeRange[1]))
				{
					FromTo.End = PeriodStart + Offset;
				}
			}
		}
	}
	// Need to clamp this into the seekable range to prevent any issues.
	if (FromTo.Start.IsValid())
	{
		if (AvailableTimeRange.Start.IsValid() && FromTo.Start < AvailableTimeRange.Start)
		{
			FromTo.Start = AvailableTimeRange.Start;
		}
		/*
		else if (AvailableTimeRange.End.IsValid() && FromTo.Start > AvailableTimeRange.End)
		{
			FromTo.Start = AvailableTimeRange.End;
		}
		*/
	}
	return FromTo;
}

FTimeValue FManifestDASHInternal::GetDefaultStartTime() const
{
	return DefaultStartTime;
}

void FManifestDASHInternal::ClearDefaultStartTime()
{
	DefaultStartTime.SetToInvalid();
}

FTimeValue FManifestDASHInternal::GetDefaultEndTime() const
{
	return DefaultEndTime;
}

void FManifestDASHInternal::ClearDefaultEndTime()
{
	DefaultEndTime.SetToInvalid();
}


} // namespace Electra

