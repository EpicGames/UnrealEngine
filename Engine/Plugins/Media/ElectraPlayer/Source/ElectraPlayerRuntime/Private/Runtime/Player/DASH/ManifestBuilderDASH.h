// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlaybackTimeline.h"
#include "Player/PlaylistReader.h"
#include "Player/PlayerSessionServices.h"
#include "PlaylistReaderDASH_Internal.h"
#include "ElectraHTTPStream.h"
#include "StreamTypes.h"
#include "ErrorDetail.h"


namespace Electra
{

class IManifestBuilderDASH
{
public:
	static IManifestBuilderDASH* Create(IPlayerSessionServices* PlayerSessionServices);

	virtual ~IManifestBuilderDASH() = default;

	/**
	 * Builds a new internal manifest from a DASH MPD
	 *
	 * @param OutMPD
	 * @param InOutMPDXML
	 * @param EffectiveURL
	 * @param ETag
	 * @param InResponseHeaders
	 *
	 * @return
	 */
	virtual FErrorDetail BuildFromMPD(TSharedPtrTS<FManifestDASHInternal>& OutMPD, TCHAR* InOutMPDXML, const FString& EffectiveURL, const FString& ETag, const TArray<FElectraHTTPStreamHeader>& InResponseHeaders) = 0;
};


} // namespace Electra




