// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "ElectraPlayerPrivate.h"
#include "ManifestDASH.h"
#include "ManifestBuilderDASH.h"
#include "PlaylistReaderDASH.h"
#include "Stats/Stats.h"
#include "Player/StreamSegmentReaderCommon.h"
#include "MP4Boxes.h"
#include "MP4Utilities.h"
#include "Demuxer/ParserISO14496-12.h"
#include "Demuxer/ParserISO14496-12_Utils.h"
#include "Demuxer/ParserMKV.h"
#include "Player/PlayerSessionServices.h"
#include "SynchronizedClock.h"
#include "Utilities/Utilities.h"
#include "Utilities/StringHelpers.h"
#include "Utilities/URLParser.h"
#include "Utilities/TimeUtilities.h"
#include "HTTP/HTTPManager.h"
#include "Player/DASH/OptionKeynamesDASH.h"
#include "Player/PlayerEntityCache.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/DRM/DRMManager.h"
#include "Player/ContentSteeringHandler.h"
#include "Player/CMCDHandler.h"

DECLARE_CYCLE_STAT(TEXT("FRepresentation::FindSegment"), STAT_ElectraPlayer_DASH_FindSegment, STATGROUP_ElectraPlayer);

namespace Electra
{

#define ERRCODE_DASH_MPD_INTERNAL						1
#define ERRCODE_DASH_MPD_BAD_REPRESENTATION				1000
#define ERRCODE_DASH_DRM_ERROR							2000


#define MAX_NEXT_SEGMENTS_FOR_CMCD	3


namespace DashUtils
{
	#define GETPLAYEROPTION(Type, Getter)																						\
		bool GetPlayerOption(IPlayerSessionServices* InPlayerSessionServices, Type& OutValue, const FName& Key, Type Default)	\
		{																														\
			if (InPlayerSessionServices->HaveOptionValue(Key))																	\
			{																													\
				OutValue = InPlayerSessionServices->GetOptionValue(Key).Getter(Default);										\
				return true;																									\
			}																													\
			OutValue = Default;																									\
			return false;																										\
		}
	GETPLAYEROPTION(FString, SafeGetFString);
	GETPLAYEROPTION(double, SafeGetDouble);
	GETPLAYEROPTION(int64, SafeGetInt64);
	GETPLAYEROPTION(bool, SafeGetBool);
	GETPLAYEROPTION(FTimeValue, SafeGetTimeValue);
	#undef GETPLAYEROPTION


	class FMatroskaParserDataReader : public IGenericDataReader, public TSharedFromThis<FMatroskaParserDataReader, ESPMode::ThreadSafe>
	{
	public:
		virtual ~FMatroskaParserDataReader() { }

		FMatroskaParserDataReader(TSharedPtrTS<FMPDLoadRequestDASH> InLoadReq)
			: LoadRequest(MoveTemp(InLoadReq))
		{ }

		int64 ReadData(void* InDestinationBuffer, int64 InNumBytesToRead, int64 InFromOffset) override
		{
			int64 nr = -1;
			if (LoadChunk(InNumBytesToRead, InFromOffset) && ResponseBuffer.IsValid())
			{
				nr = ResponseBuffer->GetLinearReadSize();
				FMemory::Memcpy(InDestinationBuffer, ResponseBuffer->GetLinearReadData(), nr);
			}
			return nr;
		}

		int64 GetCurrentOffset() const override
		{ check(!"should not be called!"); return -1; }
		int64 GetTotalSize() const override
		{ return FileSize; }
		bool HasReadBeenAborted() const override
		{ return false; }
		bool HasReachedEOF() const override
		{ check(!"this should not be called"); return false; }

	private:
		bool LoadChunk(int64 InNumBytesToRead, int64 InFromOffset)
		{
			auto Matches = [this](const TSharedPtrTS<FMPDLoadRequestDASH>& lr, int64 NumBytesToRead, int64 FromOffset) -> bool
			{
				if (lr.IsValid())
				{
					ElectraHTTPStream::FHttpRange crh;
					const HTTP::FConnectionInfo* ci = lr->GetConnectionInfo();
					if (ci && crh.ParseFromContentRangeResponse(ci->ContentRangeHeader))
					{
						if (FromOffset >= crh.GetStart() && FromOffset <= crh.GetEndIncluding())
						{
							FileSize = FileSize < 0 ? crh.GetDocumentSize() : FileSize;
							ResponseBuffer = lr->Request->GetResponseBuffer();
							return true;
						}
					}
				}
				return false;
			};

			ResponseBuffer.Reset();
			if (LoadRequest.IsValid())
			{
				if (Matches(LoadRequest, InNumBytesToRead, InFromOffset))
				{
					return true;
				}
				else
				{
					for(auto &Chained : LoadRequest->CompletedRequestChain)
					{
						if (Matches(Chained, InNumBytesToRead, InFromOffset))
						{
							return true;
						}
					}
				}
			}
			return false;
		}


		TSharedPtrTS<FMPDLoadRequestDASH> LoadRequest;
		TSharedPtrTS<FWaitableBuffer> ResponseBuffer;
		int64 FileSize = -1;
	};
}


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

class FDASHPlayPeriod : public IManifest::IPlayPeriod, public TSharedFromThis<FDASHPlayPeriod, ESPMode::ThreadSafe>
{
public:
	FDASHPlayPeriod(IPlayerSessionServices* InPlayerSessionServices, const FString& SelectedPeriodID)
		: PlayerSessionServices(InPlayerSessionServices)
		, PeriodID(SelectedPeriodID)
	{
	}

	virtual ~FDASHPlayPeriod()
	{
		if (DrmClient.IsValid())
		{
			DrmClient->UnregisterEventListener(PlayerSessionServices->GetDRMManager());
		}
	}

	//----------------------------------------------
	// Methods from IManifest::IPlayPeriod
	//
	void SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) override;
	EReadyState GetReadyState() override;
	void Load() override;
	void PrepareForPlay() override;
	int64 GetDefaultStartingBitrate() const override;
	TSharedPtrTS<FBufferSourceInfo> GetSelectedStreamBufferSourceInfo(EStreamType StreamType) override;
	FString GetSelectedAdaptationSetID(EStreamType StreamType) override;
	ETrackChangeResult ChangeTrackStreamPreference(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) override;
	TSharedPtrTS<ITimelineMediaAsset> GetMediaAsset() const override;
	void SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex) override;
	void TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload) override;
	IManifest::FResult GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, IManifest::ESearchType SearchType) override;
	IManifest::FResult GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType StreamType, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, IManifest::ESearchType SearchType) override;
	IManifest::FResult GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, IManifest::ESearchType SearchType) override;
	IManifest::FResult GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options) override;
	IManifest::FResult GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData) override;
	void IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount) override;
	void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID) override;

private:
	ELECTRA_IMPL_DEFAULT_ERROR_METHODS(DASHManifest);

	struct FPrioritizedSelection
	{
		FStreamCodecInformation CodecInfo;
		int32 Index = -1;
		int32 Priority = -1;
		int32 Bitrate = 0;
	};

	struct FInitSegmentInfo
	{
		FString AdaptationSetID;
		FString RepresentationSetID;
		FSegmentInformationCommon InitSegmentInfo;
		bool bRequested = false;
		TSharedPtrTS<FMPDLoadRequestDASH> LoadRequest;

		class FAcceptBoxes : public IParserISO14496_12::IBoxCallback
		{
		public:
			virtual ~FAcceptBoxes() = default;
			IParserISO14496_12::IBoxCallback::EParseContinuation OnFoundBox(IParserISO14496_12::FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset) override
			{ return IParserISO14496_12::IBoxCallback::EParseContinuation::Continue; }
			IParserISO14496_12::IBoxCallback::EParseContinuation OnEndOfBox(IParserISO14496_12::FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset) override
			{ return IParserISO14496_12::IBoxCallback::EParseContinuation::Continue; }
		};
	};

	TSharedPtrTS<FManifestDASHInternal> GetCurrentManifest() const;

	TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationFromAdaptationByMaxBandwidth(TWeakPtrTS<IPlaybackAssetAdaptationSet> InAdaptationSet, int32 NotExceedingBandwidth);
	TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationFromAdaptationByPriorityAndMaxBandwidth(TWeakPtrTS<IPlaybackAssetAdaptationSet> InAdaptationSet, int32 NotExceedingBandwidth, EStreamType StreamType);

	void GetRepresentationInitSegmentsFromAdaptation(TArray<FInitSegmentInfo>& OutInitSegInfos, TWeakPtrTS<IPlaybackAssetAdaptationSet> InAdaptationSet);
	void MergeRepresentationInitSegments(TArray<FInitSegmentInfo>& InOutInitSegInfos, const TArray<FInitSegmentInfo>& NewInitSegInfos);
	void HandleRepresentationInitSegmentLoading(const TArray<FInitSegmentPreload>& InitSegmentsToPreload);
	void InitSegmentMP4DownloadComplete(TSharedPtrTS<FMPDLoadRequestDASH> LoadRequest, bool bSuccess);
	void InitSegmentMKVDownloadComplete(TSharedPtrTS<FMPDLoadRequestDASH> LoadRequest, bool bSuccess);

	enum class ENextSegType
	{
		SamePeriodNext,
		SamePeriodRetry,
		SamePeriodStartOver,
		NextPeriod,
	};
	IManifest::FResult GetNextOrRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, bool& OutWaitForRemoteElement, TSharedPtrTS<const IStreamSegment> InCurrentSegment, ENextSegType InNextType, const FPlayStartOptions& Options);

	bool PrepareDRM(const TArray<FManifestDASHInternal::FAdaptationSet::FContentProtection>& InContentProtections);

	void SetupCommonSegmentRequestInfos(TSharedPtrTS<FStreamSegmentRequestCommon>& InOutSegmentRequest);

	void PrioritizeSelection(TArray<FPrioritizedSelection>& Selection, EStreamType StreamType, bool bAdaptationSetLevel, bool bSortByBitrateDescending);

	TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> SelectAdaptationSetByAttributes(TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FManifestDASHInternal::FPeriod> Period, EStreamType StreamType, const FStreamSelectionAttributes& Attributes);

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	EReadyState ReadyState = EReadyState::NotLoaded;
	FStreamSelectionAttributes VideoStreamPreferences;
	FStreamSelectionAttributes AudioStreamPreferences;
	FStreamSelectionAttributes SubtitleStreamPreferences;

	FString PeriodID;

	FString ActiveVideoAdaptationSetID;
	FString ActiveAudioAdaptationSetID;
	FString ActiveSubtitleAdaptationSetID;

	FString ActiveVideoRepresentationID;
	FString ActiveAudioRepresentationID;
	FString ActiveSubtitleRepresentationID;

	struct FSelectedQualityIndex
	{
		int32 Index = 0;
		int32 MaxIndex = 0;
		void Reset()
		{
			Index = 0;
			MaxIndex = 0;
		}
	};
	FSelectedQualityIndex ActiveVideoQualityIndex;
	FSelectedQualityIndex ActiveAudioQualityIndex;
	FSelectedQualityIndex ActiveSubtitleQualityIndex;

	TSharedPtrTS<FBufferSourceInfo> SourceBufferInfoVideo;
	TSharedPtrTS<FBufferSourceInfo> SourceBufferInfoAudio;
	TSharedPtrTS<FBufferSourceInfo> SourceBufferInfoSubtitles;

	TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> DrmClient;


	TArray<FInitSegmentInfo> VideoInitSegmentInfos;
};

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


TSharedPtrTS<FManifestDASH> FManifestDASH::Create(IPlayerSessionServices* SessionServices, TSharedPtrTS<FManifestDASHInternal> Manifest)
{
	FManifestDASH* m = new FManifestDASH(SessionServices, Manifest);
	return MakeShareable<FManifestDASH>(m);
}

FManifestDASH::FManifestDASH(IPlayerSessionServices* InSessionServices, TSharedPtrTS<FManifestDASHInternal> InManifest)
	: PlayerSessionServices(InSessionServices)
	, CurrentManifest(InManifest)
{
}

FManifestDASH::~FManifestDASH()
{
}

void FManifestDASH::UpdateInternalManifest(TSharedPtrTS<FManifestDASHInternal> UpdatedManifest)
{
	CurrentManifest = UpdatedManifest;
}

EMediaFormatType FManifestDASH::GetMediaFormatType() const
{
	return EMediaFormatType::DASH;
}

bool FManifestDASH::ReloadsPlaylistPeriodically() const
{
	return true;
}

bool FManifestDASH::SupportsMultipleTracksPerKind() const
{
	return true;
}

bool FManifestDASH::OrderOfTracksCanChangeDynamically() const
{
	return true;
}

IManifest::EType FManifestDASH::GetPresentationType() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	if (Manifest.IsValid())
	{
		return Manifest->GetPresentationType() == FManifestDASHInternal::EPresentationType::Static ? IManifest::EType::OnDemand : IManifest::EType::Live;
	}
	return IManifest::EType::OnDemand;
}

IManifest::EReplayEventType FManifestDASH::GetReplayType() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	if (Manifest.IsValid())
	{
		return Manifest->IsEpicEvent() ? IManifest::EReplayEventType::IsReplay : IManifest::EReplayEventType::NoReplay;
	}
	return IManifest::EReplayEventType::NoReplay;
}


TSharedPtrTS<const FLowLatencyDescriptor> FManifestDASH::GetLowLatencyDescriptor() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetLowLatencyDescriptor() : nullptr;
}

FTimeValue FManifestDASH::CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const
{
	FTimeValue LiveLatency;
	if (GetPresentationType() != IManifest::EType::OnDemand)
	{
		FTimeValue UTCNow = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		LiveLatency = UTCNow - InCurrentPlaybackPosition;

		if (bViaLatencyElement)
		{
			TSharedPtrTS<const FLowLatencyDescriptor> llDesc = GetLowLatencyDescriptor();
			if (llDesc.IsValid())
			{
				// Low latency Live
				TSharedPtrTS<IProducerReferenceTimeInfo> ProdRefTime = GetProducerReferenceTimeInfo(llDesc->Latency.ReferenceID);
				if (ProdRefTime.IsValid())
				{
					if (InEncoderLatency.IsValid())
					{
						LiveLatency += InEncoderLatency;
					}
				}
			}
		}
	}
	return LiveLatency;
}

FTimeValue FManifestDASH::GetAnchorTime() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetAnchorTime() : FTimeValue();
}

FTimeRange FManifestDASH::GetTotalTimeRange() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetTotalTimeRange() : FTimeRange();
}

FTimeRange FManifestDASH::GetSeekableTimeRange() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetSeekableTimeRange() : FTimeRange();
}

FTimeRange FManifestDASH::GetPlaybackRange(EPlaybackRangeType InRangeType) const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetPlayTimesFromURI(InRangeType) : FTimeRange();
}

FTimeValue FManifestDASH::GetDuration() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetDuration() : FTimeValue();
}

FTimeValue FManifestDASH::GetDefaultStartTime() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetDefaultStartTime() : FTimeValue();
}

void FManifestDASH::ClearDefaultStartTime()
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	if (Manifest.IsValid())
	{
		Manifest->ClearDefaultStartTime();
	}
}

FTimeValue FManifestDASH::GetDefaultEndTime() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetDefaultEndTime() : FTimeValue();
}

void FManifestDASH::ClearDefaultEndTime()
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	if (Manifest.IsValid())
	{
		Manifest->ClearDefaultEndTime();
	}
}

FTimeValue FManifestDASH::GetMinBufferTime() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	if (Manifest.IsValid())
	{
		TSharedPtrTS<const FDashMPD_MPDType> MPDRoot = Manifest->GetMPDRoot();
		return MPDRoot->GetMinBufferTime();
	}
	return FTimeValue();
}

FTimeValue FManifestDASH::GetDesiredLiveLatency() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetDesiredLiveLatency() : FTimeValue();
}

IManifest::ELiveEdgePlayMode FManifestDASH::GetLiveEdgePlayMode() const
{
	return IManifest::ELiveEdgePlayMode::Default;
}


TSharedPtrTS<IProducerReferenceTimeInfo> FManifestDASH::GetProducerReferenceTimeInfo(int64 ID) const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	return Manifest.IsValid() ? Manifest->GetProducerReferenceTimeElement(ID) : nullptr;
}

TRangeSet<double> FManifestDASH::GetPossiblePlaybackRates(EPlayRateType InForType) const
{
	TRangeSet<double> Ranges;
	if (InForType == IManifest::EPlayRateType::UnthinnedRate)
	{
		Ranges.Add(TRange<double>::Inclusive(0.1, 4.0));
	}
	else
	{
		Ranges.Add(TRange<double>{1.0}); // normal (real-time) playback rate
	}
	Ranges.Add(TRange<double>{0.0}); // and pause
	return Ranges;
}


void FManifestDASH::GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	if (Manifest.IsValid() && Manifest->GetPeriods().Num())
	{
		// At present we return metadata from the first period only as every period can have totally different
		// number of streams and even codecs. There is no commonality between periods.
		Manifest->PreparePeriodAdaptationSets(Manifest->GetPeriods()[0], false);
		Manifest->GetPeriods()[0]->GetMetaData(OutMetadata, StreamType);
	}
}
void FManifestDASH::UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData)
{
	// No-op.
}

void FManifestDASH::UpdateDynamicRefetchCounter()
{
	++CurrentPeriodAndAdaptationXLinkResolveID;

	// Since we don't know which streams will be used now we have to let the manifest reader know
	// that currently no stream is active that is providing inband events.
	TSharedPtrTS<IPlaylistReader> ManifestReader = PlayerSessionServices->GetManifestReader();
	IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(ManifestReader.Get());
	if (Reader)
	{
		Reader->SetStreamInbandEventUsage(EStreamType::Video, false);
		Reader->SetStreamInbandEventUsage(EStreamType::Audio, false);
		Reader->SetStreamInbandEventUsage(EStreamType::Subtitle, false);
	}
}

void FManifestDASH::PrepareForLooping(int32 InNumLoopsToAdd)
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = CurrentManifest;
	if (Manifest.IsValid() && Manifest->IsEpicEvent())
	{
		Manifest->PrepareEpicEventForLooping(InNumLoopsToAdd);
	}
}


void FManifestDASH::TriggerClockSync(IManifest::EClockSyncType InClockSyncType)
{
	IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(PlayerSessionServices->GetManifestReader().Get());
	Reader->RequestClockResync();
}

void FManifestDASH::TriggerPlaylistRefresh()
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = CurrentManifest;
	// Trigger only when updates are not expected regularly.
	if (Manifest.IsValid() && Manifest->AreUpdatesExpected() && (Manifest->GetMinimumUpdatePeriod() == FTimeValue::GetZero() || Manifest->GetMinimumUpdatePeriod() > FTimeValue(10.0)))
	{
		IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(PlayerSessionServices->GetManifestReader().Get());
		Reader->RequestMPDUpdate(IPlaylistReaderDASH::EMPDRequestType::GetLatestSegment);
	}
}

void FManifestDASH::ReachedStableBuffer()
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = CurrentManifest;
	if (Manifest.IsValid())
	{
		PlayerSessionServices->GetContentSteeringHandler()->ReachedStableBuffer();
	}
}


IStreamReader* FManifestDASH::CreateStreamReaderHandler()
{
	return new FStreamSegmentReaderCommon;
}

IManifest::FResult FManifestDASH::FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	TSharedPtrTS<FManifestDASHInternal> Manifest(CurrentManifest);
	if (!Manifest.IsValid())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotLoaded);
	}

	const TArray<TSharedPtrTS<FManifestDASHInternal::FPeriod>>& Periods = Manifest->GetPeriods();
	TSharedPtrTS<FManifestDASHInternal::FPeriod> SelectedPeriod;
	if (Periods.Num() == 0)
	{
		return IManifest::FResult(IManifest::FResult::EType::TryAgainLater).RetryAfterMilliseconds(1000);
	}

	FTimeValue PlayRangeEnd = StartPosition.Options.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());

	FTimeValue StartTime = StartPosition.Time;

	// All time values we communicate to the outside - and therefor get from the outside - are offset by the availabilityStartTime.
	StartTime -= Manifest->GetAnchorTime();
	PlayRangeEnd -= Manifest->GetAnchorTime();

	// Quick out if the time falls outside the presentation.
	FTimeValue TotalEndTime = Manifest->GetLastPeriodEndTime(true);
	TotalEndTime -= Manifest->GetAnchorTime();
	if (PlayRangeEnd.IsValid() && TotalEndTime.IsValid() && PlayRangeEnd < TotalEndTime)
	{
		TotalEndTime = PlayRangeEnd;
	}
	if (StartTime >= TotalEndTime)
	{
		return IManifest::FResult(IManifest::FResult::EType::PastEOS);
	}
	// If the time to search for is before the start of the first period we use the first period!
	else if (StartTime < Periods[0]->GetStart())
	{
		LogMessage(PlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Start time is before the start of the first period. Using first period!")));
		StartTime = Periods[0]->GetStart();
	}

	// Find the period into which the start time falls.
	for(int32 nPeriod=0; !SelectedPeriod.IsValid() && nPeriod<Periods.Num(); ++nPeriod)
	{
		if (!Periods[nPeriod]->GetIsEarlyPeriod())
		{
			FTimeValue PeriodStartTime = Periods[nPeriod]->GetStart();
			FTimeValue PeriodEndTime = Periods[nPeriod]->GetEnd();
			// When the period end time is not valid it must be the last period of a Live presentation
			if (!PeriodEndTime.IsValid())
			{
				PeriodEndTime.SetToPositiveInfinity();
			}
			// Does the time fall into this period?
			if (StartTime >= PeriodStartTime && StartTime < PeriodEndTime)
			{
				FTimeValue DiffToNextPeriod = nPeriod + 1 < Periods.Num() && !Periods[nPeriod+1]->GetIsEarlyPeriod() ? Periods[nPeriod + 1]->GetStart() - StartTime : FTimeValue::GetPositiveInfinity();
				if (!DiffToNextPeriod.IsValid())
				{
					DiffToNextPeriod = FTimeValue::GetPositiveInfinity();
				}
				FTimeValue DiffToStart = StartTime - PeriodStartTime;
				switch(SearchType)
				{
					case ESearchType::Closest:
					{
						// There is no actual choice. We have to use the period the time falls into. Why would we want
						// to snap to a different period that won't contain the segments for the time we're looking for.
						SelectedPeriod = Periods[nPeriod];
						break;
					}
					case ESearchType::Before:
					case ESearchType::Same:
					case ESearchType::After:
					{
						// Before, Same and After have no meaning when looking for a period. The period the start time falls into is the one to use.
						SelectedPeriod = Periods[nPeriod];
						break;
					}
					case ESearchType::StrictlyAfter:
					{
						if (!DiffToNextPeriod.IsInfinity())
						{
							SelectedPeriod = Periods[nPeriod + 1];
						}
						break;
					}
					case ESearchType::StrictlyBefore:
					{
						if (nPeriod)
						{
							SelectedPeriod = Periods[nPeriod - 1];
						}
						break;
					}
				}
				// Time fell into this period. We have either found a candidate or not. We're done either way.
				break;
			}
		}
	}

	if (SelectedPeriod.IsValid())
	{
		// Check if the period start is behind the end of the allowed playback range.
		if (TotalEndTime.IsValid() && TotalEndTime <= SelectedPeriod->GetStart())
		{
			return IManifest::FResult(IManifest::FResult::EType::PastEOS);
		}

		// Check if we would start outside the permitted event range.
		if (Manifest->IsDynamicEpicEvent())
		{
			FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
			FTimeValue End = Manifest->GetLastPeriodEndTime(true);
			if (Now >= End)
			{
				return IManifest::FResult(IManifest::FResult::EType::PastEOS);
			}
		}

		// Is the original period still there?
		TSharedPtrTS<FDashMPD_PeriodType> MPDPeriod = SelectedPeriod->GetMPDPeriod();
		if (MPDPeriod.IsValid())
		{
			// Does this period require onRequest xlink resolving?
			if (MPDPeriod->GetXLink().IsSet())
			{
				// Does the period require (re-)resolving?
				if (MPDPeriod->GetXLink().LastResolveID < CurrentPeriodAndAdaptationXLinkResolveID && !MPDPeriod->GetXLink().LoadRequest.IsValid())
				{
					// Need to resolve the xlink now.
					check(!"TODO");
					LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Triggering period xlink resolve")));
					return IManifest::FResult(IManifest::FResult::EType::TryAgainLater).RetryAfterMilliseconds(100);
				}
			}
			// Wrap the period in an externally accessible interface.
			TSharedPtrTS<FDASHPlayPeriod> PlayPeriod = MakeSharedTS<FDASHPlayPeriod>(PlayerSessionServices, SelectedPeriod->GetUniqueIdentifier());
			OutPlayPeriod = PlayPeriod;
			return IManifest::FResult(IManifest::FResult::EType::Found);
		}
		else
		{
			// The period has disappeared. This may happen with an MPD update and means
			// we have to try this all over with the updated one.
			LogMessage(PlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Original MPD period not available, trying again.")));
			return IManifest::FResult(IManifest::FResult::EType::TryAgainLater).RetryAfterMilliseconds(100);
		}
	}
	// Ok, we made sure to use the first period if the start time is less than that of the first one.
	// Coming here can only mean that no period was found which can only means that the time is past the last one.
	// Which means that the duration of the last period is actually less than what MPD@mediaPresentationDuration was saying.
	// So, in a nutshell, we have reached the end.
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}

IManifest::FResult FManifestDASH::FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> InCurrentSegment)
{
	const FStreamSegmentRequestCommon* CurrentRequest = static_cast<const FStreamSegmentRequestCommon*>(InCurrentSegment.Get());
	if (CurrentRequest)
	{
		FPlayStartPosition SearchTime;
		// We use the actual media segment time from the previous request in case the MPD was updated with all new or different periods.
		// That way we get whichever period is following that time. The local media time needs to be clamped to zero in case the PTO
		// would put the media time before the then current period.
		int64 MediaTime = Utils::Max((int64)0, CurrentRequest->Segment.Time - CurrentRequest->Segment.PTO);
		SearchTime.Time = CurrentRequest->AST + CurrentRequest->PeriodStart + FTimeValue(MediaTime, CurrentRequest->Segment.Timescale);
		return FindPlayPeriod(OutPlayPeriod, SearchTime, IManifest::ESearchType::StrictlyAfter);
	}
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtrTS<FManifestDASHInternal> FDASHPlayPeriod::GetCurrentManifest() const
{
	TSharedPtrTS<IPlaylistReader> ManifestReader = PlayerSessionServices->GetManifestReader();
	if (ManifestReader.IsValid())
	{
		IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(ManifestReader.Get());
		return Reader->GetCurrentMPD();
	}
	return nullptr;
}

void FDASHPlayPeriod::SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	if (ForStreamType == EStreamType::Video)
	{
		VideoStreamPreferences = StreamAttributes;
	}
	else if (ForStreamType == EStreamType::Audio)
	{
		AudioStreamPreferences = StreamAttributes;
	}
	else if (ForStreamType == EStreamType::Subtitle)
	{
		SubtitleStreamPreferences = StreamAttributes;
	}
}

IManifest::IPlayPeriod::EReadyState FDASHPlayPeriod::GetReadyState()
{
	return ReadyState;
}

void FDASHPlayPeriod::PrioritizeSelection(TArray<FPrioritizedSelection>& Selection, EStreamType StreamType, bool bAdaptationSetLevel, bool bSortByBitrateDescending)
{
	const FCodecSelectionPriorities& SelectionPriorities = PlayerSessionServices->GetCodecSelectionPriorities(StreamType);
	for(auto &Candidate : Selection)
	{
		int32 NewPriority = -1;
		if (bAdaptationSetLevel)
		{
			NewPriority = SelectionPriorities.GetClassPriority(Candidate.CodecInfo.GetCodecSpecifierRFC6381());
		}
		else
		{
			NewPriority = SelectionPriorities.GetStreamPriority(Candidate.CodecInfo.GetCodecSpecifierRFC6381());
		}
		if (NewPriority >= 0)
		{
			Candidate.Priority = NewPriority;
		}
	}
	// Sort first by descending bitrate?
	if (bSortByBitrateDescending)
	{
		Selection.StableSort([](const FPrioritizedSelection& a, const FPrioritizedSelection& b)
		{
			return a.Bitrate > b.Bitrate;
		});
	}
	// Sort by descending priority.
	Selection.StableSort([](const FPrioritizedSelection& a, const FPrioritizedSelection& b)
	{
		return a.Priority > b.Priority;
	});
}


TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> FDASHPlayPeriod::SelectAdaptationSetByAttributes(TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FManifestDASHInternal::FPeriod> Period, EStreamType StreamType, const FStreamSelectionAttributes& Attributes)
{
	TArray<FPrioritizedSelection> Selection;

	auto AddAdaptationSetToSelection = [](TArray<FPrioritizedSelection>& InOutSelection, const TSharedPtrTS<FManifestDASHInternal::FAdaptationSet>& AS, int32 Index) -> void
	{
		if (AS->GetIsUsable() && !AS->GetIsInSwitchGroup())
		{
			FPrioritizedSelection Candidate;
			Candidate.CodecInfo = AS->GetCodec();
			Candidate.Index = Index;
			Candidate.Priority = AS->GetSelectionPriority();
			InOutSelection.Emplace(MoveTemp(Candidate));
		}
	};

	TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> AS;
	int32 NumAdaptationSets = Period->GetNumberOfAdaptationSets(StreamType);
	if (NumAdaptationSets > 0)
	{
		int32 SelectedTypeIndex = 0;
		if (StreamType == EStreamType::Video || StreamType == EStreamType::Audio || StreamType == EStreamType::Subtitle)
		{
			// Check for a matching language.
			// For now we ignore the track kind.
			if (Attributes.Language_RFC4647.IsSet())
			{
				TArray<int32> CandidateIndices;
				TArray<BCP47::FLanguageTag> CandList;
				for(int32 i=0; i<NumAdaptationSets; ++i)
				{
					TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> A = StaticCastSharedPtr<FManifestDASHInternal::FAdaptationSet>(Period->GetAdaptationSetByTypeAndIndex(StreamType, i));
					CandList.Emplace(A->GetLanguageTag());
				}
				CandidateIndices = BCP47::FindExtendedFilteringMatch(CandList, Attributes.Language_RFC4647.GetValue());
				for(int32 i=0; i<CandidateIndices.Num(); ++i)
				{
					TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> A = StaticCastSharedPtr<FManifestDASHInternal::FAdaptationSet>(Period->GetAdaptationSetByTypeAndIndex(StreamType, CandidateIndices[i]));
					AddAdaptationSetToSelection(Selection, A, CandidateIndices[i]);
				}
				// Prioritize the candidates based on user configuration.
				PrioritizeSelection(Selection, StreamType, true, false);
				if (Selection.Num())
				{
					// Take the highest prioritized set for now.
					SelectedTypeIndex = Selection[0].Index;

					// Check if there is a preferred codec set. If we have multiple AdaptationSets for the same language using different codecs
					// we pick the one for which there is a preference. If there is none matching the first is chosen.
					// This is primarily to ensure the same track is kept when seeking or across period boundaries.
					if (Attributes.Codec.IsSet())
					{
						FString PreferredCodec = Attributes.Codec.GetValue();
						if (!PreferredCodec.IsEmpty())
						{
							for(int32 i=0; i<Selection.Num(); ++i)
							{
								if (Selection[i].CodecInfo.GetCodecName().Equals(PreferredCodec))
								{
									SelectedTypeIndex = Selection[i].Index;
									break;
								}
							}
						}
					}

					AS = StaticCastSharedPtr<FManifestDASHInternal::FAdaptationSet>(Period->GetAdaptationSetByTypeAndIndex(StreamType, SelectedTypeIndex));
				}
			}

			// Matching language not found. Is there an explicit index given?
			// Note: for now we use the explicit override even if a matching language was already found.
			//       there's a reason the override is specified, like to enforce a specific codec for the same language.
			//if (!AS.IsValid())
			{
				if (Attributes.OverrideIndex.IsSet() && Attributes.OverrideIndex.GetValue() >= 0 && Attributes.OverrideIndex.GetValue() < NumAdaptationSets)
				{
					SelectedTypeIndex = Attributes.OverrideIndex.GetValue();
					AS = StaticCastSharedPtr<FManifestDASHInternal::FAdaptationSet>(Period->GetAdaptationSetByTypeAndIndex(StreamType, SelectedTypeIndex));
				}
			}
			// Still nothing? Use the first one, except for subtitles that need to be explicitly selected.
			if (!AS.IsValid() && StreamType != EStreamType::Subtitle)
			{
				Selection.Empty();
				for(int32 i=0; i<NumAdaptationSets; ++i)
				{
					TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> A = StaticCastSharedPtr<FManifestDASHInternal::FAdaptationSet>(Period->GetAdaptationSetByTypeAndIndex(StreamType, i));
					AddAdaptationSetToSelection(Selection, A, i);
				}
				PrioritizeSelection(Selection, StreamType, true, false);
				if (Selection.Num())
				{
					// Take the highest prioritized set.
					SelectedTypeIndex = Selection[0].Index;
					AS = StaticCastSharedPtr<FManifestDASHInternal::FAdaptationSet>(Period->GetAdaptationSetByTypeAndIndex(StreamType, SelectedTypeIndex));
				}
			}
		}
		if (AS.IsValid())
		{
			FTrackMetadata tm;
			AS->GetMetaData(tm, StreamType);
			OutBufferSourceInfo = MakeSharedTS<FBufferSourceInfo>();
			OutBufferSourceInfo->Kind = tm.Kind;
			OutBufferSourceInfo->LanguageTag = tm.LanguageTagRFC5646;
			OutBufferSourceInfo->Codec = tm.HighestBandwidthCodec.GetCodecName();
			OutBufferSourceInfo->HardIndex = SelectedTypeIndex;
			OutBufferSourceInfo->PeriodID = Period->GetUniqueIdentifier();
			OutBufferSourceInfo->PeriodAdaptationSetID = Period->GetUniqueIdentifier() + TEXT("/") + AS->GetUniqueIdentifier();
		}
	}
	return AS;
}


int64 FDASHPlayPeriod::GetDefaultStartingBitrate() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = GetCurrentManifest();
	TSharedPtrTS<FManifestDASHInternal::FPeriod> Period = Manifest.IsValid() ? Manifest->GetPeriodByUniqueID(PeriodID) : nullptr;
	if (Period.IsValid())
	{
		TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> VideoAS = Period->GetAdaptationSetByUniqueID(ActiveVideoAdaptationSetID);
		if (VideoAS.IsValid() && VideoAS->GetNumberOfRepresentations())
		{
			TSharedPtrTS<IPlaybackAssetRepresentation> VideoRepr = VideoAS->GetRepresentationByIndex(0);
			return VideoRepr->GetBitrate();
		}

		TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> AudioAS = Period->GetAdaptationSetByUniqueID(ActiveAudioAdaptationSetID);
		if (AudioAS.IsValid() && AudioAS->GetNumberOfRepresentations())
		{
			TSharedPtrTS<IPlaybackAssetRepresentation> AudioRepr = AudioAS->GetRepresentationByIndex(0);
			return AudioRepr->GetBitrate();
		}
	}
	return 2 * 1000 * 1000;
}

void FDASHPlayPeriod::Load()
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = GetCurrentManifest();
	TSharedPtrTS<FManifestDASHInternal::FPeriod> Period = Manifest.IsValid() ? Manifest->GetPeriodByUniqueID(PeriodID) : nullptr;
	if (Period.IsValid())
	{
		TArray<FManifestDASHInternal::FAdaptationSet::FContentProtection> ContentProtections;

		// Prepare the adaptation sets and periods.
		Manifest->PreparePeriodAdaptationSets(Period, false);

		// We need to select one adaptation set per stream type we wish to play.
		TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> VideoAS = SelectAdaptationSetByAttributes(SourceBufferInfoVideo, Period, EStreamType::Video, VideoStreamPreferences);
		if (VideoAS.IsValid())
		{
			ActiveVideoAdaptationSetID = VideoAS->GetUniqueIdentifier();

			// Add encryption schemes, if any.
			if (VideoAS->GetIsSwitchGroup())
			{
				for(auto &SwitchedID : VideoAS->GetSwitchToSetIDs())
				{
					TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> SwitchedAS = Period->GetAdaptationSetByUniqueID(SwitchedID);
					if (SwitchedAS.IsValid())
					{
						ContentProtections.Append(SwitchedAS->GetPossibleContentProtections());
					}
				}
			}
			else
			{
				ContentProtections.Append(VideoAS->GetPossibleContentProtections());
			}
		}

		TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> AudioAS = SelectAdaptationSetByAttributes(SourceBufferInfoAudio, Period, EStreamType::Audio, AudioStreamPreferences);
		if (AudioAS.IsValid())
		{
			ActiveAudioAdaptationSetID = AudioAS->GetUniqueIdentifier();

			// Add encryption schemes, if any.
			if (AudioAS->GetIsSwitchGroup())
			{
				for(auto &SwitchedID : AudioAS->GetSwitchToSetIDs())
				{
					TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> SwitchedAS = Period->GetAdaptationSetByUniqueID(SwitchedID);
					if (SwitchedAS.IsValid())
					{
						ContentProtections.Append(SwitchedAS->GetPossibleContentProtections());
					}
				}
			}
			else
			{
				ContentProtections.Append(AudioAS->GetPossibleContentProtections());
			}
		}

		TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> SubtitleAS = SelectAdaptationSetByAttributes(SourceBufferInfoSubtitles, Period, EStreamType::Subtitle, SubtitleStreamPreferences);
		if (SubtitleAS.IsValid())
		{
			ActiveSubtitleAdaptationSetID = SubtitleAS->GetUniqueIdentifier();

			// Add encryption schemes, if any.
			if (SubtitleAS->GetIsSwitchGroup())
			{
				for(auto &SwitchedID : SubtitleAS->GetSwitchToSetIDs())
				{
					TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> SwitchedAS = Period->GetAdaptationSetByUniqueID(SwitchedID);
					if (SwitchedAS.IsValid())
					{
						ContentProtections.Append(SwitchedAS->GetPossibleContentProtections());
					}
				}
			}
			else
			{
				ContentProtections.Append(SubtitleAS->GetPossibleContentProtections());
			}
		}

		// Prepare the DRM system for decryption.
		if (PrepareDRM(ContentProtections))
		{
			ReadyState = EReadyState::Loaded;
		}
		else
		{
			// Set state to preparing to prevent the player from progressing while
			// the posted error works its magic.
			ReadyState = EReadyState::Loading;
		}
	}
	else
	{
		ReadyState = EReadyState::Loading;
	}
}

void FDASHPlayPeriod::PrepareForPlay()
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = GetCurrentManifest();
	TSharedPtrTS<FManifestDASHInternal::FPeriod> Period = Manifest.IsValid() ? Manifest->GetPeriodByUniqueID(PeriodID) : nullptr;
	if (Period.IsValid())
	{
		TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> VideoAS = Period->GetAdaptationSetByUniqueID(ActiveVideoAdaptationSetID);
		if (VideoAS.IsValid())
		{
			// Get the current average video bitrate with some sensible default if it is not set.
			int64 StartingBitrate = PlayerSessionServices->GetOptionValue(OptionKeyCurrentAvgStartingVideoBitrate).SafeGetInt64(2*1000*1000);

			TSharedPtrTS<IPlaybackAssetRepresentation> VideoRepr = GetRepresentationFromAdaptationByMaxBandwidth(VideoAS, (int32) StartingBitrate);
			if (VideoRepr.IsValid())
			{
				ActiveVideoRepresentationID = VideoRepr->GetUniqueIdentifier();
			}

			// Set up the list of initialization segments.
			TArray<FInitSegmentInfo> InitSegInfos;
			GetRepresentationInitSegmentsFromAdaptation(InitSegInfos, VideoAS);
			MergeRepresentationInitSegments(VideoInitSegmentInfos, InitSegInfos);
		}

		TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> AudioAS = Period->GetAdaptationSetByUniqueID(ActiveAudioAdaptationSetID);
		if (AudioAS.IsValid())
		{
			TSharedPtrTS<IPlaybackAssetRepresentation> AudioRepr = GetRepresentationFromAdaptationByPriorityAndMaxBandwidth(AudioAS, 256 * 1000, EStreamType::Audio);
			if (AudioRepr.IsValid())
			{
				ActiveAudioRepresentationID = AudioRepr->GetUniqueIdentifier();
			}
		}

		TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> SubtitleAS = Period->GetAdaptationSetByUniqueID(ActiveSubtitleAdaptationSetID);
		if (SubtitleAS.IsValid())
		{
			TSharedPtrTS<IPlaybackAssetRepresentation> SubtitleRepr = GetRepresentationFromAdaptationByPriorityAndMaxBandwidth(SubtitleAS, 256 * 1000, EStreamType::Subtitle);
			if (SubtitleRepr.IsValid())
			{
				ActiveSubtitleRepresentationID = SubtitleRepr->GetUniqueIdentifier();
			}
		}


		// If there is a low latency service description with a reference ID to a producer reference time, set it in the options.
		// This will be retrieved from there in setting up the segment requests.
		TSharedPtrTS<const FLowLatencyDescriptor> llDesc = Manifest->GetLowLatencyDescriptor();
		if (llDesc.IsValid())
		{
			if (llDesc->Latency.ReferenceID >= 0)
			{
				PlayerSessionServices->GetMutableOptions().Set(DASH::OptionKey_LatencyReferenceId, FVariantValue(llDesc->Latency.ReferenceID));
			}
			else
			{
				PlayerSessionServices->GetMutableOptions().Remove(DASH::OptionKey_LatencyReferenceId);
			}
		}

		// Emit all <EventStream> events of the period to the AEMS event handler.
		Manifest->SendEventsFromAllPeriodEventStreams(Period);

		ReadyState = EReadyState::IsReady;
	}
	else
	{
		ReadyState = EReadyState::Preparing;
	}
}

TSharedPtrTS<FBufferSourceInfo> FDASHPlayPeriod::GetSelectedStreamBufferSourceInfo(EStreamType StreamType)
{
	switch(StreamType)
	{
		case EStreamType::Video:
		{
			return SourceBufferInfoVideo;
		}
		case EStreamType::Audio:
		{
			return SourceBufferInfoAudio;
		}
		case EStreamType::Subtitle:
		{
			return SourceBufferInfoSubtitles;
		}
		default:
		{
			return nullptr;
		}
	}
}

FString FDASHPlayPeriod::GetSelectedAdaptationSetID(EStreamType StreamType)
{
	switch(StreamType)
	{
		case EStreamType::Video:
		{
			return ActiveVideoAdaptationSetID;
		}
		case EStreamType::Audio:
		{
			return ActiveAudioAdaptationSetID;
		}
		case EStreamType::Subtitle:
		{
			return ActiveSubtitleAdaptationSetID;
		}
		default:
		{
			return FString();
		}
	}
}


IManifest::IPlayPeriod::ETrackChangeResult FDASHPlayPeriod::ChangeTrackStreamPreference(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	// We cannot check if the the new stream to be selected is already the one that is active because a track change is
	// triggered at the current playback position, while we could already be in a later period!
	// Checking against a later period makes no sense, so we have to forcibly start over.
	if (ForStreamType == EStreamType::Video)
	{
		return IManifest::IPlayPeriod::ETrackChangeResult::StartOver;
	}
	return IManifest::IPlayPeriod::ETrackChangeResult::NewPeriodNeeded;
}


bool FDASHPlayPeriod::PrepareDRM(const TArray<FManifestDASHInternal::FAdaptationSet::FContentProtection>& InContentProtections)
{
	if (InContentProtections.Num())
	{
		static const TCHAR* const Delimiters[] { TEXT(" "), TEXT("\t"), TEXT("\n"), TEXT("\r") };
		// Set up DRM CRM candidates and settle on one.
		TArray<ElectraCDM::IMediaCDM::FCDMCandidate> Candidates;
		ElectraCDM::IMediaCDM::FCDMCandidate cand;
		for(int32 i=0; i<InContentProtections.Num(); ++i)
		{
			cand.SchemeId = InContentProtections[i].Descriptor->GetSchemeIdUri();
			cand.Value = InContentProtections[i].Descriptor->GetValue();
			cand.CommonScheme = InContentProtections[i].CommonScheme;
			cand.AdditionalElements = InContentProtections[i].Descriptor->GetCustomElementAndAttributeJSON();
			InContentProtections[i].DefaultKID.ParseIntoArray(cand.DefaultKIDs, Delimiters, UE_ARRAY_COUNT(Delimiters), true);
			Candidates.Emplace(MoveTemp(cand));
		}
		ElectraCDM::ECDMError Result = PlayerSessionServices->GetDRMManager()->CreateDRMClient(DrmClient, Candidates);
		if (Result == ElectraCDM::ECDMError::Success && DrmClient.IsValid())
		{
			DrmClient->RegisterEventListener(PlayerSessionServices->GetDRMManager());
			DrmClient->PrepareLicenses();
			return true;
		}
		else
		{
			PostError(PlayerSessionServices, FString::Printf(TEXT("Failed to create DRM client with error %d"), (int32)Result), ERRCODE_DASH_DRM_ERROR);
			return false;
		}
	}
	return true;
}


TSharedPtrTS<ITimelineMediaAsset> FDASHPlayPeriod::GetMediaAsset() const
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = GetCurrentManifest();
	TSharedPtrTS<FManifestDASHInternal::FPeriod> Period = Manifest.IsValid() ? Manifest->GetPeriodByUniqueID(PeriodID) : nullptr;
	if (Period.IsValid())
	{
		// Returning the asset typically means the caller wants to access the adaptation sets and representations.
		// Prepare them if necessary (if they already are this method returns immediately).
		Manifest->PreparePeriodAdaptationSets(Period, false);
		return Period;
	}
	return nullptr;
}

void FDASHPlayPeriod::SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex)
{
	// The ABR must not try to switch adaptation sets at the moment. As such the adaptation set passed in must be one of the already active ones.
	if (AdaptationSetID == ActiveVideoAdaptationSetID)
	{
		ActiveVideoRepresentationID = RepresentationID;
		ActiveVideoQualityIndex.Index = QualityIndex;
		ActiveVideoQualityIndex.MaxIndex = MaxQualityIndex;
	}
	else if (AdaptationSetID == ActiveAudioAdaptationSetID)
	{
		ActiveAudioRepresentationID = RepresentationID;
		ActiveAudioQualityIndex.Index = QualityIndex;
		ActiveAudioQualityIndex.MaxIndex = MaxQualityIndex;
	}
	else if (AdaptationSetID == ActiveSubtitleAdaptationSetID)
	{
		ActiveSubtitleRepresentationID = RepresentationID;
		ActiveSubtitleQualityIndex.Index = QualityIndex;
		ActiveSubtitleQualityIndex.MaxIndex = MaxQualityIndex;
	}
	else
	{
		LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("ABR tried to activate a stream from an inactive AdaptationSet!")));
	}
}

void FDASHPlayPeriod::TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload)
{
	HandleRepresentationInitSegmentLoading(InitSegmentsToPreload);
}


void FDASHPlayPeriod::SetupCommonSegmentRequestInfos(TSharedPtrTS<FStreamSegmentRequestCommon>& InOutSegmentRequest)
{
	FManifestDASHInternal::FRepresentation* Repr = static_cast<FManifestDASHInternal::FRepresentation*>(InOutSegmentRequest->Representation.Get());

	// Source buffer info
	InOutSegmentRequest->SourceBufferInfo[StreamTypeToArrayIndex(InOutSegmentRequest->StreamType)] =
			InOutSegmentRequest->StreamType == EStreamType::Video ? SourceBufferInfoVideo :
			InOutSegmentRequest->StreamType == EStreamType::Audio ? SourceBufferInfoAudio :
			InOutSegmentRequest->StreamType == EStreamType::Subtitle ? SourceBufferInfoSubtitles : nullptr;

	// Encryption stuff
	InOutSegmentRequest->DrmMedia.DrmClient = DrmClient;
	InOutSegmentRequest->DrmMedia.DrmMimeType = Repr->GetCodecInformation().GetMimeTypeWithCodecAndFeatures();

	// Protocol and container
	InOutSegmentRequest->StreamingProtocol = FStreamSegmentRequestCommon::EStreamingProtocol::DASH;
	InOutSegmentRequest->ExpectedContainerFormat = InOutSegmentRequest->Segment.ContainerType == FSegmentInformationCommon::EContainerType::ISO14496_12 ? FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12 : FStreamSegmentRequestCommon::EContainerFormat::Matroska_WebM;
}


IManifest::FResult FDASHPlayPeriod::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, IManifest::ESearchType SearchType)
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = GetCurrentManifest();
	if (!Manifest.IsValid())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotLoaded).SetErrorDetail(FErrorDetail().SetMessage("The manifest to locate the period start segment in has disappeared"));
	}
	TSharedPtrTS<FManifestDASHInternal::FPeriod> Period = Manifest->GetPeriodByUniqueID(PeriodID);
	if (!Period.IsValid())
	{
		// If the period has suddenly disappeared there must have been an MPD update that removed it. This is extremely rare but possible.
		return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("Period to locate start segment in has disappeared"));
	}
	Manifest->PreparePeriodAdaptationSets(Period, false);

	// Frame accurate seek required?
	bool bFrameAccurateSearch = StartPosition.Options.bFrameAccuracy;
	if (bFrameAccurateSearch)
	{
		// Get the segment that starts on or before the search time.
		SearchType = IManifest::ESearchType::Before;
	}
	FTimeValue PlayRangeEnd = StartPosition.Options.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());

	FTimeValue SegmentFetchDelay = Manifest->GetSegmentFetchDelay();
	FTimeValue AST = Manifest->GetAnchorTime();
	FTimeValue StartTime = StartPosition.Time;
	// All time values we communicate to the outside - and therefor get from the outside - are offset by the availabilityStartTime.
	StartTime -= AST;
	PlayRangeEnd -= AST;

	// Due to the way we have been searching for the period it is possible for the start time to fall (slightly) outside the actual times.
	if (StartTime < Period->GetStart())
	{
		StartTime = Period->GetStart();
	}
	else if (StartTime >= Period->GetEnd())
	{
		StartTime = Period->GetEnd();
	}
	// We are searching for a time local to the period so we need to subtract the period start time.
	StartTime -= Period->GetStart();
	// The same goes for the playback range end, but this never gets clamped to the period boundaries as it may well be somewhere else.
	PlayRangeEnd -= Period->GetStart();

	bool bUsesAST = Manifest->UsesAST();
	bool bIsStaticType = Manifest->IsStaticType() || Manifest->IsDynamicEpicEvent();

	// Create a segment request to which the individual stream segment requests will add themselves as
	// dependent streams. This is a special case for playback start.
	TSharedPtrTS<FStreamSegmentRequestCommon> StartSegmentRequest = MakeSharedTS<FStreamSegmentRequestCommon>();
	StartSegmentRequest->bIsInitialStartRequest = true;
	StartSegmentRequest->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();

	struct FSelectedStream
	{
		EStreamType	StreamType;
		FString		RepresentationID;
		FString		AdaptationSetID;
		FSelectedQualityIndex QualityIndex;
	};
	TArray<FSelectedStream> ActiveSelection;
	if (!ActiveVideoRepresentationID.IsEmpty())
	{
		ActiveSelection.Emplace(FSelectedStream({EStreamType::Video, ActiveVideoRepresentationID, ActiveVideoAdaptationSetID, ActiveVideoQualityIndex}));
	}
	if (!ActiveAudioRepresentationID.IsEmpty())
	{
		ActiveSelection.Emplace(FSelectedStream({EStreamType::Audio, ActiveAudioRepresentationID, ActiveAudioAdaptationSetID, ActiveAudioQualityIndex}));
	}
	if (!ActiveSubtitleRepresentationID.IsEmpty())
	{
		ActiveSelection.Emplace(FSelectedStream({EStreamType::Subtitle, ActiveSubtitleRepresentationID, ActiveSubtitleAdaptationSetID, ActiveSubtitleQualityIndex}));
	}

	bool bDidAdjustStartTime = false;
	bool bTryAgainLater = false;
	bool bAnyStreamAtEOS = false;
	bool bAllStreamsAtEOS = true;
	for(int32 i=0; i<ActiveSelection.Num(); ++i)
	{
		if (ActiveSelection[i].AdaptationSetID.Len() && ActiveSelection[i].RepresentationID.Len())
		{
			TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> Adapt = Period->GetAdaptationSetByUniqueID(ActiveSelection[i].AdaptationSetID);
			TSharedPtrTS<FManifestDASHInternal::FRepresentation> Repr = Adapt.IsValid()? Adapt->GetRepresentationByUniqueID(ActiveSelection[i].RepresentationID) : nullptr;
			if (!Repr.IsValid())
			{
				// If the AdaptationSet or the Representation has suddenly disappeared there must have been an MPD update that removed it, which is illegal because the
				// Period itself is still there (checked for above).
				return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("Period no longer has the selected AdaptationSet or Representation."));
			}

			FSegmentInformationCommon SegmentInfo;
			FManifestDASHInternal::FSegmentSearchOption SearchOpt;
			TArray<TWeakPtrTS<FMPDLoadRequestDASH>> RemoteElementLoadRequests;
			SearchOpt.PeriodLocalTime = StartTime;
			SearchOpt.PeriodDuration = Period->GetDuration();
			if (!SearchOpt.PeriodDuration.IsValid() || SearchOpt.PeriodDuration.IsPositiveInfinity())
			{
				SearchOpt.PeriodDuration = Manifest->GetLastPeriodEndTime(false) - AST - Period->GetStart();
			}
			SearchOpt.PeriodPresentationEnd = PlayRangeEnd;
			SearchOpt.bHasFollowingPeriod = Period->GetHasFollowingPeriod();
			SearchOpt.SearchType = SearchType;
			SearchOpt.bFrameAccurateSearch = bFrameAccurateSearch;
			SearchOpt.QualityIndex = ActiveSelection[i].QualityIndex.Index;
			SearchOpt.MaxQualityIndex = ActiveSelection[i].QualityIndex.MaxIndex;
			SearchOpt.StreamType = ActiveSelection[i].StreamType;
			FManifestDASHInternal::FRepresentation::ESearchResult SearchResult = Repr->FindSegment(PlayerSessionServices, SegmentInfo, RemoteElementLoadRequests, SearchOpt);
			if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::NeedElement)
			{
				TSharedPtrTS<IPlaylistReader> ManifestReader = PlayerSessionServices->GetManifestReader();
				if (!ManifestReader.IsValid())
				{
					return IManifest::FResult(IManifest::FResult::EType::NotLoaded).SetErrorDetail(FErrorDetail().SetMessage("Entity loader disappeared"));
				}
				IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(ManifestReader.Get());
				Reader->AddElementLoadRequests(RemoteElementLoadRequests);
				bTryAgainLater = true;
			}
			else if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS)
			{
				TSharedPtrTS<FStreamSegmentRequestCommon> SegmentRequest = MakeSharedTS<FStreamSegmentRequestCommon>();
				SegmentRequest->StreamType = ActiveSelection[i].StreamType;
				SegmentRequest->CodecInfo[StreamTypeToArrayIndex(SegmentRequest->StreamType)] = Repr->GetCodecInformation();
				SegmentRequest->Representation = Repr;
				SegmentRequest->AdaptationSet = Adapt;
				SegmentRequest->Period = Period;
				SegmentRequest->PeriodStart = Period->GetStart();
				SegmentRequest->AST = AST;
				if (bUsesAST)
				{
					SegmentRequest->ASAST = SegmentInfo.CalculateASAST(AST, Period->GetStart(), bIsStaticType) + SegmentFetchDelay;
					SegmentRequest->SAET = SegmentInfo.CalculateSAET(AST, Period->GetStart(), Manifest->GetAvailabilityEndTime(), Manifest->GetTimeshiftBufferDepth(), bIsStaticType);
				}
				SegmentRequest->Segment = MoveTemp(SegmentInfo);
				SegmentRequest->bIsEOSSegment = true;
				SegmentRequest->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
				SetupCommonSegmentRequestInfos(SegmentRequest);
				StartSegmentRequest->DependentStreams.Emplace(MoveTemp(SegmentRequest));
				bAnyStreamAtEOS = true;
			}
			else if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::Gone)
			{
				// This should only be intermittent during a playlist refresh. Try again shortly.
				bTryAgainLater = true;
			}
			else if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::BadType)
			{
				// This representation has now been disabled. Try again as soon as possible, which should pick a different representation then
				// unless the problem was that fatal that an error has been posted.
				return IManifest::FResult().RetryAfterMilliseconds(0);
			}
			else if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::Found)
			{
				if (!bFrameAccurateSearch)
				{
					// The search result will have returned a media local time of the segment to start with.
					// In order to find the best matching audio and subtitle (or other) segments we adjust
					// the search time for these now.
					// The reasoning being that these types of streams should have only SAP types 1 and
					// can begin decoding on any segment and access unit.
					if (ActiveSelection[i].StreamType == EStreamType::Video && !bDidAdjustStartTime)
					{
						bDidAdjustStartTime = true;

						// At the moment we need to start at the beginning of the segment where the IDR frame is located.
						// Frame accuracy is a problem because we need to start decoding all the frames from the start of the segment
						// anyway - and then discard them - in order to get to the frame of interest.
						// This is wasteful and prevents fast startup, so we set the start time to the beginning of the segment.
						SegmentInfo.MediaLocalFirstAUTime = SegmentInfo.Time;

						StartTime = FTimeValue().SetFromND(SegmentInfo.Time - SegmentInfo.PTO, SegmentInfo.Timescale);
						SearchType = IManifest::ESearchType::Before;
					}
				}

				TSharedPtrTS<FStreamSegmentRequestCommon> SegmentRequest = MakeSharedTS<FStreamSegmentRequestCommon>();
				SegmentRequest->StreamType = ActiveSelection[i].StreamType;
				SegmentRequest->QualityIndex = ActiveSelection[i].QualityIndex.Index;
				SegmentRequest->MaxQualityIndex = ActiveSelection[i].QualityIndex.MaxIndex;
				SegmentRequest->CodecInfo[StreamTypeToArrayIndex(SegmentRequest->StreamType)] = Repr->GetCodecInformation();
				SegmentRequest->Representation = Repr;
				SegmentRequest->AdaptationSet = Adapt;
				SegmentRequest->Period = Period;
				SegmentRequest->PeriodStart = Period->GetStart();
				SegmentRequest->AST = AST;
				if (bUsesAST)
				{
					SegmentRequest->ASAST = SegmentInfo.CalculateASAST(AST, Period->GetStart(), bIsStaticType) + SegmentFetchDelay;
					SegmentRequest->SAET = SegmentInfo.CalculateSAET(AST, Period->GetStart(), Manifest->GetAvailabilityEndTime(), Manifest->GetTimeshiftBufferDepth(), bIsStaticType);
				}
				// If the segment is known to be missing we need to instead insert filler data.
				if (SegmentInfo.bIsMissing)
				{
					SegmentRequest->bInsertFillerData = true;
				}
				SegmentRequest->Segment = MoveTemp(SegmentInfo);
				SegmentRequest->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
				if (bFrameAccurateSearch)
				{
					SegmentRequest->FrameAccurateStartTime = StartPosition.Time;
				}

				// The start segment request needs to be able to return a valid first PTS which is what the player sets
				// the playback position to. If not valid yet update it with the current stream values.
				if (!StartSegmentRequest->GetFirstPTS().IsValid())
				{
					StartSegmentRequest->AST = SegmentRequest->AST;
					StartSegmentRequest->AdditionalAdjustmentTime = SegmentRequest->AdditionalAdjustmentTime;
					StartSegmentRequest->PeriodStart = SegmentRequest->PeriodStart;
					StartSegmentRequest->Segment = SegmentRequest->Segment;
				}

				// Similarly the start segment request might need to look at a segment availability window.
				if (bUsesAST && !StartSegmentRequest->ASAST.IsValid())
				{
					StartSegmentRequest->ASAST = SegmentRequest->ASAST;
					StartSegmentRequest->SAET = SegmentRequest->SAET;
				}

				SetupCommonSegmentRequestInfos(SegmentRequest);
				StartSegmentRequest->DependentStreams.Emplace(MoveTemp(SegmentRequest));
				bAllStreamsAtEOS = false;
			}
			else
			{
				return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("Unhandled search result!"));
			}
		}
	}

	// Any waiters?
	if (bTryAgainLater)
	{
		return IManifest::FResult().RetryAfterMilliseconds(100);
	}

	// All streams already at EOS?
	if (bAnyStreamAtEOS && bAllStreamsAtEOS)
	{
		return IManifest::FResult(IManifest::FResult::EType::PastEOS);
	}

	// Done.
	OutSegment = MoveTemp(StartSegmentRequest);
	return IManifest::FResult(IManifest::FResult::EType::Found);
}


//-----------------------------------------------------------------------------
/**
 * Same as GetStartingSegment() except this is for a specific stream (video, audio, ...) only.
 * To be used when a track (language) change is made and a new segment is needed at the current playback position.
 */
IManifest::FResult FDASHPlayPeriod::GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType StreamType, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, IManifest::ESearchType SearchType)
{
	// Create a dummy request we can use to pass into GetNextOrRetrySegment().
	// Only set the values that that method requires.
	bool bNeedRemoteElement = false;
	auto DummyReq = MakeSharedTS<FStreamSegmentRequestCommon>();
	DummyReq->StreamType = StreamType;
	DummyReq->PeriodStart = StartPosition.Time;
	DummyReq->TimestampSequenceIndex = SequenceState.GetSequenceIndex();
	return GetNextOrRetrySegment(OutSegment, bNeedRemoteElement, DummyReq, ENextSegType::SamePeriodStartOver, StartPosition.Options);
}



IManifest::FResult FDASHPlayPeriod::GetNextOrRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, bool& OutWaitForRemoteElement, TSharedPtrTS<const IStreamSegment> InCurrentSegment, ENextSegType InNextType, const FPlayStartOptions& Options)
{
	OutWaitForRemoteElement = false;
	TSharedPtrTS<const FStreamSegmentRequestCommon> Current = StaticCastSharedPtr<const FStreamSegmentRequestCommon>(InCurrentSegment);
	if (Current->bIsInitialStartRequest)
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("The next segment cannot be located for the initial start request, only for an actual media request!"));
	}

	TSharedPtrTS<FManifestDASHInternal> Manifest = GetCurrentManifest();
	TSharedPtrTS<FManifestDASHInternal::FPeriod> Period = Manifest->GetPeriodByUniqueID(PeriodID);
	if (!Period.IsValid())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotLoaded).SetErrorDetail(FErrorDetail().SetMessage("Period to locate start segment in has disappeared"));
	}
	Manifest->PreparePeriodAdaptationSets(Period, false);

	FString ActiveRepresentationIDByType;
	FString ActiveAdaptationSetIDByType;
	FSelectedQualityIndex ActiveQualityIndex;
	switch(Current->GetType())
	{
		case EStreamType::Video:
		{
			ActiveAdaptationSetIDByType = ActiveVideoAdaptationSetID;
			ActiveRepresentationIDByType = ActiveVideoRepresentationID;
			ActiveQualityIndex = ActiveVideoQualityIndex;
			break;
		}
		case EStreamType::Audio:
		{
			ActiveAdaptationSetIDByType = ActiveAudioAdaptationSetID;
			ActiveRepresentationIDByType = ActiveAudioRepresentationID;
			ActiveQualityIndex = ActiveAudioQualityIndex;
			break;
		}
		case EStreamType::Subtitle:
		{
			ActiveAdaptationSetIDByType = ActiveSubtitleAdaptationSetID;
			ActiveRepresentationIDByType = ActiveSubtitleRepresentationID;
			ActiveQualityIndex = ActiveSubtitleQualityIndex;
			break;
		}
		default:
		{
			break;
		}
	}
	TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> Adapt = Period->GetAdaptationSetByUniqueID(ActiveAdaptationSetIDByType);
	TSharedPtrTS<FManifestDASHInternal::FRepresentation> Repr = Adapt.IsValid()? Adapt->GetRepresentationByUniqueID(ActiveRepresentationIDByType) : nullptr;
	if (!Repr.IsValid())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("No active stream found to get next segment for"));
	}

	FTimeValue SegmentFetchDelay = Manifest->GetSegmentFetchDelay();
	FTimeValue AST = Manifest->GetAnchorTime();
	bool bUsesAST = Manifest->UsesAST();
	bool bIsStaticType = Manifest->IsStaticType() || Manifest->IsDynamicEpicEvent();

	// Frame accurate seek required?
	bool bFrameAccurateSearch = Options.bFrameAccuracy;
	FTimeValue PlayRangeEnd = Options.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());
	PlayRangeEnd -= AST;
	PlayRangeEnd -= Period->GetStart();

	FSegmentInformationCommon SegmentInfo;
	FManifestDASHInternal::FSegmentSearchOption SearchOpt;
	TArray<TWeakPtrTS<FMPDLoadRequestDASH>> RemoteElementLoadRequests;
	if (InNextType == ENextSegType::SamePeriodNext)
	{
		// Set up the search time as the time three quarters into the current segment.
		// This is to make sure the time is sufficiently large that it won't be affected by rounding errors in timescale conversions.
		SearchOpt.PeriodLocalTime.SetFromND(Current->Segment.Time - Current->Segment.PTO + Current->Segment.Duration*3/4, Current->Segment.Timescale);
		SearchOpt.SearchType = IManifest::ESearchType::After;
		bFrameAccurateSearch = false;
		// If this representation is side loaded there is nothing to fetch next.
		// Set the period search time such that the search will have to return EOS.
		if (Repr.IsValid() && Repr->IsSideloadedSubtitle())
		{
			SearchOpt.PeriodLocalTime.SetToPositiveInfinity();
		}
	}
	else if (InNextType == ENextSegType::SamePeriodRetry)
	{
		// Set up the search time as was used to locate the current segment with a tenth of the duration into it.
		// This is to make sure the time is sufficiently large that it won't be affected by rounding errors in timescale conversions.
		SearchOpt.PeriodLocalTime.SetFromND(Current->Segment.Time - Current->Segment.PTO + Current->Segment.Duration*1/10, Current->Segment.Timescale);
		SearchOpt.SearchType = bFrameAccurateSearch ? IManifest::ESearchType::Before : IManifest::ESearchType::Closest;
	}
	else if (InNextType == ENextSegType::SamePeriodStartOver)
	{
		FTimeValue StartTime = Current->PeriodStart - AST;
		if (StartTime < Period->GetStart())
		{
			StartTime = Period->GetStart();
		}
		else if (StartTime >= Period->GetEnd())
		{
			StartTime = Period->GetEnd();
		}
		StartTime -= Period->GetStart();

		SearchOpt.PeriodLocalTime = StartTime;
		SearchOpt.SearchType = IManifest::ESearchType::Before;
	}
	else //if (InNextType == ENextSegType::NextPeriod)
	{
		SearchOpt.PeriodLocalTime.SetToZero();
		SearchOpt.SearchType = bFrameAccurateSearch ? IManifest::ESearchType::Before : IManifest::ESearchType::Closest;
	}
	SearchOpt.bHasFollowingPeriod = Period->GetHasFollowingPeriod();
	SearchOpt.bFrameAccurateSearch = bFrameAccurateSearch;
	SearchOpt.PeriodPresentationEnd = PlayRangeEnd;
	SearchOpt.PeriodDuration = Period->GetDuration();
	if (!SearchOpt.PeriodDuration.IsValid() || SearchOpt.PeriodDuration.IsPositiveInfinity())
	{
		SearchOpt.PeriodDuration = Manifest->GetLastPeriodEndTime(false) - AST;
	}
	SearchOpt.QualityIndex = ActiveQualityIndex.Index;
	SearchOpt.MaxQualityIndex = ActiveQualityIndex.MaxIndex;
	SearchOpt.StreamType = Current->GetType();

	FManifestDASHInternal::FRepresentation::ESearchResult SearchResult;
	if (Repr.IsValid())
	{
		SearchResult = Repr->FindSegment(PlayerSessionServices, SegmentInfo, RemoteElementLoadRequests, SearchOpt);
	}
	else
	{
		SearchResult = FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
	}
	if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::NeedElement)
	{
		TSharedPtrTS<IPlaylistReader> ManifestReader = PlayerSessionServices->GetManifestReader();
		if (!ManifestReader.IsValid())
		{
			return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("Entity loader disappeared"));
		}
		OutWaitForRemoteElement = true;
		IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(ManifestReader.Get());
		Reader->AddElementLoadRequests(RemoteElementLoadRequests);
		return IManifest::FResult().RetryAfterMilliseconds(100);
	}
	else if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS)
	{
		// We may have reached the end of a period or the last segment of an ongoing Live presentation.
		// Need to figure out which it is.
		if (!Manifest->AreUpdatesExpected())
		{
			// No updates of the manifest means this period is over. Try moving onto the next if there is one.
			return IManifest::FResult(IManifest::FResult::EType::PastEOS);
		}
		// Could be the end or a period. Is there a regular period following?
		if (Manifest->HasFollowingRegularPeriod(Period))
		{
			// Yes, so we can move onto the next period.
			return IManifest::FResult(IManifest::FResult::EType::PastEOS);
		}

		IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(PlayerSessionServices->GetManifestReader().Get());
		Reader->RequestMPDUpdate(IPlaylistReaderDASH::EMPDRequestType::GetLatestSegment);
		return IManifest::FResult().RetryAfterMilliseconds(250);
	}
	else if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::Gone)
	{
		// This should only be intermittent during a playlist refresh. Try again shortly.
		return IManifest::FResult().RetryAfterMilliseconds(100);
	}
	else if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::BadType)
	{
		// This representation has now been disabled. Try again as soon as possible, which should pick a different representation then
		// unless the problem was that fatal that an error has been posted.
		return IManifest::FResult().RetryAfterMilliseconds(0);
	}
	else if (SearchResult == FManifestDASHInternal::FRepresentation::ESearchResult::Found)
	{
		TSharedPtrTS<FStreamSegmentRequestCommon> SegmentRequest = MakeSharedTS<FStreamSegmentRequestCommon>();
		SegmentRequest->TimestampSequenceIndex = Current->TimestampSequenceIndex;
		SegmentRequest->StreamType = Current->GetType();
		SegmentRequest->QualityIndex = ActiveQualityIndex.Index;
		SegmentRequest->MaxQualityIndex = ActiveQualityIndex.MaxIndex;
		SegmentRequest->CodecInfo[StreamTypeToArrayIndex(SegmentRequest->StreamType)] = Repr->GetCodecInformation();
		SegmentRequest->Representation = Repr;
		SegmentRequest->AdaptationSet = Adapt;
		SegmentRequest->Period = Period;
		SegmentRequest->PeriodStart = Period->GetStart();
		SegmentRequest->AST = Manifest->GetAnchorTime();
		if (bUsesAST)
		{
			SegmentRequest->ASAST = SegmentInfo.CalculateASAST(AST, Period->GetStart(), bIsStaticType) + SegmentFetchDelay;
			SegmentRequest->SAET = SegmentInfo.CalculateSAET(AST, Period->GetStart(), Manifest->GetAvailabilityEndTime(), Manifest->GetTimeshiftBufferDepth(), bIsStaticType);
		}
		// If the segment is known to be missing we need to instead insert filler data.
		if (SegmentInfo.bIsMissing)
		{
			SegmentRequest->bInsertFillerData = true;
		}

		if (InNextType == ENextSegType::SamePeriodNext || InNextType == ENextSegType::SamePeriodRetry)
		{
			// Because we are searching for the next segment we do not want any first access units to be truncated.
			// We keep the current media local AU time for the case where with <SegmentTemplate> addressing we get greatly varying
			// segment durations from the fixed value (up to +/- 50% variation are allowed!) and the current segment did not actually
			// have any access units we wanted to have! In that case it is possible that this new segment would also have some initial
			// access units outside the time we want. By retaining the initial value this is addressed.
			// We do need to translate the value between potentially different timescales and potentially different local media times.
			SegmentInfo.MediaLocalFirstAUTime = FTimeFraction(Current->Segment.MediaLocalFirstAUTime - Current->Segment.PTO, Current->Segment.Timescale).GetAsTimebase(SegmentInfo.Timescale) + SegmentInfo.PTO;
		}
		// For a retry request we have to increate the retry count to give up after n failed attempts.
		if (InNextType == ENextSegType::SamePeriodRetry)
		{
			SegmentRequest->NumOverallRetries = Current->NumOverallRetries + 1;
		}

		// If we stayed on the same representation and the stream reader has already warned about a timescale
		// mismatch then we take on the warning flag to reduce console spam.
		if (SegmentRequest->Representation == Current->Representation)
		{
			SegmentRequest->bWarnedAboutTimescale = Current->bWarnedAboutTimescale;
		}

		SegmentRequest->Segment = MoveTemp(SegmentInfo);
		SetupCommonSegmentRequestInfos(SegmentRequest);
		OutSegment = MoveTemp(SegmentRequest);
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}
	else
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("Unhandled search result!"));
	}
}


IManifest::FResult FDASHPlayPeriod::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> InCurrentSegment, const FPlayStartOptions& Options)
{
	if (!InCurrentSegment.IsValid())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("There is no current segment to locate the next one for!"));
	}
	// Did the stream reader see a 'lmsg' brand on this segment?
	// If so then this stream has ended and there will not be a next segment.
	const FStreamSegmentRequestCommon* CurrentRequest = static_cast<const FStreamSegmentRequestCommon*>(InCurrentSegment.Get());

	// Check if we moved across a period.
	bool bNeedRemoteElement = false;
	if (CurrentRequest->Period->GetUniqueIdentifier().Equals(PeriodID))
	{
		if (CurrentRequest->Segment.bSawLMSG)
		{
			return IManifest::FResult(IManifest::FResult::EType::PastEOS);
		}
		return GetNextOrRetrySegment(OutSegment, bNeedRemoteElement, InCurrentSegment, ENextSegType::SamePeriodNext, Options);
	}
	else
	{
		// Moved into a new period. This here is the new period.
		return GetNextOrRetrySegment(OutSegment, bNeedRemoteElement, InCurrentSegment, ENextSegType::NextPeriod, Options);
	}
}

IManifest::FResult FDASHPlayPeriod::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> InCurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData)
{
	if (!InCurrentSegment.IsValid())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetMessage("There is no current segment to locate a retry segment for!"));
	}
	// To insert filler data we can use the current request over again.
	if (bReplaceWithFillerData)
	{
		const FStreamSegmentRequestCommon* CurrentRequest = static_cast<const FStreamSegmentRequestCommon*>(InCurrentSegment.Get());
		TSharedPtrTS<FStreamSegmentRequestCommon> NewRequest = MakeSharedTS<FStreamSegmentRequestCommon>();
		*NewRequest = *CurrentRequest;
		NewRequest->bInsertFillerData = true;
		// We treat replacing the segment with filler data as a retry.
		++NewRequest->NumOverallRetries;
		OutSegment = NewRequest;
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}

	// Pass the download stats bWaitingForRemoteRetryElement to convey if the retry segment needs to wait for a remote element,
	// which is either some xlink or an index segment.
	FStreamSegmentRequestCommon* CurrentRequest = const_cast<FStreamSegmentRequestCommon*>(static_cast<const FStreamSegmentRequestCommon*>(InCurrentSegment.Get()));
	IManifest::FResult Result = GetNextOrRetrySegment(OutSegment, CurrentRequest->DownloadStats.bWaitingForRemoteRetryElement, InCurrentSegment, ENextSegType::SamePeriodRetry, Options);
	return Result;
}


IManifest::FResult FDASHPlayPeriod::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, IManifest::ESearchType SearchType)
{
	return GetStartingSegment(OutSegment, InSequenceState, StartPosition, SearchType);
}


void FDASHPlayPeriod::IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount)
{
	TSharedPtrTS<FManifestDASHInternal> Manifest = GetCurrentManifest();
	if (Manifest.IsValid())
	{
		FTimeValue NewFetchDelay = Manifest->GetSegmentFetchDelay() + IncreaseAmount;
		Manifest->SetSegmentFetchDelay(NewFetchDelay);
		// If the fetch delay becomes too large then there is possibly a clock drift.
		// Trigger a resynchronization which will reset the delay when complete.
		if (NewFetchDelay.GetAsSeconds() > 0.5)
		{
			IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(PlayerSessionServices->GetManifestReader().Get());
			Reader->RequestClockResync();
		}
	}
}


void FDASHPlayPeriod::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	OutAverageSegmentDuration.SetToInvalid();

	TSharedPtrTS<FManifestDASHInternal> Manifest = GetCurrentManifest();
	if (Manifest.IsValid())
	{
		TSharedPtrTS<FManifestDASHInternal::FPeriod> Period = Manifest->GetPeriodByUniqueID(PeriodID);
		if (Period.IsValid())
		{
			Manifest->PreparePeriodAdaptationSets(Period, false);
			TSharedPtrTS<FManifestDASHInternal::FAdaptationSet> AdaptationSet = Period->GetAdaptationSetByUniqueID(AdaptationSetID);
			if (AdaptationSet.IsValid())
			{
				TSharedPtrTS<FManifestDASHInternal::FRepresentation> Repr = AdaptationSet->GetRepresentationByUniqueID(RepresentationID);
				if (Repr.IsValid())
				{
					Repr->GetAverageSegmentDuration(OutAverageSegmentDuration, AdaptationSet);
				}
			}
		}
	}
}

TSharedPtrTS<IPlaybackAssetRepresentation> FDASHPlayPeriod::GetRepresentationFromAdaptationByMaxBandwidth(TWeakPtrTS<IPlaybackAssetAdaptationSet> InAdaptationSet, int32 NotExceedingBandwidth)
{
	TSharedPtrTS<IPlaybackAssetAdaptationSet> AS = InAdaptationSet.Pin();
	TSharedPtrTS<IPlaybackAssetRepresentation> BestRepr;
	TSharedPtrTS<IPlaybackAssetRepresentation> WorstRepr;
	if (AS.IsValid())
	{
		int32 BestBW = 0;
		int32 LowestBW = TNumericLimits<int32>::Max();
		int32 NumRepr = AS->GetNumberOfRepresentations();
		for(int32 i=0; i<NumRepr; ++i)
		{
			TSharedPtrTS<IPlaybackAssetRepresentation> Repr = AS->GetRepresentationByIndex(i);
			// Is the representation enabled and usable?
			if (Repr->CanBePlayed())
			{
				if (Repr->GetBitrate() < LowestBW)
				{
					LowestBW = Repr->GetBitrate();
					WorstRepr = Repr;
				}
				if (Repr->GetBitrate() <= NotExceedingBandwidth && Repr->GetBitrate() > BestBW)
				{
					BestBW = Repr->GetBitrate();
					BestRepr= Repr;
				}
			}
		}
		if (!BestRepr.IsValid())
		{
			BestRepr = WorstRepr;
		}
	}
	return BestRepr;
}

TSharedPtrTS<IPlaybackAssetRepresentation> FDASHPlayPeriod::GetRepresentationFromAdaptationByPriorityAndMaxBandwidth(TWeakPtrTS<IPlaybackAssetAdaptationSet> InAdaptationSet, int32 NotExceedingBandwidth, EStreamType StreamType)
{
	TSharedPtrTS<IPlaybackAssetAdaptationSet> AS = InAdaptationSet.Pin();
	TSharedPtrTS<IPlaybackAssetRepresentation> BestRepr;
	if (AS.IsValid())
	{
		auto AddRepresentationToSelection = [](TArray<FPrioritizedSelection>& InOutSelection, const TSharedPtrTS<FManifestDASHInternal::FRepresentation>& R, int32 Index) -> void
		{
			if (R->CanBePlayed())
			{
				FPrioritizedSelection Candidate;
				Candidate.CodecInfo = R->GetCodecInformation();
				Candidate.Index = Index;
				Candidate.Priority = R->GetSelectionPriority();
				Candidate.Bitrate = R->GetBitrate();
				InOutSelection.Emplace(MoveTemp(Candidate));
			}
		};
		TArray<FPrioritizedSelection> Selection;

		for(int32 i=0; i<AS->GetNumberOfRepresentations(); ++i)
		{
			TSharedPtrTS<FManifestDASHInternal::FRepresentation> Repr = StaticCastSharedPtr<FManifestDASHInternal::FRepresentation>(AS->GetRepresentationByIndex(i));
			AddRepresentationToSelection(Selection, Repr, i);
		}
		PrioritizeSelection(Selection, StreamType, false, true);
		if (Selection.Num())
		{
			for(int32 i=0; i<Selection.Num(); ++i)
			{
				TSharedPtrTS<FManifestDASHInternal::FRepresentation> Repr = StaticCastSharedPtr<FManifestDASHInternal::FRepresentation>(AS->GetRepresentationByIndex(Selection[i].Index));
				if (Repr->GetBitrate() <= NotExceedingBandwidth)
				{
					BestRepr = Repr;
					break;
				}
			}
			if (!BestRepr.IsValid())
			{
				TSharedPtrTS<FManifestDASHInternal::FRepresentation> Repr = StaticCastSharedPtr<FManifestDASHInternal::FRepresentation>(AS->GetRepresentationByIndex(Selection.Last().Index));
				BestRepr = Repr;
			}
		}
	}
	return BestRepr;
}

void FDASHPlayPeriod::GetRepresentationInitSegmentsFromAdaptation(TArray<FInitSegmentInfo>& OutInitSegInfos, TWeakPtrTS<IPlaybackAssetAdaptationSet> InAdaptationSet)
{
	TSharedPtrTS<IPlaybackAssetAdaptationSet> AS = InAdaptationSet.Pin();
	if (AS.IsValid())
	{
		TArray<TWeakPtrTS<FMPDLoadRequestDASH>> DummyRequests;
		FManifestDASHInternal::FSegmentSearchOption SearchOpt;
		SearchOpt.bInitSegmentSetupOnly = true;
		int32 NumRepr = AS->GetNumberOfRepresentations();
		for(int32 i=0; i<NumRepr; ++i)
		{
			TSharedPtrTS<FManifestDASHInternal::FRepresentation> Repr = StaticCastSharedPtr<FManifestDASHInternal::FRepresentation>(AS->GetRepresentationByIndex(i));
			if (Repr->CanBePlayed())
			{
				FInitSegmentInfo SegInfo;
				if (Repr->FindSegment(PlayerSessionServices, SegInfo.InitSegmentInfo, DummyRequests, SearchOpt) == FManifestDASHInternal::FRepresentation::ESearchResult::Found)
				{
					SegInfo.AdaptationSetID = AS->GetUniqueIdentifier();
					SegInfo.RepresentationSetID = Repr->GetUniqueIdentifier();
					OutInitSegInfos.Emplace(MoveTemp(SegInfo));
				}
			}
		}
	}
}

void FDASHPlayPeriod::MergeRepresentationInitSegments(TArray<FInitSegmentInfo>& InOutInitSegInfos, const TArray<FInitSegmentInfo>& NewInitSegInfos)
{
	for(auto &ni : NewInitSegInfos)
	{
		if (!InOutInitSegInfos.ContainsByPredicate([ni](const FInitSegmentInfo& Other) { return ni.AdaptationSetID.Equals(Other.AdaptationSetID) && ni.RepresentationSetID.Equals(Other.RepresentationSetID); }))
		{
			InOutInitSegInfos.Add(ni);
		}
	}
}

void FDASHPlayPeriod::HandleRepresentationInitSegmentLoading(const TArray<FInitSegmentPreload>& InitSegmentsToPreload)
{
	TArray<TWeakPtrTS<FMPDLoadRequestDASH>> RemoteElementLoadRequests;

	auto GetInitSegInfo = [](TArray<FInitSegmentInfo>& InitSegList, const FString& InRepID) -> FInitSegmentInfo*
	{
		for(int32 i=0; i<InitSegList.Num(); ++i)
		{
			if (InitSegList[i].RepresentationSetID == InRepID)
			{
				return &InitSegList[i];
			}
		}
		return nullptr;
	};

	for(auto& pl : InitSegmentsToPreload)
	{
		FInitSegmentInfo* InitSeg = nullptr;

		if (pl.AdaptationSetID == ActiveVideoAdaptationSetID)
		{
			InitSeg = GetInitSegInfo(VideoInitSegmentInfos, pl.RepresentationID);
		}
		if (!InitSeg)
		{
			continue;
		}
		FInitSegmentInfo& is = *InitSeg;
		if (!is.bRequested)
		{
			is.bRequested = true;

			IPlayerEntityCache::FCacheItem CachedItem;
			if (!PlayerSessionServices->GetEntityCache()->GetCachedEntity(CachedItem, is.InitSegmentInfo.InitializationURL.Url.URL, is.InitSegmentInfo.InitializationURL.Range))
			{
				is.LoadRequest = MakeSharedTS<FMPDLoadRequestDASH>();
				is.LoadRequest->LoadType = FMPDLoadRequestDASH::ELoadType::InitSegment;
				is.LoadRequest->Url.URL = is.InitSegmentInfo.InitializationURL.Url.URL;
				is.LoadRequest->Range = is.InitSegmentInfo.InitializationURL.Range;
				if (is.InitSegmentInfo.InitializationURL.CustomHeader.Len())
				{
					is.LoadRequest->Headers.Emplace(HTTP::FHTTPHeader({DASH::HTTPHeaderOptionName, is.InitSegmentInfo.InitializationURL.CustomHeader}));
				}
				is.LoadRequest->PlayerSessionServices = PlayerSessionServices;
				if (is.InitSegmentInfo.ContainerType == FSegmentInformationCommon::EContainerType::ISO14496_12)
				{
					is.LoadRequest->CompleteCallback.BindThreadSafeSP(AsShared(), &FDASHPlayPeriod::InitSegmentMP4DownloadComplete);
				}
				else if (is.InitSegmentInfo.ContainerType == FSegmentInformationCommon::EContainerType::Matroska)
				{
					is.LoadRequest->CompleteCallback.BindThreadSafeSP(AsShared(), &FDASHPlayPeriod::InitSegmentMKVDownloadComplete);
				}
				else
				{
					check(!"not implemented:");
				}
				RemoteElementLoadRequests.Emplace(is.LoadRequest);
			}
		}
	}

	TSharedPtrTS<IPlaylistReader> ManifestReader = PlayerSessionServices->GetManifestReader();
	if (ManifestReader.IsValid())
	{
		IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(ManifestReader.Get());
		Reader->AddElementLoadRequests(RemoteElementLoadRequests);
	}
}

void FDASHPlayPeriod::InitSegmentMP4DownloadComplete(TSharedPtrTS<FMPDLoadRequestDASH> LoadRequest, bool bSuccess)
{
	if (bSuccess)
	{
		if (Stream::CVarElectraStreamUseLegacyMP4.GetValueOnAnyThread())
		{
			FInitSegmentInfo::FAcceptBoxes AllBoxes;
			FMP4StaticDataReader StaticDataReader;
			StaticDataReader.SetParseData(LoadRequest->Request->GetResponseBuffer());
			TSharedPtrTS<IParserISO14496_12> Init = IParserISO14496_12::CreateParser();
			UEMediaError parseError = Init->ParseHeader(&StaticDataReader, &AllBoxes, PlayerSessionServices, nullptr);
			if (parseError == UEMEDIA_ERROR_OK || parseError == UEMEDIA_ERROR_END_OF_STREAM)
			{
				// Parse the tracks of the init segment. We do this mainly to get to the CSD we might need should we have to insert filler data later.
				parseError = Init->PrepareTracks(PlayerSessionServices, TSharedPtrTS<const IParserISO14496_12>());
				if (parseError == UEMEDIA_ERROR_OK)
				{
					// Add this to the entity cache in case it needs to be retrieved again.
					IPlayerEntityCache::FCacheItem CacheItem;
					CacheItem.URL = LoadRequest->Url.URL;
					CacheItem.Range = LoadRequest->Range;
					CacheItem.Parsed14496_12Data = Init;
					PlayerSessionServices->GetEntityCache()->CacheEntity(CacheItem);
				}
			}
		}
		else
		{
			TSharedPtrTS<IPlayerEntityCache> EntityCache = PlayerSessionServices ? PlayerSessionServices->GetEntityCache() : nullptr;
			if (EntityCache)
			{
				TSharedPtr<FParsedMOOVCommon, ESPMode::ThreadSafe> ParsedMOOV = MakeShared<FParsedMOOVCommon, ESPMode::ThreadSafe>();
				if (ParsedMOOV->Parse(MakeConstArrayView<uint8>(LoadRequest->Request->GetResponseBuffer()->GetLinearReadData(), LoadRequest->Request->GetResponseBuffer()->GetLinearReadSize())))
				{
					IPlayerEntityCache::FCacheItem CacheItem;
					CacheItem.URL = LoadRequest->Url.URL;
					CacheItem.Range = LoadRequest->Range;
					CacheItem.ParsedMOOV = MoveTemp(ParsedMOOV);
					EntityCache->CacheEntity(CacheItem);
				}
			}
		}
	}
}

void FDASHPlayPeriod::InitSegmentMKVDownloadComplete(TSharedPtrTS<FMPDLoadRequestDASH> LoadRequest, bool bSuccess)
{
	check(!"this should not get called for now");
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

namespace
{

template<typename T, typename Validate, typename GetValue>
T GetAttribute(const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& Arr, GetValue Get, Validate IsValid, T Default)
{
	for(int32 i=0; i<Arr.Num(); ++i)
	{
		T v(Get(Arr[i]));
		if (IsValid(v))
		{
			return v;
		}
	}
	return Default;
}

template<typename T, typename Validate, typename GetValue>
T GetAttribute(const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& Arr, GetValue Get, Validate IsValid, T Default)
{
	for(int32 i=0; i<Arr.Num(); ++i)
	{
		T v(Get(Arr[i]));
		if (IsValid(v))
		{
			return v;
		}
	}
	return Default;
}

#define GET_ATTR(Array, GetVal, IsValid, Default) GetAttribute(Array, [](const auto& e){return e->GetVal;}, [](const auto& v){return v.IsValid;}, Default)


FTimeValue CalculateSegmentAvailabilityTimeOffset(const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& Arr)
{
	FTimeValue Sum(FTimeValue::GetZero());
	for(int32 i=0; i<Arr.Num(); ++i)
	{
		FTimeValue v = Arr[i]->GetAvailabilityTimeOffset();
		if (v.IsValid())
		{
			Sum += v;
		}
	}
	return Sum;
}
FTimeValue CalculateSegmentAvailabilityTimeOffset(const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& Arr)
{
	FTimeValue Sum(FTimeValue::GetZero());
	for(int32 i=0; i<Arr.Num(); ++i)
	{
		FTimeValue v = Arr[i]->GetAvailabilityTimeOffset();
		if (v.IsValid())
		{
			Sum += v;
		}
	}
	return Sum;
}


#if 0
TMediaOptionalValue<bool> GetSegmentAvailabilityTimeComplete(const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& Arr)
{
	TMediaOptionalValue<bool> TimeComplete;
	for(int32 i=0; i<Arr.Num(); ++i)
	{
		if (!TimeComplete.IsSet())
		{
			TimeComplete = Arr[i]->GetAvailabilityTimeComplete();
		}
		else
		{
			if (Arr[i]->GetAvailabilityTimeComplete().IsSet() && Arr[i]->GetAvailabilityTimeComplete().Value() != TimeComplete.Value())
			{
				// Inconsistent @availabilityTimeComplete
			}
		}
	}
	return TimeComplete;
}

TMediaOptionalValue<bool> GetSegmentAvailabilityTimeComplete(const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& Arr)
{
	TMediaOptionalValue<bool> TimeComplete;
	for(int32 i=0; i<Arr.Num(); ++i)
	{
		if (!TimeComplete.IsSet())
		{
			TimeComplete = Arr[i]->GetAvailabilityTimeComplete();
		}
		else
		{
			if (Arr[i]->GetAvailabilityTimeComplete().IsSet() && Arr[i]->GetAvailabilityTimeComplete().Value() != TimeComplete.Value())
			{
				// Inconsistent @availabilityTimeComplete
			}
		}
	}
	return TimeComplete;
}
#endif
}



FManifestDASHInternal::FRepresentation::ESearchResult FManifestDASHInternal::FRepresentation::PrepareSegmentIndex(IPlayerSessionServices* InPlayerSessionServices, const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& SegmentBase, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions)
{
	// If the segment index has been requested and is still pending, return right away.
	if (PendingSegmentIndexLoadRequest.IsValid())
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::NeedElement;
	}

	// Is a segment index (still) needed?
	if (!SIDXBox.IsValid() && bNeedsSegmentIndex)
	{
		// Since this method may only be called with a still valid MPD representation we can pin again and don't need to check if it's still valid.
		TSharedPtrTS<FDashMPD_RepresentationType> MPDRepresentation = Representation.Pin();

		bNeedsSegmentIndex = false;

  		// Get the index, if any.
		TSharedPtrTS<FDashMPD_URLType> RepresentationIndex = GET_ATTR(SegmentBase, GetRepresentationIndex(), IsValid(), TSharedPtrTS<FDashMPD_URLType>());
		FString RepresentationIndexSegmentURL, RepresentationIndexSegmentURLRange;
		if (RepresentationIndex.IsValid())
		{
			RepresentationIndexSegmentURLRange = RepresentationIndex->GetRange();
			RepresentationIndexSegmentURL = RepresentationIndex->GetSourceURL();
			if (RepresentationIndexSegmentURL.Len())
			{
				LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("The <RepresentationIndex> within a <SegmentBase> specifies a URL, which is probably not correct.")));
				/*
					Ignore this URL since it is not clear what that means.
					With <SegmentBase> the segment index is a `sidx` box and with almost 100% certainty the base segment file
					will be a fragmented fmp4 consisting of several `moof`+`mdat` boxes, in which case the base address of the
					moof will be the byte following the `sidx` box, but how would that work if this is represented by a
					_different_ file? At what byte offset is the first `moof` to be found then?

					This is expressed in section 8.3.3 of ISO/IEC 23009-1:2022
				*/
				RepresentationIndexSegmentURL.Empty();
			}
		}
		FString IndexRangeAttribute = GET_ATTR(SegmentBase, GetIndexRange(), Len(), FString());
		if (RepresentationIndexSegmentURLRange.Len() && IndexRangeAttribute.Len() && !RepresentationIndexSegmentURLRange.Equals(IndexRangeAttribute))
		{
			PostError(InPlayerSessionServices, FString::Printf(TEXT("The <RepresentationIndex> within a <SegmentBase> specifies a byte range different from the @indexRange attribute")), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		if (RepresentationIndexSegmentURLRange.IsEmpty())
		{
			RepresentationIndexSegmentURLRange = IndexRangeAttribute;
		}
		if (RepresentationIndexSegmentURLRange.IsEmpty())
		{
			PostError(InPlayerSessionServices, FString::Printf(TEXT("The <SegmentBase> does not specify any byte range for the index")), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}


		// If there is a range for the index, remember it since forms the base address for moof addressing.
		if (RepresentationIndexSegmentURLRange.Len())
		{
			ElectraHTTPStream::FHttpRange r;
			r.Set(RepresentationIndexSegmentURLRange);
			SegmentIndexRangeStart = r.GetStart();
			SegmentIndexRangeSize = r.GetEndIncluding() + 1 - SegmentIndexRangeStart;
		}

		if (RepresentationIndexSegmentURLRange.Len() || RepresentationIndexSegmentURL.Len())
		{
			// The URL query might need to be changed. Look for the UrlQuery properties.
			TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>> UrlQueries;
			DASHUrlHelpers::GetAllHierarchyUrlQueries(UrlQueries, MPDRepresentation, DASHUrlHelpers::EUrlQueryRequestType::Segment, true);

			TArray<DASHUrlHelpers::FDASHMediaURL> CandidateURLs;

			DASHUrlHelpers::FDASHMediaURL RootDocument;
			RootDocument.URL = MPDRepresentation->GetDocumentURL();
			// The MPD has no @serviceLocation or the DVB properties. According to DVB-DASH the @serviceLocation is the URL itself and the attributes default to 1.
			if (InPlayerSessionServices->GetContentSteeringHandler()->IsDVBDash())
			{
				RootDocument.CDN = RootDocument.URL;
			}
			RootDocument.DVBpriority = 1;
			RootDocument.DVBweight = 1;
			DASHUrlHelpers::GetAllCandidateBaseURLs(CandidateURLs, MPDRepresentation, RootDocument, InPlayerSessionServices->GetContentSteeringHandler()->IsDVBDash());
			// Build a list we can pass to the steering handler.
			TArray<FContentSteeringHandler::FCandidateURL> SteeringCandidates;
			FString SteeringMessage;
			PrepareSteeringCandidates(SteeringCandidates, InPlayerSessionServices, RootDocument.URL, CandidateURLs, UrlQueries, RepresentationIndexSegmentURL);
			FContentSteeringHandler::FSelectedCandidateURL ChosenIndex = InPlayerSessionServices->GetContentSteeringHandler()->SelectBestCandidateFrom(SteeringMessage, FContentSteeringHandler::ESelectFor::Segment, SteeringCandidates);
			if (ChosenIndex.MediaURL.URL.IsEmpty())
			{
				PostError(InPlayerSessionServices, SteeringMessage, ERRCODE_DASH_MPD_BAD_REPRESENTATION);
				return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
			}
			ChosenIndex.MediaURL.URL = DASHUrlHelpers::ApplyAnnexEByteRange(ChosenIndex.MediaURL.URL, RepresentationIndexSegmentURLRange, ChosenIndex.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_ByteRange).SafeGetFString());

			// Check with entity cache if the index has been retrieved before.
			IPlayerEntityCache::FCacheItem CachedItem;
			if (InPlayerSessionServices->GetEntityCache()->GetCachedEntity(CachedItem, ChosenIndex.MediaURL.URL, RepresentationIndexSegmentURLRange))
			{
				// Already cached. Use it.
				SIDXBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxSIDX>(CachedItem.ISOBMFFBox);
				SegmentMKV = CachedItem.ParsedMatroskaData;
			}
			else
			{
				FString RequestHeader = ChosenIndex.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_AnnexIRequestHeader).SafeGetFString();
				if (GetStreamContainerType() == EStreamContainerType::ISO14496_12)
				{
					// Create the request.
					TSharedPtrTS<FMPDLoadRequestDASH> LoadReq = MakeSharedTS<FMPDLoadRequestDASH>();
					LoadReq->LoadType = FMPDLoadRequestDASH::ELoadType::IndexSegment;
					LoadReq->Url.URL = ChosenIndex.MediaURL.URL;
					LoadReq->Range = RepresentationIndexSegmentURLRange;
					if (RequestHeader.Len())
					{
						LoadReq->Headers.Emplace(HTTP::FHTTPHeader(DASH::HTTPHeaderOptionName, RequestHeader));
					}
					LoadReq->PlayerSessionServices = InPlayerSessionServices;
					LoadReq->XLinkElement = MPDRepresentation;
					LoadReq->SegmentStreamType = InSearchOptions.StreamType;
					LoadReq->SegmentQualityIndex = InSearchOptions.QualityIndex;
					LoadReq->SegmentQualityIndexMax = InSearchOptions.MaxQualityIndex;
					LoadReq->Bitrate = Bitrate;
					LoadReq->CompleteCallback.BindThreadSafeSP(AsShared(), &FManifestDASHInternal::FRepresentation::SegmentIndexDownloadComplete);
					OutRemoteElementLoadRequests.Emplace(LoadReq);
					PendingSegmentIndexLoadRequest = MoveTemp(LoadReq);
				}
				else if (GetStreamContainerType() == EStreamContainerType::Matroska)
				{
					TSharedPtrTS<FMPDLoadRequestDASH> LoadReq = MakeSharedTS<FMPDLoadRequestDASH>();
					FSegmentInformationCommon InitSegInfo;
					if (PrepareDownloadURLs(InPlayerSessionServices, InitSegInfo, SegmentBase))
					{
						LoadReq->Range2 = InitSegInfo.InitializationURL.Range;
						LoadReq->NumRemainingInChain = 1;
					}

					LoadReq->LoadType = FMPDLoadRequestDASH::ELoadType::IndexSegment;
					LoadReq->Url.URL = ChosenIndex.MediaURL.URL;;
					LoadReq->Range = RepresentationIndexSegmentURLRange;
					if (RequestHeader.Len())
					{
						LoadReq->Headers.Emplace(HTTP::FHTTPHeader(DASH::HTTPHeaderOptionName, RequestHeader));
					}
					LoadReq->PlayerSessionServices = InPlayerSessionServices;
					LoadReq->XLinkElement = MPDRepresentation;
					LoadReq->SegmentStreamType = InSearchOptions.StreamType;
					LoadReq->SegmentQualityIndex = InSearchOptions.QualityIndex;
					LoadReq->SegmentQualityIndexMax = InSearchOptions.MaxQualityIndex;
					LoadReq->Bitrate = Bitrate;
					LoadReq->CompleteCallback.BindThreadSafeSP(AsShared(), &FManifestDASHInternal::FRepresentation::SegmentIndexDownloadComplete);
					OutRemoteElementLoadRequests.Emplace(LoadReq);
					PendingSegmentIndexLoadRequest = MoveTemp(LoadReq);
				}
				else
				{
					check(!"Not implemented yet");
				}
				return FManifestDASHInternal::FRepresentation::ESearchResult::NeedElement;
			}
		}
	}
	return FManifestDASHInternal::FRepresentation::ESearchResult::Found;
}


void FManifestDASHInternal::FRepresentation::PrepareSteeringCandidates(TArray<FContentSteeringHandler::FCandidateURL>& OutSteeringCandidates, IPlayerSessionServices* InPlayerSessionServices, const FString& InDocumentURL, const TArray<DASHUrlHelpers::FDASHMediaURL>& InCandidateURLs, const TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>>& InUrlQueries, const FString& InTemplateURL)
{
	OutSteeringCandidates.Empty();
	for(auto& cand : InCandidateURLs)
	{
		FContentSteeringHandler::FCandidateURL& sc(OutSteeringCandidates.Emplace_GetRef());
		sc.MediaURL = cand;

		// Resolve with media template string.
		if (InTemplateURL.Len())
		{
			FURL_RFC3986 AbsoluteURL;
			AbsoluteURL.Parse(sc.MediaURL.URL);
			AbsoluteURL.ResolveWith(InTemplateURL);
			sc.MediaURL.URL = AbsoluteURL.Get();
		}
		// Apply any of the Annex I UrlQuery elements
		FString AnnexIRequestHeader;
		DASHUrlHelpers::ApplyUrlQueries(InPlayerSessionServices, InDocumentURL, sc.MediaURL.URL, AnnexIRequestHeader, InUrlQueries);

		// Set DASH specific URL parameters the steering handler does not need.
		// Since the steering handler may clone a CDN on demand it needs to copy these values into the clone
		// so the best way to do that is to set them in a dictionary.
		sc.AdditionalParams.Set(DASHUrlHelpers::SteerOption_ByteRange, FVariantValue(cand.ByteRange));
		sc.AdditionalParams.Set(DASHUrlHelpers::SteerOption_ATO, FVariantValue(cand.ATO));
		sc.AdditionalParams.Set(DASHUrlHelpers::SteerOption_ATOComplete, FVariantValue((int64)(cand.ATOComplete.IsSet()?(cand.ATOComplete.Value()?1:0):-1)));
		sc.AdditionalParams.Set(DASHUrlHelpers::SteerOption_AnnexIRequestHeader, FVariantValue(AnnexIRequestHeader));
	}
}

bool FManifestDASHInternal::FRepresentation::PrepareDownloadURLs(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& InOutSegmentInfo, const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& SegmentBase)
{
	TSharedPtrTS<FDashMPD_RepresentationType> MPDRepresentation = Representation.Pin();

	// Get the initialization, if any. If there is none the representation is supposed to be self-initializing.
	TSharedPtrTS<FDashMPD_URLType> Initialization = GET_ATTR(SegmentBase, GetInitialization(), IsValid(), TSharedPtrTS<FDashMPD_URLType>());
	FString InitializationSegmentURL, InitializationSegmentURLRange;
	if (Initialization.IsValid())
	{
		InitializationSegmentURLRange = Initialization->GetRange();
		InitializationSegmentURL = Initialization->GetSourceURL();
	}

	// The URL query might need to be changed. Look for the UrlQuery properties.
	TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>> UrlQueries;
	DASHUrlHelpers::GetAllHierarchyUrlQueries(UrlQueries, MPDRepresentation, DASHUrlHelpers::EUrlQueryRequestType::Segment, true);

	TArray<DASHUrlHelpers::FDASHMediaURL> CandidateURLs;

	DASHUrlHelpers::FDASHMediaURL RootDocument;
	RootDocument.URL = MPDRepresentation->GetDocumentURL();
	// The MPD has no @serviceLocation or the DVB properties. According to DVB-DASH the @serviceLocation is the URL itself and the attributes default to 1.
	if (InPlayerSessionServices->GetContentSteeringHandler()->IsDVBDash())
	{
		RootDocument.CDN = RootDocument.URL;
	}
	RootDocument.DVBpriority = 1;
	RootDocument.DVBweight = 1;
	DASHUrlHelpers::GetAllCandidateBaseURLs(CandidateURLs, MPDRepresentation, RootDocument, InPlayerSessionServices->GetContentSteeringHandler()->IsDVBDash());
	// Build a list we can pass to the steering handler.
	TArray<FContentSteeringHandler::FCandidateURL> SteeringCandidates;
	PrepareSteeringCandidates(SteeringCandidates, InPlayerSessionServices, RootDocument.URL, CandidateURLs, UrlQueries, FString());
	// Have the steering handler select the desired CDN, potentially cloning an existing one on demand.
	FString SteeringMessage;
	FContentSteeringHandler::FSelectedCandidateURL ChosenMedia = InPlayerSessionServices->GetContentSteeringHandler()->SelectBestCandidateFrom(SteeringMessage, FContentSteeringHandler::ESelectFor::Segment, SteeringCandidates);
	if (ChosenMedia.MediaURL.URL.IsEmpty())
	{
		PostError(InPlayerSessionServices, SteeringMessage, ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		return false;
	}

	// Setup the media URL parameters in the segment request.
	InOutSegmentInfo.MediaURL.Url = ChosenMedia.MediaURL;
	InOutSegmentInfo.MediaURL.CustomHeader = ChosenMedia.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_AnnexIRequestHeader).SafeGetFString();
	InOutSegmentInfo.MediaURL.SteeringID = ChosenMedia.SteeringID;
	// Content range?
	if (InOutSegmentInfo.FirstByteOffset && InOutSegmentInfo.NumberOfBytes)
	{
		ElectraHTTPStream::FHttpRange r;
		r.SetStart(InOutSegmentInfo.FirstByteOffset);
		r.SetEndIncluding(InOutSegmentInfo.FirstByteOffset + InOutSegmentInfo.NumberOfBytes - 1);
		InOutSegmentInfo.MediaURL.Range = r.GetString();
		InOutSegmentInfo.MediaURL.Url.URL = DASHUrlHelpers::ApplyAnnexEByteRange(InOutSegmentInfo.MediaURL.Url.URL, InOutSegmentInfo.MediaURL.Range, ChosenMedia.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_ByteRange).SafeGetFString());
	}

	// Now do the same for the initialization segment if there is a dedicated one.
	if (InitializationSegmentURL.Len())
	{
		PrepareSteeringCandidates(SteeringCandidates, InPlayerSessionServices, RootDocument.URL, CandidateURLs, UrlQueries, InitializationSegmentURL);
		FContentSteeringHandler::FSelectedCandidateURL ChosenInit = InPlayerSessionServices->GetContentSteeringHandler()->SelectBestCandidateFrom(SteeringMessage, FContentSteeringHandler::ESelectFor::Segment, SteeringCandidates);
		if (ChosenInit.MediaURL.URL.IsEmpty())
		{
			PostError(InPlayerSessionServices, SteeringMessage, ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			return false;
		}

		// Setup the init URL parameters in the segment request.
		InOutSegmentInfo.InitializationURL.Url = ChosenInit.MediaURL;
		InOutSegmentInfo.InitializationURL.CustomHeader = ChosenInit.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_AnnexIRequestHeader).SafeGetFString();
		InOutSegmentInfo.InitializationURL.SteeringID = ChosenInit.SteeringID;
		InOutSegmentInfo.InitializationURL.Range = InitializationSegmentURLRange;
		InOutSegmentInfo.InitializationURL.Url.URL = DASHUrlHelpers::ApplyAnnexEByteRange(InOutSegmentInfo.InitializationURL.Url.URL, InOutSegmentInfo.InitializationURL.Range, ChosenInit.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_ByteRange).SafeGetFString());
	}
	else
	{
		// The init segment is part of the media segment, so we can just copy this across.
		InOutSegmentInfo.InitializationURL = InOutSegmentInfo.MediaURL;
		// Except for the byte range, which is different.
		InOutSegmentInfo.InitializationURL.Range = InitializationSegmentURLRange;
		InOutSegmentInfo.InitializationURL.Url.URL = DASHUrlHelpers::ApplyAnnexEByteRange(InOutSegmentInfo.InitializationURL.Url.URL, InOutSegmentInfo.InitializationURL.Range, ChosenMedia.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_ByteRange).SafeGetFString());
	}


	// Finally apply the ATO of the media segment as the ATO in general.
	// We load init segments only when requesting media segments at which time the init segment needs
	// to be available anyway.
	InOutSegmentInfo.ATO += ChosenMedia.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_ATO).SafeGetTimeValue(FTimeValue::GetZero());

	// Set up additional things.
	InOutSegmentInfo.bLowLatencyChunkedEncodingExpected = bAvailableAsLowLatency.GetWithDefault(false);
	InOutSegmentInfo.ContainerType = StreamContainerType == EStreamContainerType::ISO14496_12 ? FSegmentInformationCommon::EContainerType::ISO14496_12 : FSegmentInformationCommon::EContainerType::Matroska;
	return true;
}

bool FManifestDASHInternal::FRepresentation::PrepareDownloadURLs(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& InOutSegmentInfo, const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& SegmentTemplate)
{
	TSharedPtrTS<FDashMPD_RepresentationType> MPDRepresentation = Representation.Pin();

	// Get the media template string. While we allow for the initialization segment to be described by an <Initialization> element
	// there is no meaningful way to get the media segment without a template since there is more than just one.
	FString MediaTemplate = GET_ATTR(SegmentTemplate, GetMediaTemplate(), Len(), FString());
	if (MediaTemplate.Len() == 0)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" provides no media template!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		return false;
	}
	// Get the initialization template string. If this is not specified try any <Initialization> elements.
	FString InitializationTemplate = GET_ATTR(SegmentTemplate, GetInitializationTemplate(), Len(), FString());
	if (InitializationTemplate.Len() == 0)
	{
		TSharedPtrTS<FDashMPD_URLType> Initialization = GET_ATTR(SegmentTemplate, GetInitialization(), IsValid(), TSharedPtrTS<FDashMPD_URLType>());
		if (Initialization.IsValid())
		{
			InOutSegmentInfo.InitializationURL.Range = Initialization->GetRange();
			if (Initialization->GetSourceURL().IsEmpty())
			{
				PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" provides no initialization segment!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
				return false;
			}
			// Note: This URL should probably not be using any template strings but I can't find any evidence for this, so just treat it as a template string as well.
			InitializationTemplate = Initialization->GetSourceURL();
		}
	}

	// Substitute template parameters for the media and init segments.
	FString MediaTemplateURL = ApplyTemplateStrings(MediaTemplate, InOutSegmentInfo.Number, InOutSegmentInfo.Time, InOutSegmentInfo.SubIndex);
	FString InitTemplateURL = ApplyTemplateStrings(InitializationTemplate, InOutSegmentInfo.Number, InOutSegmentInfo.Time, InOutSegmentInfo.SubIndex);

	// The URL query might need to be changed. Look for the UrlQuery properties.
	TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>> UrlQueries;
	DASHUrlHelpers::GetAllHierarchyUrlQueries(UrlQueries, MPDRepresentation, DASHUrlHelpers::EUrlQueryRequestType::Segment, true);

	TArray<DASHUrlHelpers::FDASHMediaURL> CandidateURLs;

	DASHUrlHelpers::FDASHMediaURL RootDocument;
	RootDocument.URL = MPDRepresentation->GetDocumentURL();
	// The MPD has no @serviceLocation or the DVB properties. According to DVB-DASH the @serviceLocation is the URL itself and the attributes default to 1.
	if (InPlayerSessionServices->GetContentSteeringHandler()->IsDVBDash())
	{
		RootDocument.CDN = RootDocument.URL;
	}
	RootDocument.DVBpriority = 1;
	RootDocument.DVBweight = 1;
	DASHUrlHelpers::GetAllCandidateBaseURLs(CandidateURLs, MPDRepresentation, RootDocument, InPlayerSessionServices->GetContentSteeringHandler()->IsDVBDash());
	// Build a list we can pass to the steering handler.
	TArray<FContentSteeringHandler::FCandidateURL> SteeringCandidates;
	PrepareSteeringCandidates(SteeringCandidates, InPlayerSessionServices, RootDocument.URL, CandidateURLs, UrlQueries, MediaTemplateURL);
	// Have the steering handler select the desired CDN, potentially cloning an existing one on demand.
	FString SteeringMessage;
	FContentSteeringHandler::FSelectedCandidateURL ChosenMedia = InPlayerSessionServices->GetContentSteeringHandler()->SelectBestCandidateFrom(SteeringMessage, FContentSteeringHandler::ESelectFor::Segment, SteeringCandidates);
	if (ChosenMedia.MediaURL.URL.IsEmpty())
	{
		PostError(InPlayerSessionServices, SteeringMessage, ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		return false;
	}

	// Setup the media URL parameters in the segment request.
	InOutSegmentInfo.MediaURL.Url = ChosenMedia.MediaURL;
	InOutSegmentInfo.MediaURL.CustomHeader = ChosenMedia.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_AnnexIRequestHeader).SafeGetFString();
	InOutSegmentInfo.MediaURL.SteeringID = ChosenMedia.SteeringID;

	// If there are future segments to be requested to be conveyed via CMCD `nor`, prepare their URLs now.
	for(int32 nf=0; nf<InOutSegmentInfo.NextMediaCMCDNumberTimes.Num(); ++nf)
	{
		FString NextMediaURL = ApplyTemplateStrings(MediaTemplate, InOutSegmentInfo.NextMediaCMCDNumberTimes[nf].Number, InOutSegmentInfo.NextMediaCMCDNumberTimes[nf].Time, 0);

		TArray<FContentSteeringHandler::FCandidateURL> NextSteeringCandidates;
		PrepareSteeringCandidates(NextSteeringCandidates, InPlayerSessionServices, RootDocument.URL, CandidateURLs, UrlQueries, NextMediaURL);
		FString DummyMessage;
		FContentSteeringHandler::FSelectedCandidateURL NextChosenMedia = InPlayerSessionServices->GetContentSteeringHandler()->SelectBestCandidateFrom(DummyMessage, FContentSteeringHandler::ESelectFor::Segment, NextSteeringCandidates);
		if (NextChosenMedia.MediaURL.URL.IsEmpty())
		{
			break;
		}
		InOutSegmentInfo.NextMediaURLS.Emplace_GetRef().Url.URL = NextChosenMedia.MediaURL.URL;
	}

	// Now do the same for the initialization segment
	if (!InitTemplateURL.IsEmpty())
	{
		PrepareSteeringCandidates(SteeringCandidates, InPlayerSessionServices, RootDocument.URL, CandidateURLs, UrlQueries, InitTemplateURL);
		FContentSteeringHandler::FSelectedCandidateURL ChosenInit = InPlayerSessionServices->GetContentSteeringHandler()->SelectBestCandidateFrom(SteeringMessage, FContentSteeringHandler::ESelectFor::Segment, SteeringCandidates);
		if (ChosenInit.MediaURL.URL.IsEmpty())
		{
			PostError(InPlayerSessionServices, SteeringMessage, ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			return false;
		}

		// Setup the init URL parameters in the segment request.
		InOutSegmentInfo.InitializationURL.Url = ChosenInit.MediaURL;
		InOutSegmentInfo.InitializationURL.CustomHeader = ChosenInit.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_AnnexIRequestHeader).SafeGetFString();
		InOutSegmentInfo.InitializationURL.SteeringID = ChosenInit.SteeringID;
		// The init segment may have a byte range if provided through an <Initialization> element in the <SegmentTemplate> and that may need
		// to be handled according to Annex E byte range request rules.
		// NOTE: It is *not* possible to have a byte range for the media segments under a <SegmentTemplate>, so we do not do the same above for the media segments.
		InOutSegmentInfo.InitializationURL.Url.URL = DASHUrlHelpers::ApplyAnnexEByteRange(InOutSegmentInfo.InitializationURL.Url.URL, InOutSegmentInfo.InitializationURL.Range, ChosenInit.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_ByteRange).SafeGetFString());
	}

	// Finally apply the ATO of the media segment as the ATO in general.
	// We load init segments only when requesting media segments at which time the init segment needs
	// to be available anyway.
	InOutSegmentInfo.ATO += ChosenMedia.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_ATO).SafeGetTimeValue(FTimeValue::GetZero());

	// Set up additional things.
	InOutSegmentInfo.bLowLatencyChunkedEncodingExpected = bAvailableAsLowLatency.GetWithDefault(false);
	InOutSegmentInfo.ContainerType = StreamContainerType == EStreamContainerType::ISO14496_12 ? FSegmentInformationCommon::EContainerType::ISO14496_12 : FSegmentInformationCommon::EContainerType::Matroska;
	return true;
}

FString FManifestDASHInternal::FRepresentation::ApplyTemplateStrings(FString TemplateURL, int64 InNumber, int64 InTime, int64 InSubIndex)
{
	auto PrintWithWidth = [](int64 Value, int32 Width) -> FString
	{
		FString Out = FString::Printf(TEXT("%lld"), Value);
		while(Out.Len() < Width)
		{
			Out = TEXT("0") + Out;
		}
		return Out;
	};

	auto GetFormatWidth = [](FString In) -> int32
	{
		int32 Width = 1;
		if (In.Len() && In[0] == TCHAR('%') && In[In.Len()-1] == TCHAR('d'))
		{
			LexFromString(Width, *In.Mid(1, In.Len()-2));
		}
		return Width;
	};

	FString NewURL;
	while(!TemplateURL.IsEmpty())
	{
		int32 tokenPos = INDEX_NONE;
		if (!TemplateURL.FindChar(TCHAR('$'), tokenPos))
		{
			NewURL.Append(TemplateURL);
			break;
		}
		else
		{
			// Append everything up to the first token.
			if (tokenPos)
			{
				NewURL.Append(TemplateURL.Mid(0, tokenPos));
			}
			// Need to find another token.
			int32 token2Pos = TemplateURL.Find(TEXT("$"), ESearchCase::CaseSensitive, ESearchDir::FromStart, tokenPos+1);
			if (token2Pos != INDEX_NONE)
			{
				FString token(TemplateURL.Mid(tokenPos+1, token2Pos-tokenPos-1));
				TemplateURL.RightChopInline(token2Pos+1, EAllowShrinking::No);
				// An empty token results from "$$" used to insert a single '$'.
				if (token.IsEmpty())
				{
					NewURL.AppendChar(TCHAR('$'));
				}
				// $RepresentationID$ ?
				else if (token.Equals(TEXT("RepresentationID")))
				{
					NewURL.Append(GetUniqueIdentifier());
				}
				// $Number$ ?
				else if (token.StartsWith(TEXT("Number")))
				{
					NewURL.Append(PrintWithWidth(InNumber, GetFormatWidth(token.Mid(6))));
				}
				// $Bandwidth$ ?
				else if (token.StartsWith(TEXT("Bandwidth")))
				{
					NewURL.Append(PrintWithWidth(GetBitrate(), GetFormatWidth(token.Mid(9))));
				}
				// $Time$ ?
				else if (token.StartsWith(TEXT("Time")))
				{
					NewURL.Append(PrintWithWidth(InTime, GetFormatWidth(token.Mid(4))));
				}
				// $SubNumber$ ?
				else if (token.StartsWith(TEXT("SubNumber")))
				{
					NewURL.Append(PrintWithWidth(InSubIndex, GetFormatWidth(token.Mid(9))));
				}
				else
				{
					// Unknown. This representation is not to be used!
					NewURL.Empty();
					break;
				}
			}
			else
			{
				// Bad template string. This representation is not to be used!
				NewURL.Empty();
				break;
			}
		}
	}
	return NewURL;
}

void FManifestDASHInternal::FRepresentation::CollectInbandEventStreams(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& InOutSegmentInfo)
{
	TSharedPtrTS<FDashMPD_RepresentationType> MPDRepresentation = Representation.Pin();

	TArray<TSharedPtrTS<FDashMPD_EventStreamType>> EvS = MPDRepresentation->GetInbandEventStreams();
	for(TSharedPtrTS<const IDashMPDElement> Parent = MPDRepresentation->GetParentElement(); Parent.IsValid(); Parent=Parent->GetParentElement())
	{
		if (Parent->GetElementType() == IDashMPDElement::EType::AdaptationSet)
		{
			EvS.Append(static_cast<const FDashMPD_AdaptationSetType*>(Parent.Get())->GetInbandEventStreams());
			break;
		}
	}
	for(int32 i=0; i<EvS.Num(); ++i)
	{
		FSegmentInformationCommon::FInbandEventStream ibs;
		ibs.SchemeIdUri = EvS[i]->GetSchemeIdUri();
		ibs.Value = EvS[i]->GetValue();
		ibs.PTO = (int64) EvS[i]->GetPresentationTimeOffset().GetWithDefault(0);
		ibs.Timescale = EvS[i]->GetTimescale().GetWithDefault(1);
		InOutSegmentInfo.InbandEventStreams.Emplace(MoveTemp(ibs));
	}
}

void FManifestDASHInternal::FRepresentation::SetupProducerReferenceTimeInfo(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& InOutSegmentInfo)
{
	InOutSegmentInfo.ProducerReferenceTimeInfos = ProducerReferenceTimeInfos;
	DashUtils::GetPlayerOption(InPlayerSessionServices, InOutSegmentInfo.MeasureLatencyViaReferenceTimeInfoID, DASH::OptionKey_LatencyReferenceId, (int64)-1);
}



FManifestDASHInternal::FRepresentation::ESearchResult FManifestDASHInternal::FRepresentation::FindSegment(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_DASH_FindSegment);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, DASH_FindSegment);

	/*
		Note: We use the DASH-IF-IOP specification and timing model. This is more strict than the general DASH standard and removes ambiguities
		      and otherwise conflicting information.
			  Please refer to https://dashif-documents.azurewebsites.net/Guidelines-TimingModel/master/Guidelines-TimingModel.html
	*/

	// As attributes may be present on any of the MPD hierarchy levels we need to get all these levels locked now.
	TSharedPtrTS<FDashMPD_RepresentationType> MPDRepresentation = Representation.Pin();
	if (!MPDRepresentation.IsValid())
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::Gone;
	}
	TSharedPtrTS<FDashMPD_AdaptationSetType> MPDAdaptation = StaticCastSharedPtr<FDashMPD_AdaptationSetType>(MPDRepresentation->GetParentElement());
	if (!MPDAdaptation.IsValid())
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::Gone;
	}
	TSharedPtrTS<FDashMPD_PeriodType> MPDPeriod = StaticCastSharedPtr<FDashMPD_PeriodType>(MPDAdaptation->GetParentElement());
	if (!MPDPeriod.IsValid())
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::Gone;
	}
	TSharedPtrTS<FDashMPD_MPDType> MPD = StaticCastSharedPtr<FDashMPD_MPDType>(MPDPeriod->GetParentElement());
	if (!MPD.IsValid())
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::Gone;
	}

	// We need to consider 4 types of addressing. <SegmentBase>, <SegmentTemplate>, <SegmentTimeline> and <SegmentList> where the latter is not supported.
	// As per 5.3.9.1:
	//		"Further, if SegmentTemplate or SegmentList is present on one level of the hierarchy, then the other one shall not be present on any lower hierarchy level."
	// implies that if there is a segment list anywhere then it's SegmentList all the way and we can return here.
	if (MPDRepresentation->GetSegmentList().IsValid() || MPDAdaptation->GetSegmentList().IsValid() || MPDPeriod->GetSegmentList().IsValid())
	{
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}

	TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>> SegmentBase({MPDRepresentation->GetSegmentBase(), MPDAdaptation->GetSegmentBase(), MPDPeriod->GetSegmentBase()});
	TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>> SegmentTemplate({MPDRepresentation->GetSegmentTemplate(), MPDAdaptation->GetSegmentTemplate(), MPDPeriod->GetSegmentTemplate()});
	// On representation level there can be at most one of the others.
	if (SegmentBase[0].IsValid() && SegmentTemplate[0].IsValid())
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" must have only one of <SegmentBase> or <SegmentTemplate>!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	// It is possible there is neither, which is most common with SegmentTemplate specified on the AdaptationSet.
	else if (!SegmentBase[0].IsValid() && !SegmentTemplate[0].IsValid())
	{
		// Again, there can be at most one of the others.
		if (SegmentBase[1].IsValid() && SegmentTemplate[1].IsValid())
		{
			PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" must only inherit one of <SegmentBase> or <SegmentTemplate> from enclosing AdaptationSet!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		// And once more, if there is neither go to the Period.
		else if (!SegmentBase[1].IsValid() && !SegmentTemplate[1].IsValid())
		{
			// Again, there can be at most one of the others.
			if (SegmentBase[2].IsValid() && SegmentTemplate[2].IsValid())
			{
				PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" must only inherit one of <SegmentBase> or <SegmentTemplate> from enclosing Period!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
				bIsUsable = false;
				return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
			}
		}
	}

	if (IsSideloadedSubtitle())
	{
		return SetupSideloadedFile(InPlayerSessionServices, OutSegmentInfo, InSearchOptions, MPDRepresentation);
	}
	else
	{
		// Remove empty hierarchy levels
		SegmentBase.Remove(nullptr);
		SegmentTemplate.Remove(nullptr);
		// Nothing? Bad MPD.
		if (SegmentBase.Num() == 0 && SegmentTemplate.Num() == 0)
		{
			PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" does not have one of <SegmentBase> or <SegmentTemplate> anywhere in the MPD hierarchy!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}

		if (!InSearchOptions.bInitSegmentSetupOnly)
		{
			if (SegmentBase.Num())
			{
				if (GetStreamContainerType() == EStreamContainerType::ISO14496_12)
				{
					return FindSegment_Base_MP4(InPlayerSessionServices, OutSegmentInfo, OutRemoteElementLoadRequests, InSearchOptions, MPDRepresentation, SegmentBase);
				}
				else if (GetStreamContainerType() == EStreamContainerType::Matroska)
				{
					return FindSegment_Base_MKV(InPlayerSessionServices, OutSegmentInfo, OutRemoteElementLoadRequests, InSearchOptions, MPDRepresentation, SegmentBase);
				}
				else
				{
					PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" uses an unsupported media container format!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
					bIsUsable = false;
					return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
				}
			}
			else
			{
				// Get the segment timeline, if one is used.
				TSharedPtrTS<FDashMPD_SegmentTimelineType> SegmentTimeline = GET_ATTR(SegmentTemplate, GetSegmentTimeline(), IsValid(), TSharedPtrTS<FDashMPD_SegmentTimelineType>());
				if (SegmentTimeline.IsValid())
				{
					return FindSegment_Timeline(InPlayerSessionServices, OutSegmentInfo, OutRemoteElementLoadRequests, InSearchOptions, MPDRepresentation, SegmentTemplate, SegmentTimeline);
				}
				else
				{
					return FindSegment_Template(InPlayerSessionServices, OutSegmentInfo, OutRemoteElementLoadRequests, InSearchOptions, MPDRepresentation, SegmentTemplate);
				}
			}
		}
		else
		{
			if (SegmentBase.Num())
			{
				return PrepareDownloadURLs(InPlayerSessionServices, OutSegmentInfo, SegmentBase) ? FManifestDASHInternal::FRepresentation::ESearchResult::Found : FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
			}
			else
			{
				return PrepareDownloadURLs(InPlayerSessionServices, OutSegmentInfo, SegmentTemplate) ? FManifestDASHInternal::FRepresentation::ESearchResult::Found : FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
			}
		}
	}
}


FManifestDASHInternal::FRepresentation::ESearchResult FManifestDASHInternal::FRepresentation::FindSegment_Base_MP4(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation, const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& SegmentBase)
{
	if (!bIsUsable)
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	ESearchResult SegIndexResult = PrepareSegmentIndex(InPlayerSessionServices, SegmentBase, OutRemoteElementLoadRequests, InSearchOptions);
	if (SegIndexResult != ESearchResult::Found)
	{
		return SegIndexResult;
	}
	if (!SIDXBox.IsValid())
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("A segment index is required for Representation \"%s\""), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	uint32 SidxTimescale = SIDXBox->GetTimescale();
	if (SidxTimescale == 0)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Timescale of segment index for Representation \"%s\" is invalid!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}

	// The search time is period local time, thus starts at zero. In here it is all about media local time, so we need to map
	// the search time onto the media internal timeline.
	uint64 PTO = GET_ATTR(SegmentBase, GetPresentationTimeOffset(), IsSet(), TMediaOptionalValue<uint64>(0)).Value();
	uint32 MPDTimescale = GET_ATTR(SegmentBase, GetTimescale(), IsSet(), TMediaOptionalValue<uint32>(1)).Value();
	// Since the PTO is specified in the timescale as given in the MPD the timescales of the MPD and the segment index should better match!
	if (PTO && MPDTimescale != SidxTimescale && !bWarnedAboutTimescale)
	{
		bWarnedAboutTimescale = true;
		LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation timescale (%u) in MPD is not equal to timescale used in the segment index (%u) for Representation \"%s\"."), MPDTimescale, SidxTimescale, *MPDRepresentation->GetID()));
	}
	FTimeValue ATO = CalculateSegmentAvailabilityTimeOffset(SegmentBase);
	//TMediaOptionalValue<bool> bATOComplete = GetSegmentAvailabilityTimeComplete(SegmentBase);


	// Convert the local media search time to the timescale of the segment index.
	// The PTO (presentation time offset) which maps the internal media time to the zero point of the period must be included as well.
	// Depending on the time scale the conversion may unfortunately incur a small rounding error.
	int64 MediaLocalSearchTime = InSearchOptions.PeriodLocalTime.GetAsTimebase(SidxTimescale) + PTO;
	if (MediaLocalSearchTime < 0)
	{
		MediaLocalSearchTime = 0;
	}
	int64 MediaLocalPeriodEnd = InSearchOptions.PeriodDuration.IsValid() && !InSearchOptions.PeriodDuration.IsInfinity() ? InSearchOptions.PeriodDuration.GetAsTimebase(SidxTimescale) + PTO : TNumericLimits<int64>::Max();
	int64 MediaLocalPresentationEnd = InSearchOptions.PeriodPresentationEnd.IsValid() && !InSearchOptions.PeriodPresentationEnd.IsInfinity() ? InSearchOptions.PeriodPresentationEnd.GetAsTimebase(SidxTimescale) + PTO : TNumericLimits<int64>::Max();
	int64 MediaLocalEndTime = Utils::Min(MediaLocalPeriodEnd, MediaLocalPresentationEnd);
	if (MediaLocalSearchTime >= MediaLocalEndTime)
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
	}
	// Note: The segment index has only segment durations. If the segments have a baseMediaDecodeTime different from 0 then that value of the first segment
	//       would need to be stored in the EPT (earliest presentation time) here.
	//       The EPT also includes the very first composition time offset, so it may not be zero.
	//       We have to remember that the segment index does not necessarily have access to an edit list as this is stored in the init segment and is not
	//       available at this point in time, so any offsets that would come from an edit list need to have been applied to the EPT here already.
	int64 EarliestPresentationTime = SIDXBox->GetEarliestPresentationTime();
	int64 CurrentT = EarliestPresentationTime;
	int32 CurrentN, StartNumber=0, EndNumber=SIDXBox->GetEntries().Num();
	int32 CurrentD = 0;
	int64 PreviousT = CurrentT;
	int32 PreviousD = 0;
	int32 PreviousN = 0;
	int64 CurrentOffset = 0;
	int64 PreviousOffset = 0;
	bool bFound = false;
	for(CurrentN=0; CurrentN<EndNumber; ++CurrentN)
	{
		const MP4Boxes::FMP4BoxSIDX::FEntry& SegmentInfo = SIDXBox->GetEntries()[CurrentN];

		// We do not support hierarchical segment indices!
		if (SegmentInfo.IsReferenceType)
		{
			PostError(InPlayerSessionServices, FString::Printf(TEXT("Segment index for Representation \"%s\" must directly reference the media, not another index!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		else if (SegmentInfo.StartsWithSAP == 0)
		{
			PostError(InPlayerSessionServices, FString::Printf(TEXT("Segment index for Representation \"%s\" must have starts_with_sap set!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		// We require segments to begin with SAP type 1 or 2 (preferably 1)
		/*
		else if (SegmentInfo.SAPType == 0)
		{
			LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Segment index for Representation \"%s\" has unknown SAP_type (0). If it is not 1 or 2 playback may not work correctly"), *MPDRepresentation->GetID()));
		}
		*/
		else if (SegmentInfo.SAPType != 1 && SegmentInfo.SAPType != 2)
		{
			PostError(InPlayerSessionServices, FString::Printf(TEXT("Segment index for Representation \"%s\" must have SAP_type 1 or 2 only!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		/*

			The SAP_delta_time is merely an indication of the PTS of the SAP, it does not enter any time calculations.

		else if (SegmentInfo.SAPDeltaTime != 0)
		{
			LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Segment index for Representation \"%s\" uses SAP_delta_time. This may result in incorrect decoding."), *MPDRepresentation->GetID()));
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		*/
		CurrentD = SegmentInfo.SubSegmentDuration;
		// Does the segment start on or after the time we're looking for?
		if (CurrentT >= MediaLocalSearchTime)
		{
			bFound = true;

			// Yes, so we have now found the segment of interest. It is either this one or the previous one.
			if (InSearchOptions.SearchType == IManifest::ESearchType::Closest)
			{
				// If there is a preceeding segment check if its start time is closer
				if (CurrentN > StartNumber)
				{
					if (MediaLocalSearchTime - PreviousT < CurrentT - MediaLocalSearchTime)
					{
						--CurrentN;
						CurrentD = PreviousD;
						CurrentT = PreviousT;
						CurrentOffset = PreviousOffset;
					}
				}
				break;
			}
			else if (InSearchOptions.SearchType == IManifest::ESearchType::After || InSearchOptions.SearchType == IManifest::ESearchType::StrictlyAfter)
			{
				// The 'after' search is used to locate the next segment. For that reason the search time has been adjusted by the caller
				// to be larger than the start time of the preceeding segment.
				// Therefor, since this segment here has a larger or equal start time than the time we are searching for this segment here
				// must be the one 'after'.
				if (CurrentT >= MediaLocalEndTime)
				{
					return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
				}
				break;
			}
			else if (InSearchOptions.SearchType == IManifest::ESearchType::Same || InSearchOptions.SearchType == IManifest::ESearchType::Before)
			{
				// The 'before' search is used to locate the segment containing the search time, which could be
				// either this segment or the preceeding one.
				// The 'same' search is used exactly like 'before'. The segment is required that contains the search time.
				if (CurrentT > MediaLocalSearchTime && CurrentN > StartNumber)
				{
					// Not this segment, must be the preceeding one.
					--CurrentN;
					CurrentD = PreviousD;
					CurrentT = PreviousT;
					CurrentOffset = PreviousOffset;
				}
				break;
			}
			else if (InSearchOptions.SearchType == IManifest::ESearchType::StrictlyBefore)
			{
				// The 'strictlybefore' search is used to locate the segment just before the one the search time is in.
				// The caller is not expected to adjust the time to search for to do that since we are returning
				// the earlier segment if it exists. If not the same segment will be returned.
				if (CurrentN > StartNumber)
				{
					--CurrentN;
					CurrentD = PreviousD;
					CurrentT = PreviousT;
					CurrentOffset = PreviousOffset;
				}
				break;
			}
			else
			{
				PostError(InPlayerSessionServices, FString::Printf(TEXT("Unsupported segment search mode!")), ERRCODE_DASH_MPD_INTERNAL);
				bIsUsable = false;
				return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
			}
		}
		PreviousT = CurrentT;
		PreviousN = CurrentN;
		PreviousD = CurrentD;
		PreviousOffset = CurrentOffset;
		CurrentT += CurrentD;
		CurrentOffset += SegmentInfo.Size;
	}

	// If the search time falls into the last segment we will not have found it above.
	if (!bFound && CurrentT >= MediaLocalSearchTime && CurrentN == EndNumber)
	{
		if (InSearchOptions.SearchType == IManifest::ESearchType::Closest || InSearchOptions.SearchType == IManifest::ESearchType::Same || InSearchOptions.SearchType == IManifest::ESearchType::Before || InSearchOptions.SearchType == IManifest::ESearchType::StrictlyBefore)
		{
			--CurrentN;
			CurrentD = PreviousD;
			CurrentT = PreviousT;
			CurrentOffset -= SIDXBox->GetEntries()[CurrentN].Size;
			bFound = true;
		}
	}

	// Did we find it?
	if (bFound && CurrentT < MediaLocalEndTime)
	{
		OutSegmentInfo.Time = CurrentT;
		OutSegmentInfo.PTO = PTO;
		OutSegmentInfo.Duration = CurrentD;
		OutSegmentInfo.Number = CurrentN;
		OutSegmentInfo.NumberOfBytes = SIDXBox->GetEntries()[CurrentN].Size;
		OutSegmentInfo.FirstByteOffset = SIDXBox->GetFirstOffset() + SegmentIndexRangeStart + SegmentIndexRangeSize + CurrentOffset;
		OutSegmentInfo.MediaLocalFirstAUTime = OutSegmentInfo.MediaLocalFirstPTS = MediaLocalSearchTime;
		OutSegmentInfo.MediaLocalLastAUTime = MediaLocalEndTime;
		OutSegmentInfo.Timescale = SidxTimescale;
		OutSegmentInfo.ATO = ATO;
		OutSegmentInfo.bIsLastInPeriod = CurrentT + CurrentD >= MediaLocalEndTime;
		OutSegmentInfo.bFrameAccuracyRequired = InSearchOptions.bFrameAccurateSearch;
		// Look at the next n segment ranges for CMCD `nrr`
		if (!OutSegmentInfo.bIsLastInPeriod && InPlayerSessionServices->GetCMCDHandler()->IsEnabled())
		{
			int64 cbo = OutSegmentInfo.FirstByteOffset + OutSegmentInfo.NumberOfBytes;
			int64 nlt = CurrentT + CurrentD;
			for(int32 jj=CurrentN+1; jj<EndNumber && nlt<MediaLocalEndTime; ++jj)
			{
				int64 nss = SIDXBox->GetEntries()[jj].Size;
				int64 nsd = SIDXBox->GetEntries()[jj].SubSegmentDuration;
				int64 NextSegStart = cbo;
				int64 NextSegEnd = cbo + nss - 1;
				OutSegmentInfo.NextMediaURLS.Emplace_GetRef().Range = FString::Printf(TEXT("%lld-%lld"), (long long int)NextSegStart, (long long int)NextSegEnd);
				if (OutSegmentInfo.NextMediaURLS.Num() >= MAX_NEXT_SEGMENTS_FOR_CMCD)
				{
					break;
				}
				cbo += nss;
				nlt += nsd;
			}
		}

		CollectInbandEventStreams(InPlayerSessionServices, OutSegmentInfo);
		SetupProducerReferenceTimeInfo(InPlayerSessionServices, OutSegmentInfo);
		if (!PrepareDownloadURLs(InPlayerSessionServices, OutSegmentInfo, SegmentBase))
		{
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		else
		{
			return FManifestDASHInternal::FRepresentation::ESearchResult::Found;
		}
	}
	else
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
	}
}


FManifestDASHInternal::FRepresentation::ESearchResult FManifestDASHInternal::FRepresentation::FindSegment_Base_MKV(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation, const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& SegmentBase)
{
	ESearchResult SegIndexResult = PrepareSegmentIndex(InPlayerSessionServices, SegmentBase, OutRemoteElementLoadRequests, InSearchOptions);
	if (SegIndexResult != ESearchResult::Found)
	{
		return SegIndexResult;
	}
	if (!SegmentMKV.IsValid())
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("A segment index is required for Representation \"%s\""), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	if (SegmentMKV->GetNumberOfTracks() <= 0)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" contains no usable track"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	else if (SegmentMKV->GetNumberOfTracks() > 1)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" contains more than one track"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}

	// The search time is period local time, thus starts at zero. In here it is all about media local time, so we need to map
	// the search time onto the media internal timeline.
	int64 PTO = 0;
	FTimeValue ATO = CalculateSegmentAvailabilityTimeOffset(SegmentBase);
	//TMediaOptionalValue<bool> bATOComplete = GetSegmentAvailabilityTimeComplete(SegmentBase);
	{
		uint32 MPDTimescale = GET_ATTR(SegmentBase, GetTimescale(), IsSet(), TMediaOptionalValue<uint32>(1)).Value();
		uint64 pto = GET_ATTR(SegmentBase, GetPresentationTimeOffset(), IsSet(), TMediaOptionalValue<uint64>(0)).Value();
		FTimeFraction ptof(pto, MPDTimescale);
		PTO = ptof.GetAsTimebase(10000000);
	}


	// Convert the local media search time to the timescale of the segment index.
	// The PTO (presentation time offset) which maps the internal media time to the zero point of the period must be included as well.
	// Depending on the time scale the conversion may unfortunately incur a small rounding error.
	int64 MediaLocalSearchTime = InSearchOptions.PeriodLocalTime.GetAsHNS() + PTO;
	if (MediaLocalSearchTime < 0)
	{
		MediaLocalSearchTime = 0;
	}
	int64 MediaLocalPeriodEnd = InSearchOptions.PeriodDuration.IsValid() && !InSearchOptions.PeriodDuration.IsInfinity() ? InSearchOptions.PeriodDuration.GetAsHNS() + PTO : TNumericLimits<int64>::Max();
	int64 MediaLocalPresentationEnd = InSearchOptions.PeriodPresentationEnd.IsValid() && !InSearchOptions.PeriodPresentationEnd.IsInfinity() ? InSearchOptions.PeriodPresentationEnd.GetAsHNS() + PTO : TNumericLimits<int64>::Max();
	int64 MediaLocalEndTime = Utils::Min(MediaLocalPeriodEnd, MediaLocalPresentationEnd);
	if (MediaLocalSearchTime >= MediaLocalEndTime)
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
	}

	const IParserMKV::ITrack* Track = SegmentMKV->GetTrackByIndex(0);
	if (!Track)
	{
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	TSharedPtrTS<IParserMKV::ICueIterator> TrackIt(Track->CreateCueIterator());
	IParserMKV::ICueIterator::ESearchMode SearchMode =
		InSearchOptions.SearchType == IManifest::ESearchType::After  || InSearchOptions.SearchType == IManifest::ESearchType::StrictlyAfter  ? IParserMKV::ICueIterator::ESearchMode::After :
		InSearchOptions.SearchType == IManifest::ESearchType::Before || InSearchOptions.SearchType == IManifest::ESearchType::StrictlyBefore ? IParserMKV::ICueIterator::ESearchMode::Before : IParserMKV::ICueIterator::ESearchMode::Closest;

	UEMediaError Error;
	Error = TrackIt->StartAtTime(FTimeValue(MediaLocalSearchTime), SearchMode);
	// Did we find it?
	if (Error == UEMEDIA_ERROR_OK)
	{
		OutSegmentInfo.Time = TrackIt->GetTimestamp().GetAsHNS();
		OutSegmentInfo.PTO = PTO;
		OutSegmentInfo.Duration = TrackIt->GetClusterDuration().GetAsHNS();
		OutSegmentInfo.Number = TrackIt->GetUniqueID();
		OutSegmentInfo.NumberOfBytes = TrackIt->GetClusterFileSize();
		OutSegmentInfo.FirstByteOffset = TrackIt->GetClusterFileOffset();
		OutSegmentInfo.MediaLocalFirstAUTime = OutSegmentInfo.MediaLocalFirstPTS = MediaLocalSearchTime;
		OutSegmentInfo.MediaLocalLastAUTime = MediaLocalEndTime;
		OutSegmentInfo.Timescale = 10000000;
		OutSegmentInfo.ATO = ATO;
		OutSegmentInfo.bIsLastInPeriod = TrackIt->IsLastCluster() || (TrackIt->GetTimestamp() + TrackIt->GetClusterDuration()).GetAsHNS() >= MediaLocalEndTime;
		OutSegmentInfo.bFrameAccuracyRequired = InSearchOptions.bFrameAccurateSearch;
		if (!PrepareDownloadURLs(InPlayerSessionServices, OutSegmentInfo, SegmentBase))
		{
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		else
		{
			return FManifestDASHInternal::FRepresentation::ESearchResult::Found;
		}
	}
	else
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
	}
}

FManifestDASHInternal::FRepresentation::ESearchResult FManifestDASHInternal::FRepresentation::FindSegment_Template(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation, const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& SegmentTemplate)
{
	uint64 PTO = GET_ATTR(SegmentTemplate, GetPresentationTimeOffset(), IsSet(), TMediaOptionalValue<uint64>(0)).Value();
	uint32 MPDTimescale = GET_ATTR(SegmentTemplate, GetTimescale(), IsSet(), TMediaOptionalValue<uint32>(1)).Value();
	uint32 StartNumber = GET_ATTR(SegmentTemplate, GetStartNumber(), IsSet(), TMediaOptionalValue<uint32>(1)).Value();
	TMediaOptionalValue<uint32> EndNumber = GET_ATTR(SegmentTemplate, GetEndNumber(), IsSet(), TMediaOptionalValue<uint32>());
	TMediaOptionalValue<uint32> Duration = GET_ATTR(SegmentTemplate, GetDuration(), IsSet(), TMediaOptionalValue<uint32>());
	TMediaOptionalValue<int32> EptDelta = GET_ATTR(SegmentTemplate, GetEptDelta(), IsSet(), TMediaOptionalValue<int32>());
	FTimeValue ATO = CalculateSegmentAvailabilityTimeOffset(SegmentTemplate);
	//TMediaOptionalValue<bool> bATOComplete = GetSegmentAvailabilityTimeComplete(SegmentTemplate);

	// The timescale should in all likelihood not be 1. While certainly allowed an accuracy of only one second is more likely to be
	// an oversight when building the MPD.
	if (MPDTimescale == 0)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Timescale for Representation \"%s\" is invalid!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	else if (MPDTimescale == 1 && !bWarnedAboutTimescale)
	{
		bWarnedAboutTimescale = true;
		LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Timescale for Representation \"%s\" is given as 1. Is this intended?"), *MPDRepresentation->GetID()));
	}

	// There needs to be a segment duration here.
	if (!Duration.IsSet() || Duration.Value() == 0)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" has no valid segment duration!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	uint32 SegmentDuration = Duration.Value();

	// Get the period local time into media local timescale.
	// Note that the PTO is not relevant here since we are not calculating on internal media time.
	int64 MediaLocalSearchTime = InSearchOptions.PeriodLocalTime.GetAsTimebase(MPDTimescale);
	// If the first media segment does not fall onto the period start there will be an EPT delta that is usually negative.
	// To simplify calculation of the segment index we shift the search time such that 0 would correspond to the EPT.
	int32 EPTdelta = EptDelta.GetWithDefault(0);
	MediaLocalSearchTime -= EPTdelta;
	if (MediaLocalSearchTime < 0)
	{
		MediaLocalSearchTime = 0;
	}

	int64 MediaLocalPeriodEnd = InSearchOptions.PeriodDuration.IsValid() && !InSearchOptions.PeriodDuration.IsInfinity() ? InSearchOptions.PeriodDuration.GetAsTimebase(MPDTimescale) - EPTdelta : TNumericLimits<int64>::Max();
	int64 MediaLocalPresentationEnd = InSearchOptions.PeriodPresentationEnd.IsValid() && !InSearchOptions.PeriodPresentationEnd.IsInfinity() ? InSearchOptions.PeriodPresentationEnd.GetAsTimebase(MPDTimescale) : TNumericLimits<int64>::Max();
	int64 MediaLocalEndTime = Utils::Min(MediaLocalPeriodEnd, MediaLocalPresentationEnd);
	uint32 MaxSegmentsInPeriod = MediaLocalEndTime == TNumericLimits<int64>::Max() ? TNumericLimits<uint32>::Max() : (uint32)((MediaLocalEndTime + SegmentDuration - 1) / SegmentDuration);

	// Clamp against the number of segments described by EndNumber.
	// The assumption is that end number is inclusive, so @startNumber == @endNumber means there is 1 segment.
	if (EndNumber.IsSet())
	{
		if ((int64)MaxSegmentsInPeriod > (int64)EndNumber.Value() - StartNumber + 1)
		{
			MaxSegmentsInPeriod = (uint32)((int64)EndNumber.Value() - StartNumber + 1);
		}
	}

	// Now we calculate the number of the segment the search time falls into.
	uint32 SegmentNum = MediaLocalSearchTime / SegmentDuration;
	uint32 SegDurRemainder = MediaLocalSearchTime - SegmentNum * SegmentDuration;

	if (InSearchOptions.SearchType == IManifest::ESearchType::Closest)
	{
		// This is different from <SegmentBase> and <SegmentTimeline> handling since here we are definitely in the segment
		// the search time is in and not possibly the segment thereafter, because we calculated the index through division
		// instead of accumulating durations.
		// Therefor the segment that might be closer to the search time can only be the next one, not the preceeding one.
		if (SegDurRemainder > SegmentDuration / 2 && SegmentNum+1 < MaxSegmentsInPeriod)
		{
			++SegmentNum;
		}
	}
	else if (InSearchOptions.SearchType == IManifest::ESearchType::After || InSearchOptions.SearchType == IManifest::ESearchType::StrictlyAfter)
	{
		// The 'after' search is used to locate the next segment. For that reason the search time has been adjusted by the caller
		// to be larger than the start time of the preceeding segment, but still within the same segment!
		// So we should actually now still be in the same segment as before due to integer truncation when calculating the index
		// through division and the index we want is the next one. However, if due to dumb luck there is no remainder we need to
		// assume the time that got added by the caller (which must not have been zero!) was such that we already landed on the
		// following segment and thus do not increase the index.
		if (SegDurRemainder)
		{
			++SegmentNum;
		}
	}
	else if (InSearchOptions.SearchType == IManifest::ESearchType::Same || InSearchOptions.SearchType == IManifest::ESearchType::Before)
	{
		// The 'before' search is used to locate the segment containing the search time, which could be
		// either this segment or the preceeding one.
		// The 'same' search is used exactly like 'before'. The segment is required that contains the search time.
		// Nothing to do. We are already in that segment.
	}
	else if (InSearchOptions.SearchType == IManifest::ESearchType::StrictlyBefore)
	{
		// The 'strictlybefore' search is used to locate the segment just before the one the search time is in.
		// The caller is not expected to adjust the time to search for to do that since we are returning
		// the earlier segment if it exists. If not the same segment will be returned.
		if (SegmentNum > 0)
		{
			--SegmentNum;
		}
	}
	else
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Unsupported segment search mode!")), ERRCODE_DASH_MPD_INTERNAL);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}

	// Past the last segment?
	if (SegmentNum >= MaxSegmentsInPeriod)
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
	}

	OutSegmentInfo.Time = PTO + EPTdelta + SegmentNum * (int64)SegmentDuration;
	OutSegmentInfo.PTO = PTO;
	OutSegmentInfo.EPTdelta = EPTdelta;
	OutSegmentInfo.Duration = SegmentDuration;
	OutSegmentInfo.Number = StartNumber + SegmentNum;
	OutSegmentInfo.MediaLocalFirstAUTime = OutSegmentInfo.MediaLocalFirstPTS = MediaLocalSearchTime + PTO;
	if ((OutSegmentInfo.MediaLocalLastAUTime = MediaLocalEndTime + PTO) < 0)
	{
		OutSegmentInfo.MediaLocalLastAUTime = TNumericLimits<int64>::Max();
	}
	OutSegmentInfo.Timescale = MPDTimescale;
	OutSegmentInfo.bMayBeMissing = SegmentNum + 1 >= MaxSegmentsInPeriod;
	OutSegmentInfo.bIsLastInPeriod = OutSegmentInfo.bMayBeMissing && InSearchOptions.bHasFollowingPeriod;
	OutSegmentInfo.bFrameAccuracyRequired = InSearchOptions.bFrameAccurateSearch;
	OutSegmentInfo.ATO = ATO;

	// Look at the next n segment numbers for CMCD `nro` construction
	if (!OutSegmentInfo.bIsLastInPeriod && InPlayerSessionServices->GetCMCDHandler()->IsEnabled())
	{
		for(uint32 jj=SegmentNum+1; jj<MaxSegmentsInPeriod; ++jj)
		{
			auto& nxt = OutSegmentInfo.NextMediaCMCDNumberTimes.Emplace_GetRef();
			nxt.Number = StartNumber + jj;
			nxt.Time = PTO + EPTdelta + jj * (int64)SegmentDuration;
			if (OutSegmentInfo.NextMediaCMCDNumberTimes.Num() >= MAX_NEXT_SEGMENTS_FOR_CMCD)
			{
				break;
			}
		}
	}

	CollectInbandEventStreams(InPlayerSessionServices, OutSegmentInfo);
	SetupProducerReferenceTimeInfo(InPlayerSessionServices, OutSegmentInfo);
	if (!PrepareDownloadURLs(InPlayerSessionServices, OutSegmentInfo, SegmentTemplate))
	{
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	else
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::Found;
	}
}

FManifestDASHInternal::FRepresentation::ESearchResult FManifestDASHInternal::FRepresentation::FindSegment_Timeline(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation, const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& SegmentTemplate, const TSharedPtrTS<FDashMPD_SegmentTimelineType>& SegmentTimeline)
{
	// Segment timeline must not be empty.
	auto Selements = SegmentTimeline->GetS_Elements();
	if (Selements.Num() == 0)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" has an empty <SegmentTimeline>!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	else if (!Selements[0].HaveD)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> does not have mandatory 'd' element!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}

	uint64 PTO = GET_ATTR(SegmentTemplate, GetPresentationTimeOffset(), IsSet(), TMediaOptionalValue<uint64>(0)).Value();
	uint32 MPDTimescale = GET_ATTR(SegmentTemplate, GetTimescale(), IsSet(), TMediaOptionalValue<uint32>(1)).Value();
	uint32 StartNumber = GET_ATTR(SegmentTemplate, GetStartNumber(), IsSet(), TMediaOptionalValue<uint32>(1)).Value();
	uint32 EndNumber = GET_ATTR(SegmentTemplate, GetEndNumber(), IsSet(), TMediaOptionalValue<uint32>(~0U)).Value();
	//TMediaOptionalValue<uint32> Duration = GET_ATTR(SegmentTemplate, GetDuration(), IsSet(), TMediaOptionalValue<uint32>());
	FTimeValue ATO = CalculateSegmentAvailabilityTimeOffset(SegmentTemplate);
	//TMediaOptionalValue<bool> bATOComplete = GetSegmentAvailabilityTimeComplete(SegmentTemplate);

	// The timescale should in all likelihood not be 1. While certainly allowed an accuracy of only one second is more likely to be
	// an oversight when building the MPD.
	if (MPDTimescale == 0)
	{
		PostError(InPlayerSessionServices, FString::Printf(TEXT("Timescale for Representation \"%s\" is invalid!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	else if (MPDTimescale == 1 && !bWarnedAboutTimescale)
	{
		bWarnedAboutTimescale = true;
		LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Timescale for Representation \"%s\" is given as 1. Is this intended?"), *MPDRepresentation->GetID()));
	}

	// Get the period local time into media local timescale and add the PTO.
	int64 MediaLocalSearchTime = InSearchOptions.PeriodLocalTime.GetAsTimebase(MPDTimescale) + PTO;
	if (MediaLocalSearchTime < 0)
	{
		MediaLocalSearchTime = 0;
	}
	int64 MediaLocalPeriodEnd = InSearchOptions.PeriodDuration.IsValid() && !InSearchOptions.PeriodDuration.IsInfinity() ? InSearchOptions.PeriodDuration.GetAsTimebase(MPDTimescale) + PTO : TNumericLimits<int64>::Max();
	int64 MediaLocalPresentationEnd = InSearchOptions.PeriodPresentationEnd.IsValid() && !InSearchOptions.PeriodPresentationEnd.IsInfinity() ? InSearchOptions.PeriodPresentationEnd.GetAsTimebase(MPDTimescale) + PTO : TNumericLimits<int64>::Max();
	int64 MediaLocalEndTime = Utils::Min(MediaLocalPeriodEnd, MediaLocalPresentationEnd);

	// Note: The DASH standard has been extended with a <FailoverContent> element. If this exists we should see if the time we want falls into
	//       content that is not present in this <SegmentTimeline> (failover content does not provide actual content. It gives times for which there is no content available here!)
	//       If the failover content is not on AdaptationSet level we can look for another representation (of lower quality) for which there is content available and then use that one.
	//       Otherwise, knowing that there is no content for any representation we could create a filler segment here.

	int64 CurrentT = Selements[0].HaveT ? Selements[0].T : 0;
	int64 CurrentN = Selements[0].HaveN ? Selements[0].N : StartNumber;
	int32 CurrentR = Selements[0].HaveR ? Selements[0].R : 0;
	int64 CurrentD = Selements[0].D;
	int32 nIndex = 0;
	bool bIsCurrentlyAGap = false;

	bool bFound = false;
	bool bFirstContentMissing = false;
	// Search for the segment. It is possible that already the first segment has a larger T value than we are searching for.
	if (CurrentT > MediaLocalSearchTime)
	{
		// The first segment starts in the future. What we do now may depend on several factors.
		// If we use it the PTS will jump forward. What happens exactly depends on how the other active representations behave.
		// We could set up a dummy segment request to insert filler data for the duration until the first segment actually starts.
		// This may depend on how big of a gap we are talking about.
		double MissingContentDuration = FTimeValue(CurrentT - MediaLocalSearchTime, MPDTimescale).GetAsSeconds();
		if (MissingContentDuration > 0.1 && !bWarnedAboutTimelineStartGap)
		{
			bWarnedAboutTimelineStartGap = true;
			LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> starts with %.3f seconds of missing content that will be skipped over and might lead to playback issues."), *MPDRepresentation->GetID(), MissingContentDuration));
		}
		bFound = true;
		bFirstContentMissing = true;
	}
	else
	{
		int64 PreviousD = CurrentD;
		int64 PreviousN = CurrentN - 1;			// start with -1 so we can test if the current N is the previous+1 !
		int64 PreviousT = CurrentT - CurrentD;	// same for T
		for(nIndex=0; nIndex<Selements.Num(); ++nIndex)
		{
			if (!Selements[nIndex].HaveD)
			{
				PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> does not have mandatory 'd' element!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
				bIsUsable = false;
				return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
			}

			CurrentT = Selements[nIndex].HaveT ? Selements[nIndex].T : CurrentT;
			CurrentN = Selements[nIndex].HaveN ? Selements[nIndex].N : CurrentN;
			CurrentR = Selements[nIndex].HaveR ? Selements[nIndex].R : 0;
			CurrentD = Selements[nIndex].D;

			if (CurrentD == 0)
			{
				PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> has an entry with 'd'=0, which is invalid."), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
				bIsUsable = false;
				return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
			}

			// There is a conflict in the DASH standard in that the S@n element is an unsignedLong but both @startNumber and @endNumber are only unsignedInt.
			if (CurrentN > MAX_uint32 && !bWarnedAboutTimelineNumberOverflow)
			{
				bWarnedAboutTimelineNumberOverflow = true;
				LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> 'n' value exceeds unsignedInt (32 bits)."), *MPDRepresentation->GetID()));
			}

			// Warn if explicit numbering results in a gap or overlap. We do nothing besides warn about this.
			if (CurrentN != PreviousN+1)
			{
				if (!bWarnedAboutInconsistentNumbering)
				{
					bWarnedAboutInconsistentNumbering = true;
					LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> 'n' value %lld is not the expected %lld. This may cause playback issues"), *MPDRepresentation->GetID(), (long long int)CurrentN, (long long int)PreviousN+1));
				}
			}

			bIsCurrentlyAGap = false;
			// Check for timeline gaps or overlaps.
			int64 ExpectedT = PreviousT + PreviousD;
			if (CurrentT != ExpectedT)
			{
				// There could be an actual gap in the timeline due to a missing segment, which is probably the most common cause.
				// Another reason could be that a preceeding entry was using 'r'=-1 to repeat until the new 't' but the repeated 'd' value
				// does not result in hitting the new 't' value exactly.
				// It is also possible that the 't' value goes backwards a bit, overlapping with the preceeding segment.
				// In general it is also possible for there to be marginal rounding errors in the encoder pipeline somewhere, so small
				// enough discrepancies we will simply ignore.
				if (FTimeValue(Utils::AbsoluteValue(CurrentT - ExpectedT), MPDTimescale).GetAsMilliseconds() >= 20)
				{
					// An overlap (going backwards in time) we merely log a warning for. There is not a whole lot we can do about this.
					if (CurrentT < ExpectedT)
					{
						if (!bWarnedAboutTimelineOverlap)
						{
							bWarnedAboutTimelineOverlap = true;
							LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> 't' value %lld overlaps with preceeding segment (ends at %lld). This may cause playback issues"), *MPDRepresentation->GetID(), (long long int)CurrentT, (long long int)ExpectedT));
						}
					}
					else
					{
						// Since we do not support <FailoverContent> - and there is no guarantee that it even exists - and we also do not support
						// switching to a different representation - mostly because we have to assume the <SegmentTimeline> exists on
						// the AdaptationSet and therefor applies to all representations equally so there is no point - we need to get over this
						// gap by creating a filler data request.
						// To do this we adjust the current values to what is missing and take note for this iteration that it is missing.
						// Should we find the search time to fall into this missing range the request will be set up accordingly.
						CurrentD = CurrentT - ExpectedT;
						CurrentT = ExpectedT;
						--CurrentN;
						CurrentR = 0;
						bIsCurrentlyAGap = true;
						// We need to repeat this index!
						--nIndex;
					}
				}
			}

			if (CurrentR < 0)
			{
				// Limit the repeat count to where we are going to end.
				// This is either the next element that is required to have a 't', if it exists, or the end of the period.
				// In case the period has no end this is limited to the AvailabilityEndTime of the MPD.
				int64 EndTime = MediaLocalEndTime;
				if (nIndex+1 < Selements.Num())
				{
					if (!Selements[nIndex+1].HaveT)
					{
						if (!bWarnedAboutTimelineNoTAfterNegativeR)
						{
							bWarnedAboutTimelineNoTAfterNegativeR = true;
							LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> element following after a 'r'=-1 repeat count does not have a new 't' value!"), *MPDRepresentation->GetID()));
						}
					}
					else
					{
						EndTime = Selements[nIndex + 1].T;
					}
				}

				CurrentR = (EndTime - CurrentT + CurrentD - 1) / CurrentD - 1;

				if (EndTime == TNumericLimits<int64>::Max())
				{
					PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> repeats until infinity as last period is open-ended which is not currently supported."), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_INTERNAL);
					bIsUsable = false;
					return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
				}

				if (CurrentR < 0)
				{
					PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> repeat count of -1 failed to resolved to a positive value."), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_INTERNAL);
					bIsUsable = false;
					return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
				}
			}

			while(!bFound && CurrentR >= 0)
			{
				if (CurrentT >= MediaLocalSearchTime)
				{
					bFound = true;
					// If this segment consists of subsegments we fail. This is not currently supported.
					if (Selements[nIndex].HaveK)
					{
						PostError(InPlayerSessionServices, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> uses 'k' element which is not currently supported!"), *MPDRepresentation->GetID()), ERRCODE_DASH_MPD_BAD_REPRESENTATION);
						bIsUsable = false;
						return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
					}

					if (InSearchOptions.SearchType == IManifest::ESearchType::Closest)
					{
						if (CurrentN > StartNumber)
						{
							if (MediaLocalSearchTime - PreviousT < CurrentT - MediaLocalSearchTime)
							{
								--CurrentN;
								CurrentD = PreviousD;
								CurrentT = PreviousT;
							}
						}
						break;
					}
					else if (InSearchOptions.SearchType == IManifest::ESearchType::After || InSearchOptions.SearchType == IManifest::ESearchType::StrictlyAfter)
					{
						// The 'after' search is used to locate the next segment. For that reason the search time has been adjusted by the caller
						// to be larger than the start time of the preceeding segment.
						// Therefor, since this segment here has a larger or equal start time than the time we are searching for this segment here
						// must be the one 'after'.
						if (CurrentT >= MediaLocalEndTime)
						{
							return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
						}
						break;
					}
					else if (InSearchOptions.SearchType == IManifest::ESearchType::Same || InSearchOptions.SearchType == IManifest::ESearchType::Before)
					{
						// The 'before' search is used to locate the segment containing the search time, which could be
						// either this segment or the preceeding one.
						// The 'same' search is used exactly like 'before'. The segment is required that contains the search time.
						if (CurrentT > MediaLocalSearchTime && CurrentN > StartNumber)
						{
							// Not this segment, must be the preceeding one.
							--CurrentN;
							CurrentD = PreviousD;
							CurrentT = PreviousT;
						}
						break;
					}
					else if (InSearchOptions.SearchType == IManifest::ESearchType::StrictlyBefore)
					{
						// The 'strictlybefore' search is used to locate the segment just before the one the search time is in.
						// The caller is not expected to adjust the time to search for to do that since we are returning
						// the earlier segment if it exists. If not the same segment will be returned.
						if (CurrentN > StartNumber)
						{
							--CurrentN;
							CurrentD = PreviousD;
							CurrentT = PreviousT;
						}
						break;
					}
					else
					{
						PostError(InPlayerSessionServices, FString::Printf(TEXT("Unsupported segment search mode!")), ERRCODE_DASH_MPD_INTERNAL);
						bIsUsable = false;
						return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
					}
				}

				if (bFound)
				{
					break;
				}

				PreviousT = CurrentT;
				PreviousN = CurrentN;
				PreviousD = CurrentD;
				CurrentT += CurrentD;
				++CurrentN;
				--CurrentR;
			}

			// If the search time falls into the last segment we will not have found it above.
			if (!bFound && CurrentT >= MediaLocalSearchTime && nIndex+1 == Selements.Num())
			{
				if (InSearchOptions.SearchType == IManifest::ESearchType::Closest || InSearchOptions.SearchType == IManifest::ESearchType::Same || InSearchOptions.SearchType == IManifest::ESearchType::Before || InSearchOptions.SearchType == IManifest::ESearchType::StrictlyBefore)
				{
					--CurrentN;
					CurrentD = PreviousD;
					CurrentT = PreviousT;
					CurrentR = 0;
					bFound = true;
				}
			}

			if (bFound)
			{
				break;
			}
		}
	}

	// Did we find it?
	if (bFound && CurrentT < MediaLocalEndTime)
	{
		OutSegmentInfo.Time = CurrentT;
		OutSegmentInfo.PTO = PTO;
		OutSegmentInfo.Duration = CurrentD;
		OutSegmentInfo.Number = CurrentN;
		OutSegmentInfo.MediaLocalFirstAUTime = OutSegmentInfo.MediaLocalFirstPTS = MediaLocalSearchTime;
		OutSegmentInfo.MediaLocalLastAUTime = MediaLocalEndTime;
		OutSegmentInfo.Timescale = MPDTimescale;
		OutSegmentInfo.bMayBeMissing = CurrentT + CurrentD >= MediaLocalEndTime;
		OutSegmentInfo.bIsLastInPeriod = OutSegmentInfo.bMayBeMissing && InSearchOptions.bHasFollowingPeriod;
		OutSegmentInfo.bFrameAccuracyRequired = InSearchOptions.bFrameAccurateSearch;
		if (bIsCurrentlyAGap)
		{
			OutSegmentInfo.bMayBeMissing = true;
			OutSegmentInfo.bIsMissing = true;
			LogMessage(InPlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation \"%s\" <SegmentTimeline> gap encountered for needed 't' value of %lld. Replacing with an empty filler segment."), *MPDRepresentation->GetID(), (long long int)CurrentT));
		}
		OutSegmentInfo.ATO = ATO;

		// Look at the next n segment numbers for CMCD `nro` construction
		if (!bFirstContentMissing && !OutSegmentInfo.bIsLastInPeriod && InPlayerSessionServices->GetCMCDHandler()->IsEnabled())
		{
			bool bAllDone = false;
			while(!bAllDone)
			{
				while(--CurrentR >= 0)
				{
					CurrentT += CurrentD;
					++CurrentN;
					if (CurrentT >= MediaLocalEndTime)
					{
						bAllDone = true;
						break;
					}
					auto& nxt = OutSegmentInfo.NextMediaCMCDNumberTimes.Emplace_GetRef();
					nxt.Number = CurrentN;
					nxt.Time = CurrentT;
					if (OutSegmentInfo.NextMediaCMCDNumberTimes.Num() >= MAX_NEXT_SEGMENTS_FOR_CMCD)
					{
						bAllDone = true;
						break;
					}
				}
				if (bAllDone)
				{
					break;
				}

				if (++nIndex >= Selements.Num())
				{
					bAllDone = true;
					break;
				}
				CurrentD = Selements[nIndex].D;
				CurrentN = Selements[nIndex].HaveN ? Selements[nIndex].N - 1 : CurrentN;		// -1 since n is incremented right at the start of the loop above
				CurrentR = (Selements[nIndex].HaveR ? Selements[nIndex].R : 0) + 1;				// +1 due to the predecrement at the start of the loop
				CurrentT = Selements[nIndex].HaveT ? Selements[nIndex].T - CurrentD : CurrentT;	// subtract one d from the new time as it is added at the start of the loop
				if (CurrentD == 0)
				{
					break;
				}
			}
		}

		CollectInbandEventStreams(InPlayerSessionServices, OutSegmentInfo);
		SetupProducerReferenceTimeInfo(InPlayerSessionServices, OutSegmentInfo);
		if (!PrepareDownloadURLs(InPlayerSessionServices, OutSegmentInfo, SegmentTemplate))
		{
			bIsUsable = false;
			return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
		}
		else
		{
			return FManifestDASHInternal::FRepresentation::ESearchResult::Found;
		}
	}
	else
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
	}
}


FManifestDASHInternal::FRepresentation::ESearchResult FManifestDASHInternal::FRepresentation::SetupSideloadedFile(IPlayerSessionServices* InPlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation)
{
	// As per DASH-IF-IOP 4.3, Section 6.4.5. Guidelines for side-loaded TTML and WebVTT files
	// side loaded subtitles must not use @presentationTimeOffset, which makes sense since this attribute is not defined on <Representation> elements.
	// However, we allow its use as well as a @timescale to give the value in.
	uint32 Timescale = 1000;
	uint64 PTO = 0;
	const TArray<IDashMPDElement::FXmlAttribute>& OtherAttributes = MPDRepresentation->GetOtherAttributes();
	for(auto &Attribute : OtherAttributes)
	{
		if (Attribute.GetName().Equals(TEXT("presentationTimeOffset")))
		{
			LexFromString(PTO, *Attribute.GetValue());
		}
		else if (Attribute.GetName().Equals(TEXT("timescale")))
		{
			LexFromString(Timescale, *Attribute.GetValue());
		}
	}
	if (!Timescale)
	{
		Timescale = 1;
	}

	int64 MediaLocalSearchTime = InSearchOptions.PeriodLocalTime.GetAsTimebase(Timescale);
	if (MediaLocalSearchTime < 0)
	{
		MediaLocalSearchTime = 0;
	}
	int64 MediaLocalPeriodEnd = InSearchOptions.PeriodDuration.IsValid() && !InSearchOptions.PeriodDuration.IsInfinity() ? InSearchOptions.PeriodDuration.GetAsTimebase(Timescale) : TNumericLimits<int64>::Max();
	if (MediaLocalSearchTime >= MediaLocalPeriodEnd)
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::PastEOS;
	}

	// The time must be set to the PTO so it won't cause a problem searching for the next period (where the PTO is subtracted from the time)
	OutSegmentInfo.Time = (int64) PTO;
	OutSegmentInfo.PTO = (int64) PTO;
	OutSegmentInfo.Duration = MediaLocalPeriodEnd;
	OutSegmentInfo.Number = 0;
	OutSegmentInfo.NumberOfBytes = 0;
	OutSegmentInfo.FirstByteOffset = 0;
	OutSegmentInfo.MediaLocalFirstAUTime = OutSegmentInfo.MediaLocalFirstPTS = MediaLocalSearchTime;
	OutSegmentInfo.MediaLocalLastAUTime = MediaLocalPeriodEnd;
	OutSegmentInfo.Timescale = Timescale;
	OutSegmentInfo.ATO.SetToZero();
	OutSegmentInfo.bIsSideload = true;
	OutSegmentInfo.bIsLastInPeriod = true;
	TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>> SegmentBase;
	if (!PrepareDownloadURLs(InPlayerSessionServices, OutSegmentInfo, SegmentBase))
	{
		bIsUsable = false;
		return FManifestDASHInternal::FRepresentation::ESearchResult::BadType;
	}
	else
	{
		return FManifestDASHInternal::FRepresentation::ESearchResult::Found;
	}
}



void FManifestDASHInternal::FRepresentation::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const TSharedPtrTS<IPlaybackAssetAdaptationSet>& AdaptationSet)
{
	// This is the same as in FindSegment(), only with no error checking since this method here is not critical.
	TSharedPtrTS<FDashMPD_RepresentationType> MPDRepresentation = Representation.Pin();
	if (MPDRepresentation.IsValid())
	{
		TSharedPtrTS<FDashMPD_AdaptationSetType> MPDAdaptation = StaticCastSharedPtr<FDashMPD_AdaptationSetType>(MPDRepresentation->GetParentElement());
		if (MPDAdaptation.IsValid())
		{
			TSharedPtrTS<FDashMPD_PeriodType> MPDPeriod = StaticCastSharedPtr<FDashMPD_PeriodType>(MPDAdaptation->GetParentElement());
			if (MPDPeriod.IsValid())
			{
				TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>> SegmentBase({MPDRepresentation->GetSegmentBase(), MPDAdaptation->GetSegmentBase(), MPDPeriod->GetSegmentBase()});
				TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>> SegmentTemplate({MPDRepresentation->GetSegmentTemplate(), MPDAdaptation->GetSegmentTemplate(), MPDPeriod->GetSegmentTemplate()});
				SegmentBase.Remove(nullptr);
				SegmentTemplate.Remove(nullptr);
				if (SegmentBase.Num())
				{
					TSharedPtr<MP4Boxes::FMP4BoxSIDX, ESPMode::ThreadSafe> SI = SIDXBox;
					// If the segment index on this representation is not there we look for any segment index of another representation.
					// Since they need to be segmented the same we can at least use the segment durations from there.
					bool bExact = true;
					if (!SI.IsValid())
					{
						bExact = false;
						const FManifestDASHInternal::FAdaptationSet* ParentAdaptation = static_cast<const FManifestDASHInternal::FAdaptationSet*>(AdaptationSet.Get());
						for(int32 nR=0; nR<ParentAdaptation->GetNumberOfRepresentations(); ++nR)
						{
							FRepresentation* Rep = static_cast<FRepresentation*>(ParentAdaptation->GetRepresentationByIndex(nR).Get());
							if (Rep->SIDXBox.IsValid())
							{
								SI = Rep->SIDXBox;
								break;
							}
						}
					}
					if (SI.IsValid())
					{
						uint32 SidxTimescale = SI->GetTimescale();
						if (SidxTimescale)
						{
							int64 DurationSum = 0;
							for(int32 nI=0, nIMax=SI->GetEntries().Num(); nI<nIMax; ++nI)
							{
								DurationSum += SI->GetEntries()[nI].SubSegmentDuration;
							}
							OutAverageSegmentDuration.SetFromND(DurationSum / SI->GetEntries().Num(), SidxTimescale);
							return;
						}
					}
				}
				else if (SegmentTemplate.Num())
				{
					TSharedPtrTS<FDashMPD_SegmentTimelineType> SegmentTimeline = GET_ATTR(SegmentTemplate, GetSegmentTimeline(), IsValid(), TSharedPtrTS<FDashMPD_SegmentTimelineType>());
					if (SegmentTimeline.IsValid())
					{
						uint32 MPDTimescale = GET_ATTR(SegmentTemplate, GetTimescale(), IsSet(), TMediaOptionalValue<uint32>(1)).Value();
						auto Selements = SegmentTimeline->GetS_Elements();
						if (MPDTimescale)
						{
							int64 DurationSum = 0;
							int64 NumElements = 0;
							for(int32 nI=0, nIMax=Selements.Num(); nI<nIMax; ++nI)
							{
								// How many times does this entry repeat?
								int32 RCount = Selements[nI].R + 1;
								// If until the end of the period, for which we don't know how long that is exactly,
								// we use a repeat count guess of 10, since a single repeat might skew the average if it is
								// differing too much from the other duration values.
								if (RCount <= 0)
								{
									RCount = 10;
								}
								DurationSum += Selements[nI].D * RCount;
								NumElements += RCount;
							}
							OutAverageSegmentDuration.SetFromND(NumElements ? DurationSum / NumElements : 0, MPDTimescale);
							return;
						}
					}
					else
					{
						// Plain SegmentTemplate is trivial in that we have a fixed duration and that's it.
						TMediaOptionalValue<uint32> Duration = GET_ATTR(SegmentTemplate, GetDuration(), IsSet(), TMediaOptionalValue<uint32>());
						uint32 MPDTimescale = GET_ATTR(SegmentTemplate, GetTimescale(), IsSet(), TMediaOptionalValue<uint32>(1)).Value();
						if (Duration.IsSet() && Duration.Value() && MPDTimescale)
						{
							OutAverageSegmentDuration.SetFromND(Duration.Value(), MPDTimescale);
							return;
						}
					}
				}
			}
		}
	}
	// Nothing we could get from the actual representation.
	// Set some commonly used duration.
	OutAverageSegmentDuration.SetFromMilliseconds(4000);
}



void FManifestDASHInternal::FRepresentation::SegmentIndexDownloadComplete(TSharedPtrTS<FMPDLoadRequestDASH> LoadRequest, bool bSuccess)
{
	bool bOk = false;
	if (bSuccess)
	{
		check(LoadRequest.IsValid());
		if (GetStreamContainerType() == EStreamContainerType::ISO14496_12)
		{
			if (LoadRequest->OwningManifest.Pin().IsValid() && LoadRequest->XLinkElement == Representation)
			{
				TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe> bd = MP4Utilities::CreateBoxDataFromBuffer(MakeConstArrayView<uint8>(LoadRequest->Request->GetResponseBuffer()->GetLinearReadData(), LoadRequest->Request->GetResponseBuffer()->GetLinearReadSize()), 0);
				MP4Boxes::FMP4BoxTreeParser tp;
				if (bd && tp.ParseBoxTree(bd))
				{
					TSharedPtr<MP4Boxes::FMP4BoxSIDX, ESPMode::ThreadSafe> sidx = StaticCastSharedPtr<MP4Boxes::FMP4BoxSIDX>(tp.GetBoxTree());
					if (sidx->GetEntries().Num() > 0)
					{
						SIDXBox = MoveTemp(sidx);
						bOk = true;
						// Add this to the entity cache in case it needs to be retrieved again.
						IPlayerEntityCache::FCacheItem ci;
						ci.URL = LoadRequest->Url.URL;
						ci.Range = LoadRequest->Range;
						ci.ISOBMFFBox = SIDXBox;
						LoadRequest->PlayerSessionServices->GetEntityCache()->CacheEntity(ci);
  					}
					else
					{
						LogMessage(LoadRequest->PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation segment index is invalid. Marking representation as unusable.")));
					}
				}
				else
				{
					LogMessage(LoadRequest->PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation segment index parsing failed. Marking representation as unusable.")));
				}
			}
			else
			{
				LogMessage(LoadRequest->PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation segment index parsing failed. Marking representation as unusable.")));
			}
		}
		else if (GetStreamContainerType() == EStreamContainerType::Matroska)
		{
			if (LoadRequest->NumRemainingInChain)
			{
				TSharedPtrTS<FMPDLoadRequestDASH> NextChainedReq = MakeSharedTS<FMPDLoadRequestDASH>();
				NextChainedReq->LoadType = FMPDLoadRequestDASH::ELoadType::InitSegment;
				NextChainedReq->Url = LoadRequest->Url;
				NextChainedReq->Range = LoadRequest->Range2;
				NextChainedReq->Headers = LoadRequest->Headers;
				NextChainedReq->PlayerSessionServices = LoadRequest->PlayerSessionServices;
				NextChainedReq->XLinkElement = LoadRequest->XLinkElement;
				NextChainedReq->SegmentStreamType = LoadRequest->SegmentStreamType;
				NextChainedReq->SegmentQualityIndex = LoadRequest->SegmentQualityIndex;
				NextChainedReq->SegmentQualityIndexMax = LoadRequest->SegmentQualityIndexMax;
				NextChainedReq->CompleteCallback.BindThreadSafeSP(AsShared(), &FManifestDASHInternal::FRepresentation::SegmentIndexDownloadComplete);
				NextChainedReq->CompletedRequestChain.Emplace(MoveTemp(LoadRequest));
				TSharedPtrTS<IPlaylistReader> ManifestReader = NextChainedReq->PlayerSessionServices->GetManifestReader();
				if (ManifestReader.IsValid())
				{
					PendingSegmentIndexLoadRequest = NextChainedReq;
					TArray<TWeakPtrTS<FMPDLoadRequestDASH>> lrs {NextChainedReq};
					static_cast<IPlaylistReaderDASH*>(ManifestReader.Get())->AddElementLoadRequests(lrs);
					return;
				}
				else
				{
					LogMessage(LoadRequest->PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation segment index loading failed. Marking representation as unusable.")));
				}
			}
			else
			{
				TUniquePtr<DashUtils::FMatroskaParserDataReader> Loader = MakeUnique<DashUtils::FMatroskaParserDataReader>(LoadRequest);
				TSharedPtrTS<IParserMKV> MKVParser = IParserMKV::CreateParser(nullptr);
				FErrorDetail Error = MKVParser->ParseHeader(Loader.Get(), static_cast<Electra::IParserMKV::EParserFlags>(IParserMKV::EParserFlags::ParseFlag_OnlyEssentialLevel1 | IParserMKV::EParserFlags::ParseFlag_SuppressCueWarning));
				if (Error.IsOK())
				{
					Error = MKVParser->PrepareTracks();
					if (Error.IsOK())
					{
						// Add this to the entity cache in case it needs to be retrieved again.
						IPlayerEntityCache::FCacheItem ci;
						ci.URL = LoadRequest->Url.URL;
						ci.Range = LoadRequest->Range;
						ci.ParsedMatroskaData = MKVParser;
						LoadRequest->PlayerSessionServices->GetEntityCache()->CacheEntity(ci);
						// If there was a chained request add it, too.
						if (LoadRequest->CompletedRequestChain.Num())
						{
							ci.Range = LoadRequest->CompletedRequestChain[0]->Range;
							LoadRequest->PlayerSessionServices->GetEntityCache()->CacheEntity(ci);
						}
						SegmentMKV = MoveTemp(MKVParser);
						bOk = true;
					}
				}
				if (!bOk)
				{
					LogMessage(LoadRequest->PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation segment index parsing failed. Marking representation as unusable.")));
				}
			}
		}
	}
	else
	{
		LogMessage(LoadRequest->PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Representation segment index download failed. Marking representation as unusable.")));
	}
	bIsUsable = bOk;
	PendingSegmentIndexLoadRequest.Reset();
}

} // namespace Electra


