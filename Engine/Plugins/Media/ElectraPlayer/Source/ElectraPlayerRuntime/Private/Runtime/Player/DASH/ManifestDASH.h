// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Player/Manifest.h"

namespace Electra
{
class FManifestDASHInternal;
class FPlaylistReaderDASH;
class FDASHTimeline;


class FManifestDASH : public IManifest
{
public:
	static TSharedPtrTS<FManifestDASH> Create(IPlayerSessionServices* SessionServices, TSharedPtrTS<FManifestDASHInternal> Manifest);
	void UpdateInternalManifest(TSharedPtrTS<FManifestDASHInternal> UpdatedManifest);

	virtual ~FManifestDASH();
	EMediaFormatType GetMediaFormatType() const override;
	bool ReloadsPlaylistPeriodically() const override;
	bool SupportsMultipleTracksPerKind() const override;
	bool OrderOfTracksCanChangeDynamically() const override;
	EType GetPresentationType() const override;
	EReplayEventType GetReplayType() const override;
	TSharedPtrTS<const FLowLatencyDescriptor> GetLowLatencyDescriptor() const override;
	FTimeValue CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const override;
	FTimeValue GetAnchorTime() const override;
	FTimeRange GetTotalTimeRange() const override;
	FTimeRange GetSeekableTimeRange() const override;
	FTimeRange GetPlaybackRange(EPlaybackRangeType InRangeType) const override;
	FTimeValue GetDuration() const override;
	FTimeValue GetDefaultStartTime() const override;
	void ClearDefaultStartTime() override;
	FTimeValue GetDefaultEndTime() const override;
	void ClearDefaultEndTime() override;
	FTimeValue GetMinBufferTime() const override;
	FTimeValue GetDesiredLiveLatency() const override;
	ELiveEdgePlayMode GetLiveEdgePlayMode() const override;
	TRangeSet<double> GetPossiblePlaybackRates(EPlayRateType InForType) const override;
	TSharedPtrTS<IProducerReferenceTimeInfo> GetProducerReferenceTimeInfo(int64 ID) const override;
	void GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const override;
	void UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData) override;
	void UpdateDynamicRefetchCounter() override;
	void PrepareForLooping(int32 InNumLoopsToAdd) override;
	void TriggerClockSync(IManifest::EClockSyncType InClockSyncType) override;
	void TriggerPlaylistRefresh() override;
	void ReachedStableBuffer() override;
	IStreamReader* CreateStreamReaderHandler() override;

	FResult FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
	FResult FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment) override;

private:
	ELECTRA_IMPL_DEFAULT_ERROR_METHODS(DASHManifest);

	FManifestDASH(IPlayerSessionServices* SessionServices, TSharedPtrTS<FManifestDASHInternal> Manifest);

	IPlayerSessionServices* 					PlayerSessionServices = nullptr;
	TSharedPtrTS<FManifestDASHInternal>			CurrentManifest;
	int64										CurrentPeriodAndAdaptationXLinkResolveID = 1;
};



} // namespace Electra


