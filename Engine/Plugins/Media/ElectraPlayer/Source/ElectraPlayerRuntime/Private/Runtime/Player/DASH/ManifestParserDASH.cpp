// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "ElectraPlayerPrivate.h"
#include "Player/Manifest.h"
#include "Player/PlaybackTimeline.h"

#include "Player/PlayerSessionServices.h"
#include "Player/PlayerStreamFilter.h"

#include "ManifestParserDASH.h"

#include "Stats/Stats.h"
#include "Utilities/StringHelpers.h"
#include "SynchronizedClock.h"
#include "StreamTypes.h"
#include "ErrorDetail.h"

#include "Utilities/URLParser.h"
#include "Utilities/Utilities.h"
#include "Utilities/TimeUtilities.h"

#include "FastXml.h"


#define ERRCODE_DASH_MPD_PARSER_XML_PARSER_FAILED					1
#define ERRCODE_DASH_MPD_PARSER_XML_BAD_ENTITY						2
#define ERRCODE_DASH_MPD_PARSER_BAD_ROOT_ELEMENT					100
#define ERRCODE_DASH_MPD_PARSER_ELEMENT_CANNOT_HAVE_CHILDREN		101
#define ERRCODE_DASH_MPD_PARSER_BAD_TIME_VALUE						200
#define ERRCODE_DASH_MPD_PARSER_BAD_ELEMENT_VALUE					201

DECLARE_CYCLE_STAT(TEXT("FManifestParserDASH::BuildFromMPD"), STAT_ElectraPlayer_DASH_BuildFromMPD, STATGROUP_ElectraPlayer);


namespace Electra
{

namespace Tags
{

//-----------------------------------------------------------------------------
// Elements
#define DEF_ELEMENT(name) \
const TCHAR* const Element_##name = TEXT(#name)
#define ELEMENT(name) Tags::Element_##name
DEF_ELEMENT(MPD);
DEF_ELEMENT(EssentialProperty);
DEF_ELEMENT(SupplementalProperty);
DEF_ELEMENT(UTCTiming);
DEF_ELEMENT(ProgramInformation);
DEF_ELEMENT(Title);
DEF_ELEMENT(Source);
DEF_ELEMENT(Copyright);
DEF_ELEMENT(BaseURL);
DEF_ELEMENT(Location);
DEF_ELEMENT(Period);
DEF_ELEMENT(SegmentBase);
DEF_ELEMENT(SegmentList);
DEF_ELEMENT(SegmentTemplate);
DEF_ELEMENT(AssetIdentifier);
DEF_ELEMENT(EventStream);
DEF_ELEMENT(Event);
DEF_ELEMENT(AdaptationSet);
DEF_ELEMENT(Subset);
DEF_ELEMENT(Accessibility);
DEF_ELEMENT(Role);
DEF_ELEMENT(Rating);
DEF_ELEMENT(Viewpoint);
DEF_ELEMENT(ContentComponent);
DEF_ELEMENT(Representation);
DEF_ELEMENT(FramePacking);
DEF_ELEMENT(AudioChannelConfiguration);
DEF_ELEMENT(ContentProtection);
DEF_ELEMENT(InbandEventStream);
DEF_ELEMENT(SubRepresentation);
DEF_ELEMENT(Initialization);
DEF_ELEMENT(RepresentationIndex);
DEF_ELEMENT(SegmentTimeline);
DEF_ELEMENT(BitstreamSwitching);
DEF_ELEMENT(SegmentURL);
DEF_ELEMENT(S);
DEF_ELEMENT(FailoverContent);
DEF_ELEMENT(FCS);
DEF_ELEMENT(Switching);
DEF_ELEMENT(RandomAccess);
DEF_ELEMENT(GroupLabel);
DEF_ELEMENT(Label);
DEF_ELEMENT(EmptyAdaptationSet);
DEF_ELEMENT(ProducerReferenceTime);
DEF_ELEMENT(ContentPopularityRate);
DEF_ELEMENT(PR);
DEF_ELEMENT(InitializationSet);
DEF_ELEMENT(Preselection);
DEF_ELEMENT(InitializationGroup);
DEF_ELEMENT(InitializationPresentation);
DEF_ELEMENT(LeapSecondInformation);
DEF_ELEMENT(QualityLatency);
DEF_ELEMENT(Scope);
DEF_ELEMENT(Latency);
DEF_ELEMENT(PlaybackRate);
DEF_ELEMENT(OperatingQuality);
DEF_ELEMENT(OperatingBandwidth);
DEF_ELEMENT(ServiceDescription);
DEF_ELEMENT(Resync);
DEF_ELEMENT(ContentSteering);
DEF_ELEMENT(ClientDataReporting);
DEF_ELEMENT(CMCDParameters);

const TCHAR* const Element_up_UrlQueryInfo = TEXT("up:UrlQueryInfo");
const TCHAR* const Element_up_ExtUrlQueryInfo = TEXT("up:ExtUrlQueryInfo");
const TCHAR* const Element_up_ExtHttpHeaderInfo = TEXT("up:ExtHttpHeaderInfo");

//-----------------------------------------------------------------------------
// Attributes
#define DEF_ATTR(name) \
const TCHAR* const Attr_##name = TEXT(#name)

#define ATTR(name) Tags::Attr_##name

DEF_ATTR(id);
DEF_ATTR(profiles);
DEF_ATTR(type);
DEF_ATTR(availabilityStartTime);
DEF_ATTR(availabilityEndTime);
DEF_ATTR(publishTime);
DEF_ATTR(mediaPresentationDuration);
DEF_ATTR(minimumUpdatePeriod);
DEF_ATTR(minBufferTime);
DEF_ATTR(timeShiftBufferDepth);
DEF_ATTR(suggestedPresentationDelay);
DEF_ATTR(maxSegmentDuration);
DEF_ATTR(maxSubsegmentDuration);
DEF_ATTR(lang);
DEF_ATTR(moreInformationURL);
DEF_ATTR(serviceLocation);
DEF_ATTR(byteRange);
DEF_ATTR(availabilityTimeOffset);
DEF_ATTR(availabilityTimeComplete);
DEF_ATTR(start);
DEF_ATTR(duration);
DEF_ATTR(bitstreamSwitching);
DEF_ATTR(schemeIdUri);
DEF_ATTR(value);
DEF_ATTR(group);
DEF_ATTR(contentType);
DEF_ATTR(par);
DEF_ATTR(minBandwidth);
DEF_ATTR(maxBandwidth);
DEF_ATTR(minWidth);
DEF_ATTR(maxWidth);
DEF_ATTR(minHeight);
DEF_ATTR(maxHeight);
DEF_ATTR(minFrameRate);
DEF_ATTR(maxFrameRate);
DEF_ATTR(segmentAlignment);
DEF_ATTR(subsegmentAlignment);
DEF_ATTR(subsegmentStartsWithSAP);
DEF_ATTR(width);
DEF_ATTR(height);
DEF_ATTR(sar);
DEF_ATTR(frameRate);
DEF_ATTR(audioSamplingRate);
DEF_ATTR(mimeType);
DEF_ATTR(segmentProfiles);
DEF_ATTR(codecs);
DEF_ATTR(maximumSAPPeriod);
DEF_ATTR(startWithSAP);
DEF_ATTR(maxPlayoutRate);
DEF_ATTR(codingDependency);
DEF_ATTR(scanType);
DEF_ATTR(timescale);
DEF_ATTR(bandwidth);
DEF_ATTR(qualityRanking);
DEF_ATTR(dependencyId);
DEF_ATTR(mediaStreamStructureId);
DEF_ATTR(sourceURL);
DEF_ATTR(range);
DEF_ATTR(presentationTimeOffset);
DEF_ATTR(indexRange);
DEF_ATTR(indexRangeExact);
DEF_ATTR(startNumber);
DEF_ATTR(media);
DEF_ATTR(mediaRange);
DEF_ATTR(index);
DEF_ATTR(initialization);
DEF_ATTR(t);
DEF_ATTR(n);
DEF_ATTR(d);
DEF_ATTR(r);
DEF_ATTR(k);
DEF_ATTR(level);
DEF_ATTR(dependencyLevel);
DEF_ATTR(contentComponent);
DEF_ATTR(presentationTime);
DEF_ATTR(contentEncoding);
DEF_ATTR(messageData);
DEF_ATTR(contains);
DEF_ATTR(tag);
DEF_ATTR(presentationDuration);
DEF_ATTR(eptDelta);
DEF_ATTR(endNumber);
DEF_ATTR(selectionPriority);
DEF_ATTR(associationId);
DEF_ATTR(associationType);
DEF_ATTR(initializationSetRef);
DEF_ATTR(valid);
DEF_ATTR(interval);
DEF_ATTR(inband);
DEF_ATTR(applicationScheme);
DEF_ATTR(wallClockTime);
DEF_ATTR(popularityRate);
DEF_ATTR(source);
DEF_ATTR(source_description);
DEF_ATTR(inAllPeriods);
DEF_ATTR(preselectionComponents);
DEF_ATTR(order);
DEF_ATTR(availabilityStartLeapOffset);
DEF_ATTR(nextAvailabilityStartLeapOffset);
DEF_ATTR(nextLeapChangeTime);
DEF_ATTR(min);
DEF_ATTR(max);
DEF_ATTR(target);
DEF_ATTR(mediaType);
DEF_ATTR(maxDifference);
DEF_ATTR(referenceId);
DEF_ATTR(queryTemplate);
DEF_ATTR(useMPDUrlQuery);
DEF_ATTR(queryString);
DEF_ATTR(includeInRequests);
DEF_ATTR(headerParamSource);
DEF_ATTR(sameOriginOnly);
DEF_ATTR(dT);
DEF_ATTR(dImin);
DEF_ATTR(dImax);
DEF_ATTR(marker);
DEF_ATTR(rangeAccess);
DEF_ATTR(defaultServiceLocation);
DEF_ATTR(queryBeforeStart);
DEF_ATTR(clientRequirement);
DEF_ATTR(proxyServerURL);
DEF_ATTR(serviceLocations);
DEF_ATTR(adaptationSets);
DEF_ATTR(version);
DEF_ATTR(mode);
DEF_ATTR(keys);
DEF_ATTR(contentID);
DEF_ATTR(sessionID);


// DVB DASH attributes have a colon in them.
const TCHAR* const Attr_dvb_priority = TEXT("dvb:priority");
const TCHAR* const Attr_dvb_weight = TEXT("dvb:weight");



// xlink attributes. Due to the xlink namespace they have a colon in them
const TCHAR* const Attr_xlink_href = TEXT("xlink:href");
const TCHAR* const Attr_xlink_actuate = TEXT("xlink:actuate");

// xml namespace
const TCHAR* const Attr_xmlns = TEXT("xmlns");
const TCHAR* const Attr_xmlns_colon = TEXT("xmlns:");

#define AttrEquals(n) StringHelpers::StringEquals(ATTR(n), AttributeName)

//-----------------------------------------------------------------------------
// Constants
#define DEF_CONSTVAL(name) \
const TCHAR* const Const_##name = TEXT(#name)
#define CONSTVAL(name) Tags::Const_##name

DEF_CONSTVAL(true);
DEF_CONSTVAL(false);

const TCHAR* const Scheme_urn_mpeg_dash_urlparam_2014 = TEXT("urn:mpeg:dash:urlparam:2014");
const TCHAR* const Scheme_urn_mpeg_dash_urlparam_2016 = TEXT("urn:mpeg:dash:urlparam:2016");
}



class FManifestParserDASH;

/*********************************************************************************************************************/
namespace
{
	static bool RemoveXMLEntities(FManifestParserDASH* Builder, FString& InOutString, int32 XmlLineNumber);
	static bool ParseStringAttribute(FString& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseBooleanAttribute(bool& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseBooleanAttribute(TMediaOptionalValue<bool>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseConditionalUIntAttribute(IDashMPDElement::FBoolUInt64& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseIntAttribute(int64& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseIntAttribute(TMediaOptionalValue<int64>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseIntAttribute(TMediaOptionalValue<int32>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseIntAttribute(int32& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseUIntAttribute(uint64& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseUIntAttribute(TMediaOptionalValue<uint64>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseUIntAttribute(TMediaOptionalValue<uint32>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseDoubleAttribute(FTimeFraction& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseDoubleAttribute(FTimeValue& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseDoubleAttribute(double& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseDoubleAttribute(TMediaOptionalValue<double>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseDuration(FTimeValue& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseDateTime(FTimeValue& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseRatioAttribute(FTimeFraction& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseFramerateAttribute(FTimeFraction& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseStringArray(TArray<FString>& OutArray, const TCHAR* Delimiter, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseUIntArray(TArray<uint64>& OutArray, const TCHAR* Delimiter, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
	static bool ParseUIntArray(TArray<uint32>& OutArray, const TCHAR* Delimiter, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber);
}

/*********************************************************************************************************************/

/**
 * An <S> element within the SegmentTimelineType
 */
class FTimeline_S_ElementBuilder : public IDashMPDElement
{
public:
	FTimeline_S_ElementBuilder(const TCHAR* Name, const TCHAR* Data) : IDashMPDElement(Name, Data), CurrentElement(nullptr)
	{ }
	virtual ~FTimeline_S_ElementBuilder() = default;
	virtual EType GetElementType() const override
	{ return EType::Other; }

	void SetCurrentElement(FDashMPD_SegmentTimelineType::FSElement* InCurrentElement)
	{
		CurrentElement = InCurrentElement;
	}
protected:
	virtual bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	virtual bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
private:
	FDashMPD_SegmentTimelineType::FSElement* CurrentElement;
};

/*********************************************************************************************************************/

/**
 * An <FCS> element within the FailoverContentType
 */
class FFailOver_FCS_ElementBuilder : public IDashMPDElement
{
public:
	FFailOver_FCS_ElementBuilder(const TCHAR* Name, const TCHAR* Data) : IDashMPDElement(Name, Data), CurrentElement(nullptr)
	{ }
	virtual ~FFailOver_FCS_ElementBuilder() = default;
	virtual EType GetElementType() const override
	{ return EType::Other; }

	void SetCurrentElement(FDashMPD_FailoverContentType::FFCSElement* InCurrentElement)
	{
		CurrentElement = InCurrentElement;
	}
protected:
	virtual bool ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	virtual bool ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
private:
	FDashMPD_FailoverContentType::FFCSElement* CurrentElement;
};

/*********************************************************************************************************************/

class FManifestParserDASH : public IFastXmlCallback
{
public:
	FManifestParserDASH(IPlayerSessionServices* InPlayerSessionServices);

	virtual ~FManifestParserDASH();

	FErrorDetail BuildFromMPD(TCHAR* InOutMPDXML, const TCHAR* InExpectedRootElement);

	FDashMPD_RootEntities GetRootEntities()
	{
		return MoveTemp(RootEntities);
	}

	TArray<TWeakPtrTS<IDashMPDElement>> GetXLinkElements()
	{
		return MoveTemp(XLinkElements);
	}

	void AddXLinkElement(const TSharedPtrTS<IDashMPDElement>& Element)
	{
		XLinkElements.Emplace(Element);
	}

	bool PushNewElement(const TSharedPtrTS<IDashMPDElement>& NewElement, int32 InXmlLineNumber)
	{
		NewElement->SetXMLLineNumber(InXmlLineNumber);
		if (ElementStack.Num())
		{
			NewElement->SetParentElement(ElementStack.Last());
		}
		ElementStack.Emplace(NewElement);
		return true;
	}
	void PopLastElement(const TCHAR* ElementName)
	{
		check(ElementStack.Num());
		check(StringHelpers::StringEquals(ElementName, *ElementStack.Last()->GetName()));
		ElementStack.Pop(EAllowShrinking::No);
	}

	const TSharedPtrTS<FTimeline_S_ElementBuilder>& GetSBuilder()
	{
		return S_Builder;
	}

	const TSharedPtrTS<FFailOver_FCS_ElementBuilder>& GetFCSBuilder()
	{
		return FCS_Builder;
	}

	FString GetCurrentElementName() const
	{
		return ElementStack.Num() ? ElementStack.Last()->GetName() : FString();
	}

	bool SetLastError(const FString& InMessage, uint16 InCode, UEMediaError InError=UEMEDIA_ERROR_DETAIL)
	{
		LastErrorDetail = CreateErrorAndLog(InMessage, InCode, InError);
		return false;
	}

	FErrorDetail CreateErrorAndLog(const FString& InMessage, uint16 InCode, UEMediaError InError=UEMEDIA_ERROR_DETAIL);
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

private:
	virtual bool ProcessXmlDeclaration(const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	virtual bool ProcessElement(const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber) override;
	virtual bool ProcessAttribute(const TCHAR* AttributeName, const TCHAR* AttributeValue) override;
	virtual bool ProcessClose(const TCHAR* ElementName) override;
	virtual bool ProcessComment(const TCHAR* Comment) override;

	void PostError(const FString& InMessage, uint16 InCode, UEMediaError InError=UEMEDIA_ERROR_DETAIL);

	IPlayerSessionServices* PlayerSessionServices;

	FDashMPD_RootEntities RootEntities;
	const TCHAR* ExpectedRootElement = nullptr;

	TArray<TSharedPtrTS<IDashMPDElement>> ElementStack;
	FErrorDetail LastErrorDetail;
	TSharedPtrTS<FTimeline_S_ElementBuilder> S_Builder;
	TSharedPtrTS<FFailOver_FCS_ElementBuilder> FCS_Builder;

	TArray<TWeakPtrTS<IDashMPDElement>> XLinkElements;
};

/*********************************************************************************************************************/

namespace IManifestParserDASH
{

FErrorDetail BuildFromMPD(FDashMPD_RootEntities& OutRootEntities, TArray<TWeakPtrTS<IDashMPDElement>>& OutXLinkElements, TCHAR* InOutMPDXML, const TCHAR* InExpectedRootElement, IPlayerSessionServices* InPlayerSessionServices)
{
	TUniquePtr<FManifestParserDASH> Parser = MakeUnique<FManifestParserDASH>(InPlayerSessionServices);
	FErrorDetail Error = Parser->BuildFromMPD(InOutMPDXML, InExpectedRootElement);
	if (Error.IsOK())
	{
		OutRootEntities = Parser->GetRootEntities();
		OutXLinkElements = Parser->GetXLinkElements();
	}
	return Error;
}

}

/*********************************************************************************************************************/


FManifestParserDASH::FManifestParserDASH(IPlayerSessionServices* InPlayerSessionServices)
	: PlayerSessionServices(InPlayerSessionServices)
{
	// Create an 'S' element builder we will use over and over again.
	S_Builder = MakeSharedTS<FTimeline_S_ElementBuilder>(TEXT("S"), nullptr);
	// Create an 'FCS' element builder we will use over and over again.
	FCS_Builder = MakeSharedTS<FFailOver_FCS_ElementBuilder>(TEXT("FCS"), nullptr);
}


FManifestParserDASH::~FManifestParserDASH()
{
}


void FManifestParserDASH::PostError(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	LastErrorDetail.Clear();
	LastErrorDetail.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	LastErrorDetail.SetFacility(Facility::EFacility::DASHMPDParser);
	LastErrorDetail.SetCode(InCode);
	LastErrorDetail.SetMessage(InMessage);
	check(PlayerSessionServices);
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostError(LastErrorDetail);
	}
}


FErrorDetail FManifestParserDASH::CreateErrorAndLog(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	FErrorDetail err;
	err.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	err.SetFacility(Facility::EFacility::DASHMPDParser);
	err.SetCode(InCode);
	err.SetMessage(InMessage);
	check(PlayerSessionServices);
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::DASHMPDParser, IInfoLog::ELevel::Error, err.GetPrintable());
	}
	return err;
}


void FManifestParserDASH::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::DASHMPDParser, Level, Message);
	}
}


FErrorDetail FManifestParserDASH::BuildFromMPD(TCHAR* InOutMPDXML, const TCHAR* InExpectedRootElement)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_DASH_BuildFromMPD);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, DASH_BuildFromMPD);

	FText ErrorMsg;
	int32 ErrorLine = 0;
	const TCHAR* NoFile = TEXT("");
	ExpectedRootElement = InExpectedRootElement;
	if (InOutMPDXML)
	{
		if (!FFastXml::ParseXmlFile(this, NoFile, InOutMPDXML, nullptr, false, false, ErrorMsg, ErrorLine))
		{
			// Check if we have already set an error or if this is an error generated by the XML parser.
			if (!LastErrorDetail.IsSet())
			{
				LastErrorDetail = CreateErrorAndLog(FString::Printf(TEXT("XML parser failed with '%s' on line %d"), *(ErrorMsg.ToString()), ErrorLine), ERRCODE_DASH_MPD_PARSER_XML_PARSER_FAILED);
			}
		}
	}
	else
	{
		LastErrorDetail = CreateErrorAndLog(FString::Printf(TEXT("No XML was given to parse")), ERRCODE_DASH_MPD_PARSER_XML_PARSER_FAILED);
	}
	return LastErrorDetail;
}




/**
 * Called after the XML's header is parsed.  This is usually the first call that you'll get back.
 *
 * @param	ElementData			Optional data for this element, nullptr if none
 * @param	XmlFileLineNumber	Line number in the XML file we're on
 *
 * @return	You should return true to continue processing the file, or false to stop processing immediately.
 */
bool FManifestParserDASH::ProcessXmlDeclaration(const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	// Not needed, don't care, just continue.
	return true;
}

/**
 * Called when a new XML element is encountered, starting a new scope.  You'll receive a call to ProcessClose()
 * when this element's scope has ended.
 *
 * @param	ElementName			The name of the element
 * @param	ElementData			Optional data for this element, nullptr if none
 * @param	XmlFileLineNumber	The line number in the XML file we're on
 *
 * @return	You should return true to continue processing the file, or false to stop processing immediately.
 */
bool FManifestParserDASH::ProcessElement(const TCHAR* ElementName, const TCHAR* InElementData, int32 XmlFileLineNumber)
{
	FString ElementData(InElementData);
	if (!RemoveXMLEntities(this, ElementData, XmlFileLineNumber))
	{
		return false;
	}

	/*
		If the MPD has line breaks in an element for better human readability this will cause parsing issues because
		FastXML does not strip them. We can also NOT strip them here ourselves because FastXML will still fail internally
		because it will not find the matching element start tag when it gets to the end tag.
	*/

	// Are we awaiting a root element?
	if (ElementStack.Num() == 0)
	{
		if (ExpectedRootElement && !StringHelpers::StringEquals(ElementName, ExpectedRootElement))
		{
			LastErrorDetail = CreateErrorAndLog(FString::Printf(TEXT("Expected root element <%s> but got <%s>"), ExpectedRootElement, ElementName), ERRCODE_DASH_MPD_PARSER_BAD_ROOT_ELEMENT);
			return false;
		}
		if (StringHelpers::StringEquals(ElementName, ELEMENT(MPD)))
		{
			RootEntities.MPDs.Emplace(MakeSharedTS<FDashMPD_MPDType>(ElementName, *ElementData));
			PushNewElement(RootEntities.MPDs.Last(), XmlFileLineNumber);
		}
		else if (StringHelpers::StringEquals(ElementName, ELEMENT(Period)))
		{
			RootEntities.Periods.Emplace(MakeSharedTS<FDashMPD_PeriodType>(ElementName, *ElementData));
			PushNewElement(RootEntities.Periods.Last(), XmlFileLineNumber);
		}
		else if (StringHelpers::StringEquals(ElementName, ELEMENT(AdaptationSet)))
		{
			RootEntities.AdaptationSets.Emplace(MakeSharedTS<FDashMPD_AdaptationSetType>(ElementName, *ElementData));
			PushNewElement(RootEntities.AdaptationSets.Last(), XmlFileLineNumber);
		}
		else if (StringHelpers::StringEquals(ElementName, ELEMENT(up_UrlQueryInfo)) || StringHelpers::StringEquals(ElementName, ELEMENT(up_ExtUrlQueryInfo)) || StringHelpers::StringEquals(ElementName, ELEMENT(up_ExtHttpHeaderInfo)))
		{
			RootEntities.UrlQueries.Emplace(MakeSharedTS<FDashMPD_UrlQueryInfoType>(ElementName, *ElementData));
			PushNewElement(RootEntities.UrlQueries.Last(), XmlFileLineNumber);
		}
		else if (StringHelpers::StringEquals(ElementName, ELEMENT(InitializationSet)))
		{
			RootEntities.InitializationSets.Emplace(MakeSharedTS<FDashMPD_InitializationSetType>(ElementName, *ElementData));
			PushNewElement(RootEntities.InitializationSets.Last(), XmlFileLineNumber);
		}
		else if (StringHelpers::StringEquals(ElementName, ELEMENT(EventStream)))
		{
			RootEntities.EventStreams.Emplace(MakeSharedTS<FDashMPD_EventStreamType>(ElementName, *ElementData));
			PushNewElement(RootEntities.EventStreams.Last(), XmlFileLineNumber);
		}
		else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentList)))
		{
			RootEntities.SegmentLists.Emplace(MakeSharedTS<FDashMPD_SegmentListType>(ElementName, *ElementData));
			PushNewElement(RootEntities.SegmentLists.Last(), XmlFileLineNumber);
		}
		else
		{
			LastErrorDetail = CreateErrorAndLog(FString::Printf(TEXT("Unexpected root element <%s>"), ElementName), ERRCODE_DASH_MPD_PARSER_BAD_ROOT_ELEMENT);
			return false;
		}
	}
	else
	{
		// Defer creating of a new child element to the currently active element.
		check(ElementStack.Num());
		check(ElementStack.Last().IsValid());
		return ElementStack.Last()->ProcessElement(this, ElementName, *ElementData, XmlFileLineNumber);
	}
	return true;
}

/**
 * Called when an XML attribute is encountered for the current scope's element.
 *
 * @param	AttributeName	The name of the attribute
 * @param	AttributeValue	The value of the attribute
 *
 * @return	You should return true to continue processing the file, or false to stop processing immediately.
 */
bool FManifestParserDASH::ProcessAttribute(const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (ElementStack.Num())
	{
		check(ElementStack.Last().IsValid());

		// XML namespace?
		if (StringHelpers::StringStartsWith(AttributeName, ATTR(xmlns), 5))
		{
			FErrorDetail err = ElementStack.Last()->AddNamespace(AttributeName, AttributeValue);
			if (err.IsSet())
			{
				LastErrorDetail = err;
				return false;
			}
			return true;
		}

		return ElementStack.Last()->ProcessAttribute(this, AttributeName, AttributeValue);
	}
	else
	{
		LastErrorDetail = CreateErrorAndLog(FString::Printf(TEXT("Cannot process attribute <%s>=\"%s\" with no existing root element!"), AttributeName, AttributeValue), ERRCODE_DASH_MPD_PARSER_BAD_ROOT_ELEMENT);
		return false;
	}
}

/**
 * Called when an element's scope ends in the XML file
 *
 * @param	ElementName		Name of the element whose scope closed
 *
 * @return	You should return true to continue processing the file, or false to stop processing immediately.
 */
bool FManifestParserDASH::ProcessClose(const TCHAR* ElementName)
{
	if (ElementStack.Num() == 0)
	{
		LastErrorDetail = CreateErrorAndLog(FString::Printf(TEXT("Internal parser stack got mangled processing element <%s>"), ElementName), ERRCODE_DASH_MPD_PARSER_XML_PARSER_FAILED);
		return false;
	}
	else
	{
		// Defer creating of a new child element to the currently active element.
		// Give the current element first crack at ending its processing.
		bool bContinue = ElementStack.Last()->ProcessClose(this, ElementName);
		// Pop the current element.
		PopLastElement(ElementName);
		return bContinue;
	}
}

/**
 * Called when a comment is encountered.  This can happen pretty much anywhere in the file.
 *
 * @param	Comment		The comment text
 */
bool FManifestParserDASH::ProcessComment(const TCHAR* Comment)
{
	// Comments are fine. We do not care about them and just continue parsing.
	return true;
}


/*********************************************************************************************************************/

FString IDashMPDElement::GetDocumentURL() const
{
	// The MPD has an overloaded GetDocumentURL method so we cannot possibly get here when used properly.
	check(GetElementType() != EType::MPD);
	TSharedPtrTS<IDashMPDElement> Parent = GetParentElement();
	return Parent.IsValid() ? Parent->GetDocumentURL() : FString();
}

const TArray<TSharedPtrTS<FDashMPD_BaseURLType>>& IDashMPDElement::GetBaseURLs() const
{
	static TArray<TSharedPtrTS<FDashMPD_BaseURLType>> NoBaseURLs;
	return NoBaseURLs;
}
const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& IDashMPDElement::GetEssentialProperties() const
{
	static TArray<TSharedPtrTS<FDashMPD_DescriptorType>> NoEssentialProperties;
	return NoEssentialProperties;
}
const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& IDashMPDElement::GetSupplementalProperties() const
{
	static TArray<TSharedPtrTS<FDashMPD_DescriptorType>> NoSupplementalProperties;
	return NoSupplementalProperties;
}
IDashMPDElement::FXLink& IDashMPDElement::GetXLink()
{
	static FXLink NoXlink;
	return NoXlink;
}
int32 IDashMPDElement::ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID)
{
	check(Element.IsValid() && Element->GetParentElement().IsValid() && Element->GetParentElement().Get() == this);
	return -1;
}



bool IDashMPDElement::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	bool bEmitWarning = true;
	// Since descriptors are largely application specific we cannot know if their elements are meaningful or not, so we don't show a warning.
	// Similarly, events may carry XML elements as their payload and we will sadly parse them, too.
	if (GetElementType() == EType::Descriptor || GetElementType() == EType::Event || GetElementType() == EType::Other)
	{
		bEmitWarning = false;
	}
	if (bEmitWarning)
	{
		Builder->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Element <%s> of <%s> in line %d will be ignored"), ElementName, *Builder->GetCurrentElementName(), XmlFileLineNumber));
	}
	OtherChildren.Emplace(MakeSharedTS<FDashMPD_OtherType>(ElementName, ElementData));
	return Builder->PushNewElement(OtherChildren.Last(), XmlFileLineNumber);
}

bool IDashMPDElement::NoChildElementAllowed(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return Builder->SetLastError(FString::Printf(TEXT("<%s> element cannot have children like <%s> in line %d"), *GetName(), ElementName, GetXMLLineNumber()), ERRCODE_DASH_MPD_PARSER_ELEMENT_CANNOT_HAVE_CHILDREN);
}

bool IDashMPDElement::ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName)
{
	return true;
}

bool IDashMPDElement::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	bool bEmitWarning = true;
	// Is this an attribute prefixed with a namespace?
	FString an(AttributeName);
	int32 ColonPos = an.Find(TEXT(":"));
	if (ColonPos != INDEX_NONE)
	{
		FString NS = an.Mid(0, ColonPos);
		// Special shortcut if the namespace happens to be "xsi". While it could be named anything really, it is quite common for
		// the xml schema instance to be called xsi.
		if (NS.Equals(TEXT("xsi")))
		{
			bEmitWarning = false;
		}
		else
		{
			// We could try to find the namespace in the current element hierarchy and emit a warning if it is not.
			// However, there is often a tendency to declare the namespace AFTER having already given attributes within it
			// and since we are a one-pass SAX parser we cannot know if the namespace might be declared later.
			// In order to catch attributes with namespaces that have not been declared we would need to do a second pass.
		}
	}

	// Since descriptors are largely application specific we cannot know if their elements are meaningful or not, so we don't show a warning.
	// Similary for anything we don't really understand we also should not emit warnings.
	if (GetElementType() == EType::Descriptor || GetElementType() == EType::ContentSteering || GetElementType() == EType::ClientDataReporting || GetElementType() == EType::Other)
	{
		bEmitWarning = false;
	}

	if (bEmitWarning)
	{
		Builder->LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("Attribute '%s' on <%s> element in line %d will be ignored"), AttributeName, *Builder->GetCurrentElementName(), GetXMLLineNumber()));
	}
	OtherAttributes.Emplace(FXmlAttribute({an, FString(AttributeValue)}));
	return true;
}

bool IDashMPDElement::HandleXLink(IDashMPDElement::FXLink& OutXLink, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(xlink_href))
	{
		bool bOk = ParseStringAttribute(OutXLink.Href, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		if (bOk)
		{
			Builder->AddXLinkElement(AsShared());
		}
		return bOk;
	}
	else if (AttrEquals(xlink_actuate))
	{
		return ParseStringAttribute(OutXLink.Actuate, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	// Not xlink or an unsupported xlink attribute. Returning false will call Super::ProcessAttribute()
	return false;
}


FErrorDetail IDashMPDXMLElementBuilder::AddNamespace(const TCHAR* Name, const TCHAR* URI)
{
	FString Space, Scheme;
	// Namespace given?
	if (StringHelpers::StringStartsWith(Name, ATTR(xmlns_colon), 6))
	{
		Space = FString(Name + 6);
	}
	Namespaces.Emplace(Space, FString(URI));
	// Would it be an error if the namespace was already there?
	// It is to be unique but it may also be possible that it is given more than once.
	// For now we do not fail this.
	return FErrorDetail();
}

/*********************************************************************************************************************/

bool FDashMPD_MPDType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Period)))
	{
		Periods.Emplace(MakeSharedTS<FDashMPD_PeriodType>(ElementName, ElementData));
		return Builder->PushNewElement(Periods.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(BaseURL)))
	{
		BaseURLs.Emplace(MakeSharedTS<FDashMPD_BaseURLType>(ElementName, ElementData));
		return Builder->PushNewElement(BaseURLs.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Location)))
	{
		Locations.Emplace(MakeSharedTS<FDashMPD_OtherType>(ElementName, ElementData));
		return Builder->PushNewElement(Locations.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ProgramInformation)))
	{
		ProgramInformations.Emplace(MakeSharedTS<FDashMPD_ProgramInformationType>(ElementName, ElementData));
		return Builder->PushNewElement(ProgramInformations.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(EssentialProperty)))
	{
		EssentialProperties.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(EssentialProperties.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SupplementalProperty)))
	{
		SupplementalProperties.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(SupplementalProperties.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(UTCTiming)))
	{
		UTCTimings.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(UTCTimings.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ServiceDescription)))
	{
		ServiceDescriptions.Emplace(MakeSharedTS<FDashMPD_ServiceDescriptionType>(ElementName, ElementData));
		return Builder->PushNewElement(ServiceDescriptions.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(InitializationSet)))
	{
		InitializationSets.Emplace(MakeSharedTS<FDashMPD_InitializationSetType>(ElementName, ElementData));
		return Builder->PushNewElement(InitializationSets.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(InitializationGroup)))
	{
		InitializationGroups.Emplace(MakeSharedTS<FDashMPD_UIntVWithIDType>(ElementName, ElementData));
		return Builder->PushNewElement(InitializationGroups.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(InitializationPresentation)))
	{
		InitializationPresentations.Emplace(MakeSharedTS<FDashMPD_UIntVWithIDType>(ElementName, ElementData));
		return Builder->PushNewElement(InitializationPresentations.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(LeapSecondInformation)))
	{
		LeapSecondInformation = MakeSharedTS<FDashMPD_LeapSecondInformationType>(ElementName, ElementData);
		return Builder->PushNewElement(LeapSecondInformation, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ContentSteering)))
	{
		ContentSteering = MakeSharedTS<FDashMPD_ContentSteeringType>(ElementName, ElementData);
		return Builder->PushNewElement(ContentSteering, XmlFileLineNumber);
	}
	else
	{
		// Metrics or other elements.
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_MPDType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseStringAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(type))
	{
		return ParseStringAttribute(Type, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(profiles))
	{
		return ParseStringArray(Profiles, TEXT(","), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(mediaPresentationDuration))
	{
		return ParseDuration(MediaPresentationDuration, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(minBufferTime))
	{
		return ParseDuration(MinBufferTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(minimumUpdatePeriod))
	{
		return ParseDuration(MinimumUpdatePeriod, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(timeShiftBufferDepth))
	{
		return ParseDuration(TimeShiftBufferDepth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(suggestedPresentationDelay))
	{
		return ParseDuration(SuggestedPresentationDelay, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxSegmentDuration))
	{
		return ParseDuration(MaxSegmentDuration, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxSubsegmentDuration))
	{
		return ParseDuration(MaxSubsegmentDuration, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(availabilityStartTime))
	{
		return ParseDateTime(AvailabilityStartTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(availabilityEndTime))
	{
		return ParseDateTime(AvailabilityEndTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(publishTime))
	{
		return ParseDateTime(PublishTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

int32 FDashMPD_MPDType::ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID)
{
	if (Element->GetElementType() == IDashMPDElement::EType::Period)
	{
		bool bReplaced = false;
		for(int32 i=0; i<Periods.Num(); ++i)
		{
			if (Periods[i] == Element)
			{
				Periods.RemoveAt(i);
				if (NewRootEntities.Periods.Num())
				{
					// Update the xlink resolve ID of the new elements we are adding even if the new element is not
					// having another xlink on it. Also note that this element was brought in remotely.
					for(int32 j=0; j<NewRootEntities.Periods.Num(); ++j)
					{
						NewRootEntities.Periods[j]->SetParentElement(AsShared());
						NewRootEntities.Periods[j]->GetXLink().LastResolveID = NewResolveID;
						NewRootEntities.Periods[j]->GetXLink().bWasRemoteEntity = true;
					}
					Periods.Insert(NewRootEntities.Periods, i);
				}
				bReplaced = true;
				break;
			}
		}
		return bReplaced ? 1 : 0;
	}
	else if (Element->GetElementType() == IDashMPDElement::EType::InitializationSet)
	{
		bool bReplaced = false;
		for(int32 i=0; i<InitializationSets.Num(); ++i)
		{
			if (InitializationSets[i] == Element)
			{
				InitializationSets.RemoveAt(i);
				if (NewRootEntities.InitializationSets.Num())
				{
					// Update the xlink resolve ID of the new elements we are adding even if the new element is not
					// having another xlink on it. Also note that this element was brought in remotely.
					for(int32 j=0; j<NewRootEntities.InitializationSets.Num(); ++j)
					{
						NewRootEntities.InitializationSets[j]->SetParentElement(AsShared());
						NewRootEntities.InitializationSets[j]->GetXLink().LastResolveID = NewResolveID;
						NewRootEntities.InitializationSets[j]->GetXLink().bWasRemoteEntity = true;
					}
					InitializationSets.Insert(NewRootEntities.InitializationSets, i);
				}
				bReplaced = true;
				break;
			}
		}
		return bReplaced ? 1 : 0;
	}
	return -2;
}

/*********************************************************************************************************************/

bool FDashMPD_ProgramInformationType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Title)))
	{
		Title = MakeSharedTS<FDashMPD_OtherType>(ElementName, ElementData);
		return Builder->PushNewElement(Title, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Source)))
	{
		Source = MakeSharedTS<FDashMPD_OtherType>(ElementName, ElementData);
		return Builder->PushNewElement(Source, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Copyright)))
	{
		Copyright = MakeSharedTS<FDashMPD_OtherType>(ElementName, ElementData);
		return Builder->PushNewElement(Copyright, XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_ProgramInformationType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(lang))
	{
		return ParseStringAttribute(Language, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(moreInformationURL))
	{
		return ParseStringAttribute(MoreInformationURL, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_URLType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(sourceURL))
	{
		return ParseStringAttribute(SourceURL, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(range))
	{
		return ParseStringAttribute(Range, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_BaseURLType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_BaseURLType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(serviceLocation))
	{
		return ParseStringAttribute(ServiceLocation, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(byteRange))
	{
		return ParseStringAttribute(ByteRange, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(availabilityTimeOffset))
	{
		return ParseDoubleAttribute(AvailabilityTimeOffset, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(availabilityTimeComplete))
	{
		return ParseBooleanAttribute(bAvailabilityTimeComplete, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(timeShiftBufferDepth))
	{
		return ParseDuration(TimeShiftBufferDepth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	// DVB attributes?
	else if (AttrEquals(dvb_priority))
	{
		return ParseIntAttribute(DVBPriority, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(dvb_weight))
	{
		return ParseIntAttribute(DVBWeight, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	// Any other attribute.
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_SegmentURLType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(media))
	{
		return ParseStringAttribute(Media, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(mediaRange))
	{
		return ParseStringAttribute(MediaRange, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(index))
	{
		return ParseStringAttribute(Index, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(indexRange))
	{
		return ParseStringAttribute(IndexRange, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_UrlQueryInfoType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(queryTemplate))
	{
		return ParseStringAttribute(QueryTemplate, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(useMPDUrlQuery))
	{
		return ParseBooleanAttribute(bUseMPDUrlQuery, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(queryString))
	{
		return ParseStringAttribute(QueryString, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (HandleXLink(XLink, Builder, AttributeName, AttributeValue))
	{
		return true;
	}
	else if (AttrEquals(includeInRequests))
	{
		return ParseStringArray(IncludeInRequests, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(headerParamSource))
	{
		return ParseStringArray(HeaderParamSources, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(sameOriginOnly))
	{
		return ParseBooleanAttribute(bSameOriginOnly, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_DescriptorType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	// Handle well-known DASH types here for convenience and easier access.
	if (StringHelpers::StringEquals(ElementName, ELEMENT(up_UrlQueryInfo)) && StringHelpers::StringEquals(*SchemeIdUri, Tags::Scheme_urn_mpeg_dash_urlparam_2014))
	{
		if (StringHelpers::StringEquals(*GetName(), ELEMENT(EssentialProperty)) || StringHelpers::StringEquals(*GetName(), ELEMENT(SupplementalProperty)))
		{
			WellKnownDescriptors.Emplace(MakeSharedTS<FDashMPD_UrlQueryInfoType>(ElementName, ElementData));
			return Builder->PushNewElement(WellKnownDescriptors.Last(), XmlFileLineNumber);
		}
		else
		{
			return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
		}
	}
	else if ((StringHelpers::StringEquals(ElementName, ELEMENT(up_ExtUrlQueryInfo)) || StringHelpers::StringEquals(ElementName, ELEMENT(up_ExtHttpHeaderInfo))) && StringHelpers::StringEquals(*SchemeIdUri, Tags::Scheme_urn_mpeg_dash_urlparam_2016))
	{
		if (StringHelpers::StringEquals(*GetName(), ELEMENT(EssentialProperty)) || StringHelpers::StringEquals(*GetName(), ELEMENT(SupplementalProperty)))
		{
			WellKnownDescriptors.Emplace(MakeSharedTS<FDashMPD_UrlQueryInfoType>(ElementName, ElementData,
				StringHelpers::StringEquals(ElementName, ELEMENT(up_ExtUrlQueryInfo)) ? FDashMPD_UrlQueryInfoType::EExtendedUrlInfoType::ExtUrlQueryInfo : FDashMPD_UrlQueryInfoType::EExtendedUrlInfoType::ExtHttpHeaderInfo));
			return Builder->PushNewElement(WellKnownDescriptors.Last(), XmlFileLineNumber);
		}
		else
		{
			return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
		}
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_DescriptorType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseStringAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(schemeIdUri))
	{
		return ParseStringAttribute(SchemeIdUri, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(value))
	{
		return ParseStringAttribute(Value, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

int32 FDashMPD_DescriptorType::ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID)
{
	if (Element->GetElementType() == IDashMPDElement::EType::URLQueryInfo)
	{
		bool bReplaced = false;
		for(int32 i=0; i<WellKnownDescriptors.Num(); ++i)
		{
			if (WellKnownDescriptors[i] == Element)
			{
				WellKnownDescriptors.RemoveAt(i);
				if (NewRootEntities.UrlQueries.Num())
				{
					// Update the xlink resolve ID of the new elements we are adding even if the new element is not
					// having another xlink on it. Also note that this element was brought in remotely.
					for(int32 j=0; j<NewRootEntities.UrlQueries.Num(); ++j)
					{
						NewRootEntities.UrlQueries[j]->SetParentElement(AsShared());
						NewRootEntities.UrlQueries[j]->GetXLink().LastResolveID = NewResolveID;
						NewRootEntities.UrlQueries[j]->GetXLink().bWasRemoteEntity = true;
						WellKnownDescriptors.Insert(NewRootEntities.UrlQueries[j], i+j);
					}
				}
				bReplaced = true;
				break;
			}
		}
		return bReplaced ? 1 : 0;
	}
	return -2;
}

/*********************************************************************************************************************/

bool FDashMPD_EventType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(presentationTime))
	{
		return ParseUIntAttribute(PresentationTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(duration))
	{
		return ParseUIntAttribute(Duration, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(id))
	{
		return ParseUIntAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(contentEncoding))
	{
		return ParseStringAttribute(ContentEncoding, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(messageData))
	{
		return ParseStringAttribute(MessageData, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_EventStreamType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Event)))
	{
		Events.Emplace(MakeSharedTS<FDashMPD_EventType>(ElementName, ElementData));
		return Builder->PushNewElement(Events.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_EventStreamType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(timescale))
	{
		return ParseUIntAttribute(Timescale, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(schemeIdUri))
	{
		return ParseStringAttribute(SchemeIdUri, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(value))
	{
		return ParseStringAttribute(Value, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(presentationTimeOffset))
	{
		return ParseUIntAttribute(PresentationTimeOffset, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (HandleXLink(XLink, Builder, AttributeName, AttributeValue))
	{
		return true;
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_PeriodType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(AdaptationSet)))
	{
		AdaptationSets.Emplace(MakeSharedTS<FDashMPD_AdaptationSetType>(ElementName, ElementData));
		return Builder->PushNewElement(AdaptationSets.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(BaseURL)))
	{
		BaseURLs.Emplace(MakeSharedTS<FDashMPD_BaseURLType>(ElementName, ElementData));
		return Builder->PushNewElement(BaseURLs.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SupplementalProperty)))
	{
		SupplementalProperties.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(SupplementalProperties.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentBase)))
	{
		SegmentBase = MakeSharedTS<FDashMPD_SegmentBaseType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentBase, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentList)))
	{
		SegmentList = MakeSharedTS<FDashMPD_SegmentListType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentList, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentTemplate)))
	{
		SegmentTemplate = MakeSharedTS<FDashMPD_SegmentTemplateType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentTemplate, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(AssetIdentifier)))
	{
		AssetIdentifier = MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData);
		return Builder->PushNewElement(AssetIdentifier, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(EventStream)))
	{
		EventStreams.Emplace(MakeSharedTS<FDashMPD_EventStreamType>(ElementName, ElementData));
		return Builder->PushNewElement(EventStreams.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Subset)))
	{
		Subsets.Emplace(MakeSharedTS<FDashMPD_SubsetType>(ElementName, ElementData));
		return Builder->PushNewElement(Subsets.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ServiceDescription)))
	{
		ServiceDescriptions.Emplace(MakeSharedTS<FDashMPD_ServiceDescriptionType>(ElementName, ElementData));
		return Builder->PushNewElement(ServiceDescriptions.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(EmptyAdaptationSet)))
	{
		EmptyAdaptationSets.Emplace(MakeSharedTS<FDashMPD_AdaptationSetType>(ElementName, ElementData));
		return Builder->PushNewElement(EmptyAdaptationSets.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Preselection)))
	{
		Preselections.Emplace(MakeSharedTS<FDashMPD_PreselectionType>(ElementName, ElementData));
		return Builder->PushNewElement(Preselections.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_PeriodType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseStringAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(start))
	{
		return ParseDuration(Start, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(duration))
	{
		return ParseDuration(Duration, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(bitstreamSwitching))
	{
		return ParseBooleanAttribute(bBitstreamSwitching, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (HandleXLink(XLink, Builder, AttributeName, AttributeValue))
	{
		return true;
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

int32 FDashMPD_PeriodType::ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID)
{
	if (Element->GetElementType() == IDashMPDElement::EType::AdaptationSet)
	{
		bool bReplaced = false;
		for(int32 i=0; i<AdaptationSets.Num(); ++i)
		{
			if (AdaptationSets[i] == Element)
			{
				AdaptationSets.RemoveAt(i);
				if (NewRootEntities.AdaptationSets.Num())
				{
					// Update the xlink resolve ID of the new elements we are adding even if the new element is not
					// having another xlink on it. Also note that this element was brought in remotely.
					for(int32 j=0; j<NewRootEntities.AdaptationSets.Num(); ++j)
					{
						NewRootEntities.AdaptationSets[j]->SetParentElement(AsShared());
						NewRootEntities.AdaptationSets[j]->GetXLink().LastResolveID = NewResolveID;
						NewRootEntities.AdaptationSets[j]->GetXLink().bWasRemoteEntity = true;
					}
					AdaptationSets.Insert(NewRootEntities.AdaptationSets, i);
				}
				bReplaced = true;
				break;
			}
		}
		return bReplaced ? 1 : 0;
	}
	else if (Element->GetElementType() == IDashMPDElement::EType::EventStream)
	{
		bool bReplaced = false;
		for(int32 i=0; i<EventStreams.Num(); ++i)
		{
			if (EventStreams[i] == Element)
			{
				EventStreams.RemoveAt(i);
				if (NewRootEntities.EventStreams.Num())
				{
					// Update the xlink resolve ID of the new elements we are adding even if the new element is not
					// having another xlink on it. Also note that this element was brought in remotely.
					for(int32 j=0; j<NewRootEntities.EventStreams.Num(); ++j)
					{
						NewRootEntities.EventStreams[j]->GetXLink().LastResolveID = NewResolveID;
						NewRootEntities.EventStreams[j]->GetXLink().bWasRemoteEntity = true;
					}
					EventStreams.Insert(NewRootEntities.EventStreams, i);
				}
				bReplaced = true;
				break;
			}
		}
		return bReplaced ? 1 : 0;
	}
	else if (Element->GetElementType() == IDashMPDElement::EType::SegmentList)
	{
		bool bReplaced = false;
		if (SegmentList == Element)
		{
			SegmentList.Reset();
			if (NewRootEntities.SegmentLists.Num())
			{
				// Update the xlink resolve ID of the new elements we are adding even if the new element is not
				// having another xlink on it. Also note that this element was brought in remotely.
				for(int32 j=0; j<NewRootEntities.SegmentLists.Num(); ++j)
				{
					NewRootEntities.SegmentLists[j]->GetXLink().LastResolveID = NewResolveID;
					NewRootEntities.SegmentLists[j]->GetXLink().bWasRemoteEntity = true;
				}
				SegmentList = NewRootEntities.SegmentLists[0];
			}
			bReplaced = true;
		}
		return bReplaced ? 1 : 0;
	}
	return -2;
}

/*********************************************************************************************************************/

bool FDashMPD_RepresentationBaseType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(FramePacking)))
	{
		FramePackings.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(FramePackings.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(AudioChannelConfiguration)))
	{
		AudioChannelConfigurations.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(AudioChannelConfigurations.Last(), XmlFileLineNumber);
	}

	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ContentProtection)))
	{
		ContentProtections.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(ContentProtections.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(EssentialProperty)))
	{
		EssentialProperties.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(EssentialProperties.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SupplementalProperty)))
	{
		SupplementalProperties.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(SupplementalProperties.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(InbandEventStream)))
	{
		InbandEventStreams.Emplace(MakeSharedTS<FDashMPD_EventStreamType>(ElementName, ElementData));
		return Builder->PushNewElement(InbandEventStreams.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Switching)))
	{
		Switchings.Emplace(MakeSharedTS<FDashMPD_SwitchingType>(ElementName, ElementData));
		return Builder->PushNewElement(Switchings.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(RandomAccess)))
	{
		RandomAccesses.Emplace(MakeSharedTS<FDashMPD_RandomAccessType>(ElementName, ElementData));
		return Builder->PushNewElement(RandomAccesses.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(GroupLabel)))
	{
		GroupLabels.Emplace(MakeSharedTS<FDashMPD_LabelType>(ElementName, ElementData));
		return Builder->PushNewElement(GroupLabels.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Label)))
	{
		Labels.Emplace(MakeSharedTS<FDashMPD_LabelType>(ElementName, ElementData));
		return Builder->PushNewElement(Labels.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ProducerReferenceTime)))
	{
		ProducerReferenceTimes.Emplace(MakeSharedTS<FDashMPD_ProducerReferenceTimeType>(ElementName, ElementData));
		return Builder->PushNewElement(ProducerReferenceTimes.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ContentPopularityRate)))
	{
		ContentPopularityRates.Emplace(MakeSharedTS<FDashMPD_ContentPopularityRateType>(ElementName, ElementData));
		return Builder->PushNewElement(ContentPopularityRates.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Resync)))
	{
		Resyncs.Emplace(MakeSharedTS<FDashMPD_ResyncType>(ElementName, ElementData));
		return Builder->PushNewElement(Resyncs.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_RepresentationBaseType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(profiles))
	{
		return ParseStringArray(Profiles, TEXT(","), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(segmentProfiles))
	{
		return ParseStringArray(SegmentProfiles, TEXT(","), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(codecs))
	{
		return ParseStringArray(Codecs, TEXT(","), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(width))
	{
		return ParseUIntAttribute(Width, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(height))
	{
		return ParseUIntAttribute(Height, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(mimeType))
	{
		return ParseStringAttribute(MimeType, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(audioSamplingRate))
	{
		return ParseUIntArray(AudioSamplingRate, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(frameRate))
	{
		return ParseFramerateAttribute(FrameRate, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(startWithSAP))
	{
		return ParseIntAttribute(StartWithSAP, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(scanType))
	{
		return ParseStringAttribute(ScanType, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(sar))
	{
		return ParseRatioAttribute(SAR, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maximumSAPPeriod))
	{
		return ParseDoubleAttribute(MaximumSAPPeriod, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxPlayoutRate))
	{
		return ParseDoubleAttribute(MaxPlayoutRate, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(codingDependency))
	{
		return ParseBooleanAttribute(bCodingDependency, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(selectionPriority))
	{
		return ParseUIntAttribute(SelectionPriority, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(tag))
	{
		return ParseStringAttribute(Tag, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

int32 FDashMPD_RepresentationBaseType::ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID)
{
	if (Element->GetElementType() == IDashMPDElement::EType::EventStream)
	{
		bool bReplaced = false;
		for(int32 i=0; i<InbandEventStreams.Num(); ++i)
		{
			if (InbandEventStreams[i] == Element)
			{
				InbandEventStreams.RemoveAt(i);
				if (NewRootEntities.EventStreams.Num())
				{
					// Update the xlink resolve ID of the new elements we are adding even if the new element is not
					// having another xlink on it. Also note that this element was brought in remotely.
					for(int32 j=0; j<NewRootEntities.EventStreams.Num(); ++j)
					{
						NewRootEntities.EventStreams[j]->SetParentElement(AsShared());
						NewRootEntities.EventStreams[j]->GetXLink().LastResolveID = NewResolveID;
						NewRootEntities.EventStreams[j]->GetXLink().bWasRemoteEntity = true;
					}
					InbandEventStreams.Insert(NewRootEntities.EventStreams, i);
				}
				bReplaced = true;
				break;
			}
		}
		return bReplaced ? 1 : 0;
	}
	return -2;
}

/*********************************************************************************************************************/

bool FDashMPD_ContentComponentType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Accessibility)))
	{
		Accessibilities.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Accessibilities.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Role)))
	{
		Roles.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Roles.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Rating)))
	{
		Ratings.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Ratings.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Viewpoint)))
	{
		Viewpoints.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Viewpoints.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_ContentComponentType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		if (ParseStringAttribute(ID_AsStr, Builder, AttributeName, AttributeValue, GetXMLLineNumber()))
		{
			return ParseUIntAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		}
		return false;
	}
	else if (AttrEquals(lang))
	{
		return ParseStringAttribute(Language, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(contentType))
	{
		return ParseStringAttribute(ContentType, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(par))
	{
		return ParseRatioAttribute(PAR, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(tag))
	{
		return ParseStringAttribute(Tag, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_AdaptationSetType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Representation)))
	{
		Representations.Emplace(MakeSharedTS<FDashMPD_RepresentationType>(ElementName, ElementData));
		return Builder->PushNewElement(Representations.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentBase)))
	{
		SegmentBase = MakeSharedTS<FDashMPD_SegmentBaseType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentBase, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentList)))
	{
		SegmentList = MakeSharedTS<FDashMPD_SegmentListType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentList, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentTemplate)))
	{
		SegmentTemplate = MakeSharedTS<FDashMPD_SegmentTemplateType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentTemplate, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(BaseURL)))
	{
		BaseURLs.Emplace(MakeSharedTS<FDashMPD_BaseURLType>(ElementName, ElementData));
		return Builder->PushNewElement(BaseURLs.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Accessibility)))
	{
		Accessibilities.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Accessibilities.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Role)))
	{
		Roles.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Roles.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Rating)))
	{
		Ratings.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Ratings.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Viewpoint)))
	{
		Viewpoints.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Viewpoints.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ContentComponent)))
	{
		ContentComponents.Emplace(MakeSharedTS<FDashMPD_ContentComponentType>(ElementName, ElementData));
		return Builder->PushNewElement(ContentComponents.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_AdaptationSetType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		if (ParseStringAttribute(ID_AsStr, Builder, AttributeName, AttributeValue, GetXMLLineNumber()))
		{
			return ParseUIntAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		}
		return false;
	}
	else if (AttrEquals(group))
	{
		return ParseUIntAttribute(Group, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(lang))
	{
		return ParseStringAttribute(Language, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(contentType))
	{
		return ParseStringAttribute(ContentType, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(minBandwidth))
	{
		return ParseUIntAttribute(MinBandwidth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxBandwidth))
	{
		return ParseUIntAttribute(MaxBandwidth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(minWidth))
	{
		return ParseUIntAttribute(MinWidth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxWidth))
	{
		return ParseUIntAttribute(MaxWidth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(minHeight))
	{
		return ParseUIntAttribute(MinHeight, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxHeight))
	{
		return ParseUIntAttribute(MaxHeight, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(minFrameRate))
	{
		return ParseFramerateAttribute(MinFrameRate, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxFrameRate))
	{
		return ParseFramerateAttribute(MaxFrameRate, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(par))
	{
		return ParseRatioAttribute(PAR, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(segmentAlignment))
	{
		return ParseConditionalUIntAttribute(bSegmentAlignment, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(subsegmentAlignment))
	{
		return ParseConditionalUIntAttribute(bSubsegmentAlignment, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(subsegmentStartsWithSAP))
	{
		return ParseIntAttribute(SubsegmentStartsWithSAP, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(bitstreamSwitching))
	{
		return ParseBooleanAttribute(bBitstreamSwitching, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(initializationSetRef))
	{
		return ParseUIntArray(InitializationSetRefs, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (HandleXLink(XLink, Builder, AttributeName, AttributeValue))
	{
		return true;
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

int32 FDashMPD_AdaptationSetType::ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID)
{
	if (Element->GetElementType() == IDashMPDElement::EType::SegmentList)
	{
		bool bReplaced = false;
		if (SegmentList == Element)
		{
			SegmentList.Reset();
			if (NewRootEntities.SegmentLists.Num())
			{
				// Update the xlink resolve ID of the new elements we are adding even if the new element is not
				// having another xlink on it. Also note that this element was brought in remotely.
				for(int32 j=0; j<NewRootEntities.SegmentLists.Num(); ++j)
				{
					NewRootEntities.SegmentLists[j]->SetParentElement(AsShared());
					NewRootEntities.SegmentLists[j]->GetXLink().LastResolveID = NewResolveID;
					NewRootEntities.SegmentLists[j]->GetXLink().bWasRemoteEntity = true;
				}
				SegmentList = NewRootEntities.SegmentLists[0];
			}
			bReplaced = true;
		}
		return bReplaced ? 1 : 0;
	}
	return -2;
}


/*********************************************************************************************************************/

bool FDashMPD_SubRepresentationType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(level))
	{
		return ParseUIntAttribute(Level, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(bandwidth))
	{
		return ParseUIntAttribute(Bandwidth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(dependencyLevel))
	{
		return ParseUIntArray(DependencyLevels, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(contentComponent))
	{
		return ParseStringArray(ContentComponents, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_SubsetType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_SubsetType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(contains))
	{
		return ParseUIntArray(Contains, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(id))
	{
		return ParseStringAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_SwitchingType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_SwitchingType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(interval))
	{
		return ParseUIntAttribute(Interval, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(type))
	{
		return ParseStringAttribute(Type, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_RandomAccessType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_RandomAccessType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(interval))
	{
		return ParseUIntAttribute(Interval, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(type))
	{
		return ParseStringAttribute(Type, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(minBufferTime))
	{
		return ParseDuration(MinBufferTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(bandwidth))
	{
		return ParseUIntAttribute(Bandwidth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_LabelType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_LabelType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseUIntAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(lang))
	{
		return ParseStringAttribute(Language, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}


/*********************************************************************************************************************/

bool FDashMPD_ProducerReferenceTimeType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(UTCTiming)))
	{
		if (UTCTiming.IsValid())
		{
			return Builder->SetLastError(FString::Printf(TEXT("<%s> element cannot have more than one <%s> children in line %d"), *GetName(), ElementName, GetXMLLineNumber()), ERRCODE_DASH_MPD_PARSER_ELEMENT_CANNOT_HAVE_CHILDREN);
		}
		UTCTiming = MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData);
		return Builder->PushNewElement(UTCTiming, XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_ProducerReferenceTimeType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseUIntAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(inband))
	{
		return ParseBooleanAttribute(bInband, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(type))
	{
		return ParseStringAttribute(Type, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(applicationScheme))
	{
		return ParseStringAttribute(ApplicationScheme, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(wallClockTime))
	{
		return ParseStringAttribute(WallclockTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(presentationTime))
	{
		return ParseUIntAttribute(PresentationTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_LeapSecondInformationType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(availabilityStartLeapOffset))
	{
		return ParseIntAttribute(AvailabilityStartLeapOffset, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(nextAvailabilityStartLeapOffset))
	{
		return ParseIntAttribute(NextAvailabilityStartLeapOffset, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(nextLeapChangeTime))
	{
		return ParseDateTime(NextLeapChangeTime, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_ContentSteeringType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(defaultServiceLocation))
	{
		return ParseStringAttribute(DefaultServiceLocation, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(proxyServerURL))
	{
		return ParseStringAttribute(ProxyServerURL, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(queryBeforeStart))
	{
		return ParseBooleanAttribute(bQueryBeforeStart, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(clientRequirement))
	{
		return ParseBooleanAttribute(bClientRequirement, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_ClientDataReportingType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(CMCDParameters)))
	{
		CMCDParameters_Elements.Emplace(MakeSharedTS<FDashMPD_CMCDParametersType>(ElementName, ElementData));
		return Builder->PushNewElement(CMCDParameters_Elements.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}
bool FDashMPD_ClientDataReportingType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(serviceLocations))
	{
		return ParseStringArray(ServiceLocations, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(adaptationSets))
	{
		return ParseStringArray(AdaptationSets, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

bool FDashMPD_CMCDParametersType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(version))
	{
		return ParseIntAttribute(Version, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(includeInRequests))
	{
		return ParseStringArray(IncludeInRequests, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(keys))
	{
		return ParseStringArray(Keys, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(mode))
	{
		return ParseStringAttribute(Mode, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(contentID))
	{
		return ParseStringAttribute(ContentID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(sessionID))
	{
		return ParseStringAttribute(SessionID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(schemeIdUri))
	{
		return ParseStringAttribute(SchemeIdUri, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_ContentPopularityRateType::FPRElement::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(popularityRate))
	{
		return ParseUIntAttribute(PopularityRate, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(start))
	{
		return ParseUIntAttribute(Start, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(r))
	{
		return ParseIntAttribute(R, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

bool FDashMPD_ContentPopularityRateType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(PR)))
	{
		PR_Elements.Emplace(MakeSharedTS<FPRElement>(ElementName, ElementData));
		return Builder->PushNewElement(PR_Elements.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_ContentPopularityRateType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(source))
	{
		return ParseStringAttribute(Source, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(source_description))
	{
		return ParseStringAttribute(Source_Description, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_InitializationSetType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Accessibility)))
	{
		Accessibilities.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Accessibilities.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Role)))
	{
		Roles.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Roles.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Rating)))
	{
		Ratings.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Ratings.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Viewpoint)))
	{
		Viewpoints.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Viewpoints.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_InitializationSetType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseUIntAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(inAllPeriods))
	{
		return ParseBooleanAttribute(bInAllPeriods, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(contentType))
	{
		return ParseStringAttribute(ContentType, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(par))
	{
		return ParseRatioAttribute(PAR, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxWidth))
	{
		return ParseUIntAttribute(MaxWidth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxHeight))
	{
		return ParseUIntAttribute(MaxHeight, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxFrameRate))
	{
		return ParseFramerateAttribute(MaxFrameRate, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(initialization))
	{
		return ParseStringAttribute(Initialization, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (HandleXLink(XLink, Builder, AttributeName, AttributeValue))
	{
		return true;
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_PreselectionType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Accessibility)))
	{
		Accessibilities.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Accessibilities.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Role)))
	{
		Roles.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Roles.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Rating)))
	{
		Ratings.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Ratings.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Viewpoint)))
	{
		Viewpoints.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Viewpoints.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_PreselectionType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseStringAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(preselectionComponents))
	{
		return ParseStringArray(PreselectionComponents, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(lang))
	{
		return ParseStringAttribute(Language, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(order))
	{
		return ParseStringAttribute(Language, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_UIntVWithIDType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_UIntVWithIDType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseUIntAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(profiles))
	{
		return ParseStringArray(Profiles, TEXT(","), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(contentType))
	{
		return ParseStringAttribute(ContentType, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

bool FDashMPD_UIntVWithIDType::ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName)
{
	return ParseUIntArray(Elements, TEXT(" "), Builder, ElementName, *Data, GetXMLLineNumber());
}

/*********************************************************************************************************************/

bool FDashMPD_UIntPairsWithIDType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_UIntPairsWithIDType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(type))
	{
		return ParseStringAttribute(Type, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

bool FDashMPD_UIntPairsWithIDType::ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName)
{
	return ParseUIntArray(Elements, TEXT(" "), Builder, ElementName, *Data, GetXMLLineNumber());
}

/*********************************************************************************************************************/

bool FDashMPD_OperatingBandwidthType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_OperatingBandwidthType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(mediaType))
	{
		return ParseStringAttribute(MediaType, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(min))
	{
		return ParseUIntAttribute(Min, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(max))
	{
		return ParseUIntAttribute(Max, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(target))
	{
		return ParseUIntAttribute(Target, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_OperatingQualityType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_OperatingQualityType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(mediaType))
	{
		return ParseStringAttribute(MediaType, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(min))
	{
		return ParseUIntAttribute(Min, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(max))
	{
		return ParseUIntAttribute(Max, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(target))
	{
		return ParseUIntAttribute(Target, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(type))
	{
		return ParseStringAttribute(Type, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(maxDifference))
	{
		return ParseUIntAttribute(MaxDifference, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_PlaybackRateType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_PlaybackRateType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(min))
	{
		return ParseDoubleAttribute(Min, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(max))
	{
		return ParseDoubleAttribute(Max, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_LatencyType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(QualityLatency)))
	{
		QualityLatencies.Emplace(MakeSharedTS<FDashMPD_UIntPairsWithIDType>(ElementName, ElementData));
		return Builder->PushNewElement(QualityLatencies.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_LatencyType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(referenceId))
	{
		return ParseUIntAttribute(ReferenceID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(target))
	{
		return ParseUIntAttribute(Target, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(max))
	{
		return ParseUIntAttribute(Max, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(min))
	{
		return ParseUIntAttribute(Min, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_ServiceDescriptionType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Scope)))
	{
		Scopes.Emplace(MakeSharedTS<FDashMPD_DescriptorType>(ElementName, ElementData));
		return Builder->PushNewElement(Scopes.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(Latency)))
	{
		Latencies.Emplace(MakeSharedTS<FDashMPD_LatencyType>(ElementName, ElementData));
		return Builder->PushNewElement(Latencies.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(PlaybackRate)))
	{
		PlaybackRates.Emplace(MakeSharedTS<FDashMPD_PlaybackRateType>(ElementName, ElementData));
		return Builder->PushNewElement(PlaybackRates.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(OperatingQuality)))
	{
		OperatingQualities.Emplace(MakeSharedTS<FDashMPD_OperatingQualityType>(ElementName, ElementData));
		return Builder->PushNewElement(OperatingQualities.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(OperatingBandwidth)))
	{
		OperatingBandwidths.Emplace(MakeSharedTS<FDashMPD_OperatingBandwidthType>(ElementName, ElementData));
		return Builder->PushNewElement(OperatingBandwidths.Last(), XmlFileLineNumber);
	}
	// Content steering and client data reportings are now provided in the service description.
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ContentSteering)))
	{
		ContentSteerings.Emplace(MakeSharedTS<FDashMPD_ContentSteeringType>(ElementName, ElementData));
		return Builder->PushNewElement(ContentSteerings.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(ClientDataReporting)))
	{
		ClientDataReportings.Emplace(MakeSharedTS<FDashMPD_ClientDataReportingType>(ElementName, ElementData));
		return Builder->PushNewElement(ClientDataReportings.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_ServiceDescriptionType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseUIntAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_ResyncType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FDashMPD_ResyncType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(type))
	{
		return ParseIntAttribute(Type, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(dT))
	{
		return ParseUIntAttribute(dT, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(dImax))
	{
		return ParseDoubleAttribute(dIMax, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(dImin))
	{
		return ParseDoubleAttribute(dIMin, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(marker))
	{
		return ParseBooleanAttribute(bMarker, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(rangeAccess))
	{
		return ParseBooleanAttribute(bRangeAccess, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(index))
	{
		return ParseStringAttribute(Index, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_RepresentationType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentBase)))
	{
		SegmentBase = MakeSharedTS<FDashMPD_SegmentBaseType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentBase, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentList)))
	{
		SegmentList = MakeSharedTS<FDashMPD_SegmentListType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentList, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentTemplate)))
	{
		SegmentTemplate = MakeSharedTS<FDashMPD_SegmentTemplateType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentTemplate, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(BaseURL)))
	{
		BaseURLs.Emplace(MakeSharedTS<FDashMPD_BaseURLType>(ElementName, ElementData));
		return Builder->PushNewElement(BaseURLs.Last(), XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(SubRepresentation)))
	{
		SubRepresentations.Emplace(MakeSharedTS<FDashMPD_SubRepresentationType>(ElementName, ElementData));
		return Builder->PushNewElement(SubRepresentations.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_RepresentationType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(id))
	{
		return ParseStringAttribute(ID, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(bandwidth))
	{
		return ParseUIntAttribute(Bandwidth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(qualityRanking))
	{
		return ParseUIntAttribute(QualityRanking, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(dependencyId))
	{
		return ParseStringArray(DependencyIDs, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(mediaStreamStructureId))
	{
		return ParseStringArray(MediaStreamStructureIDs, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(associationId))
	{
		return ParseStringArray(AssociationIDs, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(associationType))
	{
		return ParseStringArray(AssociationTypes, TEXT(" "), Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

int32 FDashMPD_RepresentationType::ReplaceChildElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID)
{
	if (Element->GetElementType() == IDashMPDElement::EType::SegmentList)
	{
		bool bReplaced = false;
		if (SegmentList == Element)
		{
			SegmentList.Reset();
			if (NewRootEntities.SegmentLists.Num())
			{
				// Update the xlink resolve ID of the new elements we are adding even if the new element is not
				// having another xlink on it. Also note that this element was brought in remotely.
				for(int32 j=0; j<NewRootEntities.SegmentLists.Num(); ++j)
				{
					NewRootEntities.SegmentLists[j]->SetParentElement(AsShared());
					NewRootEntities.SegmentLists[j]->GetXLink().LastResolveID = NewResolveID;
					NewRootEntities.SegmentLists[j]->GetXLink().bWasRemoteEntity = true;
				}
				SegmentList = NewRootEntities.SegmentLists[0];
			}
			bReplaced = true;
		}
		return bReplaced ? 1 : 0;
	}
	return -2;
}


/*********************************************************************************************************************/

bool FDashMPD_SegmentBaseType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(Initialization)))
	{
		Initialization = MakeSharedTS<FDashMPD_URLType>(ElementName, ElementData);
		return Builder->PushNewElement(Initialization, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(RepresentationIndex)))
	{
		RepresentationIndex = MakeSharedTS<FDashMPD_URLType>(ElementName, ElementData);
		return Builder->PushNewElement(RepresentationIndex, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(FailoverContent)))
	{
		FailoverContent = MakeSharedTS<FDashMPD_FailoverContentType>(ElementName, ElementData);
		return Builder->PushNewElement(FailoverContent, XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_SegmentBaseType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(timescale))
	{
		return ParseUIntAttribute(Timescale, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(presentationTimeOffset))
	{
		return ParseUIntAttribute(PresentationTimeOffset, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(indexRange))
	{
		return ParseStringAttribute(IndexRange, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(indexRangeExact))
	{
		return ParseBooleanAttribute(bIndexRangeExact, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(availabilityTimeOffset))
	{
		return ParseDoubleAttribute(AvailabilityTimeOffset, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(availabilityTimeComplete))
	{
		return ParseBooleanAttribute(bAvailabilityTimeComplete, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(timeShiftBufferDepth))
	{
		return ParseDuration(TimeShiftBufferDepth, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(presentationDuration))
	{
		return ParseUIntAttribute(PresentationDuration, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(eptDelta))
	{
		return ParseIntAttribute(EptDelta, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_MultipleSegmentBaseType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentTimeline)))
	{
		SegmentTimeline = MakeSharedTS<FDashMPD_SegmentTimelineType>(ElementName, ElementData);
		return Builder->PushNewElement(SegmentTimeline, XmlFileLineNumber);
	}
	else if (StringHelpers::StringEquals(ElementName, ELEMENT(BitstreamSwitching)))
	{
		BitstreamSwitching = MakeSharedTS<FDashMPD_URLType>(ElementName, ElementData);
		return Builder->PushNewElement(BitstreamSwitching, XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_MultipleSegmentBaseType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(duration))
	{
		return ParseUIntAttribute(Duration, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(startNumber))
	{
		return ParseUIntAttribute(StartNumber, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(endNumber))
	{
		return ParseUIntAttribute(EndNumber, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_SegmentListType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(SegmentURL)))
	{
		SegmentURLs.Emplace(MakeSharedTS<FDashMPD_SegmentURLType>(ElementName, ElementData));
		return Builder->PushNewElement(SegmentURLs.Last(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_SegmentListType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (HandleXLink(XLink, Builder, AttributeName, AttributeValue))
	{
		return true;
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_SegmentTemplateType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(media))
	{
		return ParseStringAttribute(Media, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(index))
	{
		return ParseStringAttribute(Index, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(initialization))
	{
		return ParseStringAttribute(Initialization, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else if (AttrEquals(bitstreamSwitching))
	{
		return ParseStringAttribute(BitstreamSwitching, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

/*********************************************************************************************************************/

bool FDashMPD_SegmentTimelineType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(S)))
	{
		// Add a new element to our chunked array.
		Builder->GetSBuilder()->SetCurrentElement(&S_Elements[S_Elements.Add(1)]);
		return Builder->PushNewElement(Builder->GetSBuilder(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_SegmentTimelineType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
}

bool FDashMPD_SegmentTimelineType::ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName)
{
	Builder->GetSBuilder()->SetCurrentElement(nullptr);
	return true;
}

/*********************************************************************************************************************/

bool FTimeline_S_ElementBuilder::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FTimeline_S_ElementBuilder::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	check(CurrentElement);
	if (AttrEquals(t))
	{
		ParseUIntAttribute(CurrentElement->T, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		CurrentElement->HaveT = -1;
		return true;
	}
	else if (AttrEquals(n))
	{
		ParseUIntAttribute(CurrentElement->N, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		CurrentElement->HaveN = -1;
		return true;
	}
	else if (AttrEquals(d))
	{
		ParseUIntAttribute(CurrentElement->D, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		CurrentElement->HaveD = -1;
		return true;
	}
	else if (AttrEquals(r))
	{
		ParseIntAttribute(CurrentElement->R, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		CurrentElement->HaveR = -1;
		return true;
	}
	else if (AttrEquals(k))
	{
		ParseUIntAttribute(CurrentElement->K, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		CurrentElement->HaveK = -1;
		return true;
	}
	else
	{
		// We do not support additional attributes on the <S> element.
		// For storage reasons we keep it as small as possible and it is not a full element we could add additional attributes to.
		Builder->LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Attribute '%s' on <S> element in line %d will be ignored"), AttributeName, GetXMLLineNumber()));
		//return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
		return true;
	}
}

/*********************************************************************************************************************/

bool FDashMPD_FailoverContentType::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	if (StringHelpers::StringEquals(ElementName, ELEMENT(FCS)))
	{
		Builder->GetFCSBuilder()->SetCurrentElement(&FCS_Elements[FCS_Elements.Add(1)]);
		return Builder->PushNewElement(Builder->GetFCSBuilder(), XmlFileLineNumber);
	}
	else
	{
		return Super::ProcessElement(Builder, ElementName, ElementData, XmlFileLineNumber);
	}
}

bool FDashMPD_FailoverContentType::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	if (AttrEquals(valid))
	{
		return ParseBooleanAttribute(bValid, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
	}
	else
	{
		return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
	}
}

bool FDashMPD_FailoverContentType::ProcessClose(FManifestParserDASH* Builder, const TCHAR* ElementName)
{
	Builder->GetFCSBuilder()->SetCurrentElement(nullptr);
	return true;
}

/*********************************************************************************************************************/

bool FFailOver_FCS_ElementBuilder::ProcessElement(FManifestParserDASH* Builder, const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber)
{
	return NoChildElementAllowed(Builder, ElementName, ElementData, XmlFileLineNumber);
}

bool FFailOver_FCS_ElementBuilder::ProcessAttribute(FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue)
{
	check(CurrentElement);
	if (AttrEquals(t))
	{
		ParseUIntAttribute(CurrentElement->T, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		CurrentElement->HaveT = -1;
		return true;
	}
	else if (AttrEquals(d))
	{
		ParseUIntAttribute(CurrentElement->D, Builder, AttributeName, AttributeValue, GetXMLLineNumber());
		CurrentElement->HaveD = -1;
		return true;
	}
	else
	{
		// We do not support additional attributes on the <FCS> element.
		// For storage reasons we keep it as small as possible and it is not a full element we could add additional attributes to.
		Builder->LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Attribute '%s' on <FCS> element in line %d will be ignored"), AttributeName, GetXMLLineNumber()));
		//return Super::ProcessAttribute(Builder, AttributeName, AttributeValue);
		return true;
	}
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

namespace
{

static bool RemoveXMLEntities(FManifestParserDASH* Builder, FString& InOutString, int32 XmlLineNumber)
{
	FString Result;

	// We only remove the five predefined entities (see https://en.wikipedia.org/wiki/List_of_XML_and_HTML_character_entity_references#Predefined_entities_in_XML)
	// and additionally those represented numerically, but not the hundreds of named ones.
	for(StringHelpers::FStringIterator it(InOutString); it; ++it)
	{
		if (*it != TCHAR('&'))
		{
			Result += *it;
		}
		else
		{
			++it;
			// Look for the ';' ending the entity.
			StringHelpers::FStringIterator it2(InOutString, it.GetIndex());
			while(it2 && *it2 != TCHAR(';'))
			{
				++it2;
			}
			// Not found?
			if (!it2)
			{
				return Builder->SetLastError(FString::Printf(TEXT("Did not find ending ';' in \"%s\" for XML entity in line %d"), *InOutString, XmlLineNumber), ERRCODE_DASH_MPD_PARSER_XML_BAD_ENTITY);
			}
			FString Entity = InOutString.Mid(it.GetIndex(), it2.GetIndex() - it.GetIndex());
			it += Entity.Len();
			// What entity are we dealing with?
			if (Entity.Equals(TEXT("amp")))
			{
				Result += TEXT("&");
			}
			else if (Entity.Equals(TEXT("quot")))
			{
				Result += TEXT("\"");
			}
			else if (Entity.Equals(TEXT("apos")))
			{
				Result += TEXT("'");
			}
			else if (Entity.Equals(TEXT("lt")))
			{
				Result += TEXT("<");
			}
			else if (Entity.Equals(TEXT("gt")))
			{
				Result += TEXT(">");
			}
			else if (Entity.Len() > 0 && Entity[0] == TCHAR('#'))
			{
				Entity.RightChopInline(1, EAllowShrinking::No);
				// Decimal or hex?
				if (Entity.Len() > 0 && Entity[0] == TCHAR('x'))
				{
					// Hex.
					Entity.RightChopInline(1, EAllowShrinking::No);
					while(Entity.Len() > 1 && Entity[0] == TCHAR('0'))
					{
						Entity.RightChopInline(1, EAllowShrinking::No);
					}
					int32 v=0;
					LexFromStringHex(v, *Entity);
					Result += TCHAR(v);
				}
				else
				{
					// Decimal.
					Entity.RightChopInline(1, EAllowShrinking::No);
					while(Entity.Len() > 1 && Entity[0] == TCHAR('0'))
					{
						Entity.RightChopInline(1, EAllowShrinking::No);
					}
					int32 v=0;
					LexFromString(v, *Entity);
					Result += TCHAR(v);
				}
			}
			else
			{
				return Builder->SetLastError(FString::Printf(TEXT("Unhandled XML entity &\"%s\"; in line %d"), *Entity, XmlLineNumber), ERRCODE_DASH_MPD_PARSER_XML_BAD_ENTITY);
			}
		}
	}
	InOutString = Result;
	return true;
}


static bool ParseStringAttribute(FString& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	OutValue = AttributeValue;
	return RemoveXMLEntities(Builder, OutValue, XmlLineNumber);
}

static bool ParseBooleanAttribute(bool& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	if (StringHelpers::StringEquals(CONSTVAL(true), AttributeValue))
	{
		OutValue = true;
		return true;
	}
	else if (StringHelpers::StringEquals(CONSTVAL(false), AttributeValue))
	{
		OutValue = false;
		return true;
	}
	return false;
}

static bool ParseBooleanAttribute(TMediaOptionalValue<bool>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	bool v;
	if (ParseBooleanAttribute(v, Builder, AttributeName, AttributeValue, XmlLineNumber))
	{
		OutValue.Set(v);
		return true;
	}
	return false;
}

static bool ParseIntAttribute(int64& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	int64 v;
	LexFromString(v, AttributeValue);
	OutValue = v;
	return true;
}

static bool ParseIntAttribute(TMediaOptionalValue<int64>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	int64 v;
	ParseIntAttribute(v, Builder, AttributeName, AttributeValue, XmlLineNumber);
	OutValue.Set(v);
	return true;
}

static bool ParseIntAttribute(TMediaOptionalValue<int32>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	int64 v;
	ParseIntAttribute(v, Builder, AttributeName, AttributeValue, XmlLineNumber);
	OutValue.Set((int32) v);
	return true;
}

static bool ParseIntAttribute(int32& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	int64 v;
	LexFromString(v, AttributeValue);
	OutValue = (int32) v;
	return true;
}

static bool ParseUIntAttribute(uint64& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	uint64 v;
	LexFromString(v, AttributeValue);
	OutValue = v;
	return true;
}

static bool ParseUIntAttribute(TMediaOptionalValue<uint64>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	uint64 v;
	ParseUIntAttribute(v, Builder, AttributeName, AttributeValue, XmlLineNumber);
	OutValue.Set(v);
	return true;
}

static bool ParseUIntAttribute(TMediaOptionalValue<uint32>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	uint64 v;
	ParseUIntAttribute(v, Builder, AttributeName, AttributeValue, XmlLineNumber);
	OutValue.Set((uint32) v);
	return true;
}

static bool ParseDoubleAttribute(FTimeFraction& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	FTimeFraction t;
	t.SetFromFloatString(FString(AttributeValue));
	if (t.IsValid())
	{
		OutValue = t;
		return true;
	}
	return false;
}

static bool ParseDoubleAttribute(FTimeValue& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	FTimeFraction t;
	t.SetFromFloatString(FString(AttributeValue));
	if (t.IsValid())
	{
		OutValue.SetFromTimeFraction(t);
		return true;
	}
	return false;
}

static bool ParseDoubleAttribute(double& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	double v;
	LexFromString(v, AttributeValue);
	OutValue = v;
	return true;
}

static bool ParseDoubleAttribute(TMediaOptionalValue<double>& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	double v;
	LexFromString(v, AttributeValue);
	OutValue.Set(v);
	return true;
}


static bool ParseConditionalUIntAttribute(IDashMPDElement::FBoolUInt64& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	if (ParseBooleanAttribute(OutValue.bBoolean, Builder, AttributeName, AttributeValue, XmlLineNumber))
	{
		return true;
	}
	uint64 v = 0;
	if (ParseUIntAttribute(v, Builder, AttributeName, AttributeValue, XmlLineNumber))
	{
		OutValue.Number.Set(v);
		return true;
	}
	return false;
}

static bool ParseDuration(FTimeValue& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	if (!ISO8601::ParseDuration(OutValue, AttributeValue))
	{
		return Builder->SetLastError(FString::Printf(TEXT("Failed to parse time attribute '@%s'=%s of element <%s> in line %d"), AttributeName, AttributeValue, *Builder->GetCurrentElementName(), XmlLineNumber), ERRCODE_DASH_MPD_PARSER_BAD_TIME_VALUE);
	}
	return true;
}

static bool ParseDateTime(FTimeValue& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	if (!ISO8601::ParseDateTime(OutValue, FString(AttributeValue)))
	{
		return Builder->SetLastError(FString::Printf(TEXT("Failed to parse datetime attribute '@%s'=%s of element <%s> in line %d"), AttributeName, AttributeValue, *Builder->GetCurrentElementName(), XmlLineNumber), ERRCODE_DASH_MPD_PARSER_BAD_TIME_VALUE);
	}
	return true;
}

static bool ParseRatioAttribute(FTimeFraction& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	FString s(AttributeValue);
	int32 ColonPos = s.Find(TEXT(":"));
	if (ColonPos != INDEX_NONE && ColonPos != 0 && ColonPos != s.Len())
	{
		int64 n=1;
		uint32 d=1;
		LexFromString(n, *s.Mid(0, ColonPos));
		LexFromString(d, *s.Mid(ColonPos+1));
		OutValue.SetFromND(n, d);
		return true;
	}
	return false;
}

static bool ParseFramerateAttribute(FTimeFraction& OutValue, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	// Either a single decimal or a fraction like "x/y" with x and y being decimals.
	FString s(AttributeValue);
	int32 SlashPos = s.Find(TEXT("/"));
	if (SlashPos == INDEX_NONE)
	{
		int64 n;
		LexFromString(n, AttributeValue);
		OutValue.SetFromND(n, 1);
	}
	else
	{
		int64 n;
		uint32 d;
		LexFromString(n, *s.Mid(0, SlashPos));
		LexFromString(d, *s.Mid(SlashPos+1));
		OutValue.SetFromND(n, d);
	}
	return true;
}

static bool ParseStringArray(TArray<FString>& OutArray, const TCHAR* Delimiter, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	// Note: We first break the string on the delimiter and then remove the XML entities on the individual parts.
	//       This allows for the inclusion of a space in the value itself if it is using an &#32 escape.
	FString(AttributeValue).ParseIntoArray(OutArray, Delimiter, true);
	for(int32 i=0; i<OutArray.Num(); ++i)
	{
		OutArray[i].TrimStartAndEndInline();
		if (!RemoveXMLEntities(Builder, OutArray[i], XmlLineNumber))
		{
			return false;
		}
	}
	return true;
}

static bool ParseUIntArray(TArray<uint64>& OutArray, const TCHAR* Delimiter, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	TArray<FString> Temp;
	FString(AttributeValue).ParseIntoArray(Temp, Delimiter, true);
	for(int32 i=0; i<Temp.Num(); ++i)
	{
		uint64 l;
		if (!ParseUIntAttribute(l, Builder, AttributeName, *Temp[i], XmlLineNumber))
		{
			return false;
		}
		OutArray.Add(l);
	}
	return true;
}

static bool ParseUIntArray(TArray<uint32>& OutArray, const TCHAR* Delimiter, FManifestParserDASH* Builder, const TCHAR* AttributeName, const TCHAR* AttributeValue, int32 XmlLineNumber)
{
	TArray<FString> Temp;
	FString(AttributeValue).ParseIntoArray(Temp, Delimiter, true);
	for(int32 i=0; i<Temp.Num(); ++i)
	{
		uint64 l;
		if (!ParseUIntAttribute(l, Builder, AttributeName, *Temp[i], XmlLineNumber))
		{
			return false;
		}
		OutArray.Add((uint32)l);
	}
	return true;
}


} // namespace anonymous


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

namespace IManifestParserDASH
{
namespace
{
	FString RemoveNamespacePrefix(const FString& InString, bool bRemoveNS)
	{
		int32 ColonPos;
		if (bRemoveNS && InString.FindLastChar(TCHAR(':'), ColonPos))
		{
			return InString.RightChop(ColonPos + 1);
		}
		return InString;
	}

	FString MakeJSONElement(const FString& Key, const FString& Value, bool bRemoveNamespaces, const TCHAR* Prefix)
	{
		return FString::Printf(TEXT("\"%s%s\":\"%s\""), Prefix, *RemoveNamespacePrefix(Key, bRemoveNamespaces), *Value.ReplaceQuotesWithEscapedQuotes());
	}

	FString MakeJSONAttribute(const IDashMPDElement::FXmlAttribute& Attribute, bool bRemoveNamespaces, const TCHAR* Prefix)
	{
		return MakeJSONElement(Attribute.GetName(), Attribute.GetValue(), bRemoveNamespaces, Prefix);
	}

	void BuildJSONFromElementRec(FString& OutJSON, TSharedPtrTS<IDashMPDElement> InElement, bool bRemoveNamespaces, bool bForce1ElementArrays, bool bTerseObjects, const TCHAR* AttributePrefix, const TCHAR* TextPropertyName)
	{
		const TArray<TSharedPtrTS<FDashMPD_OtherType>>& Children = InElement->GetOtherChildren();
		const TArray<IDashMPDElement::FXmlAttribute>& Attributes = InElement->GetOtherAttributes();
		bool bCanBeTerse = Children.Num() == 0 && Attributes.Num() == 0;

		bool bNeedElementBraces = !bCanBeTerse || !bTerseObjects;

		if (bNeedElementBraces)
		{
			OutJSON.AppendChar(TCHAR('{'));
		}

		// Add attributes
		for(int32 i=0; i<Attributes.Num(); ++i)
		{
			OutJSON.Append(MakeJSONAttribute(Attributes[i], bRemoveNamespaces, AttributePrefix));
			if (i+1 <Attributes.Num())
			{
				OutJSON.AppendChar(TCHAR(','));
			}
		}

		// Add element data if it exists.
		bool bNeedComma = Attributes.Num() > 0;
		if (InElement->GetData().Len())
		{
			if (bNeedComma)
			{
				OutJSON.AppendChar(TCHAR(','));
			}
			if (bNeedElementBraces)
			{
				OutJSON.Append(MakeJSONElement(TextPropertyName, InElement->GetData(), false, TEXT("")));
			}
			else
			{
				OutJSON.Append(FString::Printf(TEXT("\"%s\""), *InElement->GetData().ReplaceQuotesWithEscapedQuotes()));
			}
			bNeedComma = true;
		}
		else if (!bNeedElementBraces)
		{
			// Need to add an empty string
			OutJSON.Append(TEXT("\"\""));
		}


		// Collect the children into a "map" in order to create a JSON array for those that have the same name.
		// As we need to maintain string case we can't use a TMap to do this.
		struct FCaseSensitiveChildElements
		{
			FString Key;
			TArray<TSharedPtrTS<FDashMPD_OtherType>> List;
		};
		TArray<FCaseSensitiveChildElements> ChildrenMap;
		for(int32 i=0; i<Children.Num(); ++i)
		{
			FString CaseKey = RemoveNamespacePrefix(Children[i]->GetName(), bRemoveNamespaces);
			bool bFound = false;
			for(int32 j=0; j<ChildrenMap.Num(); ++j)
			{
				if (ChildrenMap[j].Key.Equals(CaseKey, ESearchCase::CaseSensitive))
				{
					bFound = true;
					ChildrenMap[j].List.Add(Children[i]);
					break;
				}
			}
			if (!bFound)
			{
				FCaseSensitiveChildElements NewCaseSensitiveList;
				NewCaseSensitiveList.Key = CaseKey;
				NewCaseSensitiveList.List.Add(Children[i]);
				ChildrenMap.Emplace(MoveTemp(NewCaseSensitiveList));
			}
		}

		for(int32 nC=0; nC<ChildrenMap.Num(); ++nC)
		{
			if (bNeedComma)
			{
				OutJSON.AppendChar(TCHAR(','));
			}
			bNeedComma = true;
			const TArray<TSharedPtrTS<FDashMPD_OtherType>>& ArrayElements = ChildrenMap[nC].List;
			int32 NumArrayElements = ArrayElements.Num();
			if (NumArrayElements == 1 && !bForce1ElementArrays)
			{
				OutJSON.Append(FString::Printf(TEXT("\"%s\":"), *ChildrenMap[nC].Key));
			}
			else
			{
				OutJSON.Append(FString::Printf(TEXT("\"%s\":["), *ChildrenMap[nC].Key));
			}
			for(int32 i=0; i<NumArrayElements; ++i)
			{
				BuildJSONFromElementRec(OutJSON, ArrayElements[i], bRemoveNamespaces, bForce1ElementArrays, bTerseObjects, AttributePrefix, TextPropertyName);
				if (i+1 < NumArrayElements)
				{
					OutJSON.AppendChar(TCHAR(','));
				}
			}
			if (!(NumArrayElements == 1 && !bForce1ElementArrays))
			{
				OutJSON.AppendChar(TCHAR(']'));
			}
		}

		if (bNeedElementBraces)
		{
			OutJSON.AppendChar(TCHAR('}'));
		}
	}
}

void BuildJSONFromCustomElement(FString& OutJSON, TSharedPtrTS<IDashMPDElement> InElement, bool bIncludeStartElement, bool bRemoveNamespaces, bool bForce1ElementArrays, bool bTerseObjects, const TCHAR* AttributePrefix, const TCHAR* TextPropertyName)
{
	check(AttributePrefix);
	check(TextPropertyName);
	if (InElement.IsValid())
	{
		if (bIncludeStartElement)
		{
			OutJSON.Append(FString::Printf(TEXT("{\"%s\":"), *RemoveNamespacePrefix(InElement->GetName(), bRemoveNamespaces)));
		}

		BuildJSONFromElementRec(OutJSON, InElement, bRemoveNamespaces, bForce1ElementArrays, bTerseObjects, AttributePrefix, TextPropertyName);

		if (bIncludeStartElement)
		{
			OutJSON.AppendChar(TCHAR('}'));
		}
	}
	else
	{
		OutJSON.Append(TEXT("{}"));
	}
}

}

} // namespace Electra

