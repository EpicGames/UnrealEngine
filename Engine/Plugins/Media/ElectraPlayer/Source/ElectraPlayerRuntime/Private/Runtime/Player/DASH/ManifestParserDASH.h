// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlayerSessionServices.h"
#include "ErrorDetail.h"
#include "MPDElementsDASH.h"


namespace Electra
{

namespace IManifestParserDASH
{
FErrorDetail BuildFromMPD(FDashMPD_RootEntities& OutRootEntities, TArray<TWeakPtrTS<IDashMPDElement>>& OutXLinkElements, TCHAR* InOutMPDXML, const TCHAR* InExpectedRootElement, IPlayerSessionServices* InPlayerSessionServices);

void BuildJSONFromCustomElement(FString& OutJSON, TSharedPtrTS<IDashMPDElement> InElement, bool bIncludeStartElement, bool bRemoveNamespaces, bool bForce1ElementArrays, bool bTerseObjects, const TCHAR* AttributePrefix, const TCHAR* TextPropertyName);

};

} // namespace Electra

