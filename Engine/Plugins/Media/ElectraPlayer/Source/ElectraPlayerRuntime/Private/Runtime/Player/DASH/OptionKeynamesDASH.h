// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{
namespace DASH
{

const FName OptionKeyMPDLoadConnectTimeout(TEXT("mpd_connection_timeout"));					//!< (FTimeValue) value specifying connection timeout fetching the MPD
const FName OptionKeyMPDLoadNoDataTimeout(TEXT("mpd_nodata_timeout"));						//!< (FTimeValue) value specifying no-data timeout fetching the MPD
const FName OptionKeyMPDReloadConnectTimeout(TEXT("mpd_update_connection_timeout"));		//!< (FTimeValue) value specifying connection timeout fetching the MPD repeatedly
const FName OptionKeyMPDReloadNoDataTimeout(TEXT("mpd_update_nodata_timeout"));				//!< (FTimeValue) value specifying no-data timeout fetching the MPD repeatedly

const FName OptionKeyInitSegmentConnectTimeout(TEXT("initsegment_connection_timeout"));		//!< (FTimeValue) value specifying connection timeout fetching an init segment
const FName OptionKeyInitSegmentNoDataTimeout(TEXT("initsegment_nodata_timeout"));			//!< (FTimeValue) value specifying no-data timeout fetching an init segment
const FName OptionKeyMediaSegmentConnectTimeout(TEXT("mediasegment_connection_timeout"));	//!< (FTimeValue) value specifying connection timeout fetching a media segment
const FName OptionKeyMediaSegmentNoDataTimeout(TEXT("mediasegment_nodata_timeout"));		//!< (FTimeValue) value specifying no-data timeout fetching a media segment

const FName OptionKey_MinTimeBetweenMPDUpdates(TEXT("dash:min_mpd_update_interval"));		//!< (FTimeValue) value limiting the time between MPD updates unless these are required updates.

const FName OptionKey_CurrentCDN(TEXT("dash:current_cdn"));									//!< A string giving the currently selected CDN to be used with <BaseURL@serviceLocation> elements.

const FName OptionKey_LatencyReferenceId(TEXT("dash:latency_refId"));						//!< The Id of the <ProducerReferenceTime> element used by <ServiceDescription><Latency>

const FString HTTPHeaderOptionName(TEXT("MPEG-DASH-Param"));									//!< HTTP request header name carrying optional DASH specific parameters.

}

} // namespace Electra


