// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/AdaptiveStreamingPlayerEvents.h"
#include "Utilities/StringHelpers.h"
#include "Misc/Base64.h"

namespace Electra
{

namespace DASH
{

class FPlayerEvent : public IAdaptiveStreamingPlayerAEMSEvent
{
public:
	virtual ~FPlayerEvent() = default;


	void SetOrigin(EOrigin InOrigin)
	{ Origin = InOrigin; }
	void SetSchemeIdUri(FString InSchemeIdUri)
	{ SchemeIdUri = MoveTemp(InSchemeIdUri); }
	void SetValue(FString InValue)
	{ Value = MoveTemp(InValue); }
	void SetID(FString InID)
	{ ID = MoveTemp(InID); }
	void SetPresentationTime(FTimeValue InPresentationTime)
	{ PresentationTime = MoveTemp(InPresentationTime); }
	void SetDuration(FTimeValue InDuration)
	{ Duration = MoveTemp(InDuration); }
	void SetMessageData(const FString& InData, bool bDecodeBase64)
	{
		if (bDecodeBase64)
		{
			if (!FBase64::Decode(InData, MessageData))
			{
				MessageData.Empty();
			}
		}
		else
		{
			StringHelpers::StringToArray(MessageData, InData);
		}
	}
	void SetMessageData(TConstArrayView<uint8> InData)
	{
		MessageData = InData;
	}

	void SetPeriodID(const FString& InPeriodID)
	{
		PeriodID = InPeriodID;
	}
	FString GetPeriodID() const
	{
		return PeriodID;
	}

	virtual EOrigin GetOrigin() const override
	{ return Origin; }
	virtual FString GetSchemeIdUri() const override
	{ return SchemeIdUri; }
	virtual FString GetValue() const override
	{ return Value; }
	virtual FString GetID() const override
	{ return ID; }
	virtual FTimeValue GetPresentationTime() const override
	{ return PresentationTime; }
	virtual FTimeValue GetDuration() const override
	{ return Duration; }
	virtual const TArray<uint8>& GetMessageData() const override
	{ return MessageData; }

private:
	EOrigin Origin;
	FString SchemeIdUri;
	FString Value;
	FString ID;
	FTimeValue PresentationTime;
	FTimeValue Duration;
	TArray<uint8> MessageData;
	FString PeriodID;
};


}

} // namespace Electra


