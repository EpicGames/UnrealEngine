// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{

namespace DASH
{

namespace Schemes
{

namespace TimingSources
{
    const TCHAR* const Scheme_urn_mpeg_dash_utc_httphead2014 = TEXT("urn:mpeg:dash:utc:http-head:2014");
    const TCHAR* const Scheme_urn_mpeg_dash_utc_httpxsdate2014 = TEXT("urn:mpeg:dash:utc:http-xsdate:2014");
    const TCHAR* const Scheme_urn_mpeg_dash_utc_httpiso2014 = TEXT("urn:mpeg:dash:utc:http-iso:2014");
    const TCHAR* const Scheme_urn_mpeg_dash_utc_direct2014 = TEXT("urn:mpeg:dash:utc:direct:2014");
/* not currently supported
    const TCHAR* const Scheme_urn_mpeg_dash_utc_httpntp2014 = TEXT("urn:mpeg:dash:utc:http-ntp:2014");
    const TCHAR* const Scheme_urn_mpeg_dash_utc_ntp2014 = TEXT("urn:mpeg:dash:utc:ntp:2014");
    const TCHAR* const Scheme_urn_mpeg_dash_utc_sntp2014 = TEXT("urn:mpeg:dash:utc:sntp:2014");
*/
}

namespace ManifestEvents
{
    const TCHAR* const Scheme_urn_mpeg_dash_event_2012 = TEXT("urn:mpeg:dash:event:2012");
    const TCHAR* const Scheme_urn_mpeg_dash_event_callback_2015 = TEXT("urn:mpeg:dash:event:callback:2015");
    const TCHAR* const Scheme_urn_mpeg_dash_event_ttfn_2016 = TEXT("urn:mpeg:dash:event:ttfn:2016");
};

}

}

} // namespace Electra


