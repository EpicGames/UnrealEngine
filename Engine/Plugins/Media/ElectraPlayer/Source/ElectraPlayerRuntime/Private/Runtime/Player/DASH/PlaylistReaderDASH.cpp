// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderDASH.h"
#include "PlaylistReaderDASH_Internal.h"
#include "ManifestBuilderDASH.h"
#include "ManifestDASH.h"
#include "Utilities/HashFunctions.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/URLParser.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/AdaptiveStreamingPlayerEvents.h"
#include "Player/PlayerEntityCache.h"
#include "Player/DASH/OptionKeynamesDASH.h"
#include "Player/DASH/PlayerEventDASH.h"
#include "Player/DASH/PlayerEventDASH_Internal.h"
#include "Player/ContentSteeringHandler.h"
#include "Player/CMCDHandler.h"
#include "ElectraHTTPStream.h"
#include "Misc/DateTime.h"

#define ERRCODE_DASH_PARSER_ERROR							1
#define ERRCODE_DASH_MPD_DOWNLOAD_FAILED					2
#define ERRCODE_DASH_MPD_PARSING_FAILED						3
#define ERRCODE_DASH_MPD_UNSUPPORTED_DOCUMENT_ENCODING		4
#define ERRCODE_DASH_MPD_REMOTE_ENTITY_FAILED				5
#define ERRCODE_DASH_MPD_BAD_CONTENT_STEERING				6


DECLARE_CYCLE_STAT(TEXT("FPlaylistReaderDASH_WorkerThread"), STAT_ElectraPlayer_DASH_PlaylistWorker, STATGROUP_ElectraPlayer);


//#define DO_NOT_PERFORM_CONDITIONAL_GET

namespace Electra
{

/**
 * This class is responsible for downloading a DASH MPD and parsing it.
 */
class FPlaylistReaderDASH : public TSharedFromThis<FPlaylistReaderDASH, ESPMode::ThreadSafe>, public IPlaylistReaderDASH, public IAdaptiveStreamingPlayerAEMSReceiver
{
public:
	FPlaylistReaderDASH();
	void Initialize(IPlayerSessionServices* PlayerSessionServices);

	virtual ~FPlaylistReaderDASH();
	EMediaFormatType GetMediaFormatType() const override
	{ return EMediaFormatType::DASH; }
	void Close() override;
	void HandleOnce() override;
	const FString& GetPlaylistType() const override
	{
		static FString Type("dash");
		return Type;
	}
	void LoadAndParse(const FString& URL, const Playlist::FReplayEventParams& InReplayEventParams) override;
	FString GetURL() const override;
	TSharedPtrTS<IManifest> GetManifest() override;
	void AddElementLoadRequests(const TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& RemoteElementLoadRequests) override;
	void RequestMPDUpdate(EMPDRequestType InRequestType) override;
	void RequestClockResync() override;
	TSharedPtrTS<FManifestDASHInternal> GetCurrentMPD() override
	{
		return Manifest;
	}
	void SetStreamInbandEventUsage(EStreamType InStreamType, bool bUsesInbandDASHEvents) override;


	void OnMediaPlayerEventReceived(TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> InEvent, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode) override;

private:
	using FResourceLoadRequestPtr = TSharedPtrTS<FMPDLoadRequestDASH>;

	void OnHTTPResourceRequestComplete(TSharedPtrTS<FHTTPResourceRequest> InRequest);

	FErrorDetail GetXMLResponseString(FString& OutXMLString, FResourceLoadRequestPtr FromRequest);
	FErrorDetail GetResponseString(FString& OutString, FResourceLoadRequestPtr FromRequest);

	void InternalSetup();
	void InternalCleanup();

	void ExecutePendingRequests(const FTimeValue& TimeNow);
	void HandleCompletedRequests(const FTimeValue& TimeNow);
	void HandleStaticRequestCompletions(const FTimeValue& TimeNow);
	void CheckForMPDUpdate();

	void TriggerTimeSynchronization();

	void PostError(const FErrorDetail& Error);
	void PostError(const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);
	FErrorDetail CreateErrorAndLog(const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);

	void SetupRequestTimeouts(FResourceLoadRequestPtr InRequest);
	int32 CheckForRetry(FResourceLoadRequestPtr InRequest, int32 Error);

	void EnqueueResourceRequest(FResourceLoadRequestPtr InRequest);
	void EnqueueResourceRetryRequest(FResourceLoadRequestPtr InRequest, const FTimeValue& AtUTCTime);
	void EnqueueInitialXLinkRequests();

	void ManifestDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess);
	void UpdateCMCDParamsFromServiceDescriptions(const TArray<TSharedPtrTS<FDashMPD_ServiceDescriptionType>>& InServiceDescriptions);
	void FinishManifestBuildingAfterTimesync(bool bGotTheTime);
	void PrepareNextTimeSync();
	void ManifestUpdateDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess);
	void InitialMPDXLinkElementDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess);
	void ContentSteeringDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess);
	void ContentSteeringUpdateDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess);
	void TriggerContentSteeringUpdate(bool bInitial);
	void ContentSteeringUpdate(bool bInitial, FResourceLoadRequestPtr Request, bool bSuccess);

	void Timesync_httphead_Completed(FResourceLoadRequestPtr Request, bool bSuccess);
	void Timesync_httpxsdate_Completed(FResourceLoadRequestPtr Request, bool bSuccess);
	void Timesync_httpiso_Completed(FResourceLoadRequestPtr Request, bool bSuccess);
	void Callback_Completed(FResourceLoadRequestPtr Request, bool bSuccess);

	IPlayerSessionServices*									PlayerSessionServices = nullptr;
	FString													MPDURL;
	TArray<FURL_RFC3986::FQueryParam>						URLFragmentComponents;
	Playlist::FReplayEventParams							ReplayEventParams;
	bool													bIsInitialized = false;

	FCriticalSection										RequestsLock;
	TArray<FResourceLoadRequestPtr>							PendingRequests;
	TArray<FResourceLoadRequestPtr>							ActiveRequests;
	TArray<FResourceLoadRequestPtr>							CompletedRequests;
	FTimeValue												MinTimeBetweenUpdates;
	FTimeValue												NextMPDUpdateTime;
	FTimeValue												MostRecentMPDUpdateTime;
	bool													bIsMPDUpdateInProgress = false;
	struct FMPDUpdateRequest
	{
		IPlaylistReaderDASH::EMPDRequestType Type = IPlaylistReaderDASH::EMPDRequestType::MinimumUpdatePeriod;
		FString	PublishTime;
		FString	PeriodId;
		FTimeValue ValidUntil;
		FTimeValue NewDuration;
		FString NewMPD;
	};
	TArray<TSharedPtrTS<FMPDUpdateRequest>>					UpdateRequested;

	bool													bInbandEventByStreamType[3] { false, false, false };

	TUniquePtr<IManifestBuilderDASH>						Builder;

	TSharedPtrTS<FManifestDASHInternal>						Manifest;
	FErrorDetail											LastErrorDetail;

	TSharedPtrTS<FManifestDASH>								PlayerManifest;

	// Time synchronization.
	struct FInitialTimesync
	{
		FTimeValue UtcTimeThen;
		FTimeValue HttpRequestStart;
		FString HttpDateHeader;
		FString Utc_direct2014;
		TUniquePtr<HTTP::FConnectionInfo> ConnInfo;
		void Reset()
		{
			UtcTimeThen.SetToInvalid();
			HttpRequestStart.SetToInvalid();
			HttpDateHeader.Reset();
			Utc_direct2014.Reset();
			ConnInfo.Reset();
		}
	};
	FInitialTimesync										InitialTimesync;
	bool													bIsInitialTimesync = true;
	TArray<TSharedPtrTS<FDashMPD_DescriptorType>>			AttemptedTimesyncDescriptors;
	FTimeValue												NextTimeSyncTime;
	bool													bTimeSyncInProgress = false;


	// Warnings
	bool													bWarnedAboutNoUTCTimingElements = false;
	bool													bWarnedAboutUnsupportedUTCTimingElement = false;
};




/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtrTS<IPlaylistReader> IPlaylistReaderDASH::Create(IPlayerSessionServices* PlayerSessionServices)
{
	TSharedPtrTS<FPlaylistReaderDASH> PlaylistReader = MakeSharedTS<FPlaylistReaderDASH>();
	if (PlaylistReader)
	{
		PlaylistReader->Initialize(PlayerSessionServices);
	}
	return PlaylistReader;
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


FPlaylistReaderDASH::FPlaylistReaderDASH()
{
}

FPlaylistReaderDASH::~FPlaylistReaderDASH()
{
	Close();
}

FString FPlaylistReaderDASH::GetURL() const
{
	return MPDURL;
}

TSharedPtrTS<IManifest> FPlaylistReaderDASH::GetManifest()
{
	return PlayerManifest;
}

void FPlaylistReaderDASH::Initialize(IPlayerSessionServices* InPlayerSessionServices)
{
	PlayerSessionServices = InPlayerSessionServices;
}

void FPlaylistReaderDASH::Close()
{
	InternalCleanup();
}

void FPlaylistReaderDASH::HandleOnce()
{
	if (bIsInitialized)
	{
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_DASH_PlaylistWorker);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, DASH_PlaylistWorker);

		FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		// Check for time sync first. The time sync result and thus the clock gets changed in HandleCompletedRequest()
		// which may trigger a jump in time, making "Now" and NextTimeSyncTime obsolete until the next loop.
		if (NextTimeSyncTime.IsValid() && Now >= NextTimeSyncTime)
		{
			TriggerTimeSynchronization();
		}

		HandleCompletedRequests(Now);
		ExecutePendingRequests(Now);
		// Check if the MPD must be updated.
		CheckForMPDUpdate();
		// Check if a new steering manifest is needed
		if (PlayerSessionServices->GetContentSteeringHandler()->NeedToObtainNewSteeringManifestNow())
		{
			TriggerContentSteeringUpdate(false);
		}
	}
}


void FPlaylistReaderDASH::PostError(const FErrorDetail& Error)
{
	check(PlayerSessionServices);
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostError(Error);
	}
}

void FPlaylistReaderDASH::PostError(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	LastErrorDetail.Clear();
	LastErrorDetail.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	LastErrorDetail.SetFacility(Facility::EFacility::DASHMPDReader);
	LastErrorDetail.SetCode(InCode);
	LastErrorDetail.SetMessage(InMessage);
	PostError(LastErrorDetail);
}

FErrorDetail FPlaylistReaderDASH::CreateErrorAndLog(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	FErrorDetail err;
	err.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	err.SetFacility(Facility::EFacility::DASHMPDReader);
	err.SetCode(InCode);
	err.SetMessage(InMessage);
	check(PlayerSessionServices);
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::DASHMPDReader, IInfoLog::ELevel::Error, err.GetPrintable());
	}
	return err;
}


void FPlaylistReaderDASH::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::DASHMPDReader, Level, Message);
	}
}

void FPlaylistReaderDASH::LoadAndParse(const FString& URL, const Playlist::FReplayEventParams& InReplayEventParams)
{
	MPDURL = URL;
	ReplayEventParams = InReplayEventParams;
	InternalSetup();
}


void FPlaylistReaderDASH::CheckForMPDUpdate()
{
	// Get the time now and do not pass it in from the worker thread as the clock could have been
	// resynchronized to the server time in the meantime.
	FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();

	RequestsLock.Lock();
	bool bRequestNow = false;
	bool bTimedRequestAdded = false;

	// Check if the time to update the MPD through the MPD@minimumUpdatePeriod has come.
	if (NextMPDUpdateTime.IsValid())
	{
		// Limit the frequency of the updates.
		bool bIsPossible = !MostRecentMPDUpdateTime.IsValid() || Now - MostRecentMPDUpdateTime > MinTimeBetweenUpdates;
		if (NextMPDUpdateTime < Now && bIsPossible)
		{
			// When it's time to do the update add a request to the request queue. This gets handled
			// right away next thing. Doing it that way means we can put the handling into one spot.
			NextMPDUpdateTime.SetToInvalid();
			TSharedPtrTS<FMPDUpdateRequest> Request = MakeSharedTS<FMPDUpdateRequest>();
			Request->Type = IPlaylistReaderDASH::EMPDRequestType::MinimumUpdatePeriod;
			UpdateRequested.Emplace(MoveTemp(Request));
			bTimedRequestAdded = true;
		}
	}

	// Go over all currently pending update requests.
	while(UpdateRequested.Num())
	{
		TSharedPtrTS<FMPDUpdateRequest> Request = UpdateRequested[0];
		UpdateRequested.RemoveAt(0);
		if (Request->Type == IPlaylistReaderDASH::EMPDRequestType::MinimumUpdatePeriod ||
			Request->Type == IPlaylistReaderDASH::EMPDRequestType::GetLatestSegment)
		{
			// If the request to update is because we need the latest segment and there is currently
			// no MPD update by time we ask for one with the next check.
			if (Request->Type == IPlaylistReaderDASH::EMPDRequestType::GetLatestSegment)
			{
				if (!bTimedRequestAdded && !NextMPDUpdateTime.IsValid())
				{
					NextMPDUpdateTime.SetToZero();
				}
				continue;
			}

			if (!bIsMPDUpdateInProgress)
			{
				bRequestNow = true;
				/*
					If there is any stream receiving inband events we do not load the MPD through
					expiration of time. The events have sole control and responsibility here.

					Note: The expectation at the moment is that inband MPD event streams are only
						  used with Live streams and are carried in the audio stream instead of
						  each and every video stream. This here being the worker thread to
						  refresh the MPD and downloading remote entities (index segments, xlink
						  elements and such) it has no immediate knowledge which streams are
						  actively used in the playing session and whether these are carrying
						  inband event streams or not. Instead it relies on the segment downloader
						  to tell it if a particular type of stream (video, audio, etc.) is
						  using inband events as each segment is downloaded.
						  Meaning that if a stream type stops downloading (ie it reaching its end)
						  it cannot clear this flag and we assume we are still receiving events
						  from it. Hence the expectation this is used with Live streams that
						  just do not end and keep running.
						  In addition this is only done when MPD@minimumUpdatePeriod is zero
						  as per DASH-IF-IOP v4.3 Setion 4.5.
				*/
				FTimeValue mup = Manifest.IsValid() && Manifest->GetMPDRoot().IsValid() ? Manifest->GetMPDRoot()->GetMinimumUpdatePeriod() : FTimeValue::GetInvalid();
				// check video and audio only
				if ((bInbandEventByStreamType[0] || bInbandEventByStreamType[1]) && mup == FTimeValue::GetZero())
				{
					bRequestNow = false;
				}
			}
		}
		else if (Request->Type == IPlaylistReaderDASH::EMPDRequestType::EventMessage)
		{
			// Unconditional update?
			if (Request->PublishTime.IsEmpty() && !Request->NewDuration.IsValid() && !Request->ValidUntil.IsValid())
			{
				// Note: We could check the publish time against the current MPD's publish time to see if the event
				//       still applies, but that seem superfluous considering we're only reloading the MPD.
				bRequestNow = true;
			}
			else
			{
				// Does this terminate the presentation at a specified time?
				if (Request->ValidUntil.IsValid() && Request->NewDuration.IsValid())
				{
					bool bEventApplies = true;

					if (Manifest.IsValid())
					{
						// Have to check the event publish time against the current MPD publish time to see if the event still applies
						// to the current MPD.
						TSharedPtrTS<const FDashMPD_MPDType> MPDRoot = Manifest->GetMPDRoot();
						FTimeValue CurrentPUBT = MPDRoot.IsValid() ? MPDRoot->GetPublishTime() : FTimeValue::GetInvalid();
						FTimeValue EventPUBT;
						ISO8601::ParseDateTime(EventPUBT, Request->PublishTime);
						// We take it that if neither PUBT is valid the event is still to be used.
						// Only if the times are given and the event PUBT has already passed we ignore the event.
						if (CurrentPUBT.IsValid() && EventPUBT.IsValid() && CurrentPUBT > EventPUBT)
						{
							bEventApplies = false;
						}
						if (bEventApplies)
						{
							PlayerSessionServices->SetPlaybackEnd(Request->ValidUntil + Request->NewDuration, IPlayerSessionServices::EPlayEndReason::EndAll, nullptr);
							Manifest->EndPresentationAt(Request->ValidUntil + Request->NewDuration, Request->PeriodId);
						}
					}
					if (bEventApplies)
					{
						// Set internal times such that no MPD reloads will trigger based on time.
						NextMPDUpdateTime.SetToPositiveInfinity();
						MostRecentMPDUpdateTime.SetToPositiveInfinity();
						// Clear out all current requests.
						bRequestNow = false;
						UpdateRequested.Empty();
						// Send a fake update message to the player to have it evaluate the media timeline again.
						PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(FErrorDetail(), nullptr, Playlist::EListType::Main, Playlist::ELoadType::Update, 0));
					}
				}
				else
				{
					// The event could provide a new MPD to be used. This case is not supported yet though.
					if (!Request->NewMPD.IsEmpty())
					{
						// Something to implement in the future.
						LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("DASH events providing a new MPD are not supported yet. Triggering an MPD reload instead.")));
					}
					// Just trigger an MPD reload instead.
					// This is also where we a failed segment download of a segment with an inband event will arrive.
					bRequestNow = true;
				}
			}
		}
		else
		{
			check(!"Unhandled update type");
		}
	}

	RequestsLock.Unlock();
	if (bRequestNow)
	{
		TSharedPtrTS<const FDashMPD_MPDType> MPDRoot = Manifest.IsValid() ? Manifest->GetMPDRoot() : nullptr;
		if (MPDRoot.IsValid())
		{
			// The URL query might need to be changed. Look for the UrlQuery properties.
			TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>> UrlQueries;
			DASHUrlHelpers::GetAllHierarchyUrlQueries(UrlQueries, MPDRoot, DASHUrlHelpers::EUrlQueryRequestType::Mpd, false);

			DASHUrlHelpers::FDASHMediaURL RootDocument;
			RootDocument.URL = MPDRoot->GetDocumentURL();
			FURL_RFC3986 RootDocURL;
			RootDocURL.Parse(RootDocument.URL);
			// The MPD has no @serviceLocation or the DVB properties. According to DVB-DASH the @serviceLocation is the URL itself and the attributes default to 1.
			RootDocument.CDN = MPDRoot->IsDVBDash() ? RootDocument.URL : FString();
			RootDocument.DVBpriority = 1;
			RootDocument.DVBweight = 1;
			// Build a list we can pass to the steering handler.
			TArray<FContentSteeringHandler::FCandidateURL> SteeringCandidates;

			// See if there are any <Location> elements on the current MPD that tell us from where to get the update.
			// As per A.11 no <BaseURL> elements affect the construction of the new document URL. At most, if it is
			// a relative URL it will be resolved against the current document URL.
			const TArray<TSharedPtrTS<FDashMPD_OtherType>>& Locations = MPDRoot->GetLocations();
			// There can be several new locations, but the elements have no DASH defined attributes to differentiate them
			// as of the ISO/IEC 23009-1:2022 standard. For content steering ETSI TS 103 998 V1.1.1 (2024-01)
			// proposes the addition of the @serviceLocation attribute used with the <BaseURL> element.
			for(int32 i=0; i<Locations.Num(); ++i)
			{
				FContentSteeringHandler::FCandidateURL& sc = SteeringCandidates.Emplace_GetRef();
				sc.MediaURL.URL = Locations[i]->GetData();
				const TArray<IDashMPDElement::FXmlAttribute>& OtherAttributes = Locations[i]->GetOtherAttributes();
				for(auto &Attribute : OtherAttributes)
				{
					if (Attribute.GetName().Equals(TEXT("serviceLocation")))
					{
						sc.MediaURL.CDN = Attribute.GetValue();
						break;
					}
				}
				// The URL given in the location must be an absolute URL.
				if (!DASHUrlHelpers::IsAbsoluteURL(sc.MediaURL.URL))
				{
					LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("<MPD.Location> URI is not an absolute URL, ignoring!")));
					SteeringCandidates.Pop();
					continue;
				}

				// Apply any of the Annex I UrlQuery elements
				FString AnnexIRequestHeader;
				DASHUrlHelpers::ApplyUrlQueries(PlayerSessionServices, RootDocument.URL, sc.MediaURL.URL, AnnexIRequestHeader, UrlQueries);
				sc.AdditionalParams.Set(DASHUrlHelpers::SteerOption_AnnexIRequestHeader, FVariantValue(AnnexIRequestHeader));

				// If there is no service location then as with DVB-DASH the URL is to be used for that.
				if (sc.MediaURL.CDN.IsEmpty() && MPDRoot->IsDVBDash())
				{
					sc.MediaURL.CDN = sc.MediaURL.URL;
				}
			}

			FContentSteeringHandler::FSelectedCandidateURL ChosenLocation;
			// Perform steering evaluation only when there are candidates, otherwise
			// fall back to the original MPD URL.
			if (SteeringCandidates.Num())
			{
				// Have the steering handler select the desired CDN, potentially cloning an existing one on demand.
				FString SteeringMessage;
				ChosenLocation = PlayerSessionServices->GetContentSteeringHandler()->SelectBestCandidateFrom(SteeringMessage, FContentSteeringHandler::ESelectFor::Playlist, SteeringCandidates);
				if (ChosenLocation.MediaURL.URL.IsEmpty())
				{
					LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("No MPD location for reloading the MPD selected by content steering, falling back to original URL.")));
				}
			}
			if (ChosenLocation.MediaURL.URL.IsEmpty())
			{
				ChosenLocation.MediaURL = RootDocument;
				FString AnnexIRequestHeader;
				DASHUrlHelpers::ApplyUrlQueries(PlayerSessionServices, RootDocument.URL, ChosenLocation.MediaURL.URL, AnnexIRequestHeader, UrlQueries);
				ChosenLocation.AdditionalParams.Set(DASHUrlHelpers::SteerOption_AnnexIRequestHeader, FVariantValue(AnnexIRequestHeader));
				ChosenLocation.SteeringID = -1;
			}

			FString RequestHeader = ChosenLocation.AdditionalParams.GetValue(DASHUrlHelpers::SteerOption_AnnexIRequestHeader).SafeGetFString();
			FString ETag = MPDRoot->GetETag();

			TSharedPtrTS<FMPDLoadRequestDASH> PlaylistLoadRequest = MakeSharedTS<FMPDLoadRequestDASH>();
			PlaylistLoadRequest->LoadType = FMPDLoadRequestDASH::ELoadType::MPDUpdate;
			PlaylistLoadRequest->Url = ChosenLocation.MediaURL;
			PlaylistLoadRequest->SteeringID = ChosenLocation.SteeringID;
			if (RequestHeader.Len())
			{
				PlaylistLoadRequest->Headers.Emplace(HTTP::FHTTPHeader(DASH::HTTPHeaderOptionName, RequestHeader));
			}
#ifndef DO_NOT_PERFORM_CONDITIONAL_GET
			// As per the standard we should perform conditional GET requests.
			if (ETag.Len())
			{
				PlaylistLoadRequest->Headers.Emplace(HTTP::FHTTPHeader(TEXT("If-None-Match"), ETag));
			}
#endif
			PlaylistLoadRequest->CompleteCallback.BindThreadSafeSP(AsShared(), &FPlaylistReaderDASH::ManifestUpdateDownloadCompleted);

			bIsMPDUpdateInProgress = true;
			EnqueueResourceRequest(MoveTemp(PlaylistLoadRequest));
		}
	}
}


void FPlaylistReaderDASH::OnHTTPResourceRequestComplete(TSharedPtrTS<FHTTPResourceRequest> InRequest)
{
	FResourceLoadRequestPtr lr = StaticCastSharedPtr<FMPDLoadRequestDASH>(InRequest->GetObject());
	if (lr.IsValid())
	{
		FScopeLock lock(&RequestsLock);
		CompletedRequests.Emplace(lr);
		ActiveRequests.Remove(lr);
	}
}


void FPlaylistReaderDASH::EnqueueResourceRequest(FResourceLoadRequestPtr InRequest)
{
	FScopeLock lock(&RequestsLock);
	PendingRequests.Emplace(MoveTemp(InRequest));
}

void FPlaylistReaderDASH::EnqueueResourceRetryRequest(FResourceLoadRequestPtr InRequest, const FTimeValue& AtUTCTime)
{
	FScopeLock lock(&RequestsLock);
	InRequest->ExecuteAtUTC = AtUTCTime;
	++InRequest->Attempt;
	PendingRequests.Emplace(MoveTemp(InRequest));
}

void FPlaylistReaderDASH::EnqueueInitialXLinkRequests()
{
	check(Manifest.IsValid());
	TArray<TWeakPtrTS<FMPDLoadRequestDASH>> RemoteElementLoadRequests;
	Manifest->GetRemoteElementLoadRequests(RemoteElementLoadRequests);
	FScopeLock lock(&RequestsLock);
	for(int32 i=0; i<RemoteElementLoadRequests.Num(); ++i)
	{
		TSharedPtrTS<FMPDLoadRequestDASH> LReq = RemoteElementLoadRequests[i].Pin();
		if (LReq.IsValid())
		{
			LReq->OwningManifest = Manifest;
			LReq->CompleteCallback.BindThreadSafeSP(AsShared(), &FPlaylistReaderDASH::InitialMPDXLinkElementDownloadCompleted);
			PendingRequests.Emplace(MoveTemp(LReq));
		}
	}
}

void FPlaylistReaderDASH::AddElementLoadRequests(const TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& RemoteElementLoadRequests)
{
	if (RemoteElementLoadRequests.Num())
	{
		FScopeLock lock(&RequestsLock);
		for(int32 i=0; i<RemoteElementLoadRequests.Num(); ++i)
		{
			TSharedPtrTS<FMPDLoadRequestDASH> LReq = RemoteElementLoadRequests[i].Pin();
			if (LReq.IsValid())
			{
				LReq->OwningManifest = Manifest;
				PendingRequests.Emplace(MoveTemp(LReq));
			}
		}
	}
}

void FPlaylistReaderDASH::RequestMPDUpdate(IPlaylistReaderDASH::EMPDRequestType InRequestType)
{
	// This method must not be used with event messages since it does not have all the required values.
	check(InRequestType != 	IPlaylistReaderDASH::EMPDRequestType::EventMessage);

	TSharedPtrTS<FMPDUpdateRequest> Request = MakeSharedTS<FMPDUpdateRequest>();
	Request->Type = InRequestType;

	FScopeLock lock(&RequestsLock);
	UpdateRequested.Emplace(MoveTemp(Request));
}

void FPlaylistReaderDASH::RequestClockResync()
{
	NextTimeSyncTime.SetToZero();
}


void FPlaylistReaderDASH::SetStreamInbandEventUsage(EStreamType InStreamType, bool bUsesInbandDASHEvents)
{
	switch(InStreamType)
	{
		case EStreamType::Video:
		{
			bInbandEventByStreamType[0] = bUsesInbandDASHEvents;
			break;
		}
		case EStreamType::Audio:
		{
			bInbandEventByStreamType[1] = bUsesInbandDASHEvents;
			break;
		}
		default:
		{
			break;
		}
	}
}


void FPlaylistReaderDASH::TriggerTimeSynchronization()
{
	if (bTimeSyncInProgress)
	{
		return;
	}

	// We're doing a sync now. Clear the next time.
	NextTimeSyncTime.SetToInvalid();

	if (Manifest.IsValid())
	{
		// Time sync is only necessary when there is either an MPD@availabilityStartTime or MPD@type is dynamic.
		if (Manifest->UsesAST() || !Manifest->IsStaticType() || Manifest->HasInjectedTimingSources())
		{
			TSharedPtrTS<FDashMPD_MPDType> MPDRoot = Manifest->GetMPDRoot();
			if (MPDRoot.IsValid())
			{
				bool bFoundSupportedElement = false;
				FString MPDDocumentURL = MPDRoot->GetDocumentURL();
				const TArray<TSharedPtrTS<FDashMPD_DescriptorType>>& UTCTimings = MPDRoot->GetUTCTimings();

				// Emit a one-time only warning if there is no <UTCTiming> element in the MPD when there should be.
				if (UTCTimings.Num() == 0 && !bWarnedAboutNoUTCTimingElements)
				{
					bWarnedAboutNoUTCTimingElements = true;
					LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("No <UTCTiming> element found in MPD. This could result in playback failures. For detailed validation of your MPD please use https://conformance.dashif.org/")));
				}

				/*
					We go over the UTC timing elements in two passes.
					In the first we look only for the (discouraged) urn:mpeg:dash:utc:direct:2014 scheme because
					if it is available we can set it immediately. Its value is baked into the manifest and is therefore
					only really valid right at the time the MPD was fetched.
					If we try other schemes first that involve a network request and those fail then there will have
					passed that much time that the hardcoded value in the MPD will no longer make sense to associate
					with the current system time.
				*/
				for(int32 i=0; i<UTCTimings.Num(); ++i)
				{
					if (UTCTimings[i]->GetSchemeIdUri().Equals(DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_direct2014))
					{
						InitialTimesync.Utc_direct2014 = UTCTimings[i]->GetValue();
						// Remove this timing element. It can not be used indefinitely and we best get rid of it now.
						MPDRoot->RemoveUTCTimingElement(UTCTimings[i]);
						--i;
						bFoundSupportedElement = true;
					}
				}
				// 2nd pass. Use the more appropriate timing elements.
				for(int32 i=0; i<UTCTimings.Num(); ++i)
				{
					auto GetRequestURL = [MPDDocumentURL](FString ListOfURLs, int32 Index) -> FString
					{
						TArray<FString> URLs;
						ListOfURLs.ParseIntoArrayWS(URLs);
						if (Index < URLs.Num())
						{
							TArray<TSharedPtrTS<const FDashMPD_BaseURLType>> OutBaseURLs;
							FString URL, RequestHeader;
							FTimeValue UrlATO;
							TMediaOptionalValue<bool> bATOComplete;
							DASHUrlHelpers::BuildAbsoluteElementURL(URL, UrlATO, bATOComplete, MPDDocumentURL, OutBaseURLs, URLs[Index]);
							return URL;
						}
						return FString();
					};


					const FString& SchemeIdUri = UTCTimings[i]->GetSchemeIdUri();
					const FString& Value = UTCTimings[i]->GetValue();

					TSharedPtrTS<FMPDLoadRequestDASH> TimeSyncRequest;
					// Check for supported schemes. They appear in the manifest in the order the author has prioritized them.
					if (SchemeIdUri.Equals(DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_httpxsdate2014))
					{
						TimeSyncRequest = MakeSharedTS<FMPDLoadRequestDASH>();
						TimeSyncRequest->LoadType = FMPDLoadRequestDASH::ELoadType::TimeSync;
						TimeSyncRequest->Url.URL = GetRequestURL(Value, 0);
						TimeSyncRequest->CompleteCallback.BindThreadSafeSP(AsShared(), &FPlaylistReaderDASH::Timesync_httpxsdate_Completed);
					}
					else if (SchemeIdUri.Equals(DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_httpiso2014))
					{
						TimeSyncRequest = MakeSharedTS<FMPDLoadRequestDASH>();
						TimeSyncRequest->LoadType = FMPDLoadRequestDASH::ELoadType::TimeSync;
						TimeSyncRequest->Url.URL = GetRequestURL(Value, 0);
						TimeSyncRequest->CompleteCallback.BindThreadSafeSP(AsShared(), &FPlaylistReaderDASH::Timesync_httpiso_Completed);
					}
					else if (SchemeIdUri.Equals(DASH::Schemes::TimingSources::Scheme_urn_mpeg_dash_utc_httphead2014))
					{
						TimeSyncRequest = MakeSharedTS<FMPDLoadRequestDASH>();
						TimeSyncRequest->LoadType = FMPDLoadRequestDASH::ELoadType::TimeSync;
						TimeSyncRequest->Url.URL = GetRequestURL(Value, 0);
						TimeSyncRequest->Verb = TEXT("HEAD");
						TimeSyncRequest->CompleteCallback.BindThreadSafeSP(AsShared(), &FPlaylistReaderDASH::Timesync_httphead_Completed);
					}
					else
					{
						// Not supported, skip it.
					}

					// When a request has been set up, enqueue it and leave the loop.
					if (TimeSyncRequest.IsValid())
					{
						bTimeSyncInProgress = true;
						AttemptedTimesyncDescriptors.Emplace(UTCTimings[i]);
						EnqueueResourceRequest(MoveTemp(TimeSyncRequest));
						bFoundSupportedElement = true;
						break;
					}
				}
				// Warn once when there is no UTCTiming element we support.
				if (!bFoundSupportedElement && !bWarnedAboutUnsupportedUTCTimingElement)
				{
					bWarnedAboutUnsupportedUTCTimingElement = true;
					if (InitialTimesync.HttpDateHeader.Len())
					{
						LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("No supported <UTCTiming> element found in MPD, but time was synchronized to the MPD's HTTP response Date header. This may not be accurate enough.")));
					}
					else
					{
						LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("No supported <UTCTiming> element found in MPD. This could result in playback failures.")));
					}
				}

				// If no request was enqueued and this is the initial time sync we need to complete the manifest setup now.
				if (!bTimeSyncInProgress)
				{
					FinishManifestBuildingAfterTimesync(false);
				}
			}
		}
		else
		{
			FinishManifestBuildingAfterTimesync(false);
		}
	}
}


void FPlaylistReaderDASH::InternalSetup()
{
	check(!bIsInitialized);
	bInbandEventByStreamType[0] = false;
	bInbandEventByStreamType[1] = false;
	bInbandEventByStreamType[2] = false;

	NextTimeSyncTime.SetToInvalid();

	// Register event callbacks
	PlayerSessionServices->GetAEMSEventHandler()->AddAEMSReceiver(AsShared(), DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_2012, TEXT(""), IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnStart, false);
	PlayerSessionServices->GetAEMSEventHandler()->AddAEMSReceiver(AsShared(), DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_callback_2015, TEXT("1"), IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnStart, false);
	PlayerSessionServices->GetAEMSEventHandler()->AddAEMSReceiver(AsShared(), DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_ttfn_2016, TEXT(""), IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnStart, false);

	// Get the minimum MPD update time limit.
	MinTimeBetweenUpdates = PlayerSessionServices->GetOptionValue(DASH::OptionKey_MinTimeBetweenMPDUpdates).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000));

	// Setup the playlist load request for the main playlist.
	TSharedPtrTS<FMPDLoadRequestDASH> PlaylistLoadRequest = MakeSharedTS<FMPDLoadRequestDASH>();
	PlaylistLoadRequest->LoadType = FMPDLoadRequestDASH::ELoadType::MPD;
	FURL_RFC3986 UrlParser;
	UrlParser.Parse(MPDURL);
	PlaylistLoadRequest->Url.URL = UrlParser.Get(true, false);
	FURL_RFC3986::GetQueryParams(URLFragmentComponents, UrlParser.GetFragment(), false);
	PlaylistLoadRequest->CompleteCallback.BindThreadSafeSP(AsShared(), &FPlaylistReaderDASH::ManifestDownloadCompleted);
	EnqueueResourceRequest(MoveTemp(PlaylistLoadRequest));
	bIsInitialized = true;
}

void FPlaylistReaderDASH::InternalCleanup()
{
	if (bIsInitialized)
	{
		// Unregister event callbacks
		PlayerSessionServices->GetAEMSEventHandler()->RemoveAEMSReceiver(AsShared(), DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_ttfn_2016, TEXT(""), IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnStart);
		PlayerSessionServices->GetAEMSEventHandler()->RemoveAEMSReceiver(AsShared(), DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_callback_2015, TEXT("1"), IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnStart);
		PlayerSessionServices->GetAEMSEventHandler()->RemoveAEMSReceiver(AsShared(), DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_2012, TEXT(""), IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode::OnStart);

		// Cleanup!
		RequestsLock.Lock();
		PendingRequests.Empty();
		for(int32 i=0; i<ActiveRequests.Num(); ++i)
		{
			if (ActiveRequests[i]->Request.IsValid())
			{
				ActiveRequests[i]->Request->Cancel();
			}
		}
		ActiveRequests.Empty();
		CompletedRequests.Empty();
		RequestsLock.Unlock();
		Builder.Reset();
		bIsInitialized = false;
	}
}


void FPlaylistReaderDASH::ExecutePendingRequests(const FTimeValue& TimeNow)
{
	FScopeLock lock(&RequestsLock);

	for(int32 i=0; i<PendingRequests.Num(); ++i)
	{
		FResourceLoadRequestPtr Request = PendingRequests[i];
		if (!Request->ExecuteAtUTC.IsValid() || TimeNow >= Request->ExecuteAtUTC)
		{
			PendingRequests.RemoveAt(i);
			--i;

			// Check for reserved URIs
			if (Request->Url.URL.Equals(TEXT("urn:mpeg:dash:resolve-to-zero:2013")))
			{
				CompletedRequests.Emplace(Request);
			}
			else
			{
				++Request->Attempt;
				ActiveRequests.Push(Request);

				Request->Request = MakeSharedTS<FHTTPResourceRequest>();
				Request->Request->URL(Request->Url.URL).Verb(Request->Verb).Range(Request->Range).Headers(Request->Headers).Object(Request);
				// Set up timeouts and also accepted encodings depending on the type of request.
				SetupRequestTimeouts(Request);
				FString CDNId(Request->Url.CDN);
				switch(Request->GetLoadType())
				{
					case FMPDLoadRequestDASH::ELoadType::MPD:
					{
						Request->Request->AllowStaticQuery(IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Playlist);
						PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::FirstPlaylist, FCMCDHandler::EObjectType::ManifestOrPlaylist, Request->Request->GetRequest()->Parameters.URL, Request->Request->GetRequest()->Parameters.RequestHeaders, CDNId, FCMCDHandler::FRequestObjectInfo());
						break;
					}
					case FMPDLoadRequestDASH::ELoadType::MPDUpdate:
					{
						PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::PlaylistUpdate, FCMCDHandler::EObjectType::ManifestOrPlaylist, Request->Request->GetRequest()->Parameters.URL, Request->Request->GetRequest()->Parameters.RequestHeaders, CDNId, FCMCDHandler::FRequestObjectInfo());
						break;
					}
					case FMPDLoadRequestDASH::ELoadType::XLink_Period:
					case FMPDLoadRequestDASH::ELoadType::XLink_AdaptationSet:
					case FMPDLoadRequestDASH::ELoadType::XLink_EventStream:
					case FMPDLoadRequestDASH::ELoadType::XLink_SegmentList:
					case FMPDLoadRequestDASH::ELoadType::XLink_URLQuery:
					case FMPDLoadRequestDASH::ELoadType::XLink_InitializationSet:
					{
						PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::Other, FCMCDHandler::EObjectType::Other, Request->Request->GetRequest()->Parameters.URL, Request->Request->GetRequest()->Parameters.RequestHeaders, CDNId, FCMCDHandler::FRequestObjectInfo());
						break;
					}
					case FMPDLoadRequestDASH::ELoadType::Steering:
					{
						PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::Steering, FCMCDHandler::EObjectType::Other, Request->Request->GetRequest()->Parameters.URL, Request->Request->GetRequest()->Parameters.RequestHeaders, CDNId, FCMCDHandler::FRequestObjectInfo());
						break;
					}
					case FMPDLoadRequestDASH::ELoadType::IndexSegment:
					case FMPDLoadRequestDASH::ELoadType::InitSegment:
					{
						FCMCDHandler::FRequestObjectInfo ri;
						ri.MediaStreamType = Request->SegmentStreamType;
						ri.EncodedBitrate = Request->Bitrate;
						PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::InitSegment, FCMCDHandler::EObjectType::InitSegment, Request->Request->GetRequest()->Parameters.URL, Request->Request->GetRequest()->Parameters.RequestHeaders, CDNId, ri);
						break;
					}
					default:
					{
						break;
					}
				}

				Request->Request->Callback().BindThreadSafeSP(AsShared(), &FPlaylistReaderDASH::OnHTTPResourceRequestComplete);
				Request->Request->StartGet(PlayerSessionServices);
			}
		}
	}
}


void FPlaylistReaderDASH::SetupRequestTimeouts(FResourceLoadRequestPtr InRequest)
{
	switch(InRequest->LoadType)
	{
		case FMPDLoadRequestDASH::ELoadType::MPD:
		{
			InRequest->Request->ConnectionTimeout(PlayerSessionServices->GetOptionValue(DASH::OptionKeyMPDLoadConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 8)));
			InRequest->Request->NoDataTimeout(PlayerSessionServices->GetOptionValue(DASH::OptionKeyMPDLoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 5)));
			break;
		}
		case FMPDLoadRequestDASH::ELoadType::MPDUpdate:
		{
			InRequest->Request->ConnectionTimeout(PlayerSessionServices->GetOptionValue(DASH::OptionKeyMPDReloadConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 2)));
			InRequest->Request->NoDataTimeout(PlayerSessionServices->GetOptionValue(DASH::OptionKeyMPDReloadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 2)));
			break;
		}
		case FMPDLoadRequestDASH::ELoadType::Segment:
		case FMPDLoadRequestDASH::ELoadType::IndexSegment:
		case FMPDLoadRequestDASH::ELoadType::InitSegment:
		{
			InRequest->Request->ConnectionTimeout(FTimeValue(FTimeValue::MillisecondsToHNS(1000 * 5)));
			InRequest->Request->NoDataTimeout(FTimeValue(FTimeValue::MillisecondsToHNS(1000 * 2)));
			InRequest->Request->AcceptEncoding(TEXT("identity"));
			InRequest->Request->StreamTypeAndQuality(InRequest->SegmentStreamType, InRequest->SegmentQualityIndex, InRequest->SegmentQualityIndexMax);
			InRequest->Request->GetRequest()->ResponseCache = PlayerSessionServices->GetHTTPResponseCache();
		}
		case FMPDLoadRequestDASH::ELoadType::XLink_Period:
		case FMPDLoadRequestDASH::ELoadType::XLink_AdaptationSet:
		case FMPDLoadRequestDASH::ELoadType::XLink_EventStream:
		case FMPDLoadRequestDASH::ELoadType::XLink_SegmentList:
		case FMPDLoadRequestDASH::ELoadType::XLink_URLQuery:
		case FMPDLoadRequestDASH::ELoadType::XLink_InitializationSet:
		case FMPDLoadRequestDASH::ELoadType::Callback:
		default:
		{
			break;
		}
	}
}

int32 FPlaylistReaderDASH::CheckForRetry(FResourceLoadRequestPtr InRequest, int32 Error)
{
	switch(InRequest->LoadType)
	{
		case FMPDLoadRequestDASH::ELoadType::MPD:
		{
			if (Error < 100 && InRequest->Attempt < 3)
			{
				return 500 * (1 << InRequest->Attempt);
			}
			else if (Error >= 502 && Error <= 504 && InRequest->Attempt < 2)
			{
				return 1000 * (1 << InRequest->Attempt);
			}
			return -1;
		}
		case FMPDLoadRequestDASH::ELoadType::MPDUpdate:
		{
			if (Error < 100 && InRequest->Attempt < 3)
			{
				return 500 * (1 << InRequest->Attempt);
			}
			else if (InRequest->Attempt < 2 &&
					(Error == 204 || Error == 205 || Error == 404 || Error == 408 || Error == 429 || Error == 502 || Error == 503 || Error == 504))
			{
				return 1000 * (1 << InRequest->Attempt);
			}
			return -1;
		}
		case FMPDLoadRequestDASH::ELoadType::Segment:
		case FMPDLoadRequestDASH::ELoadType::IndexSegment:
		case FMPDLoadRequestDASH::ELoadType::InitSegment:
		{
			if (Error < 100 && InRequest->Attempt < 2)
			{
				return 250 * (1 << InRequest->Attempt);
			}
			else if (InRequest->Attempt < 2 &&
					(Error == 204 || Error == 205 || Error == 404 || Error == 408 || Error == 429 || Error == 502 || Error == 503 || Error == 504))
			{
				return 500 * (1 << InRequest->Attempt);
			}
			return -1;
		}
		case FMPDLoadRequestDASH::ELoadType::Steering:
		{
			// Only retry connection issues, not HTTP failures.
			if (Error < 100 && InRequest->Attempt < 2)
			{
				return 250 * (1 << InRequest->Attempt);
			}
			return -1;
		}

		case FMPDLoadRequestDASH::ELoadType::XLink_Period:
		case FMPDLoadRequestDASH::ELoadType::XLink_AdaptationSet:
		case FMPDLoadRequestDASH::ELoadType::XLink_EventStream:
		case FMPDLoadRequestDASH::ELoadType::XLink_SegmentList:
		case FMPDLoadRequestDASH::ELoadType::XLink_URLQuery:
		case FMPDLoadRequestDASH::ELoadType::XLink_InitializationSet:
		case FMPDLoadRequestDASH::ELoadType::Callback:
		case FMPDLoadRequestDASH::ELoadType::TimeSync:
		case FMPDLoadRequestDASH::ELoadType::Sideload:
		default:
		{
			break;
		}
	}
	return -1;
}


void FPlaylistReaderDASH::HandleCompletedRequests(const FTimeValue& TimeNow)
{
	RequestsLock.Lock();
	TArray<FResourceLoadRequestPtr> cr(MoveTemp(CompletedRequests));
	RequestsLock.Unlock();
	for(int32 i=0; i<cr.Num(); ++i)
	{
		TSharedPtrTS<FHTTPResourceRequest> req = cr[i]->Request;
		// Special URI requests have no actual download request. Only notify the callback of the completion.
		if (!req.IsValid())
		{
			cr[i]->CompleteCallback.ExecuteIfBound(cr[i], true);
			continue;
		}
		// Ignore canceled requests.
		if (req->GetWasCanceled())
		{
			continue;
		}
		// Did the request succeed?
		if (req->GetError() == 0)
		{
			// Set the response headers for this entity with the header cache.
			switch(cr[i]->GetLoadType())
			{
				case FMPDLoadRequestDASH::ELoadType::MPD:
				case FMPDLoadRequestDASH::ELoadType::MPDUpdate:
				{
					PlayerSessionServices->GetEntityCache()->SetRecentResponseHeaders(IPlayerEntityCache::EEntityType::Document, cr[i]->Url.URL, req->GetConnectionInfo()->ResponseHeaders);
					break;
				}
				case FMPDLoadRequestDASH::ELoadType::XLink_Period:
				case FMPDLoadRequestDASH::ELoadType::XLink_AdaptationSet:
				case FMPDLoadRequestDASH::ELoadType::XLink_EventStream:
				case FMPDLoadRequestDASH::ELoadType::XLink_SegmentList:
				case FMPDLoadRequestDASH::ELoadType::XLink_URLQuery:
				{
					PlayerSessionServices->GetEntityCache()->SetRecentResponseHeaders(IPlayerEntityCache::EEntityType::XLink, cr[i]->Url.URL, req->GetConnectionInfo()->ResponseHeaders);
					break;
				}
				case FMPDLoadRequestDASH::ELoadType::Callback:
				{
					PlayerSessionServices->GetEntityCache()->SetRecentResponseHeaders(IPlayerEntityCache::EEntityType::Callback, cr[i]->Url.URL, req->GetConnectionInfo()->ResponseHeaders);
					break;
				}
				case FMPDLoadRequestDASH::ELoadType::Segment:
				{
					PlayerSessionServices->GetEntityCache()->SetRecentResponseHeaders(IPlayerEntityCache::EEntityType::Segment, cr[i]->Url.URL, req->GetConnectionInfo()->ResponseHeaders);
					break;
				}
				default:
				{
					break;
				}
			}

			cr[i]->CompleteCallback.ExecuteIfBound(cr[i], true);
		}
		else
		{
			// Handle possible retries.
			int32 retryInMillisec = CheckForRetry(cr[i], req->GetError());
			if (retryInMillisec >= 0)
			{
				// Try again
				LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Failed to download %s (%s) from %s, retrying..."), cr[i]->GetRequestTypeName(), *cr[i]->Url.URL, *req->GetErrorString()));
				EnqueueResourceRetryRequest(cr[i], TimeNow + FTimeValue().SetFromMilliseconds(retryInMillisec));
			}
			else
			{
				// Give up. Notify caller of failure.
				LogMessage(IInfoLog::ELevel::Error, FString::Printf(TEXT("Failed to download %s (%s) from %s, giving up."), cr[i]->GetRequestTypeName(), *cr[i]->Url.URL, *req->GetErrorString()));
				cr[i]->CompleteCallback.ExecuteIfBound(cr[i], false);
			}
		}
	}
}

FErrorDetail FPlaylistReaderDASH::GetXMLResponseString(FString& OutXMLString, FResourceLoadRequestPtr FromRequest)
{
	if (FromRequest.IsValid() && FromRequest->Request.IsValid())
	{
		TSharedPtrTS<FWaitableBuffer> ResponseBuffer = FromRequest->Request->GetResponseBuffer();
		int32 NumResponseBytes = ResponseBuffer->Num();
		const uint8* ResponseBytes = (const uint8*)ResponseBuffer->GetLinearReadData();
		// Check for potential BOMs
		if (NumResponseBytes > 3 && ResponseBytes[0] == 0xEF && ResponseBytes[1] == 0xBB && ResponseBytes[2] == 0xBF)
		{
			// UTF-8 BOM
			NumResponseBytes -= 3;
			ResponseBytes += 3;
		}
		else if (NumResponseBytes >= 2 && ResponseBytes[0] == 0xFE && ResponseBytes[1] == 0xFF)
		{
			// UTF-16 BE BOM
			return CreateErrorAndLog(FString::Printf(TEXT("Document has unsupported UTF-16 BE BOM!")), ERRCODE_DASH_MPD_UNSUPPORTED_DOCUMENT_ENCODING);
		}
		else if (NumResponseBytes >= 2 && ResponseBytes[0] == 0xFF && ResponseBytes[1] == 0xFE)
		{
			// UTF-16 LE BOM
			return CreateErrorAndLog(FString::Printf(TEXT("Document has unsupported UTF-16 LE BOM!")), ERRCODE_DASH_MPD_UNSUPPORTED_DOCUMENT_ENCODING);
		}
		else if (NumResponseBytes >= 4 && ResponseBytes[0] == 0x00 && ResponseBytes[1] == 0x00 && ResponseBytes[2] == 0xFE && ResponseBytes[3] == 0xFF)
		{
			// UTF-32 BE BOM
			return CreateErrorAndLog(FString::Printf(TEXT("Document has unsupported UTF-32 BE BOM!")), ERRCODE_DASH_MPD_UNSUPPORTED_DOCUMENT_ENCODING);
		}
		else if (NumResponseBytes >= 4 && ResponseBytes[0] == 0xFF && ResponseBytes[1] == 0xFE && ResponseBytes[2] == 0x00 && ResponseBytes[3] == 0x00)
		{
			// UTF-32 LE BOM
			return CreateErrorAndLog(FString::Printf(TEXT("Document has unsupported UTF-32 LE BOM!")), ERRCODE_DASH_MPD_UNSUPPORTED_DOCUMENT_ENCODING);
		}
		FUTF8ToTCHAR TextConv((const ANSICHAR*)ResponseBytes, NumResponseBytes);
		FString XML = FString::ConstructFromPtrSize(TextConv.Get(), TextConv.Length());
		OutXMLString = MoveTemp(XML);
	}
	return FErrorDetail();
}


FErrorDetail FPlaylistReaderDASH::GetResponseString(FString& OutString, FResourceLoadRequestPtr FromRequest)
{
	if (FromRequest.IsValid() && FromRequest->Request.IsValid())
	{
		TSharedPtrTS<FWaitableBuffer> ResponseBuffer = FromRequest->Request->GetResponseBuffer();
		int32 NumResponseBytes = ResponseBuffer->Num();
		const uint8* ResponseBytes = (const uint8*)ResponseBuffer->GetLinearReadData();
		FUTF8ToTCHAR TextConv((const ANSICHAR*)ResponseBytes, NumResponseBytes);
		FString UTF8Text = FString::ConstructFromPtrSize(TextConv.Get(), TextConv.Length());
		OutString = MoveTemp(UTF8Text);
	}
	return FErrorDetail();
}


void FPlaylistReaderDASH::ManifestDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess)
{
	if (bSuccess)
	{
		if (!Request->Request.IsValid() || !Request->Request->GetResponseBuffer().IsValid())
		{
			return;
		}

		const HTTP::FConnectionInfo* ConnInfo = Request->GetConnectionInfo();
		if (!ConnInfo)
		{
			return;
		}

		FString ETag;
		TArray<FElectraHTTPStreamHeader> Headers;
		InitialTimesync.HttpRequestStart = ConnInfo->RequestStartTime;
		InitialTimesync.UtcTimeThen = MEDIAutcTime::Current();
		InitialTimesync.ConnInfo = MakeUnique<HTTP::FConnectionInfo>(*ConnInfo);
		for(int32 i=0; i<ConnInfo->ResponseHeaders.Num(); ++i)
		{
			Headers.Add(ConnInfo->ResponseHeaders[i]);
			if (ConnInfo->ResponseHeaders[i].Header.Equals(TEXT("Date"), ESearchCase::IgnoreCase))
			{
				InitialTimesync.HttpDateHeader = ConnInfo->ResponseHeaders[i].Value;
			}
			else if (ConnInfo->ResponseHeaders[i].Header.Equals(TEXT("ETag"), ESearchCase::IgnoreCase))
			{
				ETag = ConnInfo->ResponseHeaders[i].Value;
			}
		}

		PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistDownloadMessage::Create(ConnInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Request->Attempt));

		FString XML;
		LastErrorDetail = GetXMLResponseString(XML, Request);
		if (LastErrorDetail.IsOK())
		{
			FString EffectiveURL = PlayerSessionServices->GetCMCDHandler()->RemoveParamsFromURL(ConnInfo->EffectiveURL);
			Builder.Reset(IManifestBuilderDASH::Create(PlayerSessionServices));
			TSharedPtrTS<FManifestDASHInternal> NewManifest;
			LastErrorDetail = Builder->BuildFromMPD(NewManifest, XML.GetCharArray().GetData(), EffectiveURL, ETag, Headers);
			if (LastErrorDetail.IsOK() || LastErrorDetail.IsTryAgain())
			{
				if (NewManifest.IsValid())
				{
					// Update the CMCD parameters from the root level service descriptions.
					UpdateCMCDParamsFromServiceDescriptions(NewManifest->GetMPDRoot()->GetServiceDescriptions());

					NewManifest->SetURLFragmentComponents(URLFragmentComponents);
					NewManifest->SetReplayEventParams(ReplayEventParams);
					// Inject fake <UTCTiming> elements that were passed in the fragments.
					NewManifest->InjectEpicTimingSources();
				}
				Manifest = NewManifest;

				// Set the next time sync time to zero to cause a time sync as soon as possible.
				NextTimeSyncTime.SetToZero();
			}
			else
			{
				PostError(LastErrorDetail);
			}
		}
		else
		{
			PostError(LastErrorDetail);
		}
	}
	else
	{
		PostError(FString::Printf(TEXT("Failed to download MPD \"%s\" (%s)"), *Request->Url.URL, *Request->GetErrorDetail()), ERRCODE_DASH_MPD_DOWNLOAD_FAILED, UEMEDIA_ERROR_READ_ERROR);
	}
}

void FPlaylistReaderDASH::UpdateCMCDParamsFromServiceDescriptions(const TArray<TSharedPtrTS<FDashMPD_ServiceDescriptionType>>& InServiceDescriptions)
{
	// Check if the CMCD handler allows for configuration from the MPD.
	if (!PlayerSessionServices->GetCMCDHandler()->UseParametersFromPlaylist())
	{
		return;
	}
	TArray<FString> cmcdConfigs;
	for(int32 i=0; i<InServiceDescriptions.Num(); ++i)
	{
		const TArray<TSharedPtrTS<FDashMPD_ClientDataReportingType>>& cdrt = InServiceDescriptions[i]->GetClientDataReportings();
		for(int32 j=0; j<cdrt.Num(); ++j)
		{
			const TArray<FString>& ServiceLocations = cdrt[j]->GetServiceLocations();
			if (cdrt[j]->GetCMCDParameters_Elements().Num())
			{
				// This may be an array, but the spec allows for 1 entry at most.
				const TSharedPtrTS<FDashMPD_CMCDParametersType>& pt = cdrt[j]->GetCMCDParameters_Elements()[0];
				// The keys are mandatory, so if they are not present we ignore this entry.
				if (pt->GetVersion() == 1 && !pt->GetKeys().IsEmpty())
				{
					FString cfg(FString::Printf(TEXT("{\"%s\":\"request\",\"%s\":1"), FCMCDHandler::Key_Type(), FCMCDHandler::Key_Version()));
					cfg.Append(FString::Printf(TEXT(",\"%s\":\"%s\""), FCMCDHandler::Key_Mode(), *pt->GetMode()));
					if (pt->GetContentID().Len())
					{
						cfg.Append(FString::Printf(TEXT(",\"%s\":\"%s\""), FCMCDHandler::Key_ContentId(), *pt->GetContentID()));
					}
					if (pt->GetSessionID().Len())
					{
						cfg.Append(FString::Printf(TEXT(",\"%s\":\"%s\""), FCMCDHandler::Key_SessionId(), *pt->GetSessionID()));
					}
					auto MakeList = [](const TArray<FString>& InList) -> FString
					{
						FString r;
						for(int32 i=0; i<InList.Num(); ++i)
						{
							if (i)
							{
								r.Append(TEXT(" "));
							}
							r.Append(InList[i]);
						}
						return r;
					};
					cfg.Append(FString::Printf(TEXT(",\"%s\":\"%s\""), FCMCDHandler::Key_Keys(), *MakeList(pt->GetKeys())));
					if (ServiceLocations.Num())
					{
						cfg.Append(FString::Printf(TEXT(",\"%s\":\"%s\""), FCMCDHandler::Key_CDNs(), *MakeList(ServiceLocations)));
					}
					TArray<FString> IncludeIn;
					for(auto& InR : pt->GetIncludeInRequests())
					{
						if (InR.Equals(TEXT("segments")))
						{
							IncludeIn.Emplace(FCMCDHandler::IncludeInRequests_Segment());
						}
						else if (InR.Equals(TEXT("mpd")))
						{
							IncludeIn.Emplace(FCMCDHandler::IncludeInRequests_Playlist());
						}
						else if (InR.Equals(TEXT("steering")))
						{
							IncludeIn.Emplace(FCMCDHandler::IncludeInRequests_Steering());
						}
					}
					if (IncludeIn.Num())
					{
						cfg.Append(FString::Printf(TEXT(",\"%s\":\"%s\""), FCMCDHandler::Key_IncludeInRequests(), *MakeList(IncludeIn)));
					}
					cfg.Append(TEXT("}"));
					cmcdConfigs.Emplace(MoveTemp(cfg));
				}
			}
		}
	}
	// Build the overall config.
	if (cmcdConfigs.Num())
	{
		FString cfg(FString::Printf(TEXT("{\"%s\":["), FCMCDHandler::Key_CmcdArray()));
		for(int32 i=0; i<cmcdConfigs.Num(); ++i)
		{
			if (i)
			{
				cfg.Append(TEXT(","));
			}
			cfg.Append(cmcdConfigs[i]);
		}
		cfg.Append(TEXT("]}"));
		PlayerSessionServices->GetCMCDHandler()->UpdateParameters(cfg);
	}
}


void FPlaylistReaderDASH::PrepareNextTimeSync()
{
	if (Manifest.IsValid())
	{
		// Do not re-sync the time too often.
		const FTimeValue MinResyncTimeInterval(120.0);		// not sooner than every 120 seconds.
		const FTimeValue MaxResyncTimeInterval(1800.0);		// at least once every 30 minutes.

		FTimeValue ResyncTimeInterval(MinResyncTimeInterval);
		// How frequently does the MPD update?
		FTimeValue mup = Manifest->GetMinimumUpdatePeriod();
		if (mup.IsValid() && mup > MinResyncTimeInterval)
		{
			ResyncTimeInterval = mup;
		}
		if (ResyncTimeInterval > MaxResyncTimeInterval)
		{
			ResyncTimeInterval = MaxResyncTimeInterval;
		}

		NextTimeSyncTime = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime() + ResyncTimeInterval;
	}
}


void FPlaylistReaderDASH::FinishManifestBuildingAfterTimesync(bool bGotTheTime)
{
	// After a time sync clear the segment fetch delay.
	if (Manifest.IsValid())
	{
		Manifest->SetSegmentFetchDelay(FTimeValue::GetZero());
	}

	if (!bIsInitialTimesync)
	{
		PrepareNextTimeSync();
		return;
	}
	bIsInitialTimesync = false;

	FTimeValue Now = MEDIAutcTime::Current();
	FTimeValue TimeDiffSinceStart = Now - InitialTimesync.UtcTimeThen;
	if (!bGotTheTime)
	{
		// First check if the MPD contained a direct time value.
		if (InitialTimesync.Utc_direct2014.Len())
		{
			FTimeValue DirectTime;
			if (ISO8601::ParseDateTime(DirectTime, InitialTimesync.Utc_direct2014))
			{
				PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(DirectTime + TimeDiffSinceStart);
				// No need to use the Date header any more.
				InitialTimesync.HttpDateHeader.Reset();
			}
			// If parsing failed then maybe the response is just a number (possibly with frational digits) giving the
			// current Unix epoch time.
			else if (UnixEpoch::ParseFloatString(DirectTime, InitialTimesync.Utc_direct2014))
			{
				PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(DirectTime + TimeDiffSinceStart);
				// No need to use the Date header any more.
				InitialTimesync.HttpDateHeader.Reset();
			}
		}
		// As a last resort use the Date header from the HTTP response
		if (InitialTimesync.HttpDateHeader.Len())
		{
			FTimeValue DateFromHeader;
			bool bDateParsedOk = RFC7231::ParseDateTime(DateFromHeader, InitialTimesync.HttpDateHeader);
			if (bDateParsedOk && DateFromHeader.IsValid())
			{
				PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(InitialTimesync.HttpRequestStart, DateFromHeader);
			}
		}
	}

	if (Manifest.IsValid())
	{
		FTimeValue FetchTime = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime() - TimeDiffSinceStart;
		Manifest->GetMPDRoot()->SetFetchTime(FetchTime);
		MostRecentMPDUpdateTime = FetchTime;

		// If the URL has a special fragment part to turn this presentation into an event, do so.
		// Do this before preparing the default start times!
		// Note: This MAY require the client clock to be synced to the server IF the special keyword 'now' is used.
		Manifest->TransformIntoEpicEvent();

		// Do a one-time preparation of the default start time from the #t= URL fragment.
		// This is not to be repeated for manifest updates.
		Manifest->PrepareDefaultStartTime();
	}

	PlayerManifest = FManifestDASH::Create(PlayerSessionServices, Manifest);

	// Check if the MPD defines an @minimumUpdatePeriod
	FTimeValue mup = Manifest.IsValid() ? Manifest->GetMinimumUpdatePeriod() : FTimeValue::GetInvalid();
	// If the update time is zero then updates happen just in time when segments are required or through
	// an inband event stream. Either way, we do not need to update periodically.
	if (mup.IsValid() && mup > FTimeValue::GetZero())
	{
		// Warn if MUP is really small
		if (mup.GetAsMilliseconds() < 1000)
		{
			LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("MPD@minimumUpdatePeriod is set to a really small value of %lld msec. This could be a performance issue."), (long long int)mup.GetAsMilliseconds()));
		}
		RequestsLock.Lock();
		NextMPDUpdateTime = MostRecentMPDUpdateTime + mup;
		RequestsLock.Unlock();
	}

	// Notify that the "main playlist" has been parsed, successfully or not. If we still need to resolve remote entities this is not an error!
	const int32 Attempts = 0; // There are no actual attempts made here, so send '0'.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(!LastErrorDetail.IsTryAgain() ? LastErrorDetail : FErrorDetail(), InitialTimesync.ConnInfo.Get(), Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempts));
	if (LastErrorDetail.IsOK())
	{
		// Prepare content steering
		FContentSteeringHandler::FInitialParams csp;
		auto MPDRoot = Manifest->GetMPDRoot();
		auto cs = MPDRoot->GetContentSteering();
		csp.RootDocumentURL = MPDRoot->GetDocumentURL();
		csp.bHasContentSteering = cs.IsValid();
		csp.bUseDVBPriorities = !cs.IsValid() && MPDRoot->IsDVBDash();
		if (cs.IsValid())
		{
			csp.FirstSteeringURL = cs->GetContentSteeringServer();
			csp.ProxyURL = cs->GetProxyServerURL();
			csp.InitialDefaultCDN = cs->GetDefaultServiceLocation();
			csp.bQueryBeforeStart = cs->QueryBeforeStart();
			const TArray<IDashMPDElement::FXmlAttribute>& OtherAttributes = cs->GetOtherAttributes();
			for(auto &Attribute : OtherAttributes)
			{
				if (Attribute.GetName().Equals(TEXT("EpicInitialSelectionPriority")))
				{
					LexFromString(csp.CustomFirstCDNPrioritization, *Attribute.GetValue());
					break;
				}
			}
			// Without a URL we can use this only to select the CDN to start with.
			csp.bHasContentSteering = !csp.FirstSteeringURL.IsEmpty();
		}
		if (PlayerSessionServices->GetContentSteeringHandler()->InitialSetup(FContentSteeringHandler::EStreamingProtocol::DASH, csp))
		{
			// If we do not have to query the content steering server right away we are now good to go.
			if (!PlayerSessionServices->GetContentSteeringHandler()->NeedToObtainNewSteeringManifestNow())
			{
				// Notify that the "variant playlists" are ready. There are no variants in DASH, but this is the trigger that the playlists are all set up and are good to go now.
				PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, InitialTimesync.ConnInfo.Get(), Playlist::EListType::Variant, Playlist::ELoadType::Initial, Attempts));
				PrepareNextTimeSync();
			}
			else
			{
				TriggerContentSteeringUpdate(true);
			}
		}
		else
		{
			PostError(FString::Printf(TEXT("Invalid content steering configuration")), ERRCODE_DASH_MPD_BAD_CONTENT_STEERING, UEMEDIA_ERROR_DETAIL);
		}
	}
	// "try again" is returned when there are remote entities that need to be resolved first.
	else if (LastErrorDetail.IsTryAgain())
	{
		EnqueueInitialXLinkRequests();
	}
	else
	{
		PostError(LastErrorDetail);
	}

	InitialTimesync.Reset();
}

void FPlaylistReaderDASH::TriggerContentSteeringUpdate(bool bInitial)
{
	auto MPDRoot = Manifest->GetMPDRoot();
	// No <BaseURL> elements apply to content steering, but UrlQuery does.
	TArray<TSharedPtrTS<const FDashMPD_BaseURLType>> OutBaseURLs;
	FString URL, RequestHeader;
	FTimeValue UrlATO;
	TMediaOptionalValue<bool> bATOComplete;
	DASHUrlHelpers::BuildAbsoluteElementURL(URL, UrlATO, bATOComplete, MPDRoot->GetDocumentURL(), OutBaseURLs, PlayerSessionServices->GetContentSteeringHandler()->GetBaseSteeringServerRequestURL());
	TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>> UrlQueries;
	DASHUrlHelpers::GetAllHierarchyUrlQueries(UrlQueries, MPDRoot, DASHUrlHelpers::EUrlQueryRequestType::Steering, false);
	LastErrorDetail = DASHUrlHelpers::ApplyUrlQueries(PlayerSessionServices, MPDRoot->GetDocumentURL(), URL, RequestHeader, UrlQueries);
	URL = PlayerSessionServices->GetContentSteeringHandler()->GetFinalSteeringServerRequestURL(URL);
	if (LastErrorDetail.IsOK())
	{
		TSharedPtrTS<FMPDLoadRequestDASH> SteeringLoadRequest = MakeSharedTS<FMPDLoadRequestDASH>();
		SteeringLoadRequest->LoadType = FMPDLoadRequestDASH::ELoadType::Steering;
		SteeringLoadRequest->Url.URL = URL;
		if (RequestHeader.Len())
		{
			SteeringLoadRequest->Headers.Emplace(HTTP::FHTTPHeader(DASH::HTTPHeaderOptionName, RequestHeader));
		}
		SteeringLoadRequest->CompleteCallback.BindThreadSafeSP(AsShared(), bInitial ? &FPlaylistReaderDASH::ContentSteeringDownloadCompleted : &FPlaylistReaderDASH::ContentSteeringUpdateDownloadCompleted);
		EnqueueResourceRequest(MoveTemp(SteeringLoadRequest));
		PlayerSessionServices->GetContentSteeringHandler()->SetSteeringServerRequestIsPending();
	}
	else
	{
		PostError(LastErrorDetail);
	}
}


void FPlaylistReaderDASH::ManifestUpdateDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess)
{
	bIsMPDUpdateInProgress = false;

	if (bSuccess || (Request->GetConnectionInfo() && Request->GetConnectionInfo()->StatusInfo.HTTPStatus == 410))
	{
		if (!Request->Request.IsValid() || !Request->Request->GetResponseBuffer().IsValid())
		{
			return;
		}

		const HTTP::FConnectionInfo* ConnInfo = Request->GetConnectionInfo();
		if (!ConnInfo)
		{
			return;
		}
		FString ETag;
		FString EffectiveURL;
		TArray<FElectraHTTPStreamHeader> Headers;
		FTimeValue FetchTime;
		EffectiveURL = ConnInfo->EffectiveURL;
		//FetchTime = PlayerSessionServices->GetSynchronizedUTCTime()->MapToSyncTime(ConnInfo->RequestStartTime);
		FetchTime = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		for(int32 i=0; i<ConnInfo->ResponseHeaders.Num(); ++i)
		{
			Headers.Add(ConnInfo->ResponseHeaders[i]);
			if (ConnInfo->ResponseHeaders[i].Header.Equals(TEXT("ETag"), ESearchCase::IgnoreCase))
			{
				ETag = ConnInfo->ResponseHeaders[i].Value;
				break;
			}
		}
		MostRecentMPDUpdateTime = FetchTime;

		// If this was a "304 - Not modified" (RFC 7323) there is no new MPD at the moment.
		if (ConnInfo->StatusInfo.HTTPStatus == 304)
		{
			// It does however extend the validity of the MPD.
			Manifest->GetMPDRoot()->SetFetchTime(FetchTime);

			// The minimum update period still holds.
			FTimeValue mup = Manifest->GetMinimumUpdatePeriod();
			if (mup.IsValid() && mup > FTimeValue::GetZero())
			{
				RequestsLock.Lock();
				NextMPDUpdateTime = MostRecentMPDUpdateTime + mup;
				RequestsLock.Unlock();
			}
		}
		// If we got a "410 - Gone" we set the end of presentation to "now"
		else if (ConnInfo->StatusInfo.HTTPStatus == 410)
		{
			PlayerSessionServices->SetPlaybackEnd(FetchTime, IPlayerSessionServices::EPlayEndReason::EndAll, nullptr);
			Manifest->GetMPDRoot()->SetFetchTime(FetchTime);
			Manifest->EndPresentationAt(FetchTime, FString());
		}
		else
		{
			const int32 Attempt = Request->Attempt;
			PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistDownloadMessage::Create(ConnInfo, Playlist::EListType::Main, Playlist::ELoadType::Update, Attempt));

			FString XML;
			LastErrorDetail = GetXMLResponseString(XML, Request);
			if (LastErrorDetail.IsOK())
			{
				TSharedPtrTS<FManifestDASHInternal> NewManifest;
				LastErrorDetail = Builder->BuildFromMPD(NewManifest, XML.GetCharArray().GetData(), EffectiveURL, ETag, Headers);
				if (LastErrorDetail.IsOK() || LastErrorDetail.IsTryAgain())
				{
					// Set the availability start time of the new MPD to be that of the current MPD.
					// It is not supposed to change in accordance with ISO/IEC 23009-1:2022 Section 8.4.2, but we
					// have seen MPDs where it *does* change. In an attempt to make these MPDs usable we keep
					// the initial AST.
					NewManifest->GetMPDRoot()->SetAvailabilityStartTime(Manifest->GetMPDRoot()->GetAvailabilityStartTime());

					// Copy over the initial document URL fragments.
					if (NewManifest.IsValid())
					{
						NewManifest->SetURLFragmentComponents(Manifest->GetURLFragmentComponents());
					}

					// Switch over tp the new manifest.
					Manifest = NewManifest;
					// Also update in the external manifest we handed out to the player.
					PlayerManifest->UpdateInternalManifest(Manifest);

					if (Manifest.IsValid())
					{
						Manifest->GetMPDRoot()->SetFetchTime(FetchTime);
						MostRecentMPDUpdateTime = FetchTime;
					}

					// Check if the new MPD also defines an @minimumUpdatePeriod
					FTimeValue mup = Manifest.IsValid() ? Manifest->GetMinimumUpdatePeriod() : FTimeValue::GetInvalid();
					if (mup.IsValid() && mup > FTimeValue::GetZero())
					{
						if (mup.GetAsMilliseconds() < 1000)
						{
							LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("MPD@minimumUpdatePeriod is set to a really small value of %lld msec. This could be a performance issue."), (long long int)mup.GetAsMilliseconds()));
						}
						RequestsLock.Lock();
						NextMPDUpdateTime = MostRecentMPDUpdateTime + mup;
						RequestsLock.Unlock();
					}

					PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(!LastErrorDetail.IsTryAgain() ? LastErrorDetail : FErrorDetail(), ConnInfo, Playlist::EListType::Main, Playlist::ELoadType::Update, Attempt));
					if (LastErrorDetail.IsOK())
					{
						// Let content steering know which CDN we used for downloading.
						Metrics::FSegmentDownloadStats s;
						s.SegmentType = Metrics::ESegmentType::PlaylistElement;
						s.URL = Request->Url;
						s.SteeringID = Request->SteeringID;
						PlayerSessionServices->GetContentSteeringHandler()->FinishedDownloadRequestOn(s, FContentSteeringHandler::FStreamParams());
					}
					// "try again" is returned when there are remote entities that need to be resolved first.
					else if (LastErrorDetail.IsTryAgain())
					{
						EnqueueInitialXLinkRequests();
					}
					else
					{
						PostError(LastErrorDetail);
					}
				}
				else
				{
					PostError(LastErrorDetail);
				}
			}
			else
			{
				PostError(LastErrorDetail);
			}
		}
	}
	else
	{
		PostError(FString::Printf(TEXT("Failed to download MPD \"%s\" (%s)"), *Request->Url.URL, *Request->GetErrorDetail()), ERRCODE_DASH_MPD_DOWNLOAD_FAILED, UEMEDIA_ERROR_READ_ERROR);
	}
}


void FPlaylistReaderDASH::InitialMPDXLinkElementDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess)
{
	// Is the manifest for which this request was made still of interest?
	TSharedPtrTS<FManifestDASHInternal> ForManifest = Request->OwningManifest.Pin();
	if (ForManifest.IsValid())
	{
		FString XML;
		LastErrorDetail = GetXMLResponseString(XML, Request);
		LastErrorDetail = ForManifest->ResolveInitialRemoteElementRequest(Request, MoveTemp(XML), bSuccess);

		// Now check if all pending requests have finished
		if (LastErrorDetail.IsOK())
		{
			LastErrorDetail = ForManifest->BuildAfterInitialRemoteElementDownload();
			// Notify that the "variant playlists" are ready. There are no variants in DASH, but this is the trigger that the playlists are all set up and are good to go now.
			PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, nullptr, Playlist::EListType::Variant, Playlist::ELoadType::Initial, Request->Attempt));
			PrepareNextTimeSync();
		}
		else if (LastErrorDetail.IsTryAgain())
		{
			LastErrorDetail.Clear();
		}
		else
		{
			LastErrorDetail.SetMessage(FString::Printf(TEXT("Failed to process initial xlink:onLoad entities (%s)"), *LastErrorDetail.GetMessage()));
			PostError(LastErrorDetail);
		}
	}
	InitialTimesync.Reset();
}


void FPlaylistReaderDASH::ContentSteeringUpdate(bool bInitial, FResourceLoadRequestPtr Request, bool bSuccess)
{
	HTTP::FConnectionInfo ci;
	if (TSharedPtrTS<FHTTPResourceRequest> Req = Request->Request)
	{
		if (Req->GetConnectionInfo())
		{
			ci = *Req->GetConnectionInfo();
		}
	}
	FString Response;
	if (bSuccess)
	{
		GetResponseString(Response, Request);
	}
	PlayerSessionServices->GetContentSteeringHandler()->UpdateWithSteeringServerResponse(Response, ci.StatusInfo.HTTPStatus, ci.ResponseHeaders);
	if (bInitial)
	{
		// Notify that the "variant playlists" are ready. There are no variants in DASH, but this is the trigger that the playlists are all set up and are good to go now.
		PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ci, Playlist::EListType::Variant, Playlist::ELoadType::Initial, 0));
		PrepareNextTimeSync();
	}
}
void FPlaylistReaderDASH::ContentSteeringUpdateDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess)
{
	ContentSteeringUpdate(false, Request, bSuccess);
}
void FPlaylistReaderDASH::ContentSteeringDownloadCompleted(FResourceLoadRequestPtr Request, bool bSuccess)
{
	ContentSteeringUpdate(true, Request, bSuccess);
}


void FPlaylistReaderDASH::Timesync_httphead_Completed(FResourceLoadRequestPtr Request, bool bSuccess)
{
	if (bSuccess)
	{
		const HTTP::FConnectionInfo* ConnInfo = Request->GetConnectionInfo();
		if (ConnInfo)
		{
			for(int32 i=0; i<ConnInfo->ResponseHeaders.Num(); ++i)
			{
				if (ConnInfo->ResponseHeaders[i].Header.Equals(TEXT("Date"), ESearchCase::IgnoreCase))
				{
					// Parse the header
					FTimeValue DateFromHeader;
					bool bDateParsedOk = RFC7231::ParseDateTime(DateFromHeader, ConnInfo->ResponseHeaders[i].Value);
					if (bDateParsedOk && DateFromHeader.IsValid())
					{
						PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(DateFromHeader);
					}
					break;
				}
			}
		}
	}
	// Presently we do not try another time sync method if this one has failed. Clear out what we have attempted so far and be done with it.
	AttemptedTimesyncDescriptors.Empty();
	bTimeSyncInProgress = false;
	FinishManifestBuildingAfterTimesync(bSuccess);
}

void FPlaylistReaderDASH::Timesync_httpxsdate_Completed(FResourceLoadRequestPtr Request, bool bSuccess)
{
	if (bSuccess)
	{
		FString Response;
		if (GetResponseString(Response, Request).IsOK())
		{
			FTimeValue NewTime;
			// Note: Yes, this is a bit weird, but the xs:dateTime format is actually exactly the same as ISO-8601
			//       so it is not clear why there is a distinction made in the UTCTiming scheme.
			//       We keep the different callback handlers here though for completeness sake.
			if (ISO8601::ParseDateTime(NewTime, Response))
			{
				PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(NewTime);
			}
			// If parsing failed then maybe the response is just a number (possibly with frational digits) giving the
			// current Unix epoch time.
			else if (UnixEpoch::ParseFloatString(NewTime, Response))
			{
				PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(NewTime);
			}
		}
	}
	// Presently we do not try another time sync method if this one has failed. Clear out what we have attempted so far and be done with it.
	AttemptedTimesyncDescriptors.Empty();
	bTimeSyncInProgress = false;
	FinishManifestBuildingAfterTimesync(bSuccess);
}

void FPlaylistReaderDASH::Timesync_httpiso_Completed(FResourceLoadRequestPtr Request, bool bSuccess)
{
	if (bSuccess)
	{
		FString Response;
		if (GetResponseString(Response, Request).IsOK())
		{
			FTimeValue NewTime;
			if (ISO8601::ParseDateTime(NewTime, Response))
			{
				PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(NewTime);
			}
			// If parsing failed then maybe the response is just a number (possibly with frational digits) giving the
			// current Unix epoch time.
			else if (UnixEpoch::ParseFloatString(NewTime, Response))
			{
				PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(NewTime);
			}
		}
	}
	// Presently we do not try another time sync method if this one has failed. Clear out what we have attempted so far and be done with it.
	AttemptedTimesyncDescriptors.Empty();
	bTimeSyncInProgress = false;
	FinishManifestBuildingAfterTimesync(bSuccess);
}

void FPlaylistReaderDASH::OnMediaPlayerEventReceived(TSharedPtrTS<IAdaptiveStreamingPlayerAEMSEvent> InEvent, IAdaptiveStreamingPlayerAEMSReceiver::EDispatchMode InDispatchMode)
{
	if (InEvent->GetSchemeIdUri().Equals(DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_2012))
	{
		// This cast is safe because we only exist when the format is DASH and the events must therefore be of that type.
		const DASH::FPlayerEvent* DASHEvent = static_cast<const DASH::FPlayerEvent*>(InEvent.Get());
		FString EventPeriodId = DASHEvent->GetPeriodID();

		// ISO/IEC 23009-1:2019 Section 5.10.4 DASH-specific events
		if (InEvent->GetOrigin() == IAdaptiveStreamingPlayerAEMSEvent::EOrigin::EventStream)
		{
			// For an MPD triggered event there is no point in including a patch or a full MPD.
			// If either was already available when the current MPD was generated the MPD could
			// have already included the changes. As such we only handle value 1 here which
			// sets a validity expiration, which for an event coming from the MPD means to just
			// fetch a new MPD now.
			if (InEvent->GetValue().Equals(TEXT("1")))
			{
				// Add a new MPD update event. Set EventMessage for type but leave the other fields empty.
				TSharedPtrTS<FMPDUpdateRequest> Request = MakeSharedTS<FMPDUpdateRequest>();
				Request->Type = IPlaylistReaderDASH::EMPDRequestType::EventMessage;
				Request->PeriodId = EventPeriodId;
				FScopeLock lock(&RequestsLock);
				UpdateRequested.Emplace(MoveTemp(Request));
			}
		}
		else if (InEvent->GetOrigin() == IAdaptiveStreamingPlayerAEMSEvent::EOrigin::InbandEventStream)
		{
			// An inband event needs more consideration.
			TSharedPtrTS<FMPDUpdateRequest> Request = MakeSharedTS<FMPDUpdateRequest>();
			Request->Type = IPlaylistReaderDASH::EMPDRequestType::EventMessage;
			Request->PeriodId = EventPeriodId;
			Request->ValidUntil = InEvent->GetPresentationTime();
			Request->NewDuration = InEvent->GetDuration();
			int32 NULPos = INDEX_NONE;
			if (InEvent->GetMessageData().Num() && InEvent->GetMessageData().Find(0, NULPos))
			{
				Request->PublishTime = StringHelpers::ArrayToString(TArray<uint8>(InEvent->GetMessageData().GetData(), NULPos));
				FString NewMPD;
				if (InEvent->GetValue().Equals(TEXT("3")))
				{
					Request->NewMPD = StringHelpers::ArrayToString(TArray<uint8>(InEvent->GetMessageData().GetData() + NULPos + 1, InEvent->GetMessageData().Num() - NULPos - 1));
				}
			}
			FScopeLock lock(&RequestsLock);
			UpdateRequested.Emplace(MoveTemp(Request));
		}
	}
	else if (InEvent->GetSchemeIdUri().Equals(DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_callback_2015))
	{
		FString URL = StringHelpers::ArrayToString(InEvent->GetMessageData());
		if (!URL.IsEmpty())
		{
			TSharedPtrTS<FMPDLoadRequestDASH> CallbackRequest;
			CallbackRequest = MakeSharedTS<FMPDLoadRequestDASH>();
			CallbackRequest->LoadType = FMPDLoadRequestDASH::ELoadType::Callback;
			CallbackRequest->Url.URL = URL;
			CallbackRequest->CompleteCallback.BindThreadSafeSP(AsShared(), &FPlaylistReaderDASH::Callback_Completed);
			EnqueueResourceRequest(MoveTemp(CallbackRequest));
		}
	}
	else if (InEvent->GetSchemeIdUri().Equals(DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_ttfn_2016))
	{
		if (Manifest.IsValid())
		{
			// For now we do not consider the event value and just end playback.
			PlayerSessionServices->SetPlaybackEnd(InEvent->GetPresentationTime(), IPlayerSessionServices::EPlayEndReason::EndAll, nullptr);
			Manifest->EndPresentationAt(InEvent->GetPresentationTime(), FString());
		}
	}
}

void FPlaylistReaderDASH::Callback_Completed(FResourceLoadRequestPtr Request, bool bSuccess)
{
	// Callback responses are ignored.
}



} // namespace Electra


