// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlaylistReader.h"
#include "Player/PlayerSessionServices.h"


namespace Electra
{

struct FMPDLoadRequestDASH;
class FManifestDASHInternal;



class IPlaylistReaderDASH : public IPlaylistReader
{
public:
	static TSharedPtrTS<IPlaylistReader> Create(IPlayerSessionServices* PlayerSessionServices);

	virtual ~IPlaylistReaderDASH() = default;

	/**
	 * Loads and parses the MPD.
	 *
	 * @param URL URL of the playlist to load
	 * @param InReplayEventParams URL fragments pertaining to replay event which have been removed from the source URL
	 */
	virtual void LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams) = 0;

	/**
	 * Returns the URL from which the MPD was loaded (or supposed to be loaded).
	 *
	 * @return The MPD URL
	 */
	virtual FString GetURL() const = 0;

	/**
	 * Adds load requests for additional elements.
	 * 
	 * @param RemoteElementLoadRequests
	 */
	virtual void AddElementLoadRequests(const TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& RemoteElementLoadRequests) = 0;

	/**
	 * Requests an MPD update.
	 */
	enum class EMPDRequestType
	{
		MinimumUpdatePeriod,		// Triggered through MPD@minimumUpdatePeriod
		GetLatestSegment,			// Locating the segment in the Live period
		EventMessage				// Triggered through an event message
	};
	virtual void RequestMPDUpdate(EMPDRequestType InRequestType) = 0;

	/**
	 * Requests a resync of the player clock with the server clock.
	 */
	virtual void RequestClockResync() = 0;

	/**
	 * Access the current internal MPD
	 */
	virtual TSharedPtrTS<FManifestDASHInternal> GetCurrentMPD() = 0;

	/**
	 * Informs whether or not a currently playing stream is using inband events.
	 */
	virtual void SetStreamInbandEventUsage(EStreamType InStreamType, bool bUsesInbandDASHEvents) = 0;
};


} // namespace Electra

