// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "PlayerCore.h"

#include "MPDElementsDASH.h"
#include "MediaURLType.h"
#include "Player/Manifest.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/StreamSegmentReaderCommon.h"
#include "Player/ContentSteeringHandler.h"
#include "Utilities/URLParser.h"
#include "MP4Boxes.h"


namespace Electra
{
class FManifestDASHInternal;
class IParserISO14496_12;
class IParserMKV;


struct FMPDLoadRequestDASH : public IHTTPResourceRequestObject
{
	DECLARE_DELEGATE_TwoParams(FOnRequestCompleted, TSharedPtrTS<FMPDLoadRequestDASH> /*Request*/, bool /*bSuccess*/);

	enum class ELoadType
	{
		MPD,
		MPDUpdate,
		XLink_Period,
		XLink_AdaptationSet,
		XLink_EventStream,
		XLink_SegmentList,
		XLink_URLQuery,
		XLink_InitializationSet,
		Callback,
		Segment,
		IndexSegment,
		InitSegment,
		TimeSync,
		Sideload,
		Steering
	};
	const TCHAR* const GetRequestTypeName() const
	{
		switch(LoadType)
		{
			case ELoadType::MPD:					return TEXT("MPD");
			case ELoadType::MPDUpdate:				return TEXT("MPD update");
			case ELoadType::XLink_Period:			return TEXT("remote Period");
			case ELoadType::XLink_AdaptationSet:	return TEXT("remote AdaptationSet");
			case ELoadType::XLink_EventStream:		return TEXT("remote EventStream");
			case ELoadType::XLink_SegmentList:		return TEXT("remote SegmentList");
			case ELoadType::XLink_URLQuery:			return TEXT("remote URLQueryParam");
			case ELoadType::XLink_InitializationSet:return TEXT("remote InitializationSet");
			case ELoadType::Callback:				return TEXT("Callback");
			case ELoadType::Segment:				return TEXT("Segment");
			case ELoadType::IndexSegment:			return TEXT("IndexSegment");
			case ELoadType::InitSegment:			return TEXT("InitSegment");
			case ELoadType::TimeSync:				return TEXT("Time sync");
			case ELoadType::Sideload:				return TEXT("Sideload");
			default:								return TEXT("<unknown>");
		}
	}

	ELoadType GetLoadType() const
	{
		return LoadType;
	}

	FMPDLoadRequestDASH() : LoadType(ELoadType::MPD) {}
	FMediaURL Url;	// For xlink requests this could be "urn:mpeg:dash:resolve-to-zero:2013" indicating removal of the element.
	int64 SteeringID = -1;
	FString Range;
	FString Range2;
	FString Verb;
	TArray<HTTP::FHTTPHeader> Headers;
	FTimeValue ExecuteAtUTC;

	FOnRequestCompleted	CompleteCallback;

	ELoadType LoadType;
	// XLink specific information to which the remote element applies.
	TWeakPtrTS<IDashMPDElement> XLinkElement;
	// The manifest for which this request is made. Not set for an initial MPD fetch but set for everything else.
	// This allows checking if - after a dynamic MPD update - the requesting MPD is still valid and in use.
	TWeakPtrTS<FManifestDASHInternal> OwningManifest;

	EStreamType SegmentStreamType = EStreamType::Unsupported;
	int32 SegmentQualityIndex = 0;
	int32 SegmentQualityIndexMax = 0;
	int32 Bitrate = 0;

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	TSharedPtrTS<FHTTPResourceRequest> Request;
	int32 Attempt = 0;

	TArray<TSharedPtrTS<FMPDLoadRequestDASH>> CompletedRequestChain;
	int32 NumRemainingInChain = 0;

	const HTTP::FConnectionInfo* GetConnectionInfo() const
	{
		return Request.IsValid() ? Request->GetConnectionInfo() : nullptr;
	}
	FString GetErrorDetail() const
	{
		return GetConnectionInfo() ? GetConnectionInfo()->StatusInfo.ErrorDetail.GetMessage() : FString();
	}
};



namespace DASHUrlHelpers
{
	const FName SteerOption_ByteRange(TEXT("ByteRange"));
	const FName SteerOption_ATO(TEXT("ATO"));
	const FName SteerOption_ATOComplete(TEXT("ATOComplete"));
	const FName SteerOption_AnnexIRequestHeader(TEXT("AnnexIRequestHeader"));

	struct FDASHMediaURL : FMediaURL
	{
		FString ByteRange;
		FTimeValue ATO {FTimeValue::GetZero()};
		TMediaOptionalValue<bool> ATOComplete;
	};
	bool GetAllCandidateBaseURLs(TArray<FDASHMediaURL>& OutCandidateURLs, TSharedPtrTS<const IDashMPDElement> StartingAt, const FDASHMediaURL& InMPDUrl, bool bIsDVBDash);

	bool IsAbsoluteURL(const FString& URL);
	void GetAllHierarchyBaseURLs(IPlayerSessionServices* InPlayerSessionServices, TArray<TSharedPtrTS<const FDashMPD_BaseURLType>>& OutBaseURLs, TSharedPtrTS<const IDashMPDElement> StartingAt);
	enum class EUrlQueryRequestType
	{
		Segment,
		Xlink,
		Mpd,
		Callback,
		Chaining,
		Fallback,
		Steering
	};
	void GetAllHierarchyUrlQueries(TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>>& OutUrlQueries, TSharedPtrTS<const IDashMPDElement> StartingAt, EUrlQueryRequestType ForRequestType, bool bInclude2014);
	FErrorDetail ApplyUrlQueries(IPlayerSessionServices* PlayerSessionServices, const FString& InMPDUrl, FString& InOutURL, FString& OutRequestHeader, const TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>>& UrlQueries);
	bool BuildAbsoluteElementURL(FString& OutURL, FTimeValue& ATO, TMediaOptionalValue<bool>& bATOComplete, const FString& DocumentURL, const TArray<TSharedPtrTS<const FDashMPD_BaseURLType>>& BaseURLs, const FString& ElementURL);
	FString ApplyAnnexEByteRange(FString InURL, FString InRange, const TArray<TSharedPtrTS<const FDashMPD_BaseURLType>>& OutBaseURLs);
	FString ApplyAnnexEByteRange(FString InURL, FString InRange, FString InByteRangeParam);
}




class FManifestDASHInternal
{
public:
	enum class EPresentationType
	{
		Static,
		Dynamic
	};

	struct FSegmentSearchOption
	{
		FTimeValue PeriodLocalTime;					//!< Time local in the period to search a segment for.
		FTimeValue PeriodDuration;					//!< Duration of the period. Needed to determine the number of segments in the period.
		FTimeValue PeriodPresentationEnd;			//!< End time of the presetation in period local time, if not set to invalid or infinity.
		IManifest::ESearchType SearchType = IManifest::ESearchType::Closest;
		int64 RequestID = 0;						//!< Sequential request ID across all segments during playback, needed to re-resolve potential UrlQueryInfo xlinks.
		bool bHasFollowingPeriod = false;			//!< true if we know for sure there is another period following.
		bool bFrameAccurateSearch = false;			//!< true to prepare segments for frame-accurate decoding and rendering
		bool bInitSegmentSetupOnly = false;			//!< true to get the initialization segment information only.
		EStreamType StreamType = EStreamType::Unsupported;
		int32 QualityIndex = 0;
		int32 MaxQualityIndex = 0;
	};

	struct FLabelElement
	{
		FString LabelText;
		FString Language;
		uint64 ID = 0;
	};

	class FRepresentation : public IPlaybackAssetRepresentation, public TSharedFromThis<FRepresentation, ESPMode::ThreadSafe>
	{
	public:
		FRepresentation() = default;
		virtual ~FRepresentation() = default;

		enum ESearchResult
		{
			Found,									//!< Found
			PastEOS,								//!< Media is shorter than the period and no segment exists for the specified local time.
			NeedElement,							//!< An additional element is needed that must be loaded first. Execute all returned load requests and try again later.
			BadType,								//!< Representation is bad for some reason, most likely because it uses <SegmentList> addressing which is not supported.
			Gone									//!< Underlying MPD Representation (held by a weak pointer) has gone and the representation is no longer accessible.
		};

		ESearchResult FindSegment(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions);

		void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const TSharedPtrTS<IPlaybackAssetAdaptationSet>& AdaptationSet);

		int32 GetSelectionPriority() const { return SelectionPriority; }

		bool IsSideloadedSubtitle() const { return bIsSideloadedSubtitle; }

		//----------------------------------------------
		// Methods from IPlaybackAssetRepresentation
		//
		FString GetUniqueIdentifier() const override
		{
			return ID;
		}
		const FStreamCodecInformation& GetCodecInformation() const override
		{
			return CodecInfo;
		}
		int32 GetBitrate() const override
		{
			return Bitrate;
		}
		int32 GetQualityIndex() const override
		{
			return QualityIndex;
		}
		bool CanBePlayed() const override
		{
			return bIsEnabled && bIsUsable;
		}

		enum class EStreamContainerType
		{
			Undefined,
			ISO14496_12,
			Matroska
		};
		EStreamContainerType GetStreamContainerType() const
		{
			return StreamContainerType;
		}

	private:
		ESearchResult PrepareSegmentIndex(IPlayerSessionServices* PlayerSessionServices, const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& SegmentBase, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions);

		ESearchResult FindSegment_Base_MP4(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation, const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& SegmentBase);
		ESearchResult FindSegment_Base_MKV(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation, const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& SegmentBase);
		ESearchResult FindSegment_Template(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation, const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& SegmentTemplate);
		ESearchResult FindSegment_Timeline(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation, const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& SegmentTemplate, const TSharedPtrTS<FDashMPD_SegmentTimelineType>& SegmentTimeline);
		ESearchResult SetupSideloadedFile(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& OutSegmentInfo, const FSegmentSearchOption& InSearchOptions, const TSharedPtrTS<FDashMPD_RepresentationType>& MPDRepresentation);

		void PrepareSteeringCandidates(TArray<FContentSteeringHandler::FCandidateURL>& OutSteeringCandidates, IPlayerSessionServices* InPlayerSessionServices, const FString& InDocumentURL, const TArray<DASHUrlHelpers::FDASHMediaURL>& InCandidateURLs, const TArray<TSharedPtrTS<FDashMPD_UrlQueryInfoType>>& InUrlQueries, const FString& InTemplateURL);
		bool PrepareDownloadURLs(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& InOutSegmentInfo, const TArray<TSharedPtrTS<FDashMPD_SegmentBaseType>>& SegmentBase);
		bool PrepareDownloadURLs(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& InOutSegmentInfo, const TArray<TSharedPtrTS<FDashMPD_SegmentTemplateType>>& SegmentTemplate);
		FString ApplyTemplateStrings(FString TemplateURL, int64 InNumber, int64 InTime, int64 InSubIndex);
		void CollectInbandEventStreams(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& InOutSegmentInfo);
		void SetupProducerReferenceTimeInfo(IPlayerSessionServices* PlayerSessionServices, FSegmentInformationCommon& InOutSegmentInfo);

		void SegmentIndexDownloadComplete(TSharedPtrTS<FMPDLoadRequestDASH> Request, bool bSuccess);
		friend class FManifestDASHInternal;
		TWeakPtrTS<FDashMPD_RepresentationType> Representation;
		FStreamCodecInformation CodecInfo;
		FString ID;
		int32 Bitrate = MAX_int32;
		int32 QualityIndex = -1;
		int32 SelectionPriority = 1;
		bool bIsUsable = false;
		bool bIsEnabled = true;
		TMediaOptionalValue<bool> bAvailableAsLowLatency;
		bool bWarnedAboutTimelineStartGap = false;
		bool bWarnedAboutTimelineNoTAfterNegativeR = false;
		bool bWarnedAboutTimelineNumberOverflow = false;
		bool bWarnedAboutInconsistentNumbering = false;
		bool bWarnedAboutTimelineOverlap = false;
		bool bWarnedAboutTimescale = false;
		bool bWarnedAboutInconsistentAvailabilityTimeComplete = false;
		//
		bool bNeedsSegmentIndex = true;
		TSharedPtr<MP4Boxes::FMP4BoxSIDX, ESPMode::ThreadSafe> SIDXBox;
		TSharedPtrTS<const IParserMKV> SegmentMKV;
		int64 SegmentIndexRangeStart = 0;
		int64 SegmentIndexRangeSize = 0;
		TSharedPtrTS<FMPDLoadRequestDASH> PendingSegmentIndexLoadRequest;
		FString StreamMimeType;
		EStreamContainerType StreamContainerType = EStreamContainerType::Undefined;
		//
		bool bIsSideloadedSubtitle = false;

		TArray<FProducerReferenceTimeInfo> ProducerReferenceTimeInfos;
	};

	class FAdaptationSet : public IPlaybackAssetAdaptationSet
	{
	public:
		FAdaptationSet() = default;
		virtual ~FAdaptationSet() = default;

		const FStreamCodecInformation& GetCodec() const								{ return Codec; }
		const TArray<TSharedPtrTS<FRepresentation>>& GetRepresentations() const		{ return Representations; }
		const TArray<FString>& GetRoles() const										{ return Roles; }
		const TArray<FString>& GetAccessibilities() const							{ return Accessibilities; }
		const FTimeFraction& GetPAR() const											{ return PAR; }
		int32 GetMaxBandwidth() const												{ return MaxBandwidth; }
		int32 GetSelectionPriority() const											{ return SelectionPriority; }
		bool GetIsUsable() const													{ return bIsUsable; }
		bool GetIsInSwitchGroup() const												{ return bIsInSwitchGroup; }
		bool GetIsSwitchGroup() const												{ return bIsSwitchGroup; }
		const TArray<FLabelElement>& GetLabels() const								{ return Labels; }

		TSharedPtrTS<FRepresentation> GetRepresentationByUniqueID(const FString& UniqueIdentifier) const
		{
			for(int32 i=0; i<Representations.Num(); ++i)
			{
				if (Representations[i]->GetUniqueIdentifier().Equals(UniqueIdentifier))
				{
					return Representations[i];
				}
			}
			return nullptr;
		}


		void MapRoleAccessibilityToHTML5(FTrackMetadata& InOutMetadata, EStreamType StreamType) const
		{
			/*
				Role: "main", "alternate", "supplementary", "commentary", "dub", "emergency", "caption", "subtitle", "sign" or "description"
				Accessibility: "sign", "caption", "description", "enhanced-audio-intelligibility", or starts with "608:"/"708:" followed by the Value
			*/
			auto IsCEAService = [this]() -> bool
			{
				for(auto &Acc : Accessibilities)
				{
					if (Acc.StartsWith(TEXT("608:")) || Acc.StartsWith(TEXT("708:")))
					{
						return true;
					}
				}
				return false;
			};

			// See: https://dev.w3.org/html5/html-sourcing-inband-tracks/#mpegdash
			bool bMain = Roles.Contains(TEXT("main"));
			bool bAlternate = Roles.Contains(TEXT("alternate"));
			bool bSupplementary = Roles.Contains(TEXT("supplementary"));
			bool bCommentary = Roles.Contains(TEXT("commentary"));
			bool bDub = Roles.Contains(TEXT("dub"));
			bool bEmergency = Roles.Contains(TEXT("emergency"));
			bool bCaption = Roles.Contains(TEXT("caption"));
			bool bSubtitle = Roles.Contains(TEXT("subtitle"));
			bool bSign = Roles.Contains(TEXT("sign"));
			bool bDescription = Roles.Contains(TEXT("description"));
			bool bIsCEAService = IsCEAService();
			if (StreamType == EStreamType::Video || StreamType == EStreamType::Audio)
			{
				/*
					"alternative": if the role is "alternate" but not also "main" or "commentary", or "dub"
					"captions": if the role is "caption" and also "main"
					"descriptions": if the role is "description" and also "supplementary"
					"main": if the role is "main" but not also "caption", "subtitle", or "dub"
					"main-desc": if the role is "main" and also "description"
					"sign": not used
					"subtitles": if the role is "subtitle" and also "main"
					"translation": if the role is "dub" and also "main"
					"commentary": if the role is "commentary" but not also "main"
					"": otherwise
				*/
				if (bMain && !(bCaption || bSubtitle || bDub))
				{
					InOutMetadata.Kind = TEXT("main");
				}
				else if (bMain && bDescription)
				{
					InOutMetadata.Kind = TEXT("main-desc");
				}
				else if (bAlternate && !(bMain || bCommentary || bDub))
				{
					InOutMetadata.Kind = TEXT("alternative");
				}
				else if (bSubtitle && bMain)
				{
					InOutMetadata.Kind = TEXT("subtitles");
				}
				else if (bCaption && bMain)
				{
					InOutMetadata.Kind = TEXT("captions");
				}
				else if (bDescription && bSupplementary)
				{
					InOutMetadata.Kind = TEXT("descriptions");
				}
				else if (bDub && bMain)
				{
					InOutMetadata.Kind = TEXT("translation");
				}
				else if (bCommentary && !bMain)
				{
					InOutMetadata.Kind = TEXT("commentary");
				}
			}
			else if (StreamType == EStreamType::Subtitle)
			{
				/*
					Is an ISOBMFF CEA 608 or 708 caption service: "captions".
					"captions": if the Role descriptor's value is "caption"
					"subtitles": if the Role descriptor's value is "subtitle"
					"metadata": otherwise
				*/
				if (bIsCEAService || bCaption)
				{
					InOutMetadata.Kind = TEXT("captions");
				}
				else if (bSubtitle)
				{
					InOutMetadata.Kind = TEXT("subtitles");
				}
				else
				{
					InOutMetadata.Kind = TEXT("metadata");
				}
				// ID and language for CEA services
				if (bIsCEAService)
				{
					// TODO
					check(!"need to add ID and language handling");
					InOutMetadata.ID = TEXT("CC1");
					BCP47::ParseRFC5646Tag(InOutMetadata.LanguageTagRFC5646, FString(TEXT("en")));
				}
			}
		}


		void GetMetaData(FTrackMetadata& OutMetadata, EStreamType StreamType) const
		{
			OutMetadata.ID = GetUniqueIdentifier();
			OutMetadata.LanguageTagRFC5646 = GetLanguageTag();
			OutMetadata.HighestBandwidth = GetMaxBandwidth();
			OutMetadata.HighestBandwidthCodec = GetCodec();
			if (Labels.Num())
			{
				// Return the first label, regardless of its language or related group label ID.
				OutMetadata.Label = Labels[0].LabelText;
			}
			// Map role and accessibility. Do this last since this is allowed to overwrite ID and Language
			MapRoleAccessibilityToHTML5(OutMetadata, StreamType);
			const TArray<TSharedPtrTS<FRepresentation>>& Reprs = GetRepresentations();
			for(int32 j=0; j<Reprs.Num(); ++j)
			{
				if (Reprs[j]->CanBePlayed())
				{
					FStreamMetadata sd;
					sd.Bandwidth = Reprs[j]->GetBitrate();
					sd.CodecInformation = Reprs[j]->GetCodecInformation();
					OutMetadata.StreamDetails.Emplace(MoveTemp(sd));
				}
			}
		}


		struct FContentProtection
		{
			TSharedPtrTS<FDashMPD_DescriptorType> Descriptor;
			FString DefaultKID;
			FString CommonScheme;
		};
		FString GetMimeType()
		{
			TSharedPtrTS<FDashMPD_AdaptationSetType> MPDAdaptationSet = AdaptationSet.Pin();
			return MPDAdaptationSet.IsValid() ? MPDAdaptationSet->GetMimeType() : FString();
		}
		FString GetMimeTypeWithCodecs()
		{
			TSharedPtrTS<FDashMPD_AdaptationSetType> MPDAdaptationSet = AdaptationSet.Pin();
			if (MPDAdaptationSet.IsValid())
			{
				return FString::Printf(TEXT("%s; codecs=\"%s\""), *MPDAdaptationSet->GetMimeType(), *GetListOfCodecs());
			}
			return FString();
		}
		const TArray<FContentProtection>& GetPossibleContentProtections() const { return PossibleContentProtections; }
		const FString& GetCommonEncryptionScheme() const { return CommonEncryptionScheme; }
		const FString& GetDefaultKID() const { return DefaultKID; }

		const TArray<FString>& GetSwitchToSetIDs() const { return SwitchToSetIDs; }
		const TArray<FString>& GetSwitchedFromSetIDs() const { return SwitchedFromSetIDs; }

		//----------------------------------------------
		// Methods from IPlaybackAssetAdaptationSet
		//
		FString GetUniqueIdentifier() const override
		{
			return FString::Printf(TEXT("%d"), UniqueSequentialSetIndex);
		}
		FString GetListOfCodecs() const override
		{
			return Codec.GetCodecSpecifierRFC6381();
		}
		const BCP47::FLanguageTag& GetLanguageTag() const override
		{
			return LanguageTag;
		}
		int32 GetNumberOfRepresentations() const override
		{
			return Representations.Num();
		}
		bool IsLowLatencyEnabled() const override
		{
			return bAvailableAsLowLatency.GetWithDefault(false);
		}
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByIndex(int32 RepresentationIndex) const override
		{
			if (RepresentationIndex < Representations.Num())
			{
				return Representations[RepresentationIndex];
			}
			return nullptr;
		}
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByUniqueIdentifier(const FString& UniqueIdentifier) const override
		{
			return GetRepresentationByUniqueID(UniqueIdentifier);
		}
	private:
		friend class FManifestDASHInternal;
		TWeakPtrTS<FDashMPD_AdaptationSetType> AdaptationSet;
		FStreamCodecInformation Codec;
		TArray<TSharedPtrTS<FRepresentation>> Representations;
		TArray<FString> Roles;
		TArray<FString> Accessibilities;
		TArray<FLabelElement> Labels;
		FTimeFraction PAR;
		BCP47::FLanguageTag LanguageTag;
		int32 MaxBandwidth = 0;
		int32 SelectionPriority = 1;
		int32 UniqueSequentialSetIndex = 0;
		bool bIsUsable = false;
		bool bIsEnabled = true;
		TMediaOptionalValue<bool> bAvailableAsLowLatency;

		// Encryption related
		TArray<FContentProtection> PossibleContentProtections;
		FString CommonEncryptionScheme;
		FString DefaultKID;
		// Switching related
		TArray<FString> SwitchToSetIDs;
		TArray<FString> SwitchedFromSetIDs;
		bool bIsInSwitchGroup = false;
		bool bIsSwitchGroup = false;
		// Warnings
		bool bWarnedAboutInconsistentAvailabilityTimeComplete = false;
	};

	class FPeriod : public ITimelineMediaAsset
	{
	public:
		FPeriod() = default;
		virtual ~FPeriod() = default;

		bool GetHasBeenPrepared() const
		{
			return bHasBeenPrepared;
		}
		void SetHasBeenPrepared(bool bPrepared)
		{
			bHasBeenPrepared = bPrepared;
		}

		bool GetHasFollowingPeriod() const
		{
			return bHasFollowingPeriod;
		}
		void SetHasFollowingPeriod(bool bInHasFollowing)
		{
			bHasFollowingPeriod = bInHasFollowing;
		}

		const FString& GetID() const { return ID; }
		const FTimeValue& GetStart() const { return Start; }
		const FTimeValue& GetEnd() const { return End; }
		bool GetIsEarlyPeriod() const { return bIsEarlyPeriod; }
		const TArray<TSharedPtrTS<FAdaptationSet>>& GetAdaptationSets() const { return AdaptationSets; }

		TSharedPtrTS<FAdaptationSet> GetAdaptationSetByUniqueID(const FString& UniqueID) const
		{
			for(int32 i=0; i<AdaptationSets.Num(); ++i)
			{
				if (AdaptationSets[i]->GetUniqueIdentifier().Equals(UniqueID))
				{
					return AdaptationSets[i];
				}
			}
			return nullptr;
		}

		TSharedPtrTS<FAdaptationSet> GetAdaptationSetByMPDID(const FString& MPDID) const
		{
			for(int32 i=0; i<AdaptationSets.Num(); ++i)
			{
				TSharedPtrTS<FDashMPD_AdaptationSetType> MPDAdaptationSet = AdaptationSets[i]->AdaptationSet.Pin();
				if (MPDAdaptationSet.IsValid() && MPDAdaptationSet->GetID_AsStr().Equals(MPDID))
				{
					return AdaptationSets[i];
				}
			}
			return nullptr;
		}


		void EndPresentationAt(const FTimeValue& EndsAt)
		{
			FTimeValue NewDur = EndsAt - Start;
			if (NewDur >= FTimeValue::GetZero())
			{
				Duration = NewDur;
				End = Start + NewDur;
				TSharedPtrTS<FDashMPD_PeriodType> MPDPeriod = Period.Pin();
				if (MPDPeriod.IsValid())
				{
					MPDPeriod->SetDuration(NewDur);
				}
			}
		}

/*
		TSharedPtrTS<FRepresentation> GetRepresentationFromAdaptationSetIDs(const FString& AdaptationSetID, const FString& RepresentationID) const
		{
			TSharedPtrTS<FAdaptationSet> Adapt = GetAdaptationSetByUniqueID(AdaptationSetID);
			if (Adapt.IsValid())
			{
				return Adapt->GetRepresentationByUniqueID(RepresentationID);
			}
			return nullptr;
		}
*/
		TSharedPtrTS<FDashMPD_PeriodType> GetMPDPeriod() { return Period.Pin(); }

		//----------------------------------------------
		// Methods from ITimelineMediaAsset
		//
		FTimeRange GetTimeRange() const override
		{
			// Per convention the time range includes the AST
			return FTimeRange({StartAST, EndAST});
		}
		FTimeValue GetDuration() const override
		{
			return Duration;
		}
		FString GetAssetIdentifier() const override
		{
			TSharedPtrTS<FDashMPD_PeriodType> MPDPeriod = Period.Pin();
			return MPDPeriod.IsValid() && MPDPeriod->GetAssetIdentifier().IsValid() ? MPDPeriod->GetAssetIdentifier()->GetValue() : FString();
		}
		FString GetUniqueIdentifier() const override
		{
			return ID;
		}
		int32 GetNumberOfAdaptationSets(EStreamType OfStreamType) const override
		{
			int32 Num=0;
			for(int32 i=0; i<AdaptationSets.Num(); ++i)
			{
				if (AdaptationSets[i]->GetCodec().GetStreamType() == OfStreamType)
				{
					++Num;
				}
			}
			return Num;
		}
		TSharedPtrTS<IPlaybackAssetAdaptationSet> GetAdaptationSetByTypeAndIndex(EStreamType OfStreamType, int32 AdaptationSetIndex) const override
		{
			int32 Num=0;
			for(int32 i=0; i<AdaptationSets.Num(); ++i)
			{
				if (AdaptationSets[i]->GetCodec().GetStreamType() == OfStreamType)
				{
					if (Num++ == AdaptationSetIndex)
					{
						return AdaptationSets[i];
					}
				}
			}
			return nullptr;
		}

		void GetMetaData(TArray<FTrackMetadata>& OutMetadata, EStreamType OfStreamType) const override
		{
			for(int32 i=0, iMax=GetNumberOfAdaptationSets(OfStreamType); i<iMax; ++i)
			{
				TSharedPtrTS<IPlaybackAssetAdaptationSet> AdaptationSet = GetAdaptationSetByTypeAndIndex(OfStreamType, i);
				const FAdaptationSet* Adapt = static_cast<const FAdaptationSet*>(AdaptationSet.Get());
				if (Adapt && Adapt->GetIsUsable() && !Adapt->GetIsInSwitchGroup())
				{
					FTrackMetadata tm;
					Adapt->GetMetaData(tm, OfStreamType);
					OutMetadata.Emplace(MoveTemp(tm));
				}
			}
		}

		void UpdateRunningMetaData(const FString& InKindOfValue, const FVariant& InNewValue) override
		{ }

	private:
		friend class FManifestDASHInternal;
		TArray<TSharedPtrTS<FAdaptationSet>> AdaptationSets;
		TWeakPtrTS<FDashMPD_PeriodType> Period;
		FString ID;
		FTimeValue Start;
		FTimeValue End;
		FTimeValue StartAST;
		FTimeValue EndAST;
		FTimeValue Duration;
		bool bIsEarlyPeriod = false;
		bool bHasFollowingPeriod = false;
		bool bHasBeenPrepared = false;
	};

	FErrorDetail Build(IPlayerSessionServices* InPlayerSessionServices, TSharedPtr<FDashMPD_MPDType, ESPMode::ThreadSafe> InMPDRoot, TArray<TWeakPtrTS<IDashMPDElement>> InXLinkElements);

	FErrorDetail BuildAfterInitialRemoteElementDownload();

	void GetRemoteElementLoadRequests(TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests)
	{
		OutRemoteElementLoadRequests = PendingRemoteElementLoadRequests;
	}

	FErrorDetail ResolveInitialRemoteElementRequest(TSharedPtrTS<FMPDLoadRequestDASH> RequestResponse, FString XMLResponse, bool bSuccess);

	EPresentationType GetPresentationType() const
	{
		return PresentationType;
	}

	bool IsEpicEvent() const
	{
		return EpicEventType != EEpicEventType::None;
	}

	bool IsDynamicEpicEvent() const
	{
		return EpicEventType == EEpicEventType::Dynamic;
	}

	const TArray<TSharedPtrTS<FPeriod>>& GetPeriods() const
	{
		return Periods;
	}

	TSharedPtrTS<FManifestDASHInternal::FPeriod> GetPeriodByUniqueID(const FString& InUniqueID) const
	{
		for(int32 i=0, iMax=Periods.Num(); i<iMax; ++i)
		{
			if (Periods[i]->GetUniqueIdentifier().Equals(InUniqueID))
			{
				return Periods[i];
			}
		}
		return nullptr;
	}

	bool HasFollowingRegularPeriod(TSharedPtrTS<FPeriod> InPeriod) const
	{
		for(int32 i=0, iMax=Periods.Num(); i<iMax; ++i)
		{
			if (Periods[i] == InPeriod)
			{
				// Is there a following period and is it a regular period?
				if (i+1 < iMax && !Periods[i+1]->GetIsEarlyPeriod())
				{
					return true;
				}
				return false;
			}
		}
		return false;
	}

	TSharedPtrTS<const FDashMPD_MPDType> GetMPDRoot() const
	{
		return MPDRoot;
	}

	TSharedPtrTS<FDashMPD_MPDType> GetMPDRoot()
	{
		return MPDRoot;
	}

	void PreparePeriodAdaptationSets(TSharedPtrTS<FPeriod> InPeriod, bool bRequestXlink);

	void SendEventsFromAllPeriodEventStreams(TSharedPtrTS<FPeriod> InPeriod);

	const TArray<FURL_RFC3986::FQueryParam>& GetURLFragmentComponents() const
	{
		return URLFragmentComponents;
	}

	void SetURLFragmentComponents(TArray<FURL_RFC3986::FQueryParam> InURLFragmentComponents)
	{
		URLFragmentComponents = MoveTemp(InURLFragmentComponents);
	}

	const TArray<FURL_RFC3986::FQueryParam>& GetURLFragmentComponents()
	{
		return URLFragmentComponents;
	}

	void SetReplayEventParams(const Playlist::FReplayEventParams& InEventReplayParams)
	{
		ReplayEventParams = InEventReplayParams;
	}

	const Playlist::FReplayEventParams& GetReplayEventParams()
	{
		return ReplayEventParams;
	}

	void InjectEpicTimingSources();
	void TransformIntoEpicEvent();
	void PrepareEpicEventForLooping(int32 InNumLoopsToAdd);

	ELECTRA_IMPL_DEFAULT_ERROR_METHODS(DASHManifest);


	TSharedPtrTS<const FLowLatencyDescriptor> GetLowLatencyDescriptor() const
	{
		return LowLatencyDescriptor;
	}
	TSharedPtrTS<FProducerReferenceTimeInfo> GetProducerReferenceTimeElement(int64 InID) const
	{
		return InID >= 0 ? ProducerReferenceTimeElements.FindRef((uint32)InID) : nullptr;
	}

	FTimeValue GetSegmentFetchDelay() const
	{
		return SegmentFetchDelay;
	}
	void SetSegmentFetchDelay(const FTimeValue& InNewFetchDelay)
	{
		SegmentFetchDelay = InNewFetchDelay;
	}

	FTimeValue GetAnchorTime() const;
	FTimeRange GetTotalTimeRange() const;
	FTimeRange GetSeekableTimeRange() const;
	FTimeValue GetDuration() const;
	void PrepareDefaultStartTime();
	FTimeValue GetDefaultStartTime() const;
	void ClearDefaultStartTime();
	FTimeValue GetDefaultEndTime() const;
	void ClearDefaultEndTime();
	FTimeRange GetPlayTimesFromURI(IManifest::EPlaybackRangeType InRangeType) const;
	FTimeValue GetDesiredLiveLatency() const;

	FTimeValue GetMPDValidityEndTime() const;
	FTimeValue GetLastPeriodEndTime(bool bForTimeline) const;
	FTimeValue GetMinimumUpdatePeriod() const;
	bool AreUpdatesExpected() const;
	bool IsStaticType() const;
	bool UsesAST() const;
	bool HasInjectedTimingSources() const;

	FTimeValue GetAvailabilityEndTime() const;
	FTimeValue GetTimeshiftBufferDepth() const;

	void EndPresentationAt(const FTimeValue& EndsAt, const FString& InPeriod);

private:
	enum class EEpicEventType
	{
		None,
		Static,
		Dynamic
	};
	FErrorDetail PrepareRemoteElementLoadRequest(TArray<TWeakPtrTS<FMPDLoadRequestDASH>>& OutRemoteElementLoadRequests, TWeakPtrTS<IDashMPDElement> ElementWithXLink, int64 RequestID);

	int32 ReplaceElementWithRemoteEntities(TSharedPtrTS<IDashMPDElement> Element, const FDashMPD_RootEntities& NewRootEntities, int64 OldResolveID, int64 NewResolveID);

	FTimeValue CalculateDistanceToLiveEdge() const;

	bool CanUseEncryptedAdaptation(const TSharedPtrTS<FAdaptationSet>& InAdaptationSet);

	IPlayerSessionServices* PlayerSessionServices = nullptr;

	TArray<TWeakPtrTS<IDashMPDElement>> RemoteElementsToResolve;
	TArray<TWeakPtrTS<FMPDLoadRequestDASH>> PendingRemoteElementLoadRequests;

	// The parsed MPD.
	TSharedPtrTS<FDashMPD_MPDType> MPDRoot;

	// The MPD URL fragment components
	TArray<FURL_RFC3986::FQueryParam> URLFragmentComponents;
	Playlist::FReplayEventParams ReplayEventParams;

	// Type of the presentation.
	EPresentationType PresentationType;
	EEpicEventType EpicEventType = EEpicEventType::None;
	FTimeValue OriginalPresentationDuration;

	TArray<TSharedPtrTS<FPeriod>> Periods;

	TMap<uint32, TSharedPtrTS<FProducerReferenceTimeInfo>> ProducerReferenceTimeElements;
	TSharedPtrTS<FLowLatencyDescriptor> LowLatencyDescriptor;

	FTimeValue SegmentFetchDelay;

	mutable FTimeValue CalculatedLiveDistance;

	mutable FTimeRange TotalTimeRange;
	mutable FTimeRange SeekableTimeRange;
	FTimeValue DefaultStartTime;
	FTimeValue DefaultEndTime;
	mutable bool bWarnedAboutTooSmallSuggestedPresentationDelay = false;

	bool bDidInjectUTCTimingElements = false;
};


} // namespace Electra

