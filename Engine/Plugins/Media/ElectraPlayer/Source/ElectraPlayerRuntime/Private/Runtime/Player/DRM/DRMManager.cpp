// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/DRM/DRMManager.h"
#include "Player/PlayerLicenseKey.h"

namespace Electra
{

namespace
{
	ElectraCDM::IMediaCDM* Get()
	{
		return &ElectraCDM::IMediaCDM::Get();
	}
}


FString FDRMManager::MakeHexStringFromArray(const TArray<uint8>& InArray)
{
	return FString::FromHexBlob(InArray.GetData(), InArray.Num());
}

FString FDRMManager::MakePrintableStringFromUint32(uint32 InUint32)
{
	FString Out;
	// Not so much just printable as alphanumeric.
	for(uint32 i=0, Atom=InUint32; i<4; ++i, Atom<<=8)
	{
		int32 v = Atom >> 24;
		if ((v >= 'A' && v <= 'Z') || (v >= 'a' && v <= 'z') || (v >= '0' && v <= '9') || v == '_')
		{
			Out.AppendChar(v);
		}
		else
		{
			// Not alphanumeric, return it as a hex string.
			return FString::Printf(TEXT("%08x"), InUint32);
		}
	}
	return Out;
}


FDRMManager::FDRMManager(IPlayerSessionServices* InPlayerSessionServices)
	: PlayerSessionServices(InPlayerSessionServices)
{
	Session = Get()->CreatePlayerSessionID();
}

FDRMManager::~FDRMManager()
{
	if (Session)
	{
		Get()->ReleasePlayerSessionKeys(Session);
		Get()->ReleasePlayerSessionID(Session);
	}
}

TSharedPtrTS<FDRMManager> FDRMManager::Create(IPlayerSessionServices* InPlayerSessionServices)
{
	return MakeSharedTS<FDRMManager>(InPlayerSessionServices);
}

void FDRMManager::Tick()
{
	FScopeLock lock(&Lock);
	CompletedRequests.Empty();
}

void FDRMManager::Close()
{
	FScopeLock lock(&Lock);
	PendingRequests.Empty();
	CompletedRequests.Empty();
	if (Session)
	{
		Get()->ReleasePlayerSessionKeys(Session);
		Get()->ReleasePlayerSessionID(Session);
		Session = nullptr;
	}
}


void FDRMManager::GetCDMCustomJSONPrefixes(const FString& InCDMScheme, const FString& InValue, FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces)
{
	Get()->GetCDMCustomJSONPrefixes(InCDMScheme, InValue, OutAttributePrefix, OutTextPropertyName, bOutNoNamespaces);
}

TSharedPtr<ElectraCDM::IMediaCDMCapabilities, ESPMode::ThreadSafe> FDRMManager::GetCDMCapabilitiesForScheme(const FString& InCDMScheme, const FString& InValue, const FString& InAdditionalElements)
{
	return Get()->GetCDMCapabilitiesForScheme(InCDMScheme, InValue, InAdditionalElements);
}


ElectraCDM::ECDMError FDRMManager::CreateDRMClient(TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, const TArray<ElectraCDM::IMediaCDM::FCDMCandidate>& InCandidates)
{
	return Get()->CreateDRMClient(OutClient, Session, InCandidates);
}

void FDRMManager::OnHTTPResourceRequestComplete(TSharedPtrTS<FHTTPResourceRequest> InRequest)
{
	TSharedPtrTS<FDRMAsyncRequest> DrmReq = StaticCastSharedPtr<FDRMAsyncRequest>(InRequest->GetObject());
	if (DrmReq.IsValid())
	{
		FScopeLock lock(&Lock);
		PendingRequests.Remove(DrmReq);
		CompletedRequests.Emplace(DrmReq);

		TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> Client = DrmReq->Client.Pin();
		if (Client.IsValid())
		{
			TArray<uint8> LicenseResponse;

			TSharedPtrTS<FWaitableBuffer> ResponseBuffer = DrmReq->Request->GetResponseBuffer();
			if (ResponseBuffer.IsValid())
			{
				LicenseResponse.Append((const uint8*)ResponseBuffer->GetLinearReadData(), ResponseBuffer->Num());
			}
			const HTTP::FConnectionInfo* ConnInfo = DrmReq->Request->GetConnectionInfo();
			int32 HttpResponseCode = ConnInfo ? ConnInfo->StatusInfo.HTTPStatus : 500;
			PlayerSessionServices->SendMessageToPlayer(FLicenseKeyMessage::Create(FLicenseKeyMessage::EReason::LicenseKeyDownload, FErrorDetail(), ConnInfo));
			if (Client->SetLicenseKeyResponseData(DrmReq->EventId, HttpResponseCode, LicenseResponse) != ElectraCDM::ECDMError::Success)
			{
				PlayerSessionServices->SendMessageToPlayer(FLicenseKeyMessage::Create(FLicenseKeyMessage::EReason::LicenseKeyData,
																						FErrorDetail().SetError(UEMEDIA_ERROR_FORMAT_ERROR).SetFacility(Facility::EFacility::LicenseKey).SetMessage("Invalid license key"), ConnInfo));
			}
		}
	}
}


void FDRMManager::OnCDMEvent(ElectraCDM::IMediaCDMEventListener::ECDMEventType InEventType, TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> InDrmClient, void* InEventId, const TArray<uint8>& InCustomData)
{
	if (InEventType == ElectraCDM::IMediaCDMEventListener::ECDMEventType::KeyRequired)
	{
		FString LicenseURL, HttpMethod, MimeType;
		TArray<uint8> LicenseData;
		TArray<FString> Headers;
		uint32 Flags = 0;
		InDrmClient->GetLicenseKeyURL(LicenseURL);
		InDrmClient->GetLicenseKeyRequestData(LicenseData, HttpMethod, Headers, Flags);
		if (HttpMethod.IsEmpty())
		{
			HttpMethod = TEXT("POST");
		}
		if (MimeType.IsEmpty())
		{
			MimeType = TEXT("application/octet-stream");
		}
		if (LicenseURL.IsEmpty() && (Flags & ElectraCDM::IMediaCDMClient::EDRMClientFlags::EDRMFlg_AllowCustomKeyStorage) == 0)
		{
			PlayerSessionServices->PostError(FErrorDetail().SetFacility(Facility::EFacility::DRM).SetCode(1).SetError(UEMEDIA_ERROR_INTERNAL).SetMessage(FString::Printf(TEXT("No license URL specified to acquire DRM license from in event %d"), (int32)InEventType)));
			return;
		}

		TSharedPtrTS<FDRMAsyncRequest> DrmReq = MakeSharedTS<FDRMAsyncRequest>();
		DrmReq->Event = InEventType;
		DrmReq->Client = InDrmClient;
		DrmReq->EventId = InEventId;
		DrmReq->CustomData = InCustomData;
		DrmReq->Request = MakeSharedTS<FHTTPResourceRequest>();
		DrmReq->Request->URL(LicenseURL).Verb(HttpMethod).Headers(Headers).PostData(LicenseData).Object(DrmReq);
		DrmReq->Request->Callback().BindThreadSafeSP(AsShared(), &FDRMManager::OnHTTPResourceRequestComplete);
		if ((Flags & ElectraCDM::IMediaCDMClient::EDRMClientFlags::EDRMFlg_AllowCustomKeyStorage) != 0)
		{
			DrmReq->Request->AllowStaticQuery(IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::LicenseKey);
		}
		Lock.Lock();
		PendingRequests.Emplace(DrmReq);
		Lock.Unlock();
		DrmReq->Request->StartGet(PlayerSessionServices);
	}
	else
	{
		PlayerSessionServices->PostError(FErrorDetail().SetFacility(Facility::EFacility::DRM).SetCode(1).SetError(UEMEDIA_ERROR_INTERNAL).SetMessage(FString::Printf(TEXT("Unhandled DRM event %d"), (int32)InEventType)));
	}
}


} // namespace Electra




