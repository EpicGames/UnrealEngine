// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "ErrorDetail.h"
#include "Player/PlayerSessionServices.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "ElectraCDM.h"
#include "ElectraCDMClient.h"


namespace Electra
{

class FDRMManager : public TSharedFromThis<FDRMManager, ESPMode::ThreadSafe>, public ElectraCDM::IMediaCDMEventListener
{
public:
	static TSharedPtrTS<FDRMManager> Create(IPlayerSessionServices* InPlayerSessionServices);
	void Tick();
	void Close();

	FDRMManager(IPlayerSessionServices* InPlayerSessionServices);
	virtual ~FDRMManager();

	void GetCDMCustomJSONPrefixes(const FString& InCDMScheme, const FString& InValue, FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces);
	TSharedPtr<ElectraCDM::IMediaCDMCapabilities, ESPMode::ThreadSafe> GetCDMCapabilitiesForScheme(const FString& InCDMScheme, const FString& InValue, const FString& InAdditionalElements);

	ElectraCDM::ECDMError CreateDRMClient(TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, const TArray<ElectraCDM::IMediaCDM::FCDMCandidate>& InCandidates);

	virtual void OnCDMEvent(ElectraCDM::IMediaCDMEventListener::ECDMEventType InEventType, TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> InDrmClient, void* InEventId, const TArray<uint8>& InCustomData) override;

	static FString MakeHexStringFromArray(const TArray<uint8>& InArray);
	static FString MakePrintableStringFromUint32(uint32 InUint32);

private:

	struct FDRMAsyncRequest : public IHTTPResourceRequestObject
	{
		ElectraCDM::IMediaCDMEventListener::ECDMEventType Event;
		TWeakPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> Client;
		TArray<uint8> CustomData;
		void* EventId = nullptr;
		TSharedPtrTS<FHTTPResourceRequest> Request;
	};

	void OnHTTPResourceRequestComplete(TSharedPtrTS<FHTTPResourceRequest> InRequest);

	FCriticalSection Lock;
	IPlayerSessionServices* PlayerSessionServices = nullptr;
	ElectraCDM::IMediaCDM::IPlayerSession* Session = nullptr;
	TArray<TSharedPtrTS<FDRMAsyncRequest>> PendingRequests;
	TArray<TSharedPtrTS<FDRMAsyncRequest>> CompletedRequests;
};


} // namespace Electra

