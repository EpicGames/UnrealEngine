// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlaylistHLS.h"
#include "Player/PlayerStreamFilter.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/StreamSegmentReaderCommon.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/ContentSteeringHandler.h"
#include "Utilities/Utilities.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/ISO639-Map.h"
#include "Crypto/StreamCryptoAES128.h"
#include "Misc/Optional.h"
#include "Misc/SecureHash.h"
#include "ElectraPlayerPrivate.h"

namespace Electra
{
namespace HLS
{
const int32 kAssumedAudioBandwidth = 128000;
const int32 kAssumedSubtitleBandwidth = 8000;
}

namespace Util
{

static inline FString GetBaseCodec(const FString& InCodec)
{
	int32 DotPos;
	if (InCodec.FindChar(TCHAR('.'), DotPos))
	{
		return InCodec.Left(DotPos);
	}
	return InCodec;
}

}


struct FActiveHLSPlaylist::FInternalBuilder
{
	struct FGroupPrio
	{
		FGroupPrio(int32 InGroupIndex, int32 InPriority)
			: GroupIndex(InGroupIndex), Priority(InPriority)
		{}
		int32 GroupIndex;
		int32 Priority;
	};

	struct FAudioRenditionGroup
	{
		FString GroupName;
		TArray<int32> UsableRenditionIndices;
		bool operator == (const FString& InGroupName) const
		{ return GroupName.Equals(InGroupName); }
	};

	struct FVideoVariantAudio
	{
		FString GroupName;
		int32 VariantGroupIndex = -1;
		bool bIsInband = false;
		bool bIsEmpty = false;
		bool bHasNoAudio = false;
		bool operator == (const FVideoVariantAudio& rhs) const
		{ return GroupName.Equals(rhs.GroupName) && VariantGroupIndex==rhs.VariantGroupIndex && bIsInband==rhs.bIsInband && bIsEmpty==rhs.bIsEmpty && bHasNoAudio==rhs.bHasNoAudio; }
	};

	// Grouped and filtered variants and renditions
	TArray<FAudioRenditionGroup> AudioRenditionGroups;
	TArray<FGroupPrio> UsableVideoVariantGroupIndices;
	TArray<FGroupPrio> UsableAudioVariantGroupIndices;
	TArray<TArray<FVideoVariantAudio>> VideoGroupAssociatedAudio;
	int32 SelectedVideoVariantGroup = -1;
	int32 SelectedAudioVariantGroup = -1;
};


/**
 * Internal class representing a playback period.
 * A period is part (or all of) the content on the timeline.
 */
class FActiveHLSPlaylist::FPlayPeriod : public IManifest::IPlayPeriod
{
public:
	FPlayPeriod(IPlayerSessionServices* InPlayerSessionServices, const TSharedPtr<FTimelineMediaAsset, ESPMode::ThreadSafe>& InTimelineMediaAsset);
	virtual ~FPlayPeriod();
	void SetStreamPreferences(EStreamType InForStreamType, const FStreamSelectionAttributes& InStreamAttributes) override;
	EReadyState GetReadyState() override;
	void Load() override;
	void PrepareForPlay() override;
	int64 GetDefaultStartingBitrate() const override;
	TSharedPtrTS<FBufferSourceInfo> GetSelectedStreamBufferSourceInfo(EStreamType InStreamType) override;
	FString GetSelectedAdaptationSetID(EStreamType InStreamType) override;
	ETrackChangeResult ChangeTrackStreamPreference(EStreamType InStreamType, const FStreamSelectionAttributes& InStreamAttributes) override;
	TSharedPtrTS<ITimelineMediaAsset> GetMediaAsset() const override;
	void SelectStream(const FString& InAdaptationSetID, const FString& InRepresentationID, int32 InQualityIndex, int32 InMaxQualityIndex) override;
	void TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InInitSegmentsToPreload) override;
	FResult GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& InStartPosition, ESearchType InSearchType) override;
	FResult GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType InStreamType, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& InStartPosition, ESearchType InSearchType) override;
	FResult GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& InStartPosition, ESearchType InSearchType) override;
	FResult GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> InCurrentSegment, const FPlayStartOptions& InOptions) override;
	FResult GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> InCurrentSegment, const FPlayStartOptions& InOptions, bool bInReplaceWithFillerData) override;
	void IncreaseSegmentFetchDelay(const FTimeValue& InIncreaseAmount) override;
	void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& InAdaptationSetID, const FString& InRepresentationID) override;

	enum class ENextSegType
	{
		Next,
		Retry,
		StartOver
	};
	FResult GetSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FStreamSegmentRequestCommon* InSegment, const FPlayStartOptions& InOptions, ENextSegType InNextType);
	void GetActiveMediaPlaylists(TArray<TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>>& OutActivePlaylists);
private:
	void SetTimestampAdjustIfNecessary(const TSharedPtrTS<FStreamSegmentRequestCommon>& InSegment);
	void ValidateDownloadedSegmentDuration(const FStreamSegmentRequestCommon* InRequest);

	struct FSelectedTrackStream
	{
		FString MetaID;
		int32 TrackIndex = 0;
		int32 StreamIndex = 0;
		bool bIsSelected = false;
		TSharedPtrTS<FBufferSourceInfo> BufferSourceInfo;

		int32 QualityIndex = 0;
		int32 MaxQualityIndex = 0;

		TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> ActivePlaylist;
	};

	struct FStreamLoadRequest
	{
		EStreamType Type;
		TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> Request;
		TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> Playlist;
	};

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	TSharedPtr<FTimelineMediaAsset, ESPMode::ThreadSafe> TimelineMediaAsset;
	IManifest::IPlayPeriod::EReadyState CurrentReadyState = IManifest::IPlayPeriod::EReadyState::NotLoaded;
	FStreamSelectionAttributes StreamSelectionAttributes[4];
	FSelectedTrackStream SelectedTrackStream[4];
};


FActiveHLSPlaylist::FActiveHLSPlaylist()
{
	TimelineMediaAsset = MakeShared<FTimelineMediaAsset, ESPMode::ThreadSafe>();
}

FActiveHLSPlaylist::~FActiveHLSPlaylist()
{
}


FErrorDetail FActiveHLSPlaylist::PreparePathway(TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>& InOutPathway, TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InFromMultiVariantPlaylist)
{
	IPlayerStreamFilter* Filter = PlayerSessionServices->GetStreamFilter();

	TUniquePtr<FInternalBuilder> Builder=MakeUnique<FInternalBuilder>();
	// Set up the audio rendition groups filtered by supported codecs
	for(int32 i=0, iMax=InFromMultiVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)].Num(); i<iMax; ++i)
	{
		FInternalBuilder::FAudioRenditionGroup& arg = Builder->AudioRenditionGroups.Emplace_GetRef();
		arg.GroupName = InFromMultiVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)][i].GroupID;
		for(int32 j=0, jMax=InFromMultiVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)][i].Renditions.Num(); j<jMax; ++j)
		{
			if (Filter->CanDecodeStream(InFromMultiVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)][i].Renditions[j].ParsedCodecFromStreamInf))
			{
				arg.UsableRenditionIndices.Emplace(j);
			}
		}
	}

	TArray<FMultiVariantPlaylistHLS::FStreamInf>& sInfs(InOutPathway->StreamInfs);

	// If there are audio-only variant groups check the for supported codec.
	const TArray<FMultiVariantPlaylistHLS::FVideoVariantGroup>& vgrps = InOutPathway->VideoVariantGroups;
	const TArray<FMultiVariantPlaylistHLS::FAudioVariantGroup>& agrps = InOutPathway->AudioOnlyVariantGroups;
	if (agrps.Num())
	{
		const FCodecSelectionPriorities& acSel = PlayerSessionServices->GetCodecSelectionPriorities(EStreamType::Audio);
		for(int32 i=0; i<agrps.Num(); ++i)
		{
			for(int32 j=0; j<agrps[i].ParsedCodecs.Num(); ++j)
			{
				if (Filter->CanDecodeStream(agrps[i].ParsedCodecs[j]))
				{
					int32 Priority = acSel.GetClassPriority(agrps[i].ParsedCodecs[j].GetCodecSpecifierRFC6381());
					Builder->UsableAudioVariantGroupIndices.Emplace(FInternalBuilder::FGroupPrio(i, Priority));
					break;
				}
			}
		}
		// Sort by codec priority
		Builder->UsableAudioVariantGroupIndices.StableSort([](const FInternalBuilder::FGroupPrio& a, const FInternalBuilder::FGroupPrio& b)
		{
			return a.Priority > b.Priority;
		});

		// If there is no usable audio we fail if there are no video groups.
		// Video groups may reference audio rendition groups, so the audio-only groups may not even be used.
		if (Builder->UsableAudioVariantGroupIndices.IsEmpty() && vgrps.IsEmpty())
		{
			return CreateError(FString::Printf(TEXT("None of the audio-only variants on pathway \"%s\" can be played."), *InOutPathway->PathwayID), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
		}
	}

	// Check the video variant groups for supported codec.
	if (vgrps.Num())
	{
		const FCodecSelectionPriorities& vcSel = PlayerSessionServices->GetCodecSelectionPriorities(EStreamType::Video);
		for(int32 i=0; i<vgrps.Num(); ++i)
		{
			for(int32 j=0; j<vgrps[i].ParsedCodecs.Num(); ++j)
			{
				if (Filter->CanDecodeStream(vgrps[i].ParsedCodecs[j]))
				{
					int32 Priority = vcSel.GetClassPriority(vgrps[i].ParsedCodecs[j].GetCodecSpecifierRFC6381());
					Builder->UsableVideoVariantGroupIndices.Emplace(FInternalBuilder::FGroupPrio(i, Priority));
					break;
				}
			}
		}
		// Sort by codec priority
		Builder->UsableVideoVariantGroupIndices.StableSort([](const FInternalBuilder::FGroupPrio& a, const FInternalBuilder::FGroupPrio& b)
		{
			return a.Priority > b.Priority;
		});

		// If there is no usable video we fail. We do not play back audio-only in this case.
		if (Builder->UsableVideoVariantGroupIndices.IsEmpty())
		{
			return CreateError(FString::Printf(TEXT("None of the video variants on pathway \"%s\" can be played."), *InOutPathway->PathwayID), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
		}

		// For each usable video variant group determine their associated audio group.
		auto GetAudioRenditionGroup = [mvp=InFromMultiVariantPlaylist](const FString& InName) -> const FMultiVariantPlaylistHLS::FRenditionGroup*
		{
			for(int32 i=0, iMax=mvp->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)].Num(); i<iMax; ++i)
			{
				if (mvp->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)][i] == InName)
				{
					return &mvp->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)][i];
				}
			}
			return nullptr;
		};
		TArray<FString> WarnedNoPlayableStreams;
		for(int32 nvgIdx=0; nvgIdx<Builder->UsableVideoVariantGroupIndices.Num(); ++nvgIdx)
		{
			TArray<FString> AudioGroups;
			TArray<FInternalBuilder::FVideoVariantAudio> grp;
			const FMultiVariantPlaylistHLS::FVideoVariantGroup& svg = vgrps[Builder->UsableVideoVariantGroupIndices[nvgIdx].GroupIndex];
			for(int32 i=0; i<svg.StreamInfIndices.Num(); ++i)
			{
				FInternalBuilder::FVideoVariantAudio& vag = grp.Emplace_GetRef();
				if (sInfs[svg.StreamInfIndices[i]].NumAudioCodec)
				{
					vag.GroupName = sInfs[svg.StreamInfIndices[i]].AudioGroup;
					if (vag.GroupName.Len())
					{
						// References an audio group we haven't seen yet?
						if (!AudioGroups.Contains(vag.GroupName) && AudioGroups.Num())
						{
							const FMultiVariantPlaylistHLS::FRenditionGroup* rg1 = GetAudioRenditionGroup(AudioGroups.Last());
							const FMultiVariantPlaylistHLS::FRenditionGroup* rg2 = GetAudioRenditionGroup(vag.GroupName);
							// The existence of the groups has been enforced already and they cannot be non-existing.
							check(rg1 && rg2);
							// NOTE: We only check for same number of renditions, not their individual properties.
							//       That would require a deep inspection as ordering is not ensured.
							if (rg1->Renditions.Num() != rg2->Renditions.Num())
							{
								return CreateError(FString::Printf(TEXT("Audio rendition groups \"%s\" and \"%s\" referenced by grouped variant streams are mismatching"), *AudioGroups.Last(), *vag.GroupName), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
							}
						}

						// Check that the filtered-by-codec (see above) group is not empty.
						const FInternalBuilder::FAudioRenditionGroup* arg = Builder->AudioRenditionGroups.FindByKey(vag.GroupName);
						// Cannot fail.
						check(arg);
						if (arg->UsableRenditionIndices.IsEmpty())
						{
							vag.bIsEmpty = true;
							if (!WarnedNoPlayableStreams.Contains(vag.GroupName))
							{
								WarnedNoPlayableStreams.Emplace(vag.GroupName);
								LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Rendition group \"%s\" contains no playable stream"), *vag.GroupName));
							}
						}

						AudioGroups.AddUnique(vag.GroupName);
					}
					else
					{
						vag.bIsInband = true;
					}
				}
				// If the variant doesn't give an audio codec, but there is at least one usable audio variant then we set that
				// as the audio to use with the variant.
				else if (Builder->UsableAudioVariantGroupIndices.Num())
				{
					// Use the first audio-only group. It has been sorted by codec support and priority above.
					vag.VariantGroupIndex = Builder->UsableAudioVariantGroupIndices[0].GroupIndex;
				}
				else
				{
					vag.bHasNoAudio = true;
				}
			}
			Builder->VideoGroupAssociatedAudio.Emplace(MoveTemp(grp));
		}
	}
	// Is this audio-only?
	else if (agrps.Num())
	{
		// Is there a usable variant group left after filtering for supported codecs?
		if (Builder->UsableAudioVariantGroupIndices.IsEmpty())
		{
			return CreateError(FString::Printf(TEXT("There is no playable variant stream")), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
		}
		Builder->SelectedAudioVariantGroup = Builder->UsableAudioVariantGroupIndices[0].GroupIndex;
	}
	else
	{
		// There could be only subtitles in the playlist, but that is something we really do not handle.
		return CreateError(FString::Printf(TEXT("The playlist contains no playable content")), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
	}

	// For the video variant groups, assign an internal score and apply penalties for things like empty audio rendition groups.
	// Then use the group that has the highest score.
	if (Builder->UsableVideoVariantGroupIndices.Num())
	{
		struct FVideoVariantGroupScore
		{
			int32 Score;
			int32 Index;
		};
		TArray<FVideoVariantGroupScore> GroupScores;
		check(Builder->VideoGroupAssociatedAudio.Num() == Builder->UsableVideoVariantGroupIndices.Num());
		for(int32 i=0; i<Builder->UsableVideoVariantGroupIndices.Num(); ++i)
		{
			FVideoVariantGroupScore& gs = GroupScores.Emplace_GetRef();
			gs.Score = 100;
			gs.Index = Builder->UsableVideoVariantGroupIndices[i].GroupIndex;
			const TArray<FInternalBuilder::FVideoVariantAudio>& gva = Builder->VideoGroupAssociatedAudio[i];
			for(int32 j=0; j<gva.Num(); ++j)
			{
				if (gva[j].bIsEmpty)
				{
					gs.Score -= 10;
				}
			}
		}
		GroupScores.StableSort([](const FVideoVariantGroupScore& a, const FVideoVariantGroupScore& b){return a.Score > b.Score;});
		Builder->SelectedVideoVariantGroup = GroupScores[0].Index;
	}

	FErrorDetail MetadataResult = CreateTrackMetadata(PlayerSessionServices, InOutPathway, InFromMultiVariantPlaylist, Builder.Get());
	if (!MetadataResult.IsOK())
	{
		TimelineMediaAsset->MultiVariantPlaylist.Reset();
		return MetadataResult;
	}
	return FErrorDetail();
}

FErrorDetail FActiveHLSPlaylist::Create(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests, IPlayerSessionServices* InPlayerSessionServices, TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InFromMultiVariantPlaylist)
{
	if (!InPlayerSessionServices || !InPlayerSessionServices->GetStreamFilter() || !InFromMultiVariantPlaylist.IsValid())
	{
		return CreateError(FString::Printf(TEXT("Internal error")), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
	}

	TimelineMediaAsset->PlayerSessionServices = PlayerSessionServices = InPlayerSessionServices;
	IPlayerStreamFilter* Filter = PlayerSessionServices->GetStreamFilter();

	FErrorDetail LastErr;

	// Handle each possible pathway on its own since they *could* all have different variants and renditions.
	for(int32 pwIdx=0; pwIdx<InFromMultiVariantPlaylist->PathwayStreamInfs.Num(); ++pwIdx)
	{
		LastErr = PreparePathway(InFromMultiVariantPlaylist->PathwayStreamInfs[pwIdx], InFromMultiVariantPlaylist);
		if (!LastErr.IsOK())
		{
			return LastErr;
		}
	}

	TimelineMediaAsset->MultiVariantPlaylist = InFromMultiVariantPlaylist;

	// Determine the pathway to be used.
	FString CurrentPathway = TimelineMediaAsset->CurrentPathwayId;
	FString NewPathwayId;
	LastErr = DeterminePathwayToUse(PlayerSessionServices, NewPathwayId, CurrentPathway, InFromMultiVariantPlaylist);
	if (!LastErr.IsOK())
	{
		TimelineMediaAsset->MultiVariantPlaylist.Reset();
		return LastErr;
	}
	PlayerSessionServices->GetContentSteeringHandler()->SetCurrentlyActivePathway(NewPathwayId);

	// Find the pathway and set it as the current one.
	TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> NewPathway;
	for(auto& pwIt : InFromMultiVariantPlaylist->PathwayStreamInfs)
	{
		if (pwIt->PathwayID.Equals(NewPathwayId))
		{
			NewPathway = pwIt;
			break;
		}
	}
	TimelineMediaAsset->CurrentPathwayId = MoveTemp(NewPathwayId);
	TimelineMediaAsset->SetCurrentPathway(NewPathway);

	// Set up the variant and rendition playlist load requests for the streams we will be starting with.
	LastErr = GetInitialVariantPlaylistLoadRequests(OutPlaylistLoadRequests, PlayerSessionServices);
	if (!LastErr.IsOK())
	{
		TimelineMediaAsset->MultiVariantPlaylist.Reset();
	}
	return LastErr;
}

FErrorDetail FActiveHLSPlaylist::DeterminePathwayToUse(IPlayerSessionServices* InPlayerSessionServices, FString& OutPathway, const FString& InCurrentPathway, const TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe>& InFromMultiVariantPlaylist)
{
	if (!InFromMultiVariantPlaylist.IsValid())
	{
		OutPathway = FString(TEXT("."));
		return FErrorDetail();
	}
	TArray<FContentSteeringHandler::FCandidateURL> Candidates;
	// If we are on a defined pathway add it to the candidate list first so it gets selected
	// again in case there are no other matches.
	if (InCurrentPathway.Len())
	{
		FContentSteeringHandler::FCandidateURL& pw(Candidates.Emplace_GetRef());
		pw.MediaURL.CDN = InCurrentPathway;
	}
	for(auto& pwIt : InFromMultiVariantPlaylist->PathwayStreamInfs)
	{
		if (InCurrentPathway.IsEmpty() || (InCurrentPathway.Len() && !InCurrentPathway.Equals(pwIt->PathwayID)))
		{
			FContentSteeringHandler::FCandidateURL& pw(Candidates.Emplace_GetRef());
			pw.MediaURL.CDN = pwIt->PathwayID;
		}
	}
	FString SteeringMsg;
	FContentSteeringHandler::FSelectedCandidateURL Selected = InPlayerSessionServices->GetContentSteeringHandler()->SelectBestCandidateFrom(SteeringMsg, FContentSteeringHandler::ESelectFor::Playlist, Candidates);
	if (!Selected.MediaURL.CDN.IsEmpty())
	{
		OutPathway = Selected.MediaURL.CDN;
		return FErrorDetail();
	}
	return CreateError(FString::Printf(TEXT("No PATHWAY is currently viable")), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
}

void FActiveHLSPlaylist::CheckForPathwaySwitch()
{
	// This gets called when the steering manifest was updated.
	// Let's see if the update is requiring us to switch to a different pathway than we are on now.
	TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> Pwy = TimelineMediaAsset.IsValid() ? TimelineMediaAsset->GetCurrentPathway() : nullptr;
	if (Pwy.IsValid() && TimelineMediaAsset->MultiVariantPlaylist.IsValid())
	{
		FString CurrentPathway = Pwy->PathwayID;
		FString NewPathwayId;
		FErrorDetail LastErr = DeterminePathwayToUse(PlayerSessionServices, NewPathwayId, CurrentPathway, TimelineMediaAsset->MultiVariantPlaylist);
		if (LastErr.IsOK() && !NewPathwayId.Equals(CurrentPathway))
		{
			// Try to locate the new pathway
			for(int32 i=0; i<TimelineMediaAsset->MultiVariantPlaylist->PathwayStreamInfs.Num(); ++i)
			{
				if (NewPathwayId.Equals(TimelineMediaAsset->MultiVariantPlaylist->PathwayStreamInfs[i]->PathwayID))
				{
					TimelineMediaAsset->SetCurrentPathway(TimelineMediaAsset->MultiVariantPlaylist->PathwayStreamInfs[i]);
					PlayerSessionServices->GetContentSteeringHandler()->SetCurrentlyActivePathway(NewPathwayId);
					if (auto stsel = PlayerSessionServices->GetStreamSelector())
					{
						stsel->PathwayChanged(NewPathwayId);
					}
					break;
				}
			}
		}
	}
}


FErrorDetail FActiveHLSPlaylist::CreateTrackMetadata(IPlayerSessionServices* InPlayerSessionServices, TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>& InPathway, TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InFromMultiVariantPlaylist, const FInternalBuilder* InBuilder)
{
	auto GetRenditionGroup = [mvp=InFromMultiVariantPlaylist](FMultiVariantPlaylistHLS::ERenditionGroupType InType, const FString& InName) -> const FMultiVariantPlaylistHLS::FRenditionGroup*
	{
		for(int32 i=0, iMax=mvp->RenditionGroupsOfType[static_cast<int32>(InType)].Num(); i<iMax; ++i)
		{
			if (mvp->RenditionGroupsOfType[static_cast<int32>(InType)][i] == InName)
			{
				return &mvp->RenditionGroupsOfType[static_cast<int32>(InType)][i];
			}
		}
		return nullptr;
	};

	IPlayerStreamFilter* Filter = InPlayerSessionServices->GetStreamFilter();
	TArray<FString> VideoGroups;
	TArray<FString> AudioGroups;
	TArray<FString> SubtitleGroups;
	TArray<FMultiVariantPlaylistHLS::FStreamInf> GroupStreamInfs;

	// Video (with or without audio) or audio-only?
	if (InBuilder->SelectedVideoVariantGroup >= 0 || InBuilder->SelectedAudioVariantGroup >= 0)
	{
		for(auto& grSinf : InBuilder->SelectedVideoVariantGroup >= 0
				? InPathway->VideoVariantGroups[InBuilder->SelectedVideoVariantGroup].StreamInfIndices
				: InPathway->AudioOnlyVariantGroups[InBuilder->SelectedAudioVariantGroup].StreamInfIndices)
		{
			GroupStreamInfs.Emplace(InPathway->StreamInfs[grSinf]);
		}
		// Get the referenced group names from the streaminfs
		for(int32 sIdx=0; sIdx<GroupStreamInfs.Num(); ++sIdx)
		{
			const FMultiVariantPlaylistHLS::FStreamInf& si = GroupStreamInfs[sIdx];

			FString GroupName;
			GroupName = si.VideoGroup;
			if (GroupName.Len())
			{
				// References a video group we haven't seen yet?
				if (!VideoGroups.Contains(GroupName) && VideoGroups.Num())
				{
					const FMultiVariantPlaylistHLS::FRenditionGroup* rg1 = GetRenditionGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Video, VideoGroups.Last());
					const FMultiVariantPlaylistHLS::FRenditionGroup* rg2 = GetRenditionGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Video, GroupName);
					check(rg1 && rg2);
					if (rg1->Renditions.Num() != rg2->Renditions.Num())
					{
						return CreateError(FString::Printf(TEXT("Video rendition groups \"%s\" and \"%s\" referenced by grouped variant streams are mismatching"), *VideoGroups.Last(), *GroupName), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
					}
				}
				VideoGroups.AddUnique(GroupName);
			}
			// Audio groups? (they were already checked to be matching)
			GroupName = si.AudioGroup;
			if (GroupName.Len())
			{
				AudioGroups.AddUnique(GroupName);
			}
			// Subtitle groups?
			GroupName = si.SubtitleGroup;
			if (GroupName.Len())
			{
				// References a video group we haven't seen yet?
				if (!SubtitleGroups.Contains(GroupName) && SubtitleGroups.Num())
				{
					const FMultiVariantPlaylistHLS::FRenditionGroup* rg1 = GetRenditionGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Subtitles, SubtitleGroups.Last());
					const FMultiVariantPlaylistHLS::FRenditionGroup* rg2 = GetRenditionGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Subtitles, GroupName);
					check(rg1 && rg2);
					if (rg1->Renditions.Num() != rg2->Renditions.Num())
					{
						return CreateError(FString::Printf(TEXT("Subtitle rendition groups \"%s\" and \"%s\" referenced by grouped variant streams are mismatching"), *SubtitleGroups.Last(), *GroupName), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
					}
				}
				SubtitleGroups.AddUnique(GroupName);
			}
		}

		auto AssignTracks = [&](EStreamType InType, FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm) -> void
		{
			for(int32 ns=0; ns<GroupStreamInfs.Num(); ++ns)
			{
				FStreamMetadata& sm = tm.Meta.StreamDetails.Emplace_GetRef();
				sm.ID = GroupStreamInfs[ns].ID;
				sm.Bandwidth = GroupStreamInfs[ns].Bandwidth;
				sm.QualityIndex = GroupStreamInfs[ns].QualityIndex;
				for(int32 nc=0; nc<GroupStreamInfs[ns].ParsedCodecs.Num(); ++nc)
				{
					if (GroupStreamInfs[ns].ParsedCodecs[nc].IsCodec(InType))
					{
						sm.CodecInformation = GroupStreamInfs[ns].ParsedCodecs[nc];
						sm.CodecInformation.SetBitrate(sm.Bandwidth);
						break;
					}
				}
				if (sm.Bandwidth > tm.Meta.HighestBandwidth)
				{
					tm.Meta.HighestBandwidth = sm.Bandwidth;
					tm.Meta.HighestBandwidthCodec = sm.CodecInformation;
				}
			}
		};
		if (VideoGroups.Num())
		{
			// Each group contains the alternatives to a variant of a certain bandwidth and the renditions therein are the different "angles".
			// We create the video tracks from the "angles" and assign the renditions to the track.
			// This requires all groups to have matching renditions of course.
			// Get the first group (could be any, it does not matter) and create the "angle" tracks from it.
			const FMultiVariantPlaylistHLS::FRenditionGroup* rg = GetRenditionGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Video, VideoGroups[0]);
			for(int32 i=0; i<rg->Renditions.Num(); ++i)
			{
				FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm = InPathway->VideoTracks.Emplace_GetRef();
				tm.Meta.ID = FString::Printf(TEXT("vid:%s"), *rg->Renditions[i].Name);
				tm.Meta.Label = rg->Renditions[i].Name;	// Set the label to be the rendition's name. This gets used further down in comparisons!
				tm.Meta.LanguageTagRFC5646 = rg->Renditions[i].LanguageRFC5646;
				tm.Meta.Kind = i==0 ? TEXT("main") : TEXT("alternative");
			}
			for(int32 vtIdx=0; vtIdx<InPathway->VideoTracks.Num(); ++vtIdx)
			{
				// Get the variants that reference this group.
				for(int32 grpIdx=0; grpIdx<VideoGroups.Num(); ++grpIdx)
				{
					for(int32 vi=0; vi<GroupStreamInfs.Num(); ++vi)
					{
						if (!GroupStreamInfs[vi].VideoGroup.Equals(VideoGroups[grpIdx]))
						{
							continue;
						}
						FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm = InPathway->VideoTracks[vtIdx];
						rg = GetRenditionGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Video, VideoGroups[grpIdx]);
						bool bFoundRendition = false;
						for(int32 k=0; k<rg->Renditions.Num(); ++k)
						{
							if (rg->Renditions[k].Name.Equals(tm.Meta.Label))
							{
								// Test this rendition for decodability here so we do not add the `VideoVariantBaseIDs`
								// that we can't remove later.
								if (Filter->CanDecodeStream(rg->Renditions[k].ParsedCodecFromStreamInf))
								{
									tm.VideoVariantBaseIDs.Emplace(GroupStreamInfs[vi].ID);
									FStreamMetadata& sm = tm.Meta.StreamDetails.Emplace_GetRef();
									sm.Bandwidth = GroupStreamInfs[vi].Bandwidth;
									sm.QualityIndex = GroupStreamInfs[vi].QualityIndex;
									sm.ID.Empty();	// do not set an ID here to indicate this is a rendition
									sm.CodecInformation = rg->Renditions[k].ParsedCodecFromStreamInf;
									sm.CodecInformation.SetBitrate(sm.Bandwidth);
									if (sm.Bandwidth > tm.Meta.HighestBandwidth)
									{
										tm.Meta.HighestBandwidth = sm.Bandwidth;
										tm.Meta.HighestBandwidthCodec = sm.CodecInformation;
									}
								}
								bFoundRendition = true;
								break;
							}
						}
						if (!bFoundRendition)
						{
							return CreateError(FString::Printf(TEXT("Alternative rendition \"%s\" is not present in all rendition groups"), *tm.Meta.Label), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
						}
					}
				}
			}
		}
		else if (InBuilder->SelectedVideoVariantGroup >= 0)
		{
			FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm = InPathway->VideoTracks.Emplace_GetRef();
			tm.Meta.ID = TEXT("vid:");
			tm.Meta.Kind = TEXT("main");
			tm.bIsVariant = true;
			AssignTracks(EStreamType::Video, tm);
		}


		// Are there audio groups?
		if (AudioGroups.Num())
		{
			const FMultiVariantPlaylistHLS::FRenditionGroup* rg = GetRenditionGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio, AudioGroups[0]);
			for(int32 i=0; i<rg->Renditions.Num(); ++i)
			{
				FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm = InPathway->AudioTracks.Emplace_GetRef();
				for(auto& vvIt : GroupStreamInfs)
				{
					tm.VideoVariantBaseIDs.Emplace(vvIt.ID);
				}
				tm.Rendition = rg->Renditions[i];
				tm.Meta.ID = FString::Printf(TEXT("aud:%s:%s"), *AudioGroups[0], *rg->Renditions[i].Name);
				tm.Meta.Label = rg->Renditions[i].Name;
				tm.Meta.LanguageTagRFC5646 = rg->Renditions[i].LanguageRFC5646;
				tm.Meta.Kind = i==0 ? TEXT("main") : TEXT("translation");
					/*
						An AUDIO Rendition MAY include the following characteristic:
						"public.accessibility.describes-video".
					*/

				if (InBuilder->SelectedAudioVariantGroup >= 0)
				{
					AssignTracks(EStreamType::Audio, tm);
				}
				else
				{
					tm.Meta.HighestBandwidth = HLS::kAssumedAudioBandwidth;
					tm.Meta.HighestBandwidthCodec = rg->Renditions[i].ParsedCodecFromStreamInf;
					FStreamMetadata& sm = tm.Meta.StreamDetails.Emplace_GetRef();
					sm.ID.Empty();	// do not set an ID here to indicate this is a rendition
					sm.Bandwidth = tm.Meta.HighestBandwidth;
					sm.QualityIndex = 0;
					sm.CodecInformation = tm.Meta.HighestBandwidthCodec;
					sm.CodecInformation.SetBitrate(sm.Bandwidth);
				}
			}
		}
		else
		{
			if (InBuilder->SelectedAudioVariantGroup >= 0)
			{
				FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm = InPathway->AudioTracks.Emplace_GetRef();
				tm.Meta.ID = TEXT("aud:");
				tm.Meta.Kind = TEXT("main");
				//BCP47::ParseRFC5646Tag(tm.Meta.LanguageTagRFC5646, FString(TEXT("und")));
				tm.bIsVariant = true;
				AssignTracks(EStreamType::Audio, tm);
			}
			else
			{
				// This is the case where there are no audio groups but at least one audio-only variant.
				check(InBuilder->VideoGroupAssociatedAudio.Num());
				check(InBuilder->SelectedVideoVariantGroup < InBuilder->VideoGroupAssociatedAudio.Num());

				// There can be variant streams that include an audio codec and thus have inband-audio
				// and others that have no audio codec and thus need to use the audio-only variant.
				// Figure out which is which.
				TArray<FInternalBuilder::FVideoVariantAudio> va;
				for(int32 i=0; i<GroupStreamInfs.Num(); ++i)
				{
					va.Emplace(InBuilder->VideoGroupAssociatedAudio[InBuilder->SelectedVideoVariantGroup][GroupStreamInfs[i].IndexOfSelfInArray]);
					if (va.Last() != va[0])
					{
						return CreateError(FString::Printf(TEXT("Variant streams have inconsistent audio")), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
					}
				}
				if (va.Num() && !va[0].bHasNoAudio && !va[0].bIsEmpty)
				{
					FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm = InPathway->AudioTracks.Emplace_GetRef();
					tm.bIsVariant = true;
					tm.Meta.ID = TEXT("aud:");
					tm.Meta.Kind = TEXT("main");
					//BCP47::ParseRFC5646Tag(tm.Meta.LanguageTagRFC5646, FString(TEXT("und")));
					if (va[0].VariantGroupIndex < 0)
					{
						// This is the case where a variant stream includes an audio codec.
						for(auto& siIt : GroupStreamInfs)
						{
							if (siIt.Bandwidth > tm.Meta.HighestBandwidth)
							{
								for(auto& ciIt : siIt.ParsedCodecs)
								{
									if (ciIt.IsAudioCodec())
									{
										tm.Meta.HighestBandwidth = siIt.Bandwidth;
										tm.Meta.HighestBandwidthCodec = ciIt;
										break;
									}
								}
							}
						}
						// Overwrite the highest bandwidth with a value one may see for audio.
						tm.Meta.HighestBandwidth = HLS::kAssumedAudioBandwidth;
						FStreamMetadata& sm = tm.Meta.StreamDetails.Emplace_GetRef();
						sm.ID.Empty();	// do not set an ID for inband audio
						sm.Bandwidth = tm.Meta.HighestBandwidth;
						sm.QualityIndex = 0;
						sm.CodecInformation = tm.Meta.HighestBandwidthCodec;
						sm.CodecInformation.SetBitrate(sm.Bandwidth);
					}
					else
					{
						// This is the case where earlier variant streams did not include an audio codec.
						check(va[0].VariantGroupIndex < InPathway->AudioOnlyVariantGroups.Num());
						check(InPathway->AudioOnlyVariantGroups[va[0].VariantGroupIndex].ParsedCodecs.Num());
						tm.AudioVariantGroupIndex = va[0].VariantGroupIndex;
						for(int nv=0; nv<InPathway->AudioOnlyVariantGroups[tm.AudioVariantGroupIndex].StreamInfIndices.Num(); ++nv)
						{
							FStreamMetadata& sm = tm.Meta.StreamDetails.Emplace_GetRef();
							sm.ID = InPathway->StreamInfs[InPathway->AudioOnlyVariantGroups[tm.AudioVariantGroupIndex].StreamInfIndices[nv]].ID;
							sm.Bandwidth = InPathway->StreamInfs[InPathway->AudioOnlyVariantGroups[tm.AudioVariantGroupIndex].StreamInfIndices[nv]].Bandwidth;
							sm.QualityIndex = InPathway->StreamInfs[InPathway->AudioOnlyVariantGroups[tm.AudioVariantGroupIndex].StreamInfIndices[nv]].QualityIndex;
							sm.CodecInformation = InPathway->StreamInfs[InPathway->AudioOnlyVariantGroups[tm.AudioVariantGroupIndex].StreamInfIndices[nv]].ParsedCodecs[0];
							sm.CodecInformation.SetBitrate(sm.Bandwidth);
							if (sm.Bandwidth > tm.Meta.HighestBandwidth)
							{
								tm.Meta.HighestBandwidth = sm.Bandwidth;
								tm.Meta.HighestBandwidthCodec = sm.CodecInformation;
							}
						}
					}
				}
			}
		}

		// Are there subtitle groups?
		// Note: For subtitles we require there to be groups to get information on language etc.
		if (SubtitleGroups.Num())
		{
			const FMultiVariantPlaylistHLS::FRenditionGroup* rg = GetRenditionGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Subtitles, SubtitleGroups[0]);
			for(int32 i=0; i<rg->Renditions.Num(); ++i)
			{
				FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm = InPathway->SubtitleTracks.Emplace_GetRef();
				for(auto& vvIt : GroupStreamInfs)
				{
					tm.VideoVariantBaseIDs.Emplace(vvIt.ID);
				}
				tm.Rendition = rg->Renditions[i];
				tm.Meta.ID = FString::Printf(TEXT("sub:%s:%s"), *SubtitleGroups[0], *rg->Renditions[i].Name);
				tm.Meta.Label = rg->Renditions[i].Name;
				tm.Meta.LanguageTagRFC5646 = rg->Renditions[i].LanguageRFC5646;
				tm.Meta.Kind = TEXT("subtitles");
					/*
						A SUBTITLES Rendition MAY include the following characteristics:
						  "public.accessibility.transcribes-spoken-dialog",
						  "public.accessibility.describes-music-and-sound", and
						  "public.easy-to-read" (which indicates that the subtitles have
						  been edited for ease of reading).
					*/
				tm.Meta.HighestBandwidth = HLS::kAssumedSubtitleBandwidth;
				tm.Meta.HighestBandwidthCodec = rg->Renditions[i].ParsedCodecFromStreamInf;
				FStreamMetadata& sm = tm.Meta.StreamDetails.Emplace_GetRef();
				sm.ID.Empty();	// do not set an ID here to indicate this is a rendition
				sm.Bandwidth = tm.Meta.HighestBandwidth;
				sm.QualityIndex = 0;
				sm.CodecInformation = tm.Meta.HighestBandwidthCodec;
				sm.CodecInformation.SetBitrate(sm.Bandwidth);
			}
		}
	}

	// Filter out the video streams that cannot be used on this device.
	for(int32 vgIdx=0; vgIdx<InPathway->VideoTracks.Num(); ++vgIdx)
	{
		FStreamCodecInformation HighestBandwidthCodec;
		int32 HighestBandwidth = 0;

		FMultiVariantPlaylistHLS::FInternalTrackMetadata& vg(InPathway->VideoTracks[vgIdx]);
		for(int32 stIdx=0; stIdx<vg.Meta.StreamDetails.Num(); ++stIdx)
		{
			if (Filter->CanDecodeStream(vg.Meta.StreamDetails[stIdx].CodecInformation))
			{
				if (vg.Meta.StreamDetails[stIdx].Bandwidth > HighestBandwidth)
				{
					HighestBandwidth = vg.Meta.StreamDetails[stIdx].Bandwidth;
					HighestBandwidthCodec = vg.Meta.StreamDetails[stIdx].CodecInformation;
				}
			}
			else
			{
				vg.Meta.StreamDetails.RemoveAt(stIdx);
				--stIdx;
			}
		}
		if (vg.Meta.StreamDetails.Num())
		{
			vg.Meta.HighestBandwidth = HighestBandwidth;
			vg.Meta.HighestBandwidthCodec = HighestBandwidthCodec;
		}
		else
		{
			InPathway->VideoTracks.RemoveAt(vgIdx);
			--vgIdx;
		}
	}

	// Create internal AdaptationSets that are used to interface with the player.
	auto CreateAdaptationSet = [](TArray<TSharedPtrTS<FMultiVariantPlaylistHLS::FPlaybackAssetAdaptationSet>>& OutAdaptationSets, const TArray<FMultiVariantPlaylistHLS::FInternalTrackMetadata>& InTrackMetadata) -> void
	{
		for(auto& tm : InTrackMetadata)
		{
			TSharedPtrTS<FMultiVariantPlaylistHLS::FPlaybackAssetAdaptationSet> as = MakeSharedTS<FMultiVariantPlaylistHLS::FPlaybackAssetAdaptationSet>();
			as->ID = tm.Meta.ID;
			as->ListOfCodecs = tm.Meta.HighestBandwidthCodec.GetCodecSpecifierRFC6381();
			as->LanguageTag = tm.Meta.LanguageTagRFC5646;
			int32 sIdx = 0;
			for(auto& sm : tm.Meta.StreamDetails)
			{
				TSharedPtrTS<FMultiVariantPlaylistHLS::FPlaybackAssetRepresentation> repr = MakeSharedTS<FMultiVariantPlaylistHLS::FPlaybackAssetRepresentation>();

				repr->StreamCodecInformation = sm.CodecInformation;
				repr->Bandwidth = sm.Bandwidth;
				repr->QualityIndex = sm.QualityIndex;
				repr->ID = sm.ID.Len() ? sm.ID : FString::Printf(TEXT("/%d"), sIdx);
				as->Representations.Emplace(MoveTemp(repr));
				++sIdx;
			}
			OutAdaptationSets.Emplace(MoveTemp(as));
		}
	};
	CreateAdaptationSet(InPathway->VideoAdaptationSets, InPathway->VideoTracks);
	CreateAdaptationSet(InPathway->AudioAdaptationSets, InPathway->AudioTracks);
	CreateAdaptationSet(InPathway->SubtitleAdaptationSets, InPathway->SubtitleTracks);
	return FErrorDetail();
}


void FActiveHLSPlaylist::GetAllMediaPlaylistLoadRequests(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests, EStreamType InForType)
{
	// Returns playlist requests for all variants of a given type. This is used to select alternative initial variants if the first initial
	// playlist fails to download or parse.
	if (!TimelineMediaAsset.IsValid())
	{
		return;
	}

	auto Pwy = TimelineMediaAsset->GetCurrentPathway();

	FErrorDetail Result;
	TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> PlaylistLoadRequest;
	if (InForType == EStreamType::Video && Pwy->VideoTracks.Num())
	{
		const int32 kTrack = 0;
		for(int32 i=0,iMax=Pwy->VideoTracks[kTrack].Meta.StreamDetails.Num(); i<iMax; ++i)
		{
			Result = TimelineMediaAsset->GetVariantPlaylist(PlaylistLoadRequest, PlayerSessionServices, EStreamType::Video, Pwy, kTrack, i, 0, 0);
			if (Result.IsOK() && PlaylistLoadRequest.IsValid())
			{
				OutPlaylistLoadRequests.Emplace(MoveTemp(PlaylistLoadRequest));
			}
		}
	}
	else if (InForType == EStreamType::Audio && Pwy->AudioTracks.Num())
	{
		const int32 kTrack = 0;
		for(int32 i=0,iMax=Pwy->AudioTracks[kTrack].Meta.StreamDetails.Num(); i<iMax; ++i)
		{
			// TBD: Since this is used to determine alternatives for the initial variant playlist, can we safely pass 0,0 for the main stream indices?
			Result = TimelineMediaAsset->GetVariantPlaylist(PlaylistLoadRequest, PlayerSessionServices, EStreamType::Audio, Pwy, kTrack, i, 0, 0);
			if (Result.IsOK() && PlaylistLoadRequest.IsValid())
			{
				OutPlaylistLoadRequests.Emplace(MoveTemp(PlaylistLoadRequest));
			}
		}
	}
	else if (InForType == EStreamType::Subtitle && Pwy->SubtitleTracks.Num())
	{
		const int32 kTrack = 0;
		for(int32 i=0,iMax=Pwy->SubtitleTracks[kTrack].Meta.StreamDetails.Num(); i<iMax; ++i)
		{
			// TBD: Since this is used to determine alternatives for the initial variant playlist, can we safely pass 0,0 for the main stream indices?
			Result = TimelineMediaAsset->GetVariantPlaylist(PlaylistLoadRequest, PlayerSessionServices, EStreamType::Subtitle, Pwy, kTrack, i, 0, 0);
			if (Result.IsOK() && PlaylistLoadRequest.IsValid())
			{
				OutPlaylistLoadRequests.Emplace(MoveTemp(PlaylistLoadRequest));
			}
		}
	}
}


FErrorDetail FActiveHLSPlaylist::GetInitialVariantPlaylistLoadRequests(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests, IPlayerSessionServices* InPlayerSessionServices)
{
	// Get the first variant or rendition playlists to load.
	auto Pwy = TimelineMediaAsset->GetCurrentPathway();
	if (!Pwy.IsValid() || (!Pwy->VideoAdaptationSets.Num() && !Pwy->AudioAdaptationSets.Num()))
	{
		return CreateError(FString::Printf(TEXT("Nothing usable found in multivariant playlist")), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
	}

	FErrorDetail Result;
	TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> PlaylistLoadRequest;
	TArray<FString> URLs;
	bool bIsPrimary = true;
	if (Pwy->VideoAdaptationSets.Num())
	{
		Result = TimelineMediaAsset->GetVariantPlaylist(PlaylistLoadRequest, InPlayerSessionServices, EStreamType::Video, Pwy, 0, 0, 0, 0);
		if (Result.IsOK() && PlaylistLoadRequest.IsValid())
		{
			PlaylistLoadRequest->bIsPrimaryPlaylist = bIsPrimary;
			PlaylistLoadRequest->PlaylistInfo.StreamType = EStreamType::Video;
			URLs.Emplace(PlaylistLoadRequest->ResourceRequest->GetURL());
			OutPlaylistLoadRequests.Emplace(MoveTemp(PlaylistLoadRequest));
			bIsPrimary = false;
		}
	}
	if (Result.IsOK() && Pwy->AudioAdaptationSets.Num())
	{
		Result = TimelineMediaAsset->GetVariantPlaylist(PlaylistLoadRequest, InPlayerSessionServices, EStreamType::Audio, Pwy, 0, 0, 0, 0);
		if (Result.IsOK() && PlaylistLoadRequest.IsValid() && !URLs.Contains(PlaylistLoadRequest->ResourceRequest->GetURL()))
		{
			PlaylistLoadRequest->bIsPrimaryPlaylist = bIsPrimary;
			PlaylistLoadRequest->PlaylistInfo.StreamType = EStreamType::Audio;
			URLs.Emplace(PlaylistLoadRequest->ResourceRequest->GetURL());
			OutPlaylistLoadRequests.Emplace(MoveTemp(PlaylistLoadRequest));
			bIsPrimary = false;
		}
	}
	return Result;
}

void FActiveHLSPlaylist::UpdateWithMediaPlaylist(TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> InMediaPlaylist, bool bIsPrimary, bool bIsUpdate)
{
	check(TimelineMediaAsset);
	TimelineMediaAsset->UpdateWithMediaPlaylist(InMediaPlaylist, bIsPrimary, bIsUpdate);
}

void FActiveHLSPlaylist::GetNewMediaPlaylistLoadRequests(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests)
{
	check(TimelineMediaAsset);
	if (TimelineMediaAsset.IsValid())
	{
		return TimelineMediaAsset->GetNewMediaPlaylistLoadRequests(OutPlaylistLoadRequests);
	}
}

void FActiveHLSPlaylist::GetActiveMediaPlaylists(TArray<TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>>& OutActivePlaylists, const FTimeValue& InNow)
{
	TArray<TSharedPtrTS<FPlayPeriod>> Periods;
	RequestedPeriodsLock.Lock();
	for(int32 i=0; i<RequestedPeriods.Num(); ++i)
	{
		TSharedPtrTS<FPlayPeriod> p = RequestedPeriods[i].Pin();
		if (p.IsValid())
		{
			Periods.Emplace(MoveTemp(p));
		}
		else
		{
			RequestedPeriods.RemoveAt(i);
			--i;
		}
	}
	RequestedPeriodsLock.Unlock();
	for(auto& pIt : Periods)
	{
		pIt->GetActiveMediaPlaylists(OutActivePlaylists);
	}

	// Update the active media playlist states with the timeline asset.
	if (TimelineMediaAsset.IsValid())
	{
		TimelineMediaAsset->UpdateActiveMediaPlaylists(OutActivePlaylists, InNow);
	}
}


FTimeRange FActiveHLSPlaylist::GetPlaybackRange(EPlaybackRangeType InRangeType) const
{
	check(TimelineMediaAsset);
	FTimeRange Range = TimelineMediaAsset->GetPlaybackRangeFromURL(InRangeType);
	// Clamp this into the available range
	if (Range.Start.IsValid())
	{
		FTimeRange AvailableTimeRange = GetTotalTimeRange();
		if (AvailableTimeRange.Start.IsValid() && Range.Start < AvailableTimeRange.Start)
		{
			Range.Start = AvailableTimeRange.Start;
		}
	}
	return Range;
}


FTimeRange FActiveHLSPlaylist::FTimelineMediaAsset::GetSeekableTimeRange() const
{
	FTimeRange tr = GetTimeRange();
	tr.End -= GetDesiredLiveLatency();
	if (tr.End < tr.Start)
	{
		tr.End = tr.Start;
	}
	return tr;
}


FTimeValue FActiveHLSPlaylist::FTimelineMediaAsset::GetDesiredLiveLatency() const
{
	if (ReplayEvent.bIsReplay)
	{
		return ReplayEvent.SuggestedPresentationDelay.IsValid() ? ReplayEvent.SuggestedPresentationDelay : TargetDuration * 3;
	}

	// Called for a static playlist? If so, there is no desired latency.
	if (PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || bHasEndList)
	{
		return FTimeValue::GetZero();
	}

	/*
		The HLS RFC specifies the latency to be:

		HOLD-BACK
			  The value is a decimal-floating-point number of seconds that
			  indicates the server-recommended minimum distance from the end of
			  the Playlist at which clients should begin to play or to which
			  they should seek, unless PART-HOLD-BACK applies.  Its value MUST
			  be at least three times the Target Duration.

			  This attribute is OPTIONAL.  Its absence implies a value of three
			  times the Target Duration.  It MAY appear in any Media Playlist.
	*/
	FTimeValue ll = ServerControl.HoldBack;
	if (!ll.IsValid())
	{
		ll = TargetDuration * 3;
	}
	// Safety check that we do not go too far back and risk using the first segment
	// that may fall off the timeline immediately.
	if (Duration - ll < TargetDuration * 3 / 2)
	{
		ll = TargetDuration * 2;
	}
	// One final check for the cases where target duration is really large but segment durations
	// are short and few segments are available.
	if (ll >= Duration)
	{
		ll = Duration / 2;
	}
	return ll;
}

FTimeValue FActiveHLSPlaylist::FTimelineMediaAsset::CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency)
{
	FTimeValue LiveLatency;
	if ((ReplayEvent.bIsReplay && !ReplayEvent.bIsStaticEvent) || (PlaylistType != FPlaylistParserHLS::EPlaylistType::VOD && !bHasEndList))
	{
		FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		// With PDT we have the timeline locked to current wallclock 'Now', so the latency is
		// just the difference from where the playhead is to 'Now'.
		if (bHasProgramDateTime)
		{
			LiveLatency = Now - InCurrentPlaybackPosition;
		}
		else
		{
			LiveLatency = GetDesiredLiveLatency();

			InternalMediaTimelineLock.Lock();
			FInternalMediaTimeline tl(InternalMediaTimeline);
			InternalMediaTimelineLock.Unlock();
			if (tl.InitialOffsetFromNow.IsValid())
			{
				// A problem with the case is that the FirstPTS of the segment request is unknown and hence the player
				// cannot be primed with it. Instead the first PTS will be somewhere in the [0, Duration) range and not
				// the current timeline. We need to check if that is the case to prevent an incorrect latency value
				// from being returned.
				FTimeRange tr;
				tr.End = Now + tl.InitialOffsetFromNow;
				tr.Start = tr.End - Duration;
				if (InCurrentPlaybackPosition >= tr.Start)
				{
					LiveLatency = Now + tl.InitialOffsetFromNow - InCurrentPlaybackPosition;
				}
			}
		}
	}
	return LiveLatency;
}

FTimeValue FActiveHLSPlaylist::FTimelineMediaAsset::CalculatePlaylistTimeOffset(const TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe>& InPlaylist)
{
	FTimeValue tv;
	tv.SetToZero();
	if (InitialPlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || bInitialHasEndList)
	{
		if (bHasProgramDateTime)
		{
			// Note: If we wanted to - via a player option for instance - to rebase VOD with PDT to start at zero we
			//       could do this instead:
		#if 0
			tv = FTimeValue::GetZero() - InitialFirstProgramDateTime;
		#else
			tv.SetToZero();
		#endif
		}
	}
	else
	{
		const FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		if (bHasProgramDateTime)
		{
			tv = Now - InitialFirstProgramDateTime;

			check(InPlaylist.IsValid() && InPlaylist->MediaSegments.Num());
			if (InPlaylist.IsValid() && InPlaylist->MediaSegments.Num())
			{
				tv = Now - InPlaylist->MediaSegments.Last().ProgramDateTime;
			}
		}
	}
	return tv;
}

FTimeValue FActiveHLSPlaylist::FTimelineMediaAsset::CalculateStartTime(const TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe>& InPlaylist)
{
	FTimeValue tv;
	const FStartTimeHLS& StartTime(InPlaylist->StartTime);
	if (!ReplayEvent.bIsReplay && StartTime.Offset.IsValid())
	{
		FTimeRange tr = GetTimeRange();

		// Check that the start time is given in seconds somewhere in the [-Duration, Duration] range
		if (StartTime.Offset >= FTimeValue::GetZero() && bHasProgramDateTime)
		{
			FTimeValue DurLimit = Duration * 10;
			if ((StartTime.Offset > DurLimit && DurLimit < FirstProgramDateTime+BaseTimeOffset) || (StartTime.Offset > FirstProgramDateTime+BaseTimeOffset))
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("EXT-X-START has a bad value of %.4f"), StartTime.Offset.GetAsSeconds()));
				return tv;
			}
		}
		// Asked for the start time to be precise?
		if (StartTime.bPrecise)
		{
			// We can add the offset to the start of the timeline and clamp the result into it.
			if (tr.IsValid())
			{
				tv = (StartTime.Offset >= FTimeValue::GetZero() ? tr.Start : tr.End) + StartTime.Offset;
				if (tv < tr.Start)
				{
					tv = tr.Start;
				}
				else if (tv > tr.End)
				{
					tv = tr.End;
				}
			}
		}
		else
		{
			/*
				According to RFC 8216bis-15:

					PRECISE
					...... clients SHOULD start playback at the Media
					Segment containing the TIME-OFFSET .....
					..... If the value is NO, clients SHOULD attempt to render
					every media sample in that segment


				Meaning that if the time offset falls by however much or little into a segment
				the entire segment is to be displayed.
				Even if, when giving a negative value, due to rounding errors that would mean
				the time falls onto the last frame of a segment that entire segment is to play
				from its beginning!

				We need a precise time, so we have to scan through the media segments to find the one we need to start at.
			*/

			if (InPlaylist->MediaSegments.Num())
			{
				// Search forward
				if (StartTime.Offset >= FTimeValue::GetZero())
				{
					FTimeValue st = InPlaylist->MediaSegments[0].ProgramDateTime + StartTime.Offset;
					for(int32 i=0,iMax=InPlaylist->MediaSegments.Num(); i<iMax; ++i)
					{
						if (st <= InPlaylist->MediaSegments[i].ProgramDateTime + InPlaylist->MediaSegments[i].Duration)
						{
							tv = InPlaylist->MediaSegments[i].ProgramDateTime;
							break;
						}
					}
					// If not valid we are to start past the end of the timeline.
					if (!tv.IsValid())
					{
						tv = InPlaylist->MediaSegments.Last().ProgramDateTime + InPlaylist->MediaSegments.Last().Duration;
					}
				}
				else
				{
					FTimeValue st = InPlaylist->MediaSegments.Last().ProgramDateTime + InPlaylist->MediaSegments.Last().Duration + StartTime.Offset;
					for(int32 i=InPlaylist->MediaSegments.Num()-1; i>=0; --i)
					{
						if (InPlaylist->MediaSegments[i].ProgramDateTime <= st)
						{
							tv = InPlaylist->MediaSegments[i].ProgramDateTime;
							break;
						}
					}
					// If not valid we are to start before the start of the timeline.
					if (!tv.IsValid())
					{
						tv = InPlaylist->MediaSegments[0].ProgramDateTime;
					}
				}
			}
		}
	}
	tv += BaseTimeOffset;
	return tv;
}

void FActiveHLSPlaylist::FTimelineMediaAsset::UpdateWithMediaPlaylist(TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> InMediaPlaylist, bool bIsPrimary, bool bIsUpdate)
{
	if (!InMediaPlaylist.IsValid())
	{
		return;
	}
	TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> mp(InMediaPlaylist->GetPlaylist());
	if (!mp.IsValid())
	{
		return;
	}

	if (bIsPrimary)
	{
		if (!bIsUpdate)
		{
			ServerControl = mp->ServerControl;
			InitialFirstProgramDateTime = FirstProgramDateTime = mp->FirstProgramDateTime;
			TargetDuration = mp->TargetDuration;
			Duration = mp->Duration;
			InitialPlaylistType = PlaylistType = mp->PlaylistType;
			bInitialHasEndList = bHasEndList = mp->bHasEndList;
			bHasProgramDateTime = mp->bHasProgramDateTime;
			check(!bHasProgramDateTime || (bHasProgramDateTime && FirstProgramDateTime.IsValid()));
			MultiVariantURLFragmentComponents = InMediaPlaylist->MultiVariantURLFragmentComponents;
			ReplayEventParams = InMediaPlaylist->ReplayEventParams;
			// Establish the time offset between the current time and the playlist (zero for VOD)
			BaseTimeOffset = CalculatePlaylistTimeOffset(mp);
			TransformIntoReplayEvent();
			FTimeValue StartAt = CalculateStartTime(mp);
			// Get the start range from the URL fragment parameters. This overrules any EXT-X-START value.
			DefaultStartAndEndTime = GetPlaybackRangeFromURL(IManifest::EPlaybackRangeType::TemporaryPlaystartRange);
			// If the start is not given by the URL however the EXT-X-START time will be used, if it exists.
			if (!DefaultStartAndEndTime.IsValid() && StartAt.IsValid())
			{
				DefaultStartAndEndTime.Start = StartAt;
			}
		}
		else
		{
			FirstProgramDateTime = mp->FirstProgramDateTime;
			Duration = mp->Duration;
			PlaylistType = mp->PlaylistType;
			bHasEndList = mp->bHasEndList;

			if ((bHasEndList && !bInitialHasEndList) || (PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD && InitialPlaylistType != FPlaylistParserHLS::EPlaylistType::VOD))
			{
				LogMessage(PlayerSessionServices, IInfoLog::ELevel::Info, FString::Printf(TEXT("Playlist has transitioned to static")));
				TimePlaylistTransitionedToStatic = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
			}
		}
	}

	// Remove the playlist we may already have for this.
	MediaPlaylistsLock.Lock();
	MediaPlaylists.RemoveAll([Url=InMediaPlaylist->URL](const TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>& In){return In->URL.Equals(Url);});
	MediaPlaylists.Emplace(MoveTemp(InMediaPlaylist));
	MediaPlaylistsLock.Unlock();
}

FTimeRange FActiveHLSPlaylist::FTimelineMediaAsset::GetPlaybackRangeFromURL(EPlaybackRangeType InRangeType) const
{
	FTimeRange FromTo;

	// We are interested in the 't' and 'r' fragment values here.
	FString Time;
	for(int32 i=0,iMax=MultiVariantURLFragmentComponents.Num(); i<iMax; ++i)
	{
		if (InRangeType == IManifest::EPlaybackRangeType::TemporaryPlaystartRange)
		{
			if (MultiVariantURLFragmentComponents[i].Name.Equals(TEXT("t")))
			{
				Time = MultiVariantURLFragmentComponents[i].Value;
			}
		}
		else if (InRangeType == IManifest::EPlaybackRangeType::LockedPlaybackRange)
		{
			if (MultiVariantURLFragmentComponents[i].Name.Equals(TEXT("r")))
			{
				Time = MultiVariantURLFragmentComponents[i].Value;
			}
		}
	}
	if (Time.IsEmpty())
	{
		return FromTo;
	}

	if (InRangeType == IManifest::EPlaybackRangeType::TemporaryPlaystartRange && Time.Equals(TEXT("posix:now")))
	{
		FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		// A static event will not use an updated wallclock NOW, so if 'now' is used we do an init
		// with the current time.
		if (ReplayEvent.bIsReplay)
		{
			if (ReplayEvent.bIsStaticEvent)
			{
				FromTo.Start = Now;
			}
			else
			{
				// 'now' is dynamic. The time will continue to flow between here where we set the value and
				// the moment playback will begin with buffered data.
				// We do not lock 'now' with the current time but leave it unset. This results in the start
				// time to be taken from the seekable range's end value which is updating dynamically.
				FromTo.Start.SetToInvalid();
			}
		}
	}
	else
	{
		const TCHAR* const TimeDelimiter = TEXT(",");
		TArray<FString> TimeRange;
		FTimeValue Offset;
		Time.ParseIntoArray(TimeRange, TimeDelimiter, false);
		if (TimeRange.Num() && !TimeRange[0].IsEmpty())
		{
			if (RFC2326::ParseNPTTime(Offset, TimeRange[0]))
			{
				if (!FromTo.Start.IsValid())
				{
					FromTo.Start = Offset;
				}
				else
				{
					FromTo.Start += Offset;
				}
			}
		}
		if (TimeRange.Num() > 1 && !TimeRange[1].IsEmpty())
		{
			if (RFC2326::ParseNPTTime(Offset, TimeRange[1]))
			{
				FromTo.End = Offset;
			}
		}
		FromTo.Start += BaseTimeOffset;
		FromTo.End += BaseTimeOffset;
	}
	return FromTo;
}

void FActiveHLSPlaylist::FTimelineMediaAsset::GetNewMediaPlaylistLoadRequests(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests)
{
	MediaPlaylistsLock.Lock();
	OutPlaylistLoadRequests.Append(NewMediaPlaylistLoadRequests);
	NewMediaPlaylistLoadRequests.Empty();
	MediaPlaylistsLock.Unlock();
}

void FActiveHLSPlaylist::FTimelineMediaAsset::AddNewMediaPlaylistLoadRequests(const TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& InNewPlaylistLoadRequests)
{
	MediaPlaylistsLock.Lock();
	for(auto& It : InNewPlaylistLoadRequests)
	{
		check(It->ResourceRequest.IsValid());
		TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> NewPendingList = MakeShared<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>();
		NewPendingList->PlaylistState = FMediaPlaylistAndStateHLS::EPlaylistState::Requested;
		NewPendingList->URL = It->ResourceRequest->GetURL();
		MediaPlaylists.Emplace(MoveTemp(NewPendingList));
		NewMediaPlaylistLoadRequests.Emplace(It);
	}
	MediaPlaylistsLock.Unlock();
}

void FActiveHLSPlaylist::FTimelineMediaAsset::UpdateActiveMediaPlaylists(const TArray<TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>>& InActiveMediaPlaylist, const FTimeValue& InNow)
{
	// Check all the playlists against the list of currently active ones.
	// The ones that are not active and require periodic reloading must be invalidated if
	// they have expired so they will be refetched when accessed again. Otherwise they
	// would provide stale data that is of no use.
	MediaPlaylistsLock.Lock();
	for(int32 i=0; i<MediaPlaylists.Num(); ++i)
	{
		if (!InActiveMediaPlaylist.Contains(MediaPlaylists[i]) && MediaPlaylists[i]->TimeAtWhichToReload.IsValid() && InNow > MediaPlaylists[i]->TimeAtWhichToReload)
		{
			MediaPlaylists[i]->ClearPlaylist();
		}
	}
	MediaPlaylistsLock.Unlock();
}




TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> FActiveHLSPlaylist::FTimelineMediaAsset::GetExistingMediaPlaylistFromLoadRequest(TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InPlaylistLoadRequest)
{
	TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> Playlist;
	TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>* PlaylistPtr = nullptr;
	MediaPlaylistsLock.Lock();
	PlaylistPtr = MediaPlaylists.FindByPredicate([Url=InPlaylistLoadRequest->ResourceRequest->GetURL()](const TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>& In){return In->URL.Equals(Url);});
	if (PlaylistPtr)
	{
		Playlist = *PlaylistPtr;
	}
	MediaPlaylistsLock.Unlock();
	return Playlist;
}


FErrorDetail FActiveHLSPlaylist::FTimelineMediaAsset::GetVariantPlaylist(TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>& OutPlaylistLoadRequest, IPlayerSessionServices* InPlayerSessionServices, EStreamType InStreamType, const TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>& InPathway, int32 InTrackIndex, int32 InStreamIndex, int32 InMainTrackIndex, int32 InMainStreamIndex)
{
	auto GetTimeoutValue = [pss=InPlayerSessionServices](const FName& InOptionName, int32 InDefaultValueMillisec) -> FTimeValue
	{
		return pss->GetOptionValue(InOptionName).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(InDefaultValueMillisec));
	};

	const TArray<FMultiVariantPlaylistHLS::FInternalTrackMetadata>* Tracks = nullptr;
	const TArray<TSharedPtrTS<FMultiVariantPlaylistHLS::FPlaybackAssetAdaptationSet>>* AdaptationSets = nullptr;
	switch(InStreamType)
	{
		case EStreamType::Video:
		{
			Tracks = &InPathway->VideoTracks;
			AdaptationSets = &InPathway->VideoAdaptationSets;
			break;
		}
		case EStreamType::Audio:
		{
			Tracks = &InPathway->AudioTracks;
			AdaptationSets = &InPathway->AudioAdaptationSets;
			break;
		}
		case EStreamType::Subtitle:
		{
			Tracks = &InPathway->SubtitleTracks;
			AdaptationSets = &InPathway->SubtitleAdaptationSets;
			break;
		}
		default:
		{
			return FErrorDetail();
		}
	}
	if (InTrackIndex >= Tracks->Num())
	{
		return CreateError(FString::Printf(TEXT("Invalid %s track index %d"), GetStreamTypeName(InStreamType), InTrackIndex), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
	}
	const FMultiVariantPlaylistHLS::FInternalTrackMetadata& tm = (*Tracks)[InTrackIndex];
	if (InStreamIndex >= tm.Meta.StreamDetails.Num())
	{
		return CreateError(FString::Printf(TEXT("Invalid %s stream index %d for track index %d"), GetStreamTypeName(InStreamType), InStreamIndex, InTrackIndex), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
	}

	TSharedPtrTS<FMultiVariantPlaylistHLS::FPlaybackAssetAdaptationSet> AdaptationSet = (*AdaptationSets)[InTrackIndex];
	TSharedPtrTS<IPlaybackAssetRepresentation> Representation;
	const FStreamMetadata& sm = tm.Meta.StreamDetails[InStreamIndex];

	// We need to get to the variant stream that has defined this track and stream because the pathway is defined only on the variant.
	FString VariantID;
	// Variant or rendition?
	if (tm.bIsVariant)
	{
		VariantID = sm.ID;
		// Audio or subtitle renditions may be included in the variant stream.
		if (VariantID.IsEmpty())
		{
			if (InStreamType == EStreamType::Audio || InStreamType == EStreamType::Subtitle)
			{
				return GetVariantPlaylist(OutPlaylistLoadRequest, InPlayerSessionServices, EStreamType::Video, InPathway, InMainTrackIndex, InMainStreamIndex, -1, -1);
			}
			else
			{
				return CreateError(FString::Printf(TEXT("Internal error. No ID on stream metadata")), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
			}
		}
	}
	else
	{
		// A rendition. We need to check if this is the special case of a video angle.
		if (InStreamType == EStreamType::Video && tm.VideoVariantBaseIDs.Num())
		{
			check(tm.VideoVariantBaseIDs.Num() == tm.Meta.StreamDetails.Num());
			VariantID = tm.VideoVariantBaseIDs[InStreamIndex];
		}
		else
		{
			check(tm.Rendition.IsSet());
			VariantID = sm.ID;
			if (VariantID.IsEmpty())
			{
				check(tm.VideoVariantBaseIDs.Num());
				VariantID = tm.VideoVariantBaseIDs[0];
			}
		}
	}

	// Locate the variant.
	const FMultiVariantPlaylistHLS::FStreamInf* si = nullptr;
	for(int32 vIdx=0; vIdx<InPathway->StreamInfs.Num(); ++vIdx)
	{
		if (InPathway->StreamInfs[vIdx].ID.Equals(VariantID))
		{
			si = &InPathway->StreamInfs[vIdx];
			Representation = AdaptationSet->GetRepresentationByUniqueIdentifier(VariantID);
			break;
		}
	}
	if (!si)
	{
		return CreateError(FString::Printf(TEXT("Variant stream \"%s\" not found for pathway \"%s\""), *VariantID, *InPathway->PathwayID), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
	}

	// Were we looking for a variant?
	FString URL;
	if (tm.bIsVariant)
	{
		URL = si->URI;
	}
	else
	{
		// We need to find the rendition in the respective group of this variant so we are on the correct pathway.
		const TArray<FMultiVariantPlaylistHLS::FRenditionGroup>* RenditionGroups = nullptr;
		FString GroupName;
		switch(InStreamType)
		{
			case EStreamType::Video:
			{
				GroupName = si->VideoGroup;
				RenditionGroups = &MultiVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Video)];
				break;
			}
			case EStreamType::Audio:
			{
				GroupName = si->AudioGroup;
				RenditionGroups = &MultiVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)];
				break;
			}
			case EStreamType::Subtitle:
			{
				GroupName = si->SubtitleGroup;
				RenditionGroups = &MultiVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Subtitles)];
				break;
			}
			default:
			{
				return FErrorDetail();
			}
		}
		const TArray<FMultiVariantPlaylistHLS::FRendition>* Renditions = nullptr;
		for(int32 gIdx=0; gIdx<RenditionGroups->Num(); ++gIdx)
		{
			if ((*RenditionGroups)[gIdx] == GroupName)
			{
				Renditions = &(*RenditionGroups)[gIdx].Renditions;
				break;
			}
		}
		if (!Renditions)
		{
			return CreateError(FString::Printf(TEXT("Rendition group \"%s\" not found"), *GroupName), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
		}
		const FMultiVariantPlaylistHLS::FRendition* Rendition = nullptr;
		for(int32 k=0; k<Renditions->Num(); ++k)
		{
			if ((*Renditions)[k].Name.Equals(tm.Meta.Label))
			{
				Rendition = &(*Renditions)[k];
				check(AdaptationSet->GetNumberOfRepresentations() > InStreamIndex);
				Representation = AdaptationSet->GetRepresentationByIndex(InStreamIndex);
				break;
			}
		}
		if (!Rendition)
		{
			return CreateError(FString::Printf(TEXT("Rendition \"%s\" not found in group \"%s\""), *tm.Meta.Label, *GroupName), HLS::ERRCODE_PLAYLIST_SETUP_FAILED);
		}
		URL = Rendition->URI;
		// If the rendition does not have a dedicated URL it uses that of the variant.
		if (URL.IsEmpty())
		{
			URL = si->URI;
		}
	}

	if (URL.Len())
	{
		OutPlaylistLoadRequest = MakeShared<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>();
		OutPlaylistLoadRequest->LoadType = FLoadRequestHLSPlaylist::ELoadType::Variant;
		OutPlaylistLoadRequest->PlaylistInfo.StreamType = InStreamType;
		OutPlaylistLoadRequest->PlaylistInfo.AssetID = GetAssetIdentifier();
		OutPlaylistLoadRequest->PlaylistInfo.AdaptationSetID = AdaptationSet->GetUniqueIdentifier();
		check(Representation.IsValid());
		OutPlaylistLoadRequest->PlaylistInfo.RepresentationID = Representation.IsValid() ? Representation->GetUniqueIdentifier() : VariantID;
		OutPlaylistLoadRequest->PlaylistInfo.PathwayID = InPathway->PathwayID;
		OutPlaylistLoadRequest->PlaylistInfo.RepresentationBandwidth = Representation.IsValid() ? Representation->GetBitrate() : 0;
		OutPlaylistLoadRequest->ResourceRequest = MakeShared<FHTTPResourceRequest, ESPMode::ThreadSafe>();
		OutPlaylistLoadRequest->ResourceRequest->Verb(TEXT("GET")).URL(URL)
			.ConnectionTimeout(GetTimeoutValue(HLS::OptionKeyPlaylistLoadConnectTimeout, 5000))
			.NoDataTimeout(GetTimeoutValue(HLS::OptionKeyPlaylistLoadNoDataTimeout, 2000))
			.AllowStaticQuery(IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Playlist);
	}
	return FErrorDetail();
}



FActiveHLSPlaylist::FTimelineMediaAsset::ESegSearchResult FActiveHLSPlaylist::FTimelineMediaAsset::FindSegment(TSharedPtrTS<FStreamSegmentRequestCommon>& OutSegment, FTimeValue& OutTryLater, IPlayerSessionServices* InPlayerSessionServices, const TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>& InPlaylist, const FSegSearchParam& InParam)
{
	check(InPlaylist.IsValid());
	TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> mp(InPlaylist->GetPlaylist());
	check(mp.IsValid());
	check(InPlaylist->PlaylistState == FMediaPlaylistAndStateHLS::EPlaylistState::Loaded);
	check(InParam.Start.Time.IsValid());

	OutSegment = MakeSharedTS<FStreamSegmentRequestCommon>();
	OutSegment->StreamingProtocol = FStreamSegmentRequestCommon::EStreamingProtocol::HLS;
	OutSegment->HLS.Playlist = InPlaylist;
	OutSegment->QualityIndex = InParam.QualityIndex;
	OutSegment->MaxQualityIndex = InParam.MaxQualityIndex;
	OutSegment->TimestampSequenceIndex = InParam.SequenceState.GetSequenceIndex();
	OutSegment->PeriodStart.SetToZero();
	OutSegment->AST.SetToZero();
	OutSegment->Segment.ATO.SetToZero();
	OutSegment->AdditionalAdjustmentTime = BaseTimeOffset;
	OutSegment->DownloadDelayTime.SetToZero();
	OutSegment->HLS.bNoPDTMapping = bHasProgramDateTime == false && (InitialPlaylistType == FPlaylistParserHLS::EPlaylistType::Live || InitialPlaylistType == FPlaylistParserHLS::EPlaylistType::Event) && bInitialHasEndList == false;
	// Timescale values are not known in HLS. For maximum compatibility we use HNS.
	OutSegment->Segment.Timescale = 10000000U;

	const TArray<FMediaSegmentHLS>& Segments = mp->MediaSegments;
	// If there are no segments for whatever reason we assume the presentation has ended.
	if (Segments.IsEmpty())
	{
		return ESegSearchResult::Ended;
	}

	int32 SelectedSegmentIndex = -1;

	FTimeValue SearchTime(InParam.Start.Time);
	// The times passed in need to be adjusted to media internal times.
	SearchTime -= BaseTimeOffset;
	if (InParam.MediaSequenceIndex < 0 && InParam.LocalPosition < 0)
	{
		for(int32 i=0, iMax=Segments.Num(); i<iMax; ++i)
		{
			// Have we reached the time we are looking for?
			if (Segments[i].ProgramDateTime >= SearchTime)
			{
				// Do we want the segment with start time >= the search time?
				if (InParam.SearchType == IManifest::ESearchType::After)
				{
					// Yes, we're done.
					SelectedSegmentIndex = i;
					break;
				}
				// Do we want the segment with start time > the time we're looking for?
				else if (InParam.SearchType == IManifest::ESearchType::StrictlyAfter)
				{
					// Continue the loop if we hit the search time exactly.
					// The next segment, if it exists, will have a greater search time and we'll catch it then.
					if (Segments[i].ProgramDateTime == SearchTime)
					{
						continue;
					}
					SelectedSegmentIndex = i;
					break;
				}
				// Do we want the segment with start time <= the search time?
				else if (InParam.SearchType == IManifest::ESearchType::Before)
				{
					SelectedSegmentIndex = i;
					// Go back one if we did not hit the search time exactly and we're not already on the first segment.
					if (Segments[i].ProgramDateTime > SearchTime && i > 0)
					{
						--SelectedSegmentIndex;
					}
					break;
				}
				// Do we want the segment with start time < the search time?
				else if (InParam.SearchType == IManifest::ESearchType::StrictlyBefore)
				{
					// If we cannot go back one segment we can return.
					if (i == 0)
					{
						return ESegSearchResult::BeforeStart;
					}
					SelectedSegmentIndex = i-1;
					break;
				}
				// Do we want the segment whose start time is closest to the search time
				// or the segment for the exact same start time as the search time?
				else if (InParam.SearchType == IManifest::ESearchType::Closest || InParam.SearchType == IManifest::ESearchType::Same)
				{
					SelectedSegmentIndex = i;
					// If we hit the time dead on when searching for the same we are done.
					if (InParam.SearchType == IManifest::ESearchType::Same && Segments[i].ProgramDateTime == SearchTime)
					{
						break;
					}
					// If there is an earlier segment we can check which one is closer.
					if (i > 0)
					{
						FTimeValue diffHere = Segments[i].ProgramDateTime - SearchTime;
						FTimeValue diffBefore = SearchTime - Segments[i - 1].ProgramDateTime;
						// In the exceptionally rare case the difference to either segment is the same we pick the earlier one.
						if (diffBefore <= diffHere)
						{
							--SelectedSegmentIndex;
						}
					}
					break;
				}
				else
				{
					checkNoEntry();
					return ESegSearchResult::Failed;
				}
			}
		}

		// If we have not found the requested time then all segments in the list have an earlier start time.
		// We need to see if the search time falls into the duration of the last segment.
		// Whether we can use the last segment also depends on the search mode.
		if (SelectedSegmentIndex < 0 &&
		   (InParam.SearchType == IManifest::ESearchType::Closest || InParam.SearchType == IManifest::ESearchType::Before || InParam.SearchType == IManifest::ESearchType::StrictlyBefore))
		{
			int32 LastSegmentIndex = Segments.Num() - 1;
			if (SearchTime < Segments[LastSegmentIndex].ProgramDateTime + Segments[LastSegmentIndex].Duration)
			{
				SelectedSegmentIndex = LastSegmentIndex;
			}
		}
	}
	// Search by media sequence index
	else if (InParam.LocalPosition < 0)
	{
		// Find the segment with the given media sequence value
		for(int32 i=0, iMax=Segments.Num(); i<iMax; ++i)
		{
			if (Segments[i].MediaSequence == InParam.MediaSequenceIndex)
			{
				SelectedSegmentIndex = i;
				break;
			}
		}
		// If not found then it either fell off the timeline of a Live presentation or it is not available yet.
		if (SelectedSegmentIndex < 0 && Segments[0].MediaSequence > InParam.MediaSequenceIndex)
		{
			// It fell off.
			OutSegment->bIsFalloffSegment = true;
			OutSegment->HLS.LocalIndex = 0;
			OutSegment->HLS.DurationDistanceToEnd = mp->Duration;
			OutSegment->HLS.TimeWhenLoaded = InPlaylist->GetTimeWhenLoaded();
			OutSegment->Segment.Duration = Segments[0].Duration.GetAsHNS();
			OutSegment->Segment.Number = Segments[0].MediaSequence;
			OutSegment->Segment.MediaLocalLastAUTime = TNumericLimits<int64>::Max();
			OutSegment->Segment.bFrameAccuracyRequired = false;
			if (!OutSegment->HLS.bNoPDTMapping)
			{
				OutSegment->Segment.Time = Segments[0].ProgramDateTime.GetAsHNS();
				OutSegment->Segment.MediaLocalFirstAUTime = SearchTime.GetAsHNS();
				OutSegment->Segment.MediaLocalFirstPTS = SearchTime.GetAsHNS();
				OutSegment->Segment.MediaLocalLastAUTime = InParam.LastPTS.GetAsHNS();
			}
			return ESegSearchResult::Found;
		}
	}
	// Search by local position
	else
	{
		int32 LocalPosition = InParam.LocalPosition;
		if (LocalPosition >= Segments.Num())
		{
			LocalPosition = Segments.Num() - 1;
		}
		// Use the same position. Theoretically we should not be off more than one, usually because
		// the playlist update has moved the previous local position down by the addition of a new
		// segment. We will be using the previous local timestamps to reject the media data in the
		// segment if we already had it.
		SelectedSegmentIndex = LocalPosition;
	}

	// If we still have not found the requested time and this is an Event or a Live presentation
	// the segment for the time might become available with an update of the playlist.
	if (SelectedSegmentIndex < 0)
	{
		if (!mp->bHasEndList &&
			(mp->PlaylistType == FPlaylistParserHLS::EPlaylistType::Event ||
			 mp->PlaylistType == FPlaylistParserHLS::EPlaylistType::Live))
		{
			// If the playlist is no longer updating and we have used up all it has to offer
			// we change its state to having reached the end. This will put this list onto the
			// block list and be ignored from this point on.
			if (InPlaylist->LiveUpdateState == FMediaPlaylistAndStateHLS::ELiveUpdateState::NotUpdating)
			{
				InPlaylist->LiveUpdateState = FMediaPlaylistAndStateHLS::ELiveUpdateState::ReachedEnd;
			}

			// We try again quickly so we do not waste any time once the playlist has come in,
			// which is happening asynchronously. Waiting for target, segment, or even half a
			// segment duration here is detrimental.
			OutTryLater.SetFromMilliseconds(100);
			return ESegSearchResult::PastEOS;
		}
		return ESegSearchResult::Ended;
	}

	const FMediaSegmentHLS& Seg(Segments[SelectedSegmentIndex]);

	// Beyond the playback range?
	if (Seg.ProgramDateTime >= InParam.LastPTS)
	{
		return ESegSearchResult::PastEOS;
	}

	// Found the segment, fill in the remainder of the request.
	OutSegment->HLS.LocalIndex = SelectedSegmentIndex;
	OutSegment->HLS.DiscontinuitySequence = Seg.DiscontinuitySequence;
	OutSegment->HLS.bHasDiscontinuity = !!Seg.bDiscontinuity;

	OutSegment->Segment.Duration = Seg.Duration.GetAsHNS();
	OutSegment->Segment.Number = Seg.MediaSequence;
	// Time values passed in the segment have no meaning for PDT-less Live as we need to
	// rely only on the media segment internal timestamps.
	if (!OutSegment->HLS.bNoPDTMapping)
	{
		OutSegment->Segment.Time = Seg.ProgramDateTime.GetAsHNS();
		OutSegment->Segment.MediaLocalLastAUTime = InParam.LastPTS.GetAsHNS();
		OutSegment->Segment.bFrameAccuracyRequired = InParam.bFrameAccurateSearch;
		if (InParam.bFrameAccurateSearch)
		{
			OutSegment->Segment.MediaLocalFirstAUTime = SearchTime.GetAsHNS();
			OutSegment->Segment.MediaLocalFirstPTS = SearchTime.GetAsHNS();
		}
	}
	else
	{
		OutSegment->HLS.DurationDistanceToEnd = mp->Duration - Seg.ProgramDateTime;
		OutSegment->HLS.TimeWhenLoaded = InPlaylist->GetTimeWhenLoaded();
		OutSegment->Segment.MediaLocalLastAUTime = TNumericLimits<int64>::Max();
		OutSegment->Segment.bFrameAccuracyRequired = false;
	}

	if (Seg.InitSegment.IsValid())
	{
		FURL_RFC3986 UrlParserInit(mp->ParsedURL);
		UrlParserInit.ResolveWith(Seg.InitSegment->URL);
		OutSegment->Segment.InitializationURL.Url.URL = UrlParserInit.Get(true, true);
		OutSegment->Segment.InitializationURL.Range = Seg.InitSegment->ByteRange.GetForHTTP();
		OutSegment->Segment.InitializationURL.Url.CDN = InPlaylist->PlaylistInfo.PathwayID;
		if (Seg.InitSegment->Encryption.IsValid())
		{
			FDRMClientCacheHLS::FEntry DrmClientEntry;
			LastError = LicenseKeyCache.GetClient(DrmClientEntry, Seg.InitSegment->Encryption, InPlayerSessionServices, mp->ParsedURL);
			if (LastError.IsSet())
			{
				return ESegSearchResult::UnsupportedDRM;
			}
			if (DrmClientEntry.DrmIV.IsEmpty())
			{
				LastError.SetError(UEMEDIA_ERROR_NOT_SUPPORTED).SetFacility(Facility::EFacility::HLSPlaylistBuilder).SetCode(HLS::ERRCODE_PLAYLIST_NO_SUPPORTED_DRM).SetMessage(TEXT("Encrypted init segment requires an IV"));
				return ESegSearchResult::UnsupportedDRM;
			}
			OutSegment->DrmInit.DrmClient = DrmClientEntry.DrmClient;
			OutSegment->DrmInit.DrmKID= DrmClientEntry.DrmKID;
			OutSegment->DrmInit.DrmIV = DrmClientEntry.DrmIV;
			OutSegment->DrmInit.DrmMimeType = DrmClientEntry.DrmMimeType;
		}
	}

	FURL_RFC3986 UrlParserMedia(mp->ParsedURL);
	UrlParserMedia.ResolveWith(Seg.URL);
	OutSegment->Segment.MediaURL.Url.URL = UrlParserMedia.Get(true, true);
	OutSegment->Segment.MediaURL.Range = Seg.ByteRange.GetForHTTP();
	OutSegment->Segment.MediaURL.Url.CDN = InPlaylist->PlaylistInfo.PathwayID;
	if (Seg.Encryption.IsValid())
	{
		FDRMClientCacheHLS::FEntry DrmClientEntry;
		LastError = LicenseKeyCache.GetClient(DrmClientEntry, Seg.Encryption, InPlayerSessionServices, mp->ParsedURL);
		if (LastError.IsSet())
		{
			return ESegSearchResult::UnsupportedDRM;
		}
		OutSegment->DrmMedia.DrmClient = DrmClientEntry.DrmClient;
		OutSegment->DrmMedia.DrmKID= DrmClientEntry.DrmKID;
		OutSegment->DrmMedia.DrmMimeType = DrmClientEntry.DrmMimeType;
		OutSegment->DrmMedia.DrmIV = DrmClientEntry.DrmIV;
		if (OutSegment->DrmMedia.DrmIV.IsEmpty())
		{
			ElectraCDM::IStreamDecrypterAES128::MakePaddedIVFromUInt64(OutSegment->DrmMedia.DrmIV, OutSegment->Segment.Number);
		}
	}

	OutSegment->Segment.bIsLastInPeriod = (mp->bHasEndList || mp->PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD) && SelectedSegmentIndex+1 >= Segments.Num();
	OutSegment->bIsGapSegment = OutSegment->Segment.bMayBeMissing = !!Seg.bGap;

	// Add next segments for CMCD?
	if (!OutSegment->Segment.bIsLastInPeriod)
	{
		const int32 kMaxNextSegmentsForCMCD = 3;
		while(++SelectedSegmentIndex < Segments.Num() && OutSegment->Segment.NextMediaURLS.Num() < kMaxNextSegmentsForCMCD)
		{
			const FMediaSegmentHLS& NextSeg(Segments[SelectedSegmentIndex]);
			FSegmentInformationCommon::FURL& Nxt = OutSegment->Segment.NextMediaURLS.Emplace_GetRef();
			FURL_RFC3986 NxtUrl(mp->ParsedURL);
			NxtUrl.ResolveWith(NextSeg.URL);
			Nxt.Url.URL = NxtUrl.Get(true, true);
			Nxt.Range = NextSeg.ByteRange.GetForHTTP();
		}
	}

	return ESegSearchResult::Found;
}


const FMultiVariantPlaylistHLS::FInternalTrackMetadata* FActiveHLSPlaylist::FTimelineMediaAsset::GetInternalTrackMetadata(const FString& InForID) const
{
	auto Pwy = GetCurrentPathway();
	if (Pwy.IsValid())
	{
		for(int32 i=0; i<Pwy->VideoTracks.Num(); ++i)
		{
			if (Pwy->VideoTracks[i].Meta.ID.Equals(InForID))
			{
				return &Pwy->VideoTracks[i];
			}
		}
		for(int32 i=0; i<Pwy->AudioTracks.Num(); ++i)
		{
			if (Pwy->AudioTracks[i].Meta.ID.Equals(InForID))
			{
				return &Pwy->AudioTracks[i];
			}
		}
		for(int32 i=0; i<Pwy->SubtitleTracks.Num(); ++i)
		{
			if (Pwy->SubtitleTracks[i].Meta.ID.Equals(InForID))
			{
				return &Pwy->SubtitleTracks[i];
			}
		}
	}
	return nullptr;
}

TSharedPtrTS<FMultiVariantPlaylistHLS::FPlaybackAssetAdaptationSet> FActiveHLSPlaylist::FTimelineMediaAsset::GetAdaptationSet(const FString& InForID) const
{
	auto Pwy = GetCurrentPathway();
	if (Pwy.IsValid())
	{
		for(int32 i=0; i<Pwy->VideoAdaptationSets.Num(); ++i)
		{
			if (Pwy->VideoAdaptationSets[i]->ID.Equals(InForID))
			{
				return Pwy->VideoAdaptationSets[i];
			}
		}
		for(int32 i=0; i<Pwy->AudioAdaptationSets.Num(); ++i)
		{
			if (Pwy->AudioAdaptationSets[i]->ID.Equals(InForID))
			{
				return Pwy->AudioAdaptationSets[i];
			}
		}
		for(int32 i=0; i<Pwy->SubtitleAdaptationSets.Num(); ++i)
		{
			if (Pwy->SubtitleAdaptationSets[i]->ID.Equals(InForID))
			{
				return Pwy->SubtitleAdaptationSets[i];
			}
		}
	}
	return nullptr;
}





FActiveHLSPlaylist::FTimelineMediaAsset::~FTimelineMediaAsset()
{
}

void FActiveHLSPlaylist::FTimelineMediaAsset::UpdateTimelineFromMediaSegment(TSharedPtrTS<FStreamSegmentRequestCommon> InSegment)
{
	FScopeLock lock(&InternalMediaTimelineLock);
	InternalMediaTimeline.MediaSegmentBaseTime = InSegment->TimestampVars.Local.First[StreamTypeToArrayIndex(InSegment->GetType())];
	InternalMediaTimeline.AvailableDurationUntilEnd = InSegment->HLS.DurationDistanceToEnd;
	InternalMediaTimeline.TimeWhenLoaded = InSegment->HLS.TimeWhenLoaded;
	if (InternalMediaTimeline.bLockInitial)
	{
		InternalMediaTimeline.bLockInitial = false;
		InternalMediaTimeline.InitialMediaSegmentBaseTime = InternalMediaTimeline.MediaSegmentBaseTime;
		InternalMediaTimeline.InitialAvailableDurationUntilEnd = InternalMediaTimeline.AvailableDurationUntilEnd;
		InternalMediaTimeline.InitialTimeWhenLoaded = InternalMediaTimeline.TimeWhenLoaded;
	}
}

void FActiveHLSPlaylist::FTimelineMediaAsset::ResetInternalTimeline()
{
	InternalMediaTimeline.ResyncNeeded();
}

void FActiveHLSPlaylist::FTimelineMediaAsset::TransformIntoReplayEvent()
{
	// Can only transform VOD into a replay event.
	if (!(PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || bHasEndList))
	{
		return;
	}

	FString Arg;
	bool bStaticStart = false;
	bool bDynamicStart = false;

	const TArray<FURL_RFC3986::FQueryParam>& frgs(ReplayEventParams.GetUrlFragmentParams());
	// Which type of replay event?
	for(int32 i=0,iMax=frgs.Num(); i<iMax; ++i)
	{
		bStaticStart = frgs[i].Name.Equals(CustomOptions::Custom_EpicStaticStart);
		bDynamicStart = frgs[i].Name.Equals(CustomOptions::Custom_EpicDynamicStart);
		if (bStaticStart || bDynamicStart)
		{
			Arg = frgs[i].Value;
			break;
		}
	}

	// Any indication that this is a replay event?
	if (!bStaticStart && !bDynamicStart)
	{
		return;
	}
	// Cannot use playlists that contain an #EXT-X-PROGRAM-DATE-TIME tag.
	if (bHasProgramDateTime)
	{
		UE_LOGF(LogElectraPlayer, Warning, "HLS stream uses program-date-time, cannot use as replay event.");
		return;
	}

	// Get the replay base start time.
	for(int32 i=0,iMax=frgs.Num(); i<iMax; ++i)
	{
		if (frgs[i].Name.Equals(CustomOptions::Custom_EpicUTCNow))
		{
			if (!UnixEpoch::ParseFloatString(ReplayEvent.BaseStartTime, frgs[i].Value))
			{
				UE_LOGF(LogElectraPlayer, Warning, "Could not parse the custom UTC now value, disabling replay event.");
				return;
			}
			break;
		}
		else if (frgs[i].Name.Equals(CustomOptions::Custom_EpicUTCUrl))
		{
			UE_LOGF(LogElectraPlayer, Warning, "Fetching UTC time from a custom URL for a replay event is not supported with HLS.");
		}
	}

	TArray<FString> Params;
	if (!Arg.IsEmpty())
	{
		const TCHAR* const Delimiter = TEXT(",");
		Arg.ParseIntoArray(Params, Delimiter, true);
	}
	if (Params.Num())
	{
		FTimeValue Start;
		ReplayEvent.bIsReplay = true;
		ReplayEvent.bIsStaticEvent = bStaticStart;
		if (ReplayEvent.BaseStartTime.IsValid())
		{
			PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(ReplayEvent.BaseStartTime);
		}
		// Get the event start time. This is either a Posix time in seconds since the Epoch (1/1/1970) or the special
		// word 'now' optionally followed by a value to be added or subtracted from now.
		FString Time = Params[0];
		if (Time.StartsWith(TEXT("now")))
		{
			Time.RightChopInline(3);
			Start = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		}
		if (!Time.IsEmpty())
		{
			FTimeValue Offset = FTimeValue().SetFromTimeFraction(FTimeFraction().SetFromFloatString(Time));
			if (Offset.IsValid())
			{
				if (Start.IsValid())
				{
					Start += Offset;
				}
				else
				{
					Start = Offset;
				}
			}
		}
		if (!Start.IsValid())
		{
			ReplayEvent.Reset();
			return;
		}
		ReplayEvent.BaseStartTime = Start;
		ReplayEvent.OriginalBaseTimeOffset = BaseTimeOffset;
		BaseTimeOffset = ReplayEvent.BaseStartTime;
		if (Params.Num() > 1)
		{
			UnixEpoch::ParseFloatString(ReplayEvent.SuggestedPresentationDelay, Params[1]);
		}
	}
}

void FActiveHLSPlaylist::FTimelineMediaAsset::PrepareReplayEventForLooping(int32 InNumLoopsToAdd)
{
	if (ReplayEvent.bIsReplay)
	{
		BaseTimeOffset += Duration * InNumLoopsToAdd;
	}
}


FTimeRange FActiveHLSPlaylist::FTimelineMediaAsset::GetTimeRange() const
{
	FTimeRange tr;
	if (ReplayEvent.bIsReplay)
	{
		tr.Start = BaseTimeOffset;
		tr.End = tr.Start + Duration;
	}
	// Was this a static asset from the get-go?
	else if (InitialPlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || bInitialHasEndList)
	{
		// If the #EXT-X-PROGRAM-DATE-TIME tags are in use we need to abide by them
		// even for VOD content because the fmp4 segments are bound to have an EPT
		// in that range. When a Live stream is changed over to VOD it is not
		// feasible to convert the segments.
		if (bHasProgramDateTime)
		{
			tr.Start = InitialFirstProgramDateTime + BaseTimeOffset;
			tr.End = tr.Start + Duration;
		}
		else
		{
			// Without program-date-time things start at zero.
			tr.Start.SetToZero();
			tr.End = Duration;
		}
	}
	else
	{
		FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();

		auto RefreshTimeline = [&]() -> void
		{
			tr.Start.SetToZero();
			tr.End = Duration;

			InternalMediaTimelineLock.Lock();
			if (InternalMediaTimeline.bNeedResync && !InternalMediaTimeline.bLockInitial && InternalMediaTimeline.InitialMediaSegmentBaseTime.IsValid())
			{
				InternalMediaTimeline.bNeedResync = false;
				InternalMediaTimeline.InitialOffsetFromNow = InternalMediaTimeline.InitialMediaSegmentBaseTime + InternalMediaTimeline.InitialAvailableDurationUntilEnd - InternalMediaTimeline.InitialTimeWhenLoaded;
			}
			if (InternalMediaTimeline.InitialOffsetFromNow.IsValid())
			{
				tr.End = Now + InternalMediaTimeline.InitialOffsetFromNow;
				tr.Start = tr.End - Duration;
			}
			InternalMediaTimelineLock.Unlock();
		};

		// Started out as a Live or Event presentation. Did it transition to a static presentation now?
		if (TimePlaylistTransitionedToStatic.IsValid())
		{
			// With PDT values we lock the timeline into its final range.
			if (bHasProgramDateTime)
			{
				tr.Start = FirstProgramDateTime + BaseTimeOffset;
				tr.End = tr.Start + Duration;
			}
			else
			{
				if (LastKnownTimeRange.IsValid())
				{
					tr = LastKnownTimeRange;
				}
				else
				{
					RefreshTimeline();
					LastKnownTimeRange = tr;
				}
			}
		}
		// Still an ongoing presentation.
		else
		{
			// When PDT is in use we update the timeline such that 'Now' is the end and the
			// beginning is that minus the total duration of the available segments.
			if (bHasProgramDateTime)
			{
				tr.End = Now;
				tr.Start = tr.End - Duration;
			}
			else
			{
				// A Live presentation that does not use PDT. This is tricky in that we have
				// nothing to use as a reference time regarding the media segment's internal
				// timestamps, which are unknown.
				// We need to rely entirely on the media timestamps that could literally be
				// anything. The timeline will therefore be 0-duration for the start and
				// then be shifted once we have the first media timestamps, but it will never
				// be adjusted to correspond in any way to 'Now'.
				RefreshTimeline();
			}
		}
	}
	return tr;
}

FTimeValue FActiveHLSPlaylist::FTimelineMediaAsset::GetDuration() const
{
	if (PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || bHasEndList)
	{
		return Duration;
	}
	return FTimeValue::GetPositiveInfinity();
}

FString FActiveHLSPlaylist::FTimelineMediaAsset::GetAssetIdentifier() const
{
	return FString(TEXT("$Asset.1"));
}

FString FActiveHLSPlaylist::FTimelineMediaAsset::GetUniqueIdentifier() const
{
	return FString(TEXT("1"));
}

int32 FActiveHLSPlaylist::FTimelineMediaAsset::GetNumberOfAdaptationSets(EStreamType InStreamType) const
{
	auto Pwy = GetCurrentPathway();
	if (!Pwy.IsValid())
	{
		return 0;
	}
	switch(InStreamType)
	{
		case EStreamType::Video:
		{
			return Pwy->VideoTracks.Num();
		}
		case EStreamType::Audio:
		{
			return Pwy->AudioTracks.Num();
		}
		case EStreamType::Subtitle:
		{
			return Pwy->SubtitleTracks.Num();
		}
		default:
		{
			return 0;
		}
	}
}

TSharedPtrTS<IPlaybackAssetAdaptationSet> FActiveHLSPlaylist::FTimelineMediaAsset::GetAdaptationSetByTypeAndIndex(EStreamType InStreamType, int32 InAdaptationSetIndex) const
{
	auto Pwy = GetCurrentPathway();
	if (Pwy.IsValid())
	{
		if (InStreamType == EStreamType::Video && InAdaptationSetIndex >= 0 && InAdaptationSetIndex < Pwy->VideoAdaptationSets.Num())
		{
			return Pwy->VideoAdaptationSets[InAdaptationSetIndex];
		}
		else if (InStreamType == EStreamType::Audio && InAdaptationSetIndex >= 0 && InAdaptationSetIndex < Pwy->AudioAdaptationSets.Num())
		{
			return Pwy->AudioAdaptationSets[InAdaptationSetIndex];
		}
		else if (InStreamType == EStreamType::Subtitle && InAdaptationSetIndex >= 0 && InAdaptationSetIndex < Pwy->SubtitleAdaptationSets.Num())
		{
			return Pwy->SubtitleAdaptationSets[InAdaptationSetIndex];
		}
	}
	return nullptr;
}

void FActiveHLSPlaylist::FTimelineMediaAsset::GetMetaData(TArray<FTrackMetadata>& OutMetadata, EStreamType InStreamType) const
{
	auto Pwy = GetCurrentPathway();
	if (Pwy.IsValid())
	{
		const TArray<FMultiVariantPlaylistHLS::FInternalTrackMetadata>* Tracks = nullptr;
		if (InStreamType == EStreamType::Video)
		{
			Tracks = &Pwy->VideoTracks;
		}
		else if (InStreamType == EStreamType::Audio)
		{
			Tracks = &Pwy->AudioTracks;
		}
		else if (InStreamType == EStreamType::Subtitle)
		{
			Tracks = &Pwy->SubtitleTracks;
		}
		if (Tracks)
		{
			for(auto& trkIt : *Tracks)
			{
				OutMetadata.Emplace(trkIt.Meta);
			}
		}
	}
}

void FActiveHLSPlaylist::FTimelineMediaAsset::UpdateRunningMetaData(const FString& InKindOfValue, const FVariant& InNewValue)
{
	// No-op.
}


EMediaFormatType FActiveHLSPlaylist::GetMediaFormatType() const
{
	return EMediaFormatType::HLS;
}

bool FActiveHLSPlaylist::ReloadsPlaylistPeriodically() const
{
	// This is true even for VoD since each variant has its own playlist that gets loaded just in time.
	return true;
}

bool FActiveHLSPlaylist::SupportsMultipleTracksPerKind() const
{
	return true;
}

bool FActiveHLSPlaylist::OrderOfTracksCanChangeDynamically() const
{
	return false;
}


IManifest::EType FActiveHLSPlaylist::GetPresentationType() const
{
	if (TimelineMediaAsset->ReplayEvent.bIsReplay)
	{
		return TimelineMediaAsset->ReplayEvent.bIsStaticEvent ? IManifest::EType::OnDemand : IManifest::EType::Live;
	}
	return (TimelineMediaAsset->PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || TimelineMediaAsset->bHasEndList) ? IManifest::EType::OnDemand : IManifest::EType::Live;
}

IManifest::EReplayEventType FActiveHLSPlaylist::GetReplayType() const
{
	return TimelineMediaAsset->ReplayEvent.bIsReplay ? IManifest::EReplayEventType::IsReplay : IManifest::EReplayEventType::NoReplay;
}

TSharedPtrTS<const FLowLatencyDescriptor> FActiveHLSPlaylist::GetLowLatencyDescriptor() const
{
	return nullptr;
}

FTimeValue FActiveHLSPlaylist::CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const
{
	FTimeValue LiveLatency;
	if (TimelineMediaAsset.IsValid())
	{
		LiveLatency = TimelineMediaAsset->CalculateCurrentLiveLatency(InCurrentPlaybackPosition, InEncoderLatency);
	}
	return LiveLatency;
}

FTimeValue FActiveHLSPlaylist::GetAnchorTime() const
{
	// HLS does not have the concept of an AvailabilityStartTime like DASH does.
	return FTimeValue::GetZero();
}

FTimeRange FActiveHLSPlaylist::GetTotalTimeRange() const
{
	return TimelineMediaAsset.IsValid() ? TimelineMediaAsset->GetTimeRange() : FTimeRange();
}

FTimeRange FActiveHLSPlaylist::GetSeekableTimeRange() const
{
	return TimelineMediaAsset.IsValid() ? TimelineMediaAsset->GetSeekableTimeRange() : FTimeRange();
}

FTimeValue FActiveHLSPlaylist::GetDuration() const
{
	return TimelineMediaAsset.IsValid() ? TimelineMediaAsset->GetDuration() : FTimeValue();
}

FTimeValue FActiveHLSPlaylist::GetDefaultStartTime() const
{
	return TimelineMediaAsset.IsValid() ? TimelineMediaAsset->DefaultStartAndEndTime.Start : FTimeValue();
}

void FActiveHLSPlaylist::ClearDefaultStartTime()
{
	if (TimelineMediaAsset.IsValid())
	{
		TimelineMediaAsset->DefaultStartAndEndTime.Start.SetToInvalid();
	}
}

FTimeValue FActiveHLSPlaylist::GetDefaultEndTime() const
{
	return TimelineMediaAsset.IsValid() ? TimelineMediaAsset->DefaultStartAndEndTime.End : FTimeValue();
}

void FActiveHLSPlaylist::ClearDefaultEndTime()
{
	if (TimelineMediaAsset.IsValid())
	{
		TimelineMediaAsset->DefaultStartAndEndTime.End.SetToInvalid();
	}
}

FTimeValue FActiveHLSPlaylist::GetMinBufferTime() const
{
	// HLS does not offer a minimum duration to be in the buffers at all times. For expedited startup we use 2 seconds here.
	return FTimeValue().SetFromSeconds(2.0);
}

FTimeValue FActiveHLSPlaylist::GetDesiredLiveLatency() const
{
	return TimelineMediaAsset.IsValid() ? TimelineMediaAsset->GetDesiredLiveLatency() : FTimeValue();
}

IManifest::ELiveEdgePlayMode FActiveHLSPlaylist::GetLiveEdgePlayMode() const
{
	return IManifest::ELiveEdgePlayMode::Default;
}

TRangeSet<double> FActiveHLSPlaylist::GetPossiblePlaybackRates(EPlayRateType InForType) const
{
	TRangeSet<double> Ranges;
	if (InForType == IManifest::EPlayRateType::UnthinnedRate)
	{
		Ranges.Add(TRange<double>::Inclusive(0.1, 4.0));
	}
	else
	{
		Ranges.Add(TRange<double>{1.0}); // normal (real-time) playback rate
	}
	Ranges.Add(TRange<double>{0.0}); // and pause
	return Ranges;
}

TSharedPtrTS<IProducerReferenceTimeInfo> FActiveHLSPlaylist::GetProducerReferenceTimeInfo(int64 InID) const
{
	// Not used with HLS
	return nullptr;
}

void FActiveHLSPlaylist::GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType InStreamType) const
{
	if (TimelineMediaAsset.IsValid())
	{
		auto Pwy = TimelineMediaAsset->GetCurrentPathway();
		if (Pwy.IsValid())
		{
			const TArray<FMultiVariantPlaylistHLS::FInternalTrackMetadata>* Tracks = nullptr;
			if (InStreamType == EStreamType::Video)
			{
				Tracks = &Pwy->VideoTracks;
			}
			else if (InStreamType == EStreamType::Audio)
			{
				Tracks = &Pwy->AudioTracks;
			}
			else if (InStreamType == EStreamType::Subtitle)
			{
				Tracks = &Pwy->SubtitleTracks;
			}
			if (Tracks)
			{
				for(auto& trkIt : *Tracks)
				{
					OutMetadata.Emplace(trkIt.Meta);
				}
			}
		}
	}
}

void FActiveHLSPlaylist::UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData)
{
	// Not used with HLS
}

void FActiveHLSPlaylist::UpdateDynamicRefetchCounter()
{
	// Not used with HLS
}

void FActiveHLSPlaylist::PrepareForLooping(int32 InNumLoopsToAdd)
{
	if (TimelineMediaAsset.IsValid())
	{
		TimelineMediaAsset->PrepareReplayEventForLooping(InNumLoopsToAdd);
	}
}

void FActiveHLSPlaylist::TriggerClockSync(EClockSyncType InClockSyncType)
{
	// Not used with HLS
}

void FActiveHLSPlaylist::TriggerPlaylistRefresh()
{
	// Not used with HLS
}

void FActiveHLSPlaylist::ReachedStableBuffer()
{
	if (TimelineMediaAsset.IsValid())
	{
		PlayerSessionServices->GetContentSteeringHandler()->ReachedStableBuffer();
	}
}

IStreamReader *FActiveHLSPlaylist::CreateStreamReaderHandler()
{
	// We reset the internal timeline used with Live playback at this point since we either
	//  - start playback after who-knows-when-we-loaded-the-playlist
	//  - are rebuffering and need the most up to date playlist with all new timing info
	if (TimelineMediaAsset.IsValid())
	{
		TimelineMediaAsset->ResetInternalTimeline();
	}
	return new FStreamSegmentReaderCommon;
}

IManifest::FResult FActiveHLSPlaylist::FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	if (!TimelineMediaAsset.IsValid())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotLoaded);
	}

	FTimeValue PlayRangeEnd = StartPosition.Options.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());
	FTimeValue StartTime = StartPosition.Time;
	FTimeRange MediaRange = TimelineMediaAsset->GetTimeRange();
	FTimeValue TotalEndTime(MediaRange.End);
	if (PlayRangeEnd.IsValid() && TotalEndTime.IsValid() && PlayRangeEnd < TotalEndTime)
	{
		TotalEndTime = PlayRangeEnd;
	}
	if (StartTime >= TotalEndTime)
	{
		return IManifest::FResult(IManifest::FResult::EType::PastEOS);
	}
	else if (StartTime < MediaRange.Start)
	{
		StartTime = MediaRange.Start;
	}

	TSharedPtrTS<FPlayPeriod> Period(new FPlayPeriod(PlayerSessionServices, TimelineMediaAsset));
	OutPlayPeriod = Period;

	// Add that period to the list of requested periods.
	// We need that list to determine which media playlists are being referenced.
	// Although one and the same HLS presentation, there could be multiple differently configured play periods
	// with different languages.
	RequestedPeriodsLock.Lock();
	for(int32 i=0; i<RequestedPeriods.Num(); ++i)
	{
		if (!RequestedPeriods[i].IsValid())
		{
			RequestedPeriods.RemoveAt(i);
			--i;
		}
	}
	RequestedPeriods.Add(Period);
	RequestedPeriodsLock.Unlock();

	return IManifest::FResult(IManifest::FResult::EType::Found);
}

IManifest::FResult FActiveHLSPlaylist::FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment)
{
	// Since there is only a single logical period with HLS there is no following one.
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}





FActiveHLSPlaylist::FPlayPeriod::FPlayPeriod(IPlayerSessionServices* InPlayerSessionServices, const TSharedPtr<FTimelineMediaAsset, ESPMode::ThreadSafe>& InTimelineMediaAsset)
	: PlayerSessionServices(InPlayerSessionServices), TimelineMediaAsset(InTimelineMediaAsset)
{
}

FActiveHLSPlaylist::FPlayPeriod::~FPlayPeriod()
{
}

void FActiveHLSPlaylist::FPlayPeriod::SetStreamPreferences(EStreamType InStreamType, const FStreamSelectionAttributes& InStreamAttributes)
{
	StreamSelectionAttributes[StreamTypeToArrayIndex(InStreamType)] = InStreamAttributes;
}

IManifest::IPlayPeriod::EReadyState FActiveHLSPlaylist::FPlayPeriod::GetReadyState()
{
	// While the state is preparing, call PrepareForPlay() again to check on media playlist load progress.
	if (CurrentReadyState == IManifest::IPlayPeriod::EReadyState::Preparing)
	{
		PrepareForPlay();
	}
	return CurrentReadyState;
}

void FActiveHLSPlaylist::FPlayPeriod::Load()
{
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::Loaded;
}

void FActiveHLSPlaylist::FPlayPeriod::PrepareForPlay()
{
	auto Pwy = TimelineMediaAsset->GetCurrentPathway();
	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> NewLoadReq;
	int32 NumPending = 0;
	// Select streams by preference, or the first one of the type if no preference is given.
	for(int32 nStreamTypeIdx=0; nStreamTypeIdx<3; ++nStreamTypeIdx)
	{
		FSelectedTrackStream& st = SelectedTrackStream[nStreamTypeIdx];
		const TArray<FMultiVariantPlaylistHLS::FInternalTrackMetadata>* Tracks = nullptr;
		const EStreamType StreamType = StreamArrayIndexToType(nStreamTypeIdx);

		if (nStreamTypeIdx == 0 && Pwy->VideoTracks.Num())
		{
			Tracks = &Pwy->VideoTracks;
		}
		else if (nStreamTypeIdx == 1 && Pwy->AudioTracks.Num())
		{
			Tracks = &Pwy->AudioTracks;
		}
		else if (nStreamTypeIdx == 2 && Pwy->SubtitleTracks.Num())
		{
			// Subtitle tracks are not selected by default.
			// If there is no explicit selection asked for we ignore them.
			if (StreamSelectionAttributes[nStreamTypeIdx].IsSet())
			{
				Tracks = &Pwy->SubtitleTracks;
			}
		}

		int32 SelectedTrackIndex = 0;
		if (Tracks && StreamSelectionAttributes[nStreamTypeIdx].IsSet())
		{
			const FStreamSelectionAttributes& Sel(StreamSelectionAttributes[nStreamTypeIdx]);

			// Is this a hard choice?
			if (Sel.OverrideIndex.IsSet() && Sel.OverrideIndex.GetValue() < Tracks->Num())
			{
				SelectedTrackIndex = Sel.OverrideIndex.GetValue();
			}
			else
			{
				TArray<int32> CandidateIndices;
				// Choose language?
				if (Sel.Language_RFC4647.IsSet())
				{
					TArray<BCP47::FLanguageTag> CandList;
					for(int32 i=0; i<Tracks->Num(); ++i)
					{
						CandList.Emplace((*Tracks)[i].Meta.LanguageTagRFC5646);
					}
					CandidateIndices = BCP47::FindExtendedFilteringMatch(CandList, Sel.Language_RFC4647.GetValue());
				}
				// If there are multiple language candidates narrow the list down by kind.
				if (CandidateIndices.Num() && Sel.Kind.IsSet())
				{
					TArray<int32> TempList;
					for(int32 i=0; i<CandidateIndices.Num(); ++i)
					{
						if ((*Tracks)[CandidateIndices[i]].Meta.Kind.Equals(Sel.Kind.GetValue()))
						{
							TempList.Emplace(CandidateIndices[i]);
						}
					}
					// If there are new candidates update the list. If everything is filtered out, keep the previous list.
					if (TempList.Num())
					{
						Swap(TempList, CandidateIndices);
					}
				}
				// TODO: In the future we could narrow the list down by codec if necessary.

				// Use the first candidate's index even if there are several possibilities. If there are none, use the first track.
				SelectedTrackIndex = CandidateIndices.Num() ? CandidateIndices[0] : 0;
			}
		}
		if (Tracks && SelectedTrackIndex < Tracks->Num() && (*Tracks)[SelectedTrackIndex].Meta.StreamDetails.Num())
		{
			const FMultiVariantPlaylistHLS::FInternalTrackMetadata& Track((*Tracks)[SelectedTrackIndex]);

			st.MetaID = Track.Meta.ID;
			st.TrackIndex = SelectedTrackIndex;
			st.StreamIndex = 0;
			st.bIsSelected = true;

			st.BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>();
			st.BufferSourceInfo->PeriodID = TimelineMediaAsset->GetAssetIdentifier();
			st.BufferSourceInfo->PeriodAdaptationSetID = st.MetaID;
			st.BufferSourceInfo->Kind = Track.Meta.Kind;
			st.BufferSourceInfo->LanguageTag = Track.Meta.LanguageTagRFC5646;
			st.BufferSourceInfo->Codec = Util::GetBaseCodec(Track.Meta.HighestBandwidthCodec.GetCodecSpecifierRFC6381());
			st.BufferSourceInfo->HardIndex = st.TrackIndex;

			FStreamLoadRequest LoadReq;
			if (TimelineMediaAsset->GetVariantPlaylist(LoadReq.Request, PlayerSessionServices, StreamType, Pwy, SelectedTrackIndex,0, SelectedTrackStream[0].TrackIndex,0).IsOK())
			{
				st.ActivePlaylist = TimelineMediaAsset->GetExistingMediaPlaylistFromLoadRequest(LoadReq.Request);
				if (!st.ActivePlaylist.IsValid())
				{
					NewLoadReq.Add(LoadReq.Request);
					++NumPending;
				}
				else if (!st.ActivePlaylist->ActivateIsReady())
				{
					++NumPending;
				}
			}
		}
	}
	if (NumPending)
	{
		TimelineMediaAsset->AddNewMediaPlaylistLoadRequests(NewLoadReq);
		CurrentReadyState = IManifest::IPlayPeriod::EReadyState::Preparing;
	}
	else
	{
		CurrentReadyState = IManifest::IPlayPeriod::EReadyState::IsReady;
	}
}

int64 FActiveHLSPlaylist::FPlayPeriod::GetDefaultStartingBitrate() const
{
	auto Pwy = TimelineMediaAsset->GetCurrentPathway();
	if (Pwy.IsValid())
	{
		const FMultiVariantPlaylistHLS::FInternalTrackMetadata* Track = nullptr;
		if (Pwy->VideoTracks.Num())
		{
			Track = &Pwy->VideoTracks[0];
		}
		else if (Pwy->AudioTracks.Num())
		{
			Track = &Pwy->AudioTracks[0];
		}
		if (Track)
		{
			check(Track->Meta.StreamDetails.Num());
			if (Track->Meta.StreamDetails.Num())
			{
				return Track->Meta.StreamDetails[0].Bandwidth;
			}
		}
	}
	return -1;
}

TSharedPtrTS<FBufferSourceInfo> FActiveHLSPlaylist::FPlayPeriod::GetSelectedStreamBufferSourceInfo(EStreamType InStreamType)
{
	if (InStreamType == EStreamType::Video && SelectedTrackStream[0].bIsSelected)
	{
		return SelectedTrackStream[0].BufferSourceInfo;
	}
	else if (InStreamType == EStreamType::Audio && SelectedTrackStream[1].bIsSelected)
	{
		return SelectedTrackStream[1].BufferSourceInfo;
	}
	else if (InStreamType == EStreamType::Subtitle && SelectedTrackStream[2].bIsSelected)
	{
		return SelectedTrackStream[2].BufferSourceInfo;
	}
	return nullptr;
}

FString FActiveHLSPlaylist::FPlayPeriod::GetSelectedAdaptationSetID(EStreamType InStreamType)
{
	const FSelectedTrackStream& ts = SelectedTrackStream[StreamTypeToArrayIndex(InStreamType)];
	return ts.bIsSelected ? ts.MetaID : FString();
}

IManifest::IPlayPeriod::ETrackChangeResult FActiveHLSPlaylist::FPlayPeriod::ChangeTrackStreamPreference(EStreamType InStreamType, const FStreamSelectionAttributes& InStreamAttributes)
{
	// Video cannot be switched seamlessly as this might also contain audio and subtitles.
	if (InStreamType == EStreamType::Video)
	{
		return IManifest::IPlayPeriod::ETrackChangeResult::StartOver;
	}
	// On ongoing Live presentation without PDT has no information to locate the startover segment because
	// there will no "previous" segment request to get the information from. See GetContinuationSegment()
	if (TimelineMediaAsset->InitialPlaylistType == FPlaylistParserHLS::EPlaylistType::Live && !TimelineMediaAsset->bHasProgramDateTime && !TimelineMediaAsset->bInitialHasEndList)
	{
		return IManifest::IPlayPeriod::ETrackChangeResult::StartOver;
	}
	// Create a temporary period and prepare it for playback. This may result in media playlist load requests!
	FPlayPeriod TempPeriod(PlayerSessionServices, TimelineMediaAsset);
	TempPeriod.SetStreamPreferences(InStreamType, InStreamAttributes);
	TempPeriod.PrepareForPlay();
	const TArray<FMultiVariantPlaylistHLS::FInternalTrackMetadata>* Tracks = nullptr;
	const int32 stIdx = StreamTypeToArrayIndex(InStreamType);
	auto Pwy = TimelineMediaAsset->GetCurrentPathway();
	if (Pwy.IsValid())
	{
		if (stIdx == 0 && Pwy->VideoTracks.Num())
		{
			Tracks = &Pwy->VideoTracks;
		}
		else if (stIdx == 1 && Pwy->AudioTracks.Num())
		{
			Tracks = &Pwy->AudioTracks;
		}
		else if (stIdx == 2 && Pwy->SubtitleTracks.Num())
		{
			Tracks = &Pwy->SubtitleTracks;
		}
	}
	// If either the stream we are leaving or the one we want to switch to is a variant we have to start over.
	if (!Tracks || ((*Tracks)[SelectedTrackStream[stIdx].TrackIndex].bIsVariant || (*Tracks)[TempPeriod.SelectedTrackStream[stIdx].TrackIndex].bIsVariant))
	{
		return IManifest::IPlayPeriod::ETrackChangeResult::StartOver;
	}
	return IManifest::IPlayPeriod::ETrackChangeResult::NewPeriodNeeded;
}

TSharedPtrTS<ITimelineMediaAsset> FActiveHLSPlaylist::FPlayPeriod::GetMediaAsset() const
{
	return TimelineMediaAsset;
}

void FActiveHLSPlaylist::FPlayPeriod::SelectStream(const FString& InAdaptationSetID, const FString& InRepresentationID, int32 InQualityIndex, int32 InMaxQualityIndex)
{
	int32 TypeIndex = -1;
	if (SelectedTrackStream[0].MetaID.Equals(InAdaptationSetID) && SelectedTrackStream[0].bIsSelected)
	{
		TypeIndex = 0;
	}
	else if (SelectedTrackStream[1].MetaID.Equals(InAdaptationSetID) && SelectedTrackStream[1].bIsSelected)
	{
		TypeIndex = 1;
	}
	else if (SelectedTrackStream[2].MetaID.Equals(InAdaptationSetID) && SelectedTrackStream[2].bIsSelected)
	{
		TypeIndex = 2;
	}
	else
	{
		LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("ABR tried to activate a stream from an inactive AdaptationSet!")));
		return;
	}
	TSharedPtrTS<FMultiVariantPlaylistHLS::FPlaybackAssetAdaptationSet> Adapt = TimelineMediaAsset->GetAdaptationSet(InAdaptationSetID);
	if (!Adapt.IsValid())
	{
		LogMessage(PlayerSessionServices, IInfoLog::ELevel::Error, FString::Printf(TEXT("ABR tried to activate a bad AdaptationSet!")));
		return;
	}
	for(int32 nRepIdx=0; nRepIdx<Adapt->Representations.Num(); ++nRepIdx)
	{
		if (Adapt->GetRepresentationByIndex(nRepIdx)->GetUniqueIdentifier().Equals(InRepresentationID))
		{
			SelectedTrackStream[TypeIndex].StreamIndex = nRepIdx;
			SelectedTrackStream[TypeIndex].QualityIndex = InQualityIndex;
			SelectedTrackStream[TypeIndex].MaxQualityIndex = InMaxQualityIndex;
			return;
		}
	}
	LogMessage(PlayerSessionServices, IInfoLog::ELevel::Warning, FString::Printf(TEXT("ABR tried to activate a representation that was not found in the active AdaptationSet!")));
}

void FActiveHLSPlaylist::FPlayPeriod::TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InInitSegmentsToPreload)
{
	// Not possible with HLS since the individual media playlists and hence the information on the
	// init segment has not (and must not) be loaded upfront.
}

void FActiveHLSPlaylist::FPlayPeriod::GetActiveMediaPlaylists(TArray<TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>>& OutActivePlaylists)
{
	for(int32 i=0; i<3; ++i)
	{
		TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> MediaPlaylist(SelectedTrackStream[i].ActivePlaylist);
		if (SelectedTrackStream[i].bIsSelected && MediaPlaylist.IsValid())
		{
			OutActivePlaylists.Emplace(MoveTemp(MediaPlaylist));
		}
	}
}

void FActiveHLSPlaylist::FPlayPeriod::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& InAdaptationSetID, const FString& /*InRepresentationID*/)
{
	// The segments of a variant should have equal durations across all variants.
	// For renditions they should be similar (although they can't really be due to probably different codecs (AAC has different block sizes than AC3)).
	// So for our purposes it is sufficient to look at any of the active playlists of the appropriate stream type.
	TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> MediaPlaylist;
	// Video?
	if (SelectedTrackStream[0].bIsSelected && SelectedTrackStream[0].MetaID.Equals(InAdaptationSetID))
	{
		MediaPlaylist = SelectedTrackStream[0].ActivePlaylist;
	}
	// Audio?
	else if (SelectedTrackStream[1].bIsSelected && SelectedTrackStream[1].MetaID.Equals(InAdaptationSetID))
	{
		MediaPlaylist = SelectedTrackStream[1].ActivePlaylist;
	}
	// Subtitles?
	else if (SelectedTrackStream[2].bIsSelected && SelectedTrackStream[2].MetaID.Equals(InAdaptationSetID))
	{
		MediaPlaylist = SelectedTrackStream[2].ActivePlaylist;
	}
	TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> mp(MediaPlaylist.IsValid() ? MediaPlaylist->GetPlaylist() : nullptr);
	if (mp.IsValid())
	{
		if (mp->MediaSegments.Num())
		{
			OutAverageSegmentDuration = mp->Duration / mp->MediaSegments.Num();
		}
		else
		{
			OutAverageSegmentDuration = mp->TargetDuration;
		}
	}
	// If we do not have the media playlist yet, we leave the average duration unset.
	// The ABR will use a default value instead. Eventually the media playlists will become available and can be used
	// in subsequent calls.
}


void FActiveHLSPlaylist::FPlayPeriod::IncreaseSegmentFetchDelay(const FTimeValue& InIncreaseAmount)
{
	// No-op for HLS. Segments are announced in the playlist so they cannot 404 now to become available a bit later.
}



void FActiveHLSPlaylist::FPlayPeriod::SetTimestampAdjustIfNecessary(const TSharedPtrTS<FStreamSegmentRequestCommon>& InSegment)
{
	// If this is a VoD or an Event, or was a Live stream once, we need to rebase the segment timestamps if the presentation's
	// first segment does not start at zero.
	TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> mp(InSegment->HLS.Playlist->GetPlaylist());

	if (mp->PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || mp->PlaylistType == FPlaylistParserHLS::EPlaylistType::Event ||
		(mp->PlaylistType == FPlaylistParserHLS::EPlaylistType::Live && TimelineMediaAsset->GetInitialMediaPlaylistHadEndOfList()) ||
		(mp->PlaylistType == FPlaylistParserHLS::EPlaylistType::Live && mp->bHasProgramDateTime))
	{
		InSegment->TimestampVars.bGetAndAdjustByFirstTimestamp = true;
	}
}

IManifest::FResult FActiveHLSPlaylist::FPlayPeriod::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& InStartPosition, ESearchType InSearchType)
{
	TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> Pwy = TimelineMediaAsset.IsValid() ? TimelineMediaAsset->GetCurrentPathway() : nullptr;
	if (!Pwy.IsValid())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotLoaded);
	}

	// Determine the media playlists we need.
	TArray<FStreamLoadRequest> LoadReq;
	for(int32 nStreamTypeIdx=0; nStreamTypeIdx<3; ++nStreamTypeIdx)
	{
		// When starting we clear out the currently active media playlist.
		SelectedTrackStream[nStreamTypeIdx].ActivePlaylist.Reset();
		// Then prepare fresh ones.
		if (SelectedTrackStream[nStreamTypeIdx].bIsSelected)
		{
			EStreamType StreamType = nStreamTypeIdx==0 ? EStreamType::Video:nStreamTypeIdx==1 ?EStreamType::Audio : EStreamType::Subtitle;
			LoadReq.Emplace_GetRef().Type = StreamType;
			FErrorDetail Error = TimelineMediaAsset->GetVariantPlaylist(LoadReq.Last().Request, PlayerSessionServices, StreamType, Pwy,
				SelectedTrackStream[nStreamTypeIdx].TrackIndex, SelectedTrackStream[nStreamTypeIdx].StreamIndex,
				SelectedTrackStream[0].TrackIndex, SelectedTrackStream[0].StreamIndex);
			if (Error.IsSet())
			{
				return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(Error);
			}
		}
	}
	// This should never happen, but if it does it probably means we're not loaded.
	if (LoadReq.IsEmpty())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotLoaded);
	}

	// See if all of them are present and accounted for.
	int32 NumPending = 0;
	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> NewLoadReq;
	for(int32 i=0; i<LoadReq.Num(); ++i)
	{
		LoadReq[i].Playlist = TimelineMediaAsset->GetExistingMediaPlaylistFromLoadRequest(LoadReq[i].Request);
		if (!LoadReq[i].Playlist.IsValid())
		{
			NewLoadReq.Add(LoadReq[i].Request);
			++NumPending;
		}
		else
		{
			// Remember the playlist that is now active for this stream
			SelectedTrackStream[StreamTypeToArrayIndex(LoadReq[i].Type)].ActivePlaylist = LoadReq[i].Playlist;
			if (!LoadReq[i].Playlist->ActivateIsReady())
			{
				++NumPending;
			}
		}
	}
	TimelineMediaAsset->AddNewMediaPlaylistLoadRequests(NewLoadReq);
	if (NumPending)
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound).RetryAfterMilliseconds(50);
	}

	TSharedPtrTS<FStreamSegmentRequestCommon> Segment;
	FTimelineMediaAsset::FSegSearchParam SegParam;
	FTimeValue TryAgainAt;

	// Create a segment request to which the individual stream segment requests will add themselves as
	// dependent streams. This is a special case for playback start.
	TSharedPtrTS<FStreamSegmentRequestCommon> StartSegmentRequest = MakeSharedTS<FStreamSegmentRequestCommon>();
	StartSegmentRequest->bIsInitialStartRequest = true;
	StartSegmentRequest->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();

	bool bFrameAccurateSearch = InStartPosition.Options.bFrameAccuracy;
	if (bFrameAccurateSearch)
	{
		// Get the segment that starts on or before the search time.
		InSearchType = IManifest::ESearchType::Before;
	}
	FTimeValue PlayRangeEnd = InStartPosition.Options.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());

	bool bAnyStreamAtEOS = false;
	bool bAllStreamsAtEOS = true;
	TArray<TSharedPtrTS<FStreamSegmentRequestCommon>> DependentStreams;
	for(int32 nStr=0; nStr<LoadReq.Num(); ++nStr)
	{
		int32 SelectedTrackTypeIndex = StreamTypeToArrayIndex(LoadReq[nStr].Type);
		SegParam.SearchType = InSearchType;
		SegParam.Start = InStartPosition;
		SegParam.QualityIndex = SelectedTrackStream[SelectedTrackTypeIndex].QualityIndex;
		SegParam.MaxQualityIndex = SelectedTrackStream[SelectedTrackTypeIndex].MaxQualityIndex;
		SegParam.SequenceState = InSequenceState;
		SegParam.bFrameAccurateSearch = bFrameAccurateSearch;
		SegParam.LastPTS = PlayRangeEnd;

		FTimelineMediaAsset::ESegSearchResult SegRes = TimelineMediaAsset->FindSegment(Segment, TryAgainAt, PlayerSessionServices, LoadReq[nStr].Playlist, SegParam);
		if (SegRes == FTimelineMediaAsset::ESegSearchResult::Failed || SegRes == FTimelineMediaAsset::ESegSearchResult::BeforeStart)
		{
			return IManifest::FResult(SegRes == FTimelineMediaAsset::ESegSearchResult::Failed ? IManifest::FResult::EType::NotFound : IManifest::FResult::EType::BeforeStart).SetErrorDetail(FErrorDetail().SetMessage("Failed to locate start segment"));
		}
		else if (SegRes == FTimelineMediaAsset::ESegSearchResult::UnsupportedDRM)
		{
			return IManifest::FResult(IManifest::FResult::EType::Error).SetErrorDetail(TimelineMediaAsset->GetLastError());
		}
		else if (SegRes == FTimelineMediaAsset::ESegSearchResult::PastEOS || SegRes == FTimelineMediaAsset::ESegSearchResult::Ended)
		{
			Segment->bIsEOSSegment = true;
			bAnyStreamAtEOS = true;
		}
		check(SegRes == FTimelineMediaAsset::ESegSearchResult::Found || SegRes == FTimelineMediaAsset::ESegSearchResult::PastEOS || SegRes == FTimelineMediaAsset::ESegSearchResult::Ended);
		check(Segment.IsValid());
		if (SegRes == FTimelineMediaAsset::ESegSearchResult::Found)
		{
			bAllStreamsAtEOS = false;
		}
		Segment->StreamType = LoadReq[nStr].Type;
		Segment->Period = TimelineMediaAsset;
		Segment->AdaptationSet = TimelineMediaAsset->GetAdaptationSetByTypeAndIndex(LoadReq[nStr].Type, SelectedTrackStream[SelectedTrackTypeIndex].TrackIndex);
		Segment->Representation = Segment->AdaptationSet->GetRepresentationByIndex(SelectedTrackStream[SelectedTrackTypeIndex].StreamIndex);
		Segment->SourceBufferInfo[SelectedTrackTypeIndex] = SelectedTrackStream[SelectedTrackTypeIndex].BufferSourceInfo;
		Segment->CodecInfo[SelectedTrackTypeIndex] = Segment->Representation->GetCodecInformation();
		Segment->bIgnoreVideo = Segment->StreamType != EStreamType::Video;
		Segment->bIgnoreAudio = Segment->StreamType != EStreamType::Audio;
		Segment->bIgnoreSubtitles = Segment->StreamType != EStreamType::Subtitle;
		if (bFrameAccurateSearch)
		{
			Segment->FrameAccurateStartTime = InStartPosition.Time;
		}
		SetTimestampAdjustIfNecessary(Segment);

		DependentStreams.Emplace(MoveTemp(Segment));
	}
	// Look for duplicates. We have this when a variant is a using multiplexed segments.
	for(int32 i=1; i<DependentStreams.Num(); ++i)
	{
		// Referencing the same playlist?
		if (DependentStreams[i]->HLS.Playlist == DependentStreams[i-1]->HLS.Playlist)
		{
			switch(DependentStreams[i]->StreamType)
			{
				case EStreamType::Video:
				{
					DependentStreams[i-1]->bIsMultiplex = true;
					DependentStreams[i-1]->bIgnoreVideo = false;
					DependentStreams[i-1]->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Video)] = DependentStreams[i]->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Video)];
					DependentStreams[i-1]->CodecInfo[StreamTypeToArrayIndex(EStreamType::Video)] = DependentStreams[i]->CodecInfo[StreamTypeToArrayIndex(EStreamType::Video)];
					break;
				}
				case EStreamType::Audio:
				{
					DependentStreams[i-1]->bIsMultiplex = true;
					DependentStreams[i-1]->bIgnoreAudio = false;
					DependentStreams[i-1]->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Audio)] = DependentStreams[i]->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Audio)];
					DependentStreams[i-1]->CodecInfo[StreamTypeToArrayIndex(EStreamType::Audio)] = DependentStreams[i]->CodecInfo[StreamTypeToArrayIndex(EStreamType::Audio)];
					break;
				}
				case EStreamType::Subtitle:
				{
					DependentStreams[i-1]->bIsMultiplex = true;
					DependentStreams[i-1]->bIgnoreSubtitles = false;
					DependentStreams[i-1]->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)] = DependentStreams[i]->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)];
					DependentStreams[i-1]->CodecInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)] = DependentStreams[i]->CodecInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)];
					break;
				}
			}
			DependentStreams.RemoveAt(i);
			--i;
		}
	}

	// Remember the now active playlists.
	for(int32 i=0; i<3; ++i)
	{
		SelectedTrackStream[i].ActivePlaylist.Reset();
	}
	for(int32 i=0; i<DependentStreams.Num(); ++i)
	{
		// When there is no PDT mapping we want to know when we got the first media segment timestamp.
		if (i == 0 && DependentStreams[i]->HLS.bNoPDTMapping)
		{
			// For non-PDT Live streams we need to resync the timeline
			TimelineMediaAsset->InternalMediaTimeline.ResyncNeeded();
			DependentStreams[i]->FirstTimestampReceivedDelegate.BindSPLambda(TimelineMediaAsset.ToSharedRef(), [That=TimelineMediaAsset.ToWeakPtr()](TSharedPtrTS<FStreamSegmentRequestCommon> InSegment)
			{
				TSharedPtr<FTimelineMediaAsset, ESPMode::ThreadSafe> ma = That.Pin();
				if (ma.IsValid())
				{
					ma->UpdateTimelineFromMediaSegment(InSegment);
				}
			});
		}

		SelectedTrackStream[StreamTypeToArrayIndex(DependentStreams[i]->GetType())].ActivePlaylist = DependentStreams[i]->HLS.Playlist;
	}

	// The start segment request needs to be able to return a valid first PTS which is what the player sets
	// the playback position to. If not valid yet update it with the current stream values.
	if (!StartSegmentRequest->GetFirstPTS().IsValid())
	{
		check(DependentStreams.Num());
		StartSegmentRequest->AST = DependentStreams[0]->AST;
		StartSegmentRequest->AdditionalAdjustmentTime = DependentStreams[0]->AdditionalAdjustmentTime;
		StartSegmentRequest->PeriodStart = DependentStreams[0]->PeriodStart;
		StartSegmentRequest->Segment = DependentStreams[0]->Segment;
	}

	StartSegmentRequest->DependentStreams = MoveTemp(DependentStreams);
	OutSegment = MoveTemp(StartSegmentRequest);

	// All streams already at EOS?
	if (bAnyStreamAtEOS && bAllStreamsAtEOS)
	{
		return IManifest::FResult(IManifest::FResult::EType::PastEOS);
	}

	return IManifest::FResult(IManifest::FResult::EType::Found);
}

IManifest::FResult FActiveHLSPlaylist::FPlayPeriod::GetSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FStreamSegmentRequestCommon* InSegment, const FPlayStartOptions& InOptions, ENextSegType InNextType)
{
	check(InSegment);

	int32 SelectedTrackTypeIndex = StreamTypeToArrayIndex(InSegment->StreamType);
	if (!SelectedTrackStream[SelectedTrackTypeIndex].bIsSelected)
	{
		// The track may not be selected, which is ok and happens when switching tracks as start-over requests are made for all
		// track types that are not the ones being switched.
		return IManifest::FResult(IManifest::FResult::EType::NotFound);
	}

	auto Pwy = TimelineMediaAsset->GetCurrentPathway();
	FStreamLoadRequest LoadReq;
	LoadReq.Type = InSegment->StreamType;
	FErrorDetail Error = TimelineMediaAsset->GetVariantPlaylist(LoadReq.Request, PlayerSessionServices, InSegment->StreamType, Pwy,
		SelectedTrackStream[SelectedTrackTypeIndex].TrackIndex, SelectedTrackStream[SelectedTrackTypeIndex].StreamIndex,
		SelectedTrackStream[0].TrackIndex, SelectedTrackStream[0].StreamIndex);
	if (Error.IsSet())
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(Error);
	}
	for(int32 i=0; i<3; ++i)
	{
		if (SelectedTrackStream[i].bIsSelected)
		{
			LoadReq.Request->bIsPrimaryPlaylist = i==SelectedTrackTypeIndex;
			break;
		}
	}

	int32 NumPending = 0;
	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> NewLoadReq;
	LoadReq.Playlist = TimelineMediaAsset->GetExistingMediaPlaylistFromLoadRequest(LoadReq.Request);
	if (!LoadReq.Playlist.IsValid())
	{
		NewLoadReq.Add(LoadReq.Request);
		++NumPending;
	}
	else
	{
		// Remember the playlist that is now active for this stream
		SelectedTrackStream[SelectedTrackTypeIndex].ActivePlaylist = LoadReq.Playlist;
		if (!LoadReq.Playlist->ActivateIsReady())
		{
			++NumPending;
		}
	}
	if (NumPending)
	{
		TimelineMediaAsset->AddNewMediaPlaylistLoadRequests(NewLoadReq);
		return IManifest::FResult(IManifest::FResult::EType::NotFound).RetryAfterMilliseconds(50);
	}


	TSharedPtrTS<FStreamSegmentRequestCommon> Segment;
	FTimelineMediaAsset::FSegSearchParam SegParam;
	FTimeValue TryAgainAfter;

	SegParam.QualityIndex = SelectedTrackStream[SelectedTrackTypeIndex].QualityIndex;
	SegParam.MaxQualityIndex = SelectedTrackStream[SelectedTrackTypeIndex].MaxQualityIndex;
	SegParam.SequenceState.SetSequenceIndex(InSegment->TimestampSequenceIndex);
	SegParam.bFrameAccurateSearch = false;
	FTimeValue PlayRangeEnd = InOptions.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());
	SegParam.LastPTS = PlayRangeEnd;

	// Are we still on the same playlist?
	bool bSetNextExpectedTime = false;
	if (LoadReq.Playlist == InSegment->HLS.Playlist)
	{
		check(InNextType != ENextSegType::StartOver);	// starting over has no current information on sequence index, so we must not get here.
		SegParam.MediaSequenceIndex = InSegment->Segment.Number;
		// Not actually needed, but set as a safe value.
		SegParam.Start.Time.SetFromHNS(InSegment->Segment.Time + InSegment->Segment.Duration);
		SegParam.Start.Time += InSegment->AdditionalAdjustmentTime;
		if (InNextType == ENextSegType::Next)
		{
			++SegParam.MediaSequenceIndex;
		}
		// If we failed the timestamp check on the current request after a playlist switch, we need to re-check
		// this segment as well as it might also be too old to be used.
		bSetNextExpectedTime = InSegment->TimestampVars.Next.bFailed;
	}
	else
	{
		// For VOD presentations, or an Event (to which only new segments may be added), or anything that had already finished
		// we can make the playlist switch by just looking at the time in the playlist.
		if (TimelineMediaAsset->bHasProgramDateTime || TimelineMediaAsset->bInitialHasEndList || TimelineMediaAsset->InitialPlaylistType != FPlaylistParserHLS::EPlaylistType::Live)
		{
			if (InNextType == ENextSegType::Next)
			{
				SegParam.SearchType = IManifest::ESearchType::StrictlyAfter;
				SegParam.Start.Time.SetFromHNS(InSegment->Segment.Time + InSegment->Segment.Duration*3/4);
				SegParam.Start.Time += InSegment->AdditionalAdjustmentTime;
			}
			else if (InNextType == ENextSegType::StartOver)
			{
				SegParam.SearchType = IManifest::ESearchType::Before;
				SegParam.Start.Time.SetFromHNS(InSegment->Segment.Time);
				SegParam.Start.Time += InSegment->AdditionalAdjustmentTime;
			}
			else
			{
				SegParam.SearchType = IManifest::ESearchType::Same;
				SegParam.Start.Time.SetFromHNS(InSegment->Segment.Time);
				SegParam.Start.Time += InSegment->AdditionalAdjustmentTime;
			}
		}
		else
		{
			check(InNextType != ENextSegType::StartOver);	// starting over has no current information on sequence index, so we must not get here.
			// Otherwise - in a Live presentation that has no #EXT-X-PROGRAM-DATE-TIME values - things are a bit more difficult.
			SegParam.DiscontinuityIndex = InSegment->HLS.DiscontinuitySequence;
			// We got back one segment on purpose and risk loading a segment we already got.
			// This is still better than accidentally skipping one because the playlists are not in sync.
			// Here it does not matter if we are looking for the next segment, a retry or a startover segment.
			// It works like this either way.
			SegParam.LocalPosition = InSegment->HLS.LocalIndex > 0 ? InSegment->HLS.LocalIndex - 1 : InSegment->HLS.LocalIndex;
			// Not actually needed, but set as a safe value.
			SegParam.Start.Time.SetFromHNS(InSegment->Segment.Time + InSegment->Segment.Duration);
			SegParam.Start.Time += InSegment->AdditionalAdjustmentTime;
			// Perform time check in download.
			bSetNextExpectedTime = true;
		}
	}


	FTimelineMediaAsset::ESegSearchResult SegRes = TimelineMediaAsset->FindSegment(Segment, TryAgainAfter, PlayerSessionServices, LoadReq.Playlist, SegParam);
	if (SegRes == FTimelineMediaAsset::ESegSearchResult::Failed || SegRes == FTimelineMediaAsset::ESegSearchResult::BeforeStart)
	{
		return IManifest::FResult(SegRes == FTimelineMediaAsset::ESegSearchResult::Failed ? IManifest::FResult::EType::NotFound : IManifest::FResult::EType::BeforeStart).SetErrorDetail(FErrorDetail().SetMessage("Failed to locate start segment"));
	}
	else if (SegRes == FTimelineMediaAsset::ESegSearchResult::UnsupportedDRM)
	{
		return IManifest::FResult(IManifest::FResult::EType::Error).SetErrorDetail(TimelineMediaAsset->GetLastError());
	}
	else if (SegRes == FTimelineMediaAsset::ESegSearchResult::PastEOS || SegRes == FTimelineMediaAsset::ESegSearchResult::Ended)
	{
		return IManifest::FResult(IManifest::FResult::EType::PastEOS).RetryAfter(TryAgainAfter);
	}
	else
	{
		check(Segment.IsValid());
		if (Segment.IsValid())
		{
			Segment->StreamType = LoadReq.Type;
			Segment->Period = TimelineMediaAsset;
			Segment->AdaptationSet = TimelineMediaAsset->GetAdaptationSetByTypeAndIndex(LoadReq.Type, SelectedTrackStream[SelectedTrackTypeIndex].TrackIndex);
			Segment->Representation = Segment->AdaptationSet->GetRepresentationByIndex(SelectedTrackStream[SelectedTrackTypeIndex].StreamIndex);
			Segment->SourceBufferInfo[SelectedTrackTypeIndex] = SelectedTrackStream[SelectedTrackTypeIndex].BufferSourceInfo;
			Segment->CodecInfo[SelectedTrackTypeIndex] = Segment->Representation->GetCodecInformation();
			Segment->bIgnoreVideo = Segment->StreamType != EStreamType::Video;
			Segment->bIgnoreAudio = Segment->StreamType != EStreamType::Audio;
			Segment->bIgnoreSubtitles = Segment->StreamType != EStreamType::Subtitle;
			// When there is no PDT mapping we want to know when we got the first media segment timestamp.
			if (Segment->HLS.bNoPDTMapping && LoadReq.Request->bIsPrimaryPlaylist)
			{
				Segment->FirstTimestampReceivedDelegate.BindSPLambda(TimelineMediaAsset.ToSharedRef(), [That=TimelineMediaAsset.ToWeakPtr()](TSharedPtrTS<FStreamSegmentRequestCommon> InSegment)
				{
					TSharedPtr<FTimelineMediaAsset, ESPMode::ThreadSafe> ma = That.Pin();
					if (ma.IsValid())
					{
						ma->UpdateTimelineFromMediaSegment(InSegment);
					}
				});
			}

			// Copy the timestamp adjustment settings across if there is no discontinuity.
			if (!Segment->HLS.bHasDiscontinuity || InSegment->bIsFalloffSegment)
			{
				// Copy the internal timestamp variables across.
				Segment->TimestampVars.Internal = InSegment->TimestampVars.Internal;
				if (bSetNextExpectedTime)
				{
					static_assert(sizeof(Segment->TimestampVars.Next.ExpectedLargerThan) == sizeof(InSegment->TimestampVars.Local.First));

					// If we failed before, do not update the timestamps for the next check but keep the previous ones.
					// This is necessary because we purposely went back an additional segment and must not use this one's
					// values, since we know those already failed and we would keep re-checking the same segment repeatedly.
					if (InSegment->TimestampVars.Next.bFailed)
					{
						for(int32 i=0; i<UE_ARRAY_COUNT(Segment->TimestampVars.Next.ExpectedLargerThan); ++i)
						{
							Segment->TimestampVars.Next.ExpectedLargerThan[i] = InSegment->TimestampVars.Next.ExpectedLargerThan[i];
						}
					}
					else
					{
						FTimeValue HalfSegmentDuration(InSegment->Segment.Duration / 2, InSegment->Segment.Timescale, 0);
						for(int32 i=0; i<UE_ARRAY_COUNT(Segment->TimestampVars.Next.ExpectedLargerThan); ++i)
						{
							Segment->TimestampVars.Next.ExpectedLargerThan[i] = InSegment->TimestampVars.Local.First[i] + HalfSegmentDuration;
						}
					}
					Segment->TimestampVars.Next.bCheck = true;
				}
			}
			else
			{
				// With a discontinuity we may need to re-adjust the timestamps.
				SetTimestampAdjustIfNecessary(Segment);
				// And increase the timestamp index since the times might internally get smaller than before.
				FPlayerSequenceState seqState;
				seqState.SetSequenceIndex(Segment->TimestampSequenceIndex);
				++seqState.PrimaryIndex;
				Segment->TimestampSequenceIndex = seqState.GetSequenceIndex();
			}

			// Was the request that finished a multiplex?
			if (InSegment->bIsMultiplex)
			{
				// Then this request will also need to be a multiplex.
				Segment->bIsMultiplex = true;
				if (Segment->StreamType == EStreamType::Video)
				{
					check(!InSegment->bIgnoreVideo);
					if (!InSegment->bIgnoreAudio)
					{
						Segment->bIgnoreAudio = false;
						Segment->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Audio)] = SelectedTrackStream[StreamTypeToArrayIndex(EStreamType::Audio)].BufferSourceInfo;
						Segment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Audio)] = InSegment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Audio)];
					}
					if (!InSegment->bIgnoreSubtitles)
					{
						Segment->bIgnoreSubtitles = false;
						Segment->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)] = SelectedTrackStream[StreamTypeToArrayIndex(EStreamType::Subtitle)].BufferSourceInfo;
						Segment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)] = InSegment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)];
					}
				}
				else if (Segment->StreamType == EStreamType::Audio)
				{
					check(!InSegment->bIgnoreAudio);
					if (!InSegment->bIgnoreVideo)
					{
						Segment->bIgnoreVideo = false;
						Segment->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Video)] = SelectedTrackStream[StreamTypeToArrayIndex(EStreamType::Video)].BufferSourceInfo;
						Segment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Video)] = InSegment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Video)];
					}
					if (!InSegment->bIgnoreSubtitles)
					{
						Segment->bIgnoreSubtitles = false;
						Segment->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)] = SelectedTrackStream[StreamTypeToArrayIndex(EStreamType::Subtitle)].BufferSourceInfo;
						Segment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)] = InSegment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)];
					}
				}
				else if (Segment->StreamType == EStreamType::Subtitle)
				{
					check(!InSegment->bIgnoreSubtitles);
					if (!InSegment->bIgnoreVideo)
					{
						Segment->bIgnoreVideo = false;
						Segment->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Video)] = SelectedTrackStream[StreamTypeToArrayIndex(EStreamType::Video)].BufferSourceInfo;
						Segment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Video)] = InSegment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Video)];
					}
					if (!InSegment->bIgnoreAudio)
					{
						Segment->bIgnoreAudio = false;
						Segment->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Audio)] = SelectedTrackStream[StreamTypeToArrayIndex(EStreamType::Audio)].BufferSourceInfo;
						Segment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Audio)] = InSegment->CodecInfo[StreamTypeToArrayIndex(EStreamType::Audio)];
					}
				}
			}
			// If retrying increase the count to keep track of the retries performed.
			if (InNextType == ENextSegType::Retry)
			{
				++Segment->NumOverallRetries;
			}
		}
	}
	OutSegment = MoveTemp(Segment);
	return IManifest::FResult(IManifest::FResult::EType::Found);
}


IManifest::FResult FActiveHLSPlaylist::FPlayPeriod::GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType InStreamType, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& InStartPosition, ESearchType InSearchType)
{
	FStreamSegmentRequestCommon DummyReq;
	DummyReq.StreamType = InStreamType;
	DummyReq.Segment.Time = InStartPosition.Time.GetAsHNS();
	DummyReq.TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
	return GetSegment(OutSegment, &DummyReq, InStartPosition.Options, ENextSegType::StartOver);
}

IManifest::FResult FActiveHLSPlaylist::FPlayPeriod::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& InStartPosition, ESearchType InSearchType)
{
	return GetStartingSegment(OutSegment, InSequenceState, InStartPosition, InSearchType);
}

void FActiveHLSPlaylist::FPlayPeriod::ValidateDownloadedSegmentDuration(const FStreamSegmentRequestCommon* InRequest)
{
	// For successful downloads check if the segment duration was as specified.
	if (InRequest && InRequest->DownloadStats.bWasSuccessful && !InRequest->DownloadStats.bInsertedFillerData && !InRequest->TimestampVars.Next.bFailed)
	{
		// The #EXTINF duration needs to be as precise to the actual media duration as possible.
		// Otherwise playback errors could occur and switching across renditions may not work correctly.
		double AbsDelta = Utils::AbsoluteValue(InRequest->DownloadStats.Duration - InRequest->DownloadStats.DurationDownloaded);
		if (AbsDelta > 1.0)
		{
			FURL_RFC3986 MediaURL, PlaylistURL;
			MediaURL.Parse(InRequest->DownloadStats.URL.URL);
			PlaylistURL.Parse(InRequest->HLS.Playlist.IsValid() ? InRequest->HLS.Playlist->URL : TEXT(""));
			PlayerSessionServices->PostLog(Facility::EFacility::HLSPlaylistHandler, IInfoLog::ELevel::Warning, FString::Printf(TEXT("Media segment duration for segment %s in variant playlist %s was given as %.3f seconds but really was %.3f seconds!"), *MediaURL.GetLastPathComponent(), *PlaylistURL.GetLastPathComponent(), InRequest->DownloadStats.Duration, InRequest->DownloadStats.DurationDownloaded));
		}
	}
}

IManifest::FResult FActiveHLSPlaylist::FPlayPeriod::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> InCurrentSegment, const FPlayStartOptions& InOptions)
{
	FStreamSegmentRequestCommon* CurrentRequest = const_cast<FStreamSegmentRequestCommon*>(static_cast<const FStreamSegmentRequestCommon*>(InCurrentSegment.Get()));
	ValidateDownloadedSegmentDuration(CurrentRequest);
	IManifest::FResult Result = GetSegment(OutSegment, CurrentRequest, InOptions, ENextSegType::Next);
	if (Result.GetRetryAgainAtTime().IsValid())
	{
		CurrentRequest->DownloadStats.bWaitingForRemoteRetryElement = true;
	}
	return Result;
}

IManifest::FResult FActiveHLSPlaylist::FPlayPeriod::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> InCurrentSegment, const FPlayStartOptions& InOptions, bool bInReplaceWithFillerData)
{
	FStreamSegmentRequestCommon* CurrentRequest = const_cast<FStreamSegmentRequestCommon*>(static_cast<const FStreamSegmentRequestCommon*>(InCurrentSegment.Get()));
	ValidateDownloadedSegmentDuration(CurrentRequest);
	// To insert filler data we can use the current request over again.
	if (bInReplaceWithFillerData)
	{
		TSharedPtrTS<FStreamSegmentRequestCommon> NewRequest = MakeSharedTS<FStreamSegmentRequestCommon>();
		*NewRequest = *CurrentRequest;
		NewRequest->bInsertFillerData = true;
		// We treat replacing the segment with filler data as a retry.
		++NewRequest->NumOverallRetries;
		OutSegment = NewRequest;
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}
	// Request retry.
	IManifest::FResult Result = GetSegment(OutSegment, CurrentRequest, InOptions, ENextSegType::Retry);
	if (Result.GetRetryAgainAtTime().IsValid())
	{
		CurrentRequest->DownloadStats.bWaitingForRemoteRetryElement = true;
	}
	return Result;
}





void FMediaPlaylistAndStateHLS::SetPlaylist(IPlayerSessionServices* InPlayerSessionServices, TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> InPlaylist, FTimeValue InNow)
{
	check(InPlaylist.IsValid());

	auto CalcUpdateTime = [&]() -> void
	{
		if (Playlist->MediaSegments.Num())
		{
			TimeAtWhichToReload = TimeWhenLoaded + Playlist->MediaSegments.Last().Duration;
		}
		else
		{
			TimeAtWhichToReload = TimeWhenLoaded + Playlist->TargetDuration;
		}
	};
	FScopeLock lock(&PlaylistLock);
	// Fresh playlist?
	if (!Playlist.IsValid())
	{
		Playlist = MoveTemp(InPlaylist);
		PlaylistState = EPlaylistState::Loaded;
		TimeWhenLoaded = InNow;
		// Will this playlist need to be reloaded?
		if (Playlist->PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || Playlist->bHasEndList)
		{
			TimeAtWhichToReload.SetToInvalid();
		}
		else
		{
			CalcUpdateTime();
		}
	}
	else
	{
		check(Playlist.IsValid());
		// Is the updated playlist ending the presentation?
		if (InPlaylist->PlaylistType == FPlaylistParserHLS::EPlaylistType::VOD || InPlaylist->bHasEndList)
		{
			Playlist = MoveTemp(InPlaylist);
			TimeWhenLoaded = InNow;
			TimeAtWhichToReload.SetToInvalid();
			ReloadCount = 0;
		}
		// Does the updated playlist add new content?
		else if (InPlaylist->NextMediaSequence > Playlist->NextMediaSequence)
		{
			Playlist = MoveTemp(InPlaylist);
			TimeWhenLoaded = InNow;
			CalcUpdateTime();
			ReloadCount = 0;
		}
		// No new content yet.
		else
		{
			// Use the new playlist anyway as old segments may have been removed
			// or other permitted changes made.
			Playlist = MoveTemp(InPlaylist);

			/*
				According to the RFC:

					If the client reloads a Playlist file and finds that it has not
					changed, then it MUST wait for a period of one-half the Target
					Duration before retrying.  If the Playlist file remains unchanged
					when reloaded and it has been at least 1.5 times the Target Duration
					since the last time the client loaded a changed Playlist then the
					client MAY conclude that the server is not behaving properly and
					switch to a different Variant Stream or trigger a playback error.
			*/
			++ReloadCount;
			const FTimeValue RequiredUpdateInterval = (Playlist->TargetDuration * 3) / 2;
			if (InNow < TimeWhenLoaded + RequiredUpdateInterval)
			{
				// Try again after half a target duration.
				TimeAtWhichToReload = InNow + (Playlist->TargetDuration / 2);
			}
			else
			{
				// If three target durations have passed we call it quits.
				if (InNow > TimeWhenLoaded + Playlist->TargetDuration * 3)
				{
					InPlayerSessionServices->PostLog(Facility::EFacility::HLSPlaylistBuilder, IInfoLog::ELevel::Warning, FString::Printf(TEXT("HLS Live variant playlist still did not update after %.3f seconds but had to after %.3f seconds, giving up!"), (InNow - TimeWhenLoaded).GetAsSeconds(), RequiredUpdateInterval.GetAsSeconds()));
					// Stop reloading and mark it as no longer updating.
					TimeWhenLoaded = InNow;
					TimeAtWhichToReload.SetToInvalid();
					ReloadCount = 0;
					LiveUpdateState = ELiveUpdateState::NotUpdating;
				}
				else
				{
					InPlayerSessionServices->PostLog(Facility::EFacility::HLSPlaylistBuilder, IInfoLog::ELevel::Warning, FString::Printf(TEXT("HLS Live variant playlist did not update after %.3f seconds, but must every %.3f seconds!"), (InNow - TimeWhenLoaded).GetAsSeconds(), RequiredUpdateInterval.GetAsSeconds()));
					// Try again after another half a target duration.
					TimeAtWhichToReload = InNow + (Playlist->TargetDuration / 2);
				}
			}
		}
	}
}



FErrorDetail FDRMClientCacheHLS::GetClient(FEntry& OutDrmClient, const TSharedPtr<FMediaEncryptionHLS, ESPMode::ThreadSafe>& InEncryption, IPlayerSessionServices* InPlayerSessionServices, const FURL_RFC3986& InPlaylistURL)
{
	enum class EMethod
	{
		Unsupported,
		AES128,
		SampleAES,
		SampleAESCTR
	};
	check(InEncryption.IsValid());

	const FEntry* CachedEntry = Cache.FindAndTouch(InEncryption);
	if (CachedEntry)
	{
		OutDrmClient = *CachedEntry;
		return FErrorDetail();
	}

	TSharedPtrTS<FDRMManager> DRMManager = InPlayerSessionServices->GetDRMManager();
	if (DRMManager.IsValid())
	{
		for(auto& encIt : InEncryption->KeyInfos)
		{
			EMethod Method = encIt.Method.Equals(TEXT("AES-128")) ? EMethod::AES128 :
							 encIt.Method.Equals(TEXT("SAMPLE-AES")) ? EMethod::SampleAES :
							 encIt.Method.Equals(TEXT("SAMPLE-AES-CTR")) ? EMethod::SampleAESCTR : EMethod::Unsupported;
			if (Method != EMethod::Unsupported)
			{

				FString Scheme, Cipher;
				if (Method == EMethod::AES128)
				{
					Scheme = TEXT("AES-128");
					Cipher = TEXT("cbc7");
				}
				else
				{
					Scheme = TEXT("SAMPLE-AES");
					Cipher = Method == EMethod::SampleAESCTR ? TEXT("cenc") : TEXT("cbcs");
				}

				const FString& Keyformat(encIt.KeyFormat);
				const FString& KeyformatVersions(encIt.KeyFormatVersions);

				TSharedPtr<ElectraCDM::IMediaCDMCapabilities, ESPMode::ThreadSafe> DRMCapabilities;
				DRMCapabilities = DRMManager->GetCDMCapabilitiesForScheme(Scheme, Keyformat, KeyformatVersions);
				if (DRMCapabilities.IsValid())
				{
					ElectraCDM::IMediaCDMCapabilities::ESupportResult ResultCaps = DRMCapabilities->SupportsCipher(Cipher);
					if (ResultCaps == ElectraCDM::IMediaCDMCapabilities::ESupportResult::Supported)
					{
						TArray<ElectraCDM::IMediaCDM::FCDMCandidate> Candidates;
						ElectraCDM::IMediaCDM::FCDMCandidate& cand(Candidates.Emplace_GetRef());
						cand.SchemeId = Scheme;
						cand.Value = Keyformat;
						cand.CommonScheme = Cipher;

						FString LicenseKeyURL = FURL_RFC3986(InPlaylistURL).ResolveWith(encIt.URI).Get(true, false);

						FString ParamsJSON(TEXT("{"));
						ParamsJSON.Append(FString::Printf(TEXT("\"METHOD\":\"%s\","), *encIt.Method));
						ParamsJSON.Append(FString::Printf(TEXT("\"KEYFORMAT\":\"%s\","), *Keyformat));
						ParamsJSON.Append(FString::Printf(TEXT("\"KEYFORMATVERSIONS\":\"%s\","), *KeyformatVersions));
						ParamsJSON.Append(FString::Printf(TEXT("\"URI\":\"%s\","), *LicenseKeyURL));
						ParamsJSON.Append(FString::Printf(TEXT("\"IV\":\"%s\""), *encIt.IV));
						ParamsJSON.Append(TEXT("}"));
						cand.AdditionalElements = MoveTemp(ParamsJSON);
						FSHA1 Hash;
						Hash.UpdateWithString(*LicenseKeyURL, LicenseKeyURL.Len());
						cand.DefaultKIDs.Emplace(Hash.Finalize().ToString());

						TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> DrmClient;
						ElectraCDM::ECDMError ResultClient = DRMManager->CreateDRMClient(DrmClient, Candidates);
						if (ResultClient == ElectraCDM::ECDMError::Success && DrmClient.IsValid())
						{
							DrmClient->RegisterEventListener(DRMManager);
							DrmClient->PrepareLicenses();

							FEntry CacheEntry;
							CacheEntry.DrmClient = MoveTemp(DrmClient);

							// Parse out the default IV, if any, into a hex array
							FString IV(encIt.IV);
							if (IV.Len())
							{
								// Strip off the hex prefix if there is one.
								if (IV.StartsWith(TEXT("0x")))
								{
									IV.MidInline(2);
								}
								CacheEntry.DrmIV.SetNumUninitialized((IV.Len() + 1) / 2);
								FString::ToHexBlob(IV, CacheEntry.DrmIV.GetData(), CacheEntry.DrmIV.Num());
							}
							// Convert the default KID to a hex array.
							CacheEntry.DrmKID.SetNumUninitialized((cand.DefaultKIDs[0].Len() + 1) / 2);
							FString::ToHexBlob(cand.DefaultKIDs[0], CacheEntry.DrmKID.GetData(), CacheEntry.DrmKID.Num());
							// Set the cipher as a "mime type"
							CacheEntry.DrmMimeType = MoveTemp(Cipher);

							Cache.Add(InEncryption, CacheEntry);

							OutDrmClient = MoveTemp(CacheEntry);
							return FErrorDetail();
						}
					}
				}
			}
		}
	}
	return FErrorDetail().SetError(UEMEDIA_ERROR_NOT_SUPPORTED).SetFacility(Facility::EFacility::HLSPlaylistBuilder)
							.SetCode(HLS::ERRCODE_PLAYLIST_NO_SUPPORTED_DRM).SetMessage(TEXT("None of the DRM schemes is supported"));
}


} // namespace Electra
