// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PlayerCore.h"
#include "StreamTypes.h"
#include "Containers/Array.h"
#include "Containers/LruCache.h"
#include "Utilities/URLParser.h"
#include "Utilities/BCP47-Helpers.h"
#include "Player/PlayerSessionServices.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/HLS/PlaylistParserHLS.h"
#include "Player/Manifest.h"
#include "Player/StreamSegmentReaderCommon.h"
#include "Player/DRM/DRMManager.h"
#include "PlayerFacility.h"
#include "MediaStreamMetadata.h"
#include "ErrorDetail.h"

namespace Electra
{
class IStreamReader;


namespace HLS
{
const FName OptionKeyPlaylistLoadConnectTimeout(TEXT("playlist_connection_timeout"));
const FName OptionKeyPlaylistLoadNoDataTimeout(TEXT("playlist_nodata_timeout"));
const FName OptionKeyPlaylistReloadConnectTimeout(TEXT("playlist_reload_connection_timeout"));
const FName OptionKeyPlaylistReloadNoDataTimeout(TEXT("playlist_reload_nodata_timeout"));

const FTimeValue kProgramDateTimeGapThreshold(0.5);

const int32 ERRCODE_MAIN_PLAYLIST_DOWNLOAD_FAILED = 1;
const int32 ERRCODE_PLAYLIST_PARSING_FAILED = 2;
const int32 ERRCODE_PLAYLIST_SETUP_FAILED = 3;
const int32 ERRCODE_PLAYLIST_NO_SUPPORTED_DRM = 4;

}


struct FServerControlHLS
{
	FTimeValue CanSkipUntil;
	FTimeValue HoldBack;
	FTimeValue PartHoldBack;
	bool bCanSkipDateRanges = false;
	bool bCanBlockReload = false;
};

struct FStartTimeHLS
{
	FTimeValue Offset;
	bool bPrecise = false;
};

struct FMultiVariantPlaylistHLS
{
	class FPlaybackAssetRepresentation : public IPlaybackAssetRepresentation
	{
	public:
		virtual ~FPlaybackAssetRepresentation() = default;
		FString GetUniqueIdentifier() const override
		{ return ID; }
		const FStreamCodecInformation& GetCodecInformation() const override
		{ return StreamCodecInformation; }
		int32 GetBitrate() const override
		{ return Bandwidth; }
		int32 GetQualityIndex() const override
		{ return QualityIndex; }
		bool CanBePlayed() const override
		{ return true; }
		FStreamCodecInformation StreamCodecInformation;
		FString ID;
		int32 Bandwidth = 0;
		int32 QualityIndex = 0;
	};

	class FPlaybackAssetAdaptationSet : public IPlaybackAssetAdaptationSet
	{
	public:
		virtual ~FPlaybackAssetAdaptationSet() = default;
		FString GetUniqueIdentifier() const override
		{ return ID; }
		FString GetListOfCodecs() const override
		{ return ListOfCodecs; }
		const BCP47::FLanguageTag& GetLanguageTag() const override
		{ return LanguageTag; }
		int32 GetNumberOfRepresentations() const override
		{ return Representations.Num(); }
		bool IsLowLatencyEnabled() const override
		{ return false; }
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByIndex(int32 InRepresentationIndex) const override
		{ return InRepresentationIndex >= 0 && InRepresentationIndex < Representations.Num() ? Representations[InRepresentationIndex] : nullptr; }
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByUniqueIdentifier(const FString& InUniqueIdentifier) const override
		{
			for(int32 i=0; i<Representations.Num(); ++i)
			{
				if (Representations[i]->GetUniqueIdentifier().Equals(InUniqueIdentifier))
				{
					return Representations[i];
				}
			}
			return nullptr;
		}

		TArray<TSharedPtrTS<FPlaybackAssetRepresentation>> Representations;
		BCP47::FLanguageTag LanguageTag;
		FString ID;
		FString ListOfCodecs;
	};


	struct FRendition
	{
		// Sadly there is no explicit codec. We *TRY* to associate it with a codec from the EXT-X-STREAM-INF, if present.
		FStreamCodecInformation ParsedCodecFromStreamInf;
		FString CodecNameFromStreamInf;

		FString Name;								// NAME
		BCP47::FLanguageTag LanguageRFC5646;		// LANGUAGE
		BCP47::FLanguageTag AssocLanguageRFC5646;	// ASSOC-LANGUAGE
		FString StableRenditionId;					// STABLE-RENDITION-ID
		FString URI;								// URI
		FString InstreamId;							// INSTREAM-ID
		FString Characteristics;					// CHARACTERISTICS
		int32 SampleRate = 0;						// SAMPLE-RATE
		int32 Channels = 0;							// CHANNELS (first channel count part only)
		int32 BitDepth = 0;							// BIT-DEPTH
		bool bDefault = false;						// DEFAULT
		bool bAutoSelect = false;					// AUTOSELECT
		bool bForced = false;						// FORCED
	};

	struct FRenditionGroup
	{
		TArray<FRendition> Renditions;
		FString GroupID;				// GROUP-ID
		bool bIsReferenced = false;		// true if referenced by a variant stream, false if not (an orphaned group)

		// The CODEC strings applicable to this type of group from the referencing EXT-X-STREAM-INF
		TArray<FStreamCodecInformation> ParsedCodecsFromStreamInf;
		TArray<FString> CodecNamesFromStreamInf;

		bool operator == (const FString& InGroupID) const
		{ return GroupID.Equals(InGroupID); }
	};
	enum class ERenditionGroupType : int32
	{
		Invalid = -1,
		Video = 0,
		Audio = 1,
		Subtitles = 2,
		ClosedCaptions = 3
	};

	struct FStreamInf
	{
		TArray<FStreamCodecInformation> ParsedCodecs;
		TArray<FString> Codecs;				// CODECS
		TArray<FString> SupplementalCodecs;	//SUPPLEMENTAL-CODECS
		FString VideoRange {TEXT("SDR")};	// VIDEO-RANGE
		FString URI;						// The associated URI
		FString VideoGroup;					// VIDEO
		FString AudioGroup;					// AUDIO
		FString SubtitleGroup;				// SUBTITLES
		FString ClosedCaptionGroup;			// CLOSED-CAPTIONS
		FString StableVariantId;			// STABLE-VARIANT-ID
		FString PathwayId;					// PATHWAY-ID
		FTimeFraction FrameRate;			// FRAME-RATE
		int64 Bandwidth = -1;				// BANDWIDTH
		int32 ResolutionW = 0;				// RESOLUTION (width)
		int32 ResolutionH = 0;				// RESOLUTION (height)
		float Score = -1.0f;				// SCORE
		// Number of parsed codecs per type.
		int32 NumVideoCodec = 0;
		int32 NumAudioCodec = 0;
		int32 NumSubtitleCodec = 0;
		// Generated ID
		FString ID;
		// Generated indices
		int32 IndexOfSelfInArray = -1;
		int32 QualityIndex = 0;
		// Temp check
		bool bReferencesAudioRenditionWithoutCodec = false;
	};

	struct FVideoVariantGroup
	{
		TArray<FStreamCodecInformation> ParsedCodecs;
		TArray<FString> BaseSupplementalCodecs;
		FString VideoRange;
		TArray<int32> StreamInfIndices;
		TArray<int32> SameAsVideoVariantGroupIndex;
	};

	struct FAudioVariantGroup
	{
		TArray<FStreamCodecInformation> ParsedCodecs;
		TArray<int32> StreamInfIndices;
	};

	struct FInternalTrackMetadata
	{
		FTrackMetadata Meta;

		/*
			If set this is the rendition this has been sourced from.
			If not set this is generated from a variant.
		*/
		TOptional<FRendition> Rendition;

		/*
			If `true` this indicates that this track is being sourced from a variant
			and not a rendition.
			If `false` the track may be sourced from a separate rendition if the URI in
			the rendition is set, otherwise it is included in the variant.
		*/
		bool bIsVariant = false;

		/*
			Used only when there are audio-only variant streams AND video-only variant streams, in which
			case this gives the index of the audio-only group we have associated with the video.
			`bIsVariant` will be `true` when this is set.
		*/
		int32 AudioVariantGroupIndex = -1;

		/*
			Used only when there are multiple video groups, aka "angles".
			Each "angle" is one video track, with the streams in the track metadata being
			the different quality levels of that "angle".
			This array contains the variant IDs that correspond to the respective quality level.
		*/
		TArray<FString> VideoVariantBaseIDs;
	};

	struct FPathwayStreamInfs
	{
		FString PathwayID;
		TArray<FStreamInf> StreamInfs;
		TArray<FVideoVariantGroup> VideoVariantGroups;
		TArray<FAudioVariantGroup> AudioOnlyVariantGroups;
		// Generated metadata
		TArray<FInternalTrackMetadata> VideoTracks;
		TArray<FInternalTrackMetadata> AudioTracks;
		TArray<FInternalTrackMetadata> SubtitleTracks;
		TArray<TSharedPtrTS<FPlaybackAssetAdaptationSet>> VideoAdaptationSets;
		TArray<TSharedPtrTS<FPlaybackAssetAdaptationSet>> AudioAdaptationSets;
		TArray<TSharedPtrTS<FPlaybackAssetAdaptationSet>> SubtitleAdaptationSets;
	};

	struct FContentSteeringParams
	{
		FString PrimaryPathwayId;
		FString SteeringURI;
		FString CustomInitialSelectionPriority;
		bool bQueryBeforeStart = false;
		bool bHaveContentSteering = false;
	};

	// Variable names are case sensitive so we use an array instead of a map.
	TArray<FPlaylistParserHLS::FVariableSubstitution> VariableSubstitutions;
	TArray<FRenditionGroup> RenditionGroupsOfType[4];
	TArray<TSharedPtrTS<FPathwayStreamInfs>> PathwayStreamInfs;
	FString URL;
	FURL_RFC3986 ParsedURL;

	FServerControlHLS ServerControl;
	FStartTimeHLS StartTime;
	FContentSteeringParams ContentSteeringParams;

	// Initial bucket for all #EXT-X-STREAM-INF's before grouping into their respective PATHWAY-IDs
	TArray<FStreamInf> InitialStreamInfs;
};


struct FMediaEncryptionHLS
{
	struct FKeyInfo
	{
		FString Method;
		FString URI;
		FString IV;
		FString KeyFormat;
		FString KeyFormatVersions;
	};
	TArray<FKeyInfo> KeyInfos;
};

struct FMediaByteRangeHLS
{
	int64 NumBytes = -1;
	int64 Offset = -1;
	FString GetForHTTP() const
	{
		if (Offset >= 0 && NumBytes > 0)
		{
			return FString::Printf(TEXT("%lld-%lld"), (long long int) Offset, (long long int)(Offset+NumBytes-1));
		}
		return FString();
	}
};

struct FMediaInitSegment
{
	TSharedPtr<FMediaEncryptionHLS, ESPMode::ThreadSafe> Encryption;
	FString URL;
	FMediaByteRangeHLS ByteRange;
};

struct FMediaSegmentHLS
{
	TSharedPtr<FMediaInitSegment, ESPMode::ThreadSafe> InitSegment;
	TSharedPtr<FMediaEncryptionHLS, ESPMode::ThreadSafe> Encryption;
	FString URL;
	FMediaByteRangeHLS ByteRange;
	FTimeValue Duration;
	FTimeValue ProgramDateTime;
	int64 MediaSequence = 0;
	int64 DiscontinuitySequence = 0;
	uint8 bDiscontinuity : 1 = 0;
	uint8 bGap : 1 = 0;
};

struct FMediaPlaylistHLS
{
	FServerControlHLS ServerControl;
	FStartTimeHLS StartTime;
	FTimeValue FirstProgramDateTime;
	FTimeValue TargetDuration;
	FTimeValue Duration;
	int64 NextMediaSequence = 0;
	int64 NextDiscontinuitySequence = 0;
	FPlaylistParserHLS::EPlaylistType PlaylistType = FPlaylistParserHLS::EPlaylistType::Live;
	TArray<FMediaSegmentHLS> MediaSegments;
	bool bHasEndList = false;
	bool bHasProgramDateTime = false;

	TArray<FPlaylistParserHLS::FVariableSubstitution> VariableSubstitutions;
	FString URL;
	FURL_RFC3986 ParsedURL;
};


struct FMediaPlaylistInformationHLS
{
	EStreamType StreamType = EStreamType::Unsupported;
	FString AssetID;
	FString AdaptationSetID;
	FString RepresentationID;
	FString PathwayID;
	int32 RepresentationBandwidth = 0;
	bool Equals(const FMediaPlaylistInformationHLS& InOther) const
	{
		return AssetID.Equals(InOther.AssetID) &&
			   AdaptationSetID.Equals(InOther.AdaptationSetID) &&
			   RepresentationID.Equals(InOther.RepresentationID) &&
			   PathwayID.Equals(InOther.PathwayID);
	}
};

struct FMediaPlaylistAndStateHLS
{
	enum class EPlaylistState
	{
		NotLoaded,
		Requested,
		Loaded,
		Invalid
	};
	enum class ELiveUpdateState
	{
		Normal,
		NotUpdating,
		ReachedEnd,
		Stopped
	};
	// Media playlist URL as given in the multivariant playlist. This is not necessarily the effective URL.
	FString URL;
	TArray<FURL_RFC3986::FQueryParam> MultiVariantURLFragmentComponents;
	Playlist::FReplayEventParams ReplayEventParams;

	// Current state of the playlist.
	EPlaylistState PlaylistState = EPlaylistState::NotLoaded;
	// Time at which the response was received, from the Date HTTP response header
	FTimeValue ResponseDateHeaderTime;
	// Time at which the playlist needs to be reloaded, if at all.
	FTimeValue TimeAtWhichToReload;
	// Whether or not this is the primary variant playlist.
	bool bIsPrimaryPlaylist = false;
	// State of Live updates
	ELiveUpdateState LiveUpdateState = ELiveUpdateState::Normal;

	FMediaPlaylistInformationHLS PlaylistInfo;

	bool ActivateIsReady()
	{
		// If requested it's in the process of loading and is not ready yet.
		if (PlaylistState == FMediaPlaylistAndStateHLS::EPlaylistState::Requested)
		{
			return false;
		}
		// If the playlist has been cleared we need to fetch it again.
		if (!Playlist.IsValid())
		{
			// We do this by setting the next reload time to zero so it gets fetched immediately.
			PlaylistState = FMediaPlaylistAndStateHLS::EPlaylistState::Requested;
			TimeAtWhichToReload.SetToZero();
			return false;
		}
		check(Playlist.IsValid() && PlaylistState == EPlaylistState::Loaded);
		return true;
	}

	TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> GetPlaylist()
	{
		FScopeLock lock(&PlaylistLock);
		TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> pl(Playlist);
		return pl;
	}
	void SetPlaylist(IPlayerSessionServices* InPlayerSessionServices, TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> InPlaylist, FTimeValue InNow);
	void ClearPlaylist()
	{
		TimeAtWhichToReload.SetToInvalid();
		PlaylistLock.Lock();
		TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> pl;
		Swap(Playlist, pl);
		PlaylistLock.Unlock();
		pl.Reset();
	}
	void LoadFailed()
	{
		PlaylistLock.Lock();
		TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> pl;
		Swap(Playlist, pl);
		TimeAtWhichToReload.SetToInvalid();
		PlaylistState = EPlaylistState::NotLoaded;
		PlaylistLock.Unlock();
		pl.Reset();
	}
	FTimeValue GetTimeWhenLoaded() const
	{
		return TimeWhenLoaded;
	}

private:
	// The loaded playlist.
	TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> Playlist;
	// The time when the playlist was loaded and processed.
	FTimeValue TimeWhenLoaded;
	// Number of times we had to reload before new media segments showed up.
	int32 ReloadCount = 0;

	FCriticalSection PlaylistLock;
};


struct FLoadRequestHLSPlaylist
{
	~FLoadRequestHLSPlaylist()
	{
		if (ResourceRequest.IsValid())
		{
			ResourceRequest->Cancel();
			ResourceRequest.Reset();
		}
	}

	enum class ELoadType
	{
		Undefined,
		Main,
		Steering,
		Variant,
		InitialVariant
	};

	ELoadType LoadType = ELoadType::Undefined;
	TSharedPtr<FHTTPResourceRequest, ESPMode::ThreadSafe> ResourceRequest;
	FTimeValue ExecuteAtUTC;
	int32 Attempt = 1;
	bool bIsPrimaryPlaylist = false;
	bool bIsPreStartSteering = false;
	bool bAddedCMCDParameters = false;

	FMediaPlaylistInformationHLS PlaylistInfo;

	// Which previous playlist this is an update request for, if any.
	TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> UpdateRequestFor;

	// Time at which the response was received, from the Date HTTP response header
	FTimeValue ResponseDateHeaderTime;
};



class FDRMClientCacheHLS
{
public:
	FDRMClientCacheHLS()
		: Cache(32) // set sufficiently large to accommodate n variant streams with differently encrypted init segments and/or key rotation
	{ }

	struct FEntry
	{
		TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> DrmClient;
		FString DrmMimeType;
		TArray<uint8> DrmIV;
		TArray<uint8> DrmKID;
	};

	FErrorDetail GetClient(FEntry& OutDrmClient, const TSharedPtr<FMediaEncryptionHLS, ESPMode::ThreadSafe>& InEncryption, IPlayerSessionServices* InPlayerSessionServices, const FURL_RFC3986& InPlaylistURL);
private:
	TLruCache<TSharedPtr<FMediaEncryptionHLS, ESPMode::ThreadSafe>, FEntry> Cache;
};




class FActiveHLSPlaylist : public IManifest
{
public:
	FActiveHLSPlaylist();
	~FActiveHLSPlaylist();
	FErrorDetail Create(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests, IPlayerSessionServices* InPlayerSessionServices, TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InFromMultiVariantPlaylist);
	void UpdateWithMediaPlaylist(TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> InMediaPlaylist, bool bIsPrimary, bool bIsUpdate);
	void GetNewMediaPlaylistLoadRequests(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests);
	void GetActiveMediaPlaylists(TArray<TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>>& OutActivePlaylists, const FTimeValue& InNow);
	void GetAllMediaPlaylistLoadRequests(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests, EStreamType InForType);
	void CheckForPathwaySwitch();

	EMediaFormatType GetMediaFormatType() const override;
	bool ReloadsPlaylistPeriodically() const override;
	bool SupportsMultipleTracksPerKind() const override;
	bool OrderOfTracksCanChangeDynamically() const override;
	EType GetPresentationType() const override;
	EReplayEventType GetReplayType() const override;
	TSharedPtrTS<const FLowLatencyDescriptor> GetLowLatencyDescriptor() const override;
	FTimeValue CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const override;
	FTimeValue GetAnchorTime() const override;
	FTimeRange GetTotalTimeRange() const override;
	FTimeRange GetSeekableTimeRange() const override;
	FTimeRange GetPlaybackRange(EPlaybackRangeType InRangeType) const override;
	FTimeValue GetDuration() const override;
	FTimeValue GetDefaultStartTime() const override;
	void ClearDefaultStartTime() override;
	FTimeValue GetDefaultEndTime() const override;
	void ClearDefaultEndTime() override;
	FTimeValue GetMinBufferTime() const override;
	FTimeValue GetDesiredLiveLatency() const override;
	ELiveEdgePlayMode GetLiveEdgePlayMode() const override;
	TRangeSet<double> GetPossiblePlaybackRates(EPlayRateType InForType) const override;
	TSharedPtrTS<IProducerReferenceTimeInfo> GetProducerReferenceTimeInfo(int64 InID) const override;
	void GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType InStreamType) const override;
	void UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData) override;
	void UpdateDynamicRefetchCounter() override;
	void PrepareForLooping(int32 InNumLoopsToAdd) override;
	void TriggerClockSync(EClockSyncType InClockSyncType) override;
	void TriggerPlaylistRefresh() override;
	void ReachedStableBuffer() override;
	IStreamReader *CreateStreamReaderHandler() override;
	FResult FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
	FResult FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment) override;

	static FErrorDetail DeterminePathwayToUse(IPlayerSessionServices* InPlayerSessionServices, FString& OutPathway, const FString& InCurrentPathway, const TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe>& InFromMultiVariantPlaylist);
	FErrorDetail PreparePathway(TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>& InOutPathway, TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InFromMultiVariantPlaylist);
private:
	ELECTRA_IMPL_DEFAULT_ERROR_METHODS(HLSPlaylistBuilder);

	struct FInternalBuilder;
	class FPlayPeriod;

	FErrorDetail CreateTrackMetadata(IPlayerSessionServices* InPlayerSessionServices, TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>& InPathway, TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InFromMultiVariantPlaylist, const FInternalBuilder* InBuilder);
	FErrorDetail GetInitialVariantPlaylistLoadRequests(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests, IPlayerSessionServices* InPlayerSessionServices);

	class FTimelineMediaAsset : public ITimelineMediaAsset
	{
	public:
		virtual ~FTimelineMediaAsset();
		FTimeRange GetTimeRange() const override;
		FTimeValue GetDuration() const override;
		FString GetAssetIdentifier() const override;
		FString GetUniqueIdentifier() const override;
		int32 GetNumberOfAdaptationSets(EStreamType InStreamType) const override;
		TSharedPtrTS<IPlaybackAssetAdaptationSet> GetAdaptationSetByTypeAndIndex(EStreamType InStreamType, int32 InAdaptationSetIndex) const override;
		void GetMetaData(TArray<FTrackMetadata>& OutMetadata, EStreamType InStreamType) const override;
		void UpdateRunningMetaData(const FString& InKindOfValue, const FVariant& InNewValue) override;

		FErrorDetail GetVariantPlaylist(TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>& OutPlaylistLoadRequest, IPlayerSessionServices* InPlayerSessionServices, EStreamType InStreamType, const TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>& InPathway, int32 InTrackIndex, int32 InStreamIndex, int32 InMainTrackIndex, int32 InMainStreamIndex);
		void UpdateWithMediaPlaylist(TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> InMediaPlaylist, bool bIsPrimary, bool bIsUpdate);
		void GetNewMediaPlaylistLoadRequests(TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& OutPlaylistLoadRequests);
		void AddNewMediaPlaylistLoadRequests(const TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>>& InNewPlaylistLoadRequests);
		void UpdateActiveMediaPlaylists(const TArray<TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>>& InActiveMediaPlaylist, const FTimeValue& InNow);

		TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> GetExistingMediaPlaylistFromLoadRequest(TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InPlaylistLoadRequest);
		FTimeRange GetPlaybackRangeFromURL(EPlaybackRangeType InRangeType) const;
		FTimeValue CalculatePlaylistTimeOffset(const TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe>& InPlaylist);
		FTimeValue CalculateStartTime(const TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe>& InPlaylist);
		FTimeValue GetDesiredLiveLatency() const;
		FTimeRange GetSeekableTimeRange() const;
		FTimeValue CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency);
		void UpdateTimelineFromMediaSegment(TSharedPtrTS<FStreamSegmentRequestCommon> InSegment);
		void ResetInternalTimeline();
		void TransformIntoReplayEvent();
		void PrepareReplayEventForLooping(int32 InNumLoopsToAdd);

		TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> GetCurrentPathway() const
		{
			return CurrentPathway;
		}
		void SetCurrentPathway(const TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>& InNewPathway)
		{
			CurrentPathway = InNewPathway;
		}

		struct FSegSearchParam
		{
			IManifest::ESearchType SearchType = IManifest::ESearchType::Closest;
			FPlayStartPosition Start;
			int32 QualityIndex = 0;
			int32 MaxQualityIndex = 0;
			FTimeValue LastPTS;
			bool bFrameAccurateSearch = false;
			FPlayerSequenceState SequenceState;
			int64 MediaSequenceIndex = -1;
			int64 DiscontinuityIndex = -1;
			int32 LocalPosition = -1;
		};
		enum class ESegSearchResult
		{
			Failed,
			Found,
			PastEOS,
			BeforeStart,
			Ended,
			UnsupportedDRM
		};
		ESegSearchResult FindSegment(TSharedPtrTS<FStreamSegmentRequestCommon>& OutSegment, FTimeValue& OutTryLater, IPlayerSessionServices* InPlayerSessionServices, const TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>& InPlaylist, const FSegSearchParam& InParam);

		FErrorDetail GetLastError() const
		{ return LastError; }

		const FMultiVariantPlaylistHLS::FInternalTrackMetadata* GetInternalTrackMetadata(const FString& InForID) const;
		TSharedPtrTS<FMultiVariantPlaylistHLS::FPlaybackAssetAdaptationSet> GetAdaptationSet(const FString& InForID) const;

		// Returns `true` if the very first media playlist loaded already had an EXT-X-ENDLIST
		// If this is an EVENT or a Live presentation and it did not, we can tell that if it
		// appears in a playlist reload that we are transitioning from Live to VOD.
		bool GetInitialMediaPlaylistHadEndOfList() const
		{ return bInitialHasEndList; }

	public:
		IPlayerSessionServices* PlayerSessionServices = nullptr;

		TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> MultiVariantPlaylist;

		// Metadata from the primary media playlist
		FServerControlHLS ServerControl;
		FTimeValue FirstProgramDateTime;
		FTimeValue InitialFirstProgramDateTime;
		FTimeValue TargetDuration;
		FTimeValue Duration;
		FPlaylistParserHLS::EPlaylistType PlaylistType = FPlaylistParserHLS::EPlaylistType::Live;
		FPlaylistParserHLS::EPlaylistType InitialPlaylistType = FPlaylistParserHLS::EPlaylistType::Live;
		bool bHasEndList = false;
		bool bInitialHasEndList = false;
		bool bHasProgramDateTime = false;
		TArray<FURL_RFC3986::FQueryParam> MultiVariantURLFragmentComponents;
		Playlist::FReplayEventParams ReplayEventParams;
		FTimeRange DefaultStartAndEndTime;

		// A one-time only established base time to shift the playlist timeline around.
		FTimeValue BaseTimeOffset;
		// The time at which a live playlist was turned into a static one.
		FTimeValue TimePlaylistTransitionedToStatic;
		// Holds the last generated time range to be used when a Live presentation transitions to static.
		mutable FTimeRange LastKnownTimeRange;

		// Timestamps from the media segment, used with non-PDT Live streams
		struct FInternalMediaTimeline
		{
			void ResyncNeeded()
			{
				bNeedResync = true;
				bLockInitial = true;
				InitialMediaSegmentBaseTime.SetToInvalid();
				InitialAvailableDurationUntilEnd.SetToInvalid();
				InitialTimeWhenLoaded.SetToInvalid();
				InitialOffsetFromNow.SetToInvalid();
			}
			// Running values as received with every media segment (if reporting is enabled)
			FTimeValue MediaSegmentBaseTime;
			FTimeValue AvailableDurationUntilEnd;
			FTimeValue TimeWhenLoaded;

			// Initial one-time set values after a seek.
			FTimeValue InitialMediaSegmentBaseTime;
			FTimeValue InitialAvailableDurationUntilEnd;
			FTimeValue InitialTimeWhenLoaded;
			FTimeValue InitialOffsetFromNow;
			bool bNeedResync = false;
			bool bLockInitial = true;
		};
		mutable FInternalMediaTimeline InternalMediaTimeline;
		mutable FCriticalSection InternalMediaTimelineLock;


		FCriticalSection MediaPlaylistsLock;
		TArray<TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>> MediaPlaylists;
		TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> NewMediaPlaylistLoadRequests;
		FDRMClientCacheHLS LicenseKeyCache;
		FErrorDetail LastError;
		// The currently active pathway
		TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> CurrentPathway;
		FString CurrentPathwayId;

		struct FReplayEvent
		{
			FTimeValue BaseStartTime;
			FTimeValue OriginalBaseTimeOffset;
			FTimeValue SuggestedPresentationDelay;
			bool bIsReplay = false;
			bool bIsStaticEvent = false;
			void Reset()
			{
				BaseStartTime.SetToInvalid();
				OriginalBaseTimeOffset.SetToInvalid();
				SuggestedPresentationDelay.SetToInvalid();
				bIsReplay = false;
				bIsStaticEvent = false;
			}
		};
		FReplayEvent ReplayEvent;
	};

	FCriticalSection RequestedPeriodsLock;
	TArray<TWeakPtrTS<FPlayPeriod>> RequestedPeriods;

	TSharedPtr<FTimelineMediaAsset, ESPMode::ThreadSafe> TimelineMediaAsset;
	IPlayerSessionServices* PlayerSessionServices = nullptr;
};



} // namespace Electra
