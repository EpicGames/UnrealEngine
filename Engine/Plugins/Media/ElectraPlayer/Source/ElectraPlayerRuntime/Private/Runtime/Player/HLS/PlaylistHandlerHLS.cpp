// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlaylistHandlerHLS.h"
#include "PlaylistParserHLS.h"
#include "PlaylistHLS.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/ContentSteeringHandler.h"
#include "Player/CMCDHandler.h"
#include "Utilities/Utilities.h"
#include "Utilities/URLParser.h"
#include "Utilities/TimeUtilities.h"
#include "ElectraPlayerMisc.h"
#include "StreamTypes.h"
#include "Misc/SecureHash.h"
#include "Misc/Base64.h"
#include "Internationalization/Regex.h"

namespace Electra
{

class FPlaylistHandlerHLS : public IPlaylistHandlerHLS
{
public:
	FPlaylistHandlerHLS(IPlayerSessionServices* InPlayerSessionServices);
	virtual ~FPlaylistHandlerHLS();

	EMediaFormatType GetMediaFormatType() const override
	{ return EMediaFormatType::HLS; }
	void Close() override;
	void HandleOnce() override;
	const FString& GetPlaylistType() const override
	{
		static const FString PlaylistType(TEXT("hls"));
		return PlaylistType;
	}
	void LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams) override;
	FString GetURL() const override;
	TSharedPtrTS<IManifest> GetManifest() override;

private:
	ELECTRA_CLASS_DEFAULT_ERROR_METHODS(PlayerSessionServices, HLSPlaylistHandler);

	static const FRegexPattern VariableSubstitutionPattern()
	{
		static const FRegexPattern vsp(TEXT("(\\{\\$.+?\\})"));
		return vsp;
	}

	static bool ValidateNumbersOnly(const FString& In)
	{
		for(StringHelpers::FStringIterator It(In); It; ++It)
		{
			if (!(*It >= TCHAR('0') && *It <= TCHAR('9')))
			{
				return false;
			}
		}
		return true;
	}
	static bool ValidatePositiveFloatOnly(const FString& In)
	{
		for(StringHelpers::FStringIterator It(In); It; ++It)
		{
			if (!((*It >= TCHAR('0') && *It <= TCHAR('9')) || *It == TCHAR('.')))
			{
				return false;
			}
		}
		return true;
	};

	void LoadMainPlaylist(const FString& InUrl);
	TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> CreateMediaPlaylistLoadRequest(const TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>& InPlaylist);
	void RepeatPlaylistLoad(TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InRequestToRepeat, const FTimeValue& InAtUTC);
	void RetryInitialPlaylistWith(TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InRequestToRepeat, bool bIsPrimary);
	void SetupActivePlaylist();
	void PerformContentSteeringCloning(bool bIsPreStartFetch);

	FTimeValue GetTimeoutValue(const FName& InOptionName, int32 InDefaultValueMillisec);
	TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> HandleMainPlaylist(const FTimeValue& Now, TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InRequest);
	TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> HandleVariantPlaylist(const FTimeValue& Now, TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InRequest);

	bool ParseStartTime(FStartTimeHLS& InOutStartTime, const TUniquePtr<FPlaylistParserHLS::FElement>& InElement);
	bool ParseServerControl(FServerControlHLS& InOutServerControl, const TUniquePtr<FPlaylistParserHLS::FElement>& InElement);
	bool PrepareSubstitutionVariables(TArray<FPlaylistParserHLS::FVariableSubstitution>& OutVariableSubstitutions, TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> InPlaylist, const TArray<FPlaylistParserHLS::FVariableSubstitution>& InParentSubstitutions);
	bool BuildPlaylist(TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> InPlaylist, bool bIsMainPlaylist);
	bool BuildMultiVariantPlaylist(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe>& OutMultiVariantPlaylist, TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> InPlaylist);
	bool BuildMediaPlaylist(TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe>& OutMediaPlaylist, TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> InPlaylist, TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void GroupVariantStreamsByPathways(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void AssignInternalVariantStreamIDs(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void GroupVariantStreamsByVideoProperties(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void GroupAudioOnlyVariantStreams(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	enum EFillInOptions
	{
		None = 0,
		FallbackCDNs = 1<<0,
		Codecs = 1<<1,
		Resolution = 1<<2,
		RenditionCodecs = 1<<3,
		Scores = 1<<4,
		All = (1<<5) - 1
	};
	void FillInMissingInformation(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist, EFillInOptions InFillInOpts);
	void CheckForScore(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void CheckForFallbackStreams(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void CheckForMissingCodecs(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void CheckForMissingResolution(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void AssignResolutionAndFrameRateToCodecs(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);
	void AssignCodecsToRenditions(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist);

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	FString MultiVariantPlaylistEffectiveURL;
	TArray<FURL_RFC3986::FQueryParam> MultiVariantURLFragmentComponents;
	Playlist::FReplayEventParams ReplayEventParams;

	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> PendingLoadRequests;
	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> RunningLoadRequests;
	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> CurrentlyFailedPlaylistRequests;
	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> NewlyFailedPlaylistRequests;

	struct FFailedPlaylist
	{
		FMediaPlaylistInformationHLS Info;
		int32 NumFailures = 0;
	};
	TArray<FFailedPlaylist> RepeatedlyFailedPlaylistRequests;

	FErrorDetail LastError;

	TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> MultiVariantPlaylist;
	int32 NumPendingInitialVariantRequest = 0;

	TSharedPtr<FActiveHLSPlaylist, ESPMode::ThreadSafe> ActivePlaylist;
};



TSharedPtrTS<IPlaylistReader> IPlaylistHandlerHLS::Create(IPlayerSessionServices* PlayerSessionServices)
{
	return MakeShared<FPlaylistHandlerHLS, ESPMode::ThreadSafe>(PlayerSessionServices);
}


FPlaylistHandlerHLS::FPlaylistHandlerHLS(IPlayerSessionServices* InPlayerSessionServices)
{
	check(InPlayerSessionServices);
	PlayerSessionServices = InPlayerSessionServices;
}

FPlaylistHandlerHLS::~FPlaylistHandlerHLS()
{
}


void FPlaylistHandlerHLS::Close()
{
	for(int32 i=0; i<PendingLoadRequests.Num(); ++i)
	{
		PendingLoadRequests[i]->ResourceRequest.Reset();
		PendingLoadRequests[i].Reset();
	}
	for(int32 i=0; i<RunningLoadRequests.Num(); ++i)
	{
		RunningLoadRequests[i]->ResourceRequest->Cancel();
		RunningLoadRequests[i]->ResourceRequest.Reset();
		RunningLoadRequests[i].Reset();
	}
}

void FPlaylistHandlerHLS::SetupActivePlaylist()
{
	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> PlaylistLoadRequests;
	ActivePlaylist = MakeShared<FActiveHLSPlaylist, ESPMode::ThreadSafe>();
	LastError = ActivePlaylist->Create(PlaylistLoadRequests, PlayerSessionServices, MultiVariantPlaylist);
	if (LastError.IsOK())
	{
		for(int32 nr=0; nr<PlaylistLoadRequests.Num(); ++nr)
		{
			PlaylistLoadRequests[nr]->LoadType = FLoadRequestHLSPlaylist::ELoadType::InitialVariant;
			PendingLoadRequests.Emplace(MoveTemp(PlaylistLoadRequests[nr]));
			++NumPendingInitialVariantRequest;
		}
	}
	else
	{
		PostError(LastError);
		ActivePlaylist.Reset();
	}
}

void FPlaylistHandlerHLS::HandleOnce()
{
	FTimeValue Now = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
	bool bIsPreStartSteering = false;

	// Get any new media playlist load requests.
	if (ActivePlaylist.IsValid())
	{
		TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> NewPlaylistLoadRequests;
		ActivePlaylist->GetNewMediaPlaylistLoadRequests(NewPlaylistLoadRequests);
		for(int32 nr=0; nr<NewPlaylistLoadRequests.Num(); ++nr)
		{
			PendingLoadRequests.Emplace(MoveTemp(NewPlaylistLoadRequests[nr]));
		}

		// Get the currently active media playlists which we may have to reload periodically.
		TArray<TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>> ActiveMediaPlaylist;
		ActivePlaylist->GetActiveMediaPlaylists(ActiveMediaPlaylist, Now);
		for(int32 i=0; i<ActiveMediaPlaylist.Num(); ++i)
		{
			// Failed to update and reached the end?
			// Note: This is detected only if the playlist is currently active.
			if (ActiveMediaPlaylist[i]->LiveUpdateState == FMediaPlaylistAndStateHLS::ELiveUpdateState::ReachedEnd)
			{
				// This playlist has stopped.
				ActiveMediaPlaylist[i]->LiveUpdateState = FMediaPlaylistAndStateHLS::ELiveUpdateState::Stopped;
				// Create a fake load request that we can add to the list of failed playlists.
				TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> lr = CreateMediaPlaylistLoadRequest(ActiveMediaPlaylist[i]);
				lr->UpdateRequestFor = ActiveMediaPlaylist[i];
				lr->bIsPrimaryPlaylist = ActiveMediaPlaylist[i]->bIsPrimaryPlaylist;
				NewlyFailedPlaylistRequests.Emplace(MoveTemp(lr));
				continue;
			}

			// Needs to reload?
			if (!ActiveMediaPlaylist[i]->TimeAtWhichToReload.IsValid() || ActiveMediaPlaylist[i]->TimeAtWhichToReload > Now)
			{
				continue;
			}

			// Set reload time to INF to indicate we are processing it and not trigger it again.
			ActiveMediaPlaylist[i]->TimeAtWhichToReload.SetToPositiveInfinity();
			TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> lr = CreateMediaPlaylistLoadRequest(ActiveMediaPlaylist[i]);
			lr->UpdateRequestFor = ActiveMediaPlaylist[i];
			lr->bIsPrimaryPlaylist = ActiveMediaPlaylist[i]->bIsPrimaryPlaylist;
			PendingLoadRequests.Emplace(MoveTemp(lr));
		}
	}

	// Execute the pending requests for which the time to run them has come.
	for(int32 i=0; i<PendingLoadRequests.Num(); ++i)
	{
		if (!PendingLoadRequests[i]->ExecuteAtUTC.IsValid() || PendingLoadRequests[i]->ExecuteAtUTC <= Now)
		{
			IElectraHttpManager::FParams& Params(PendingLoadRequests[i]->ResourceRequest->GetRequest()->Parameters);
			FCMCDHandler::FRequestObjectInfo ri;
			FString RequestedURL(Params.URL);
			FString PathwayID(PendingLoadRequests[i]->PlaylistInfo.PathwayID);
			switch(PendingLoadRequests[i]->LoadType)
			{
				case FLoadRequestHLSPlaylist::ELoadType::Main:
				{
					PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::FirstPlaylist, FCMCDHandler::EObjectType::ManifestOrPlaylist, Params.URL, Params.RequestHeaders, PathwayID, ri);
					break;
				}
				case FLoadRequestHLSPlaylist::ELoadType::InitialVariant:
				{
					PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::Playlist, FCMCDHandler::EObjectType::ManifestOrPlaylist, Params.URL, Params.RequestHeaders, PathwayID, ri);
					break;
				}
				case FLoadRequestHLSPlaylist::ELoadType::Variant:
				{
					PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::PlaylistUpdate, FCMCDHandler::EObjectType::ManifestOrPlaylist, Params.URL, Params.RequestHeaders, PathwayID, ri);
					break;
				}
				case FLoadRequestHLSPlaylist::ELoadType::Steering:
				{
					PlayerSessionServices->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::Steering, FCMCDHandler::EObjectType::Other, Params.URL, Params.RequestHeaders, PathwayID, ri);
					break;
				}
			}
			PendingLoadRequests[i]->bAddedCMCDParameters = !RequestedURL.Equals(Params.URL);
			PendingLoadRequests[i]->ResourceRequest->StartGet(PlayerSessionServices);
			RunningLoadRequests.Add(PendingLoadRequests[i]);
			PendingLoadRequests.RemoveAt(i);
			--i;
		}
	}
	// Get the requests that have finished into a separate list.
	TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> FinishedLoadRequests;
	for(int32 i=0; i<RunningLoadRequests.Num(); ++i)
	{
		if (RunningLoadRequests[i]->ResourceRequest->GetHasFinished())
		{
			FinishedLoadRequests.Add(RunningLoadRequests[i]);
			RunningLoadRequests.RemoveAt(i);
			--i;
		}
	}

	// Handle the finished requests.
	for(int32 i=0; i<FinishedLoadRequests.Num(); ++i)
	{
		if (FinishedLoadRequests[i]->ResourceRequest->GetWasCanceled())
		{
			continue;
		}
		// Did we add CMCD query parameters?
		if (FinishedLoadRequests[i]->bAddedCMCDParameters)
		{
			if (FinishedLoadRequests[i]->ResourceRequest->GetRequest())
			{
				IElectraHttpManager::FParams& Params(FinishedLoadRequests[i]->ResourceRequest->GetRequest()->Parameters);
				Params.URL = PlayerSessionServices->GetCMCDHandler()->RemoveParamsFromURL(Params.URL);
			}
			const_cast<HTTP::FConnectionInfo*>(FinishedLoadRequests[i]->ResourceRequest->GetConnectionInfo())->EffectiveURL = PlayerSessionServices->GetCMCDHandler()->RemoveParamsFromURL(FinishedLoadRequests[i]->ResourceRequest->GetConnectionInfo()->EffectiveURL);
		}

		auto MakeVariantMediaPlaylist = [&](const TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>& FinishedReq, bool bGotUpdate) -> void
		{
			TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> MediaPlaylist = FinishedReq->UpdateRequestFor;
			if (!MediaPlaylist.IsValid())
			{
				MediaPlaylist = MakeShared<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>();
				MediaPlaylist->URL = FinishedReq->ResourceRequest->GetURL();
				MediaPlaylist->MultiVariantURLFragmentComponents = MultiVariantURLFragmentComponents;
				MediaPlaylist->ReplayEventParams = ReplayEventParams;
				MediaPlaylist->PlaylistState = FMediaPlaylistAndStateHLS::EPlaylistState::Invalid;
				MediaPlaylist->ResponseDateHeaderTime = FinishedReq->ResponseDateHeaderTime;
				MediaPlaylist->bIsPrimaryPlaylist = FinishedReq->bIsPrimaryPlaylist;
				MediaPlaylist->PlaylistInfo = FinishedReq->PlaylistInfo;
			}

			TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> vp = HandleVariantPlaylist(Now, FinishedReq);
			if (vp.IsValid())
			{
				TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> NewMediaPlaylist;
				if (BuildMediaPlaylist(NewMediaPlaylist, vp, MultiVariantPlaylist))
				{
					MediaPlaylist->SetPlaylist(PlayerSessionServices, NewMediaPlaylist, Now);
					if (bGotUpdate)
					{
						PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastError, FinishedReq->ResourceRequest->GetConnectionInfo(), Playlist::EListType::Variant, Playlist::ELoadType::Update, FinishedReq->Attempt));
					}
					ActivePlaylist->UpdateWithMediaPlaylist(MediaPlaylist, FinishedReq->bIsPrimaryPlaylist, bGotUpdate);
				}
			}
		};


		// Get the date and time from the Date header of the playlist response
		FTimeValue ResponseDateHeaderTime;
		if (const HTTP::FConnectionInfo* ci = FinishedLoadRequests[i]->ResourceRequest->GetConnectionInfo())
		{
			for(auto &hdr : ci->ResponseHeaders)
			{
				if (hdr.Header.Equals(TEXT("Date"), ESearchCase::IgnoreCase))
				{
					if (RFC7231::ParseDateTime(ResponseDateHeaderTime, hdr.Value) && ResponseDateHeaderTime.IsValid())
					{
						FinishedLoadRequests[i]->ResponseDateHeaderTime = ResponseDateHeaderTime;
					}
					break;
				}
			}
		}

		switch(FinishedLoadRequests[i]->LoadType)
		{
			case FLoadRequestHLSPlaylist::ELoadType::Main:
			{
				// Set the clock to the Date header of the playlist response
				if (ResponseDateHeaderTime.IsValid())
				{
					PlayerSessionServices->GetSynchronizedUTCTime()->SetTime(ResponseDateHeaderTime);
				}

				TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> MainPlaylist = HandleMainPlaylist(Now, FinishedLoadRequests[i]);
				if (LastError.IsSet())
				{
					PostError(LastError);
				}
				// When retrying there is no LastError, but also no MainPlaylist either!
				else if (MainPlaylist.IsValid())
				{
					PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastError, FinishedLoadRequests[i]->ResourceRequest->GetConnectionInfo(), Playlist::EListType::Main, Playlist::ELoadType::Initial, FinishedLoadRequests[i]->Attempt));

					if (BuildPlaylist(MainPlaylist, true))
					{
						// Configure content steering now. This allows us to obtain an initial steering manifest before deciding
						// which CDN to use.
						FContentSteeringHandler::FInitialParams csp;
						if (MultiVariantPlaylist->ContentSteeringParams.bHaveContentSteering)
						{
							csp.RootDocumentURL = MultiVariantPlaylist->URL;
							csp.FirstSteeringURL = MultiVariantPlaylist->ContentSteeringParams.SteeringURI;
							csp.InitialDefaultCDN = MultiVariantPlaylist->ContentSteeringParams.PrimaryPathwayId;
							csp.bQueryBeforeStart = MultiVariantPlaylist->ContentSteeringParams.bQueryBeforeStart;
							csp.CustomFirstCDNPrioritization = MultiVariantPlaylist->ContentSteeringParams.CustomInitialSelectionPriority;
							// Without a URL we can use this only to select the PATHWAY to start with.
							csp.bHasContentSteering = !csp.FirstSteeringURL.IsEmpty();
						}
						// If we do not have an initial CDN specified in the multi variant playlist we still
						// need to set up a list of pathways that have been specified wither through explicit
						// PATHWAY-ID attributes on the #EXT-X-STREAM-INF or through generated ones when
						// fallback variants have been detected.
						if (csp.InitialDefaultCDN.IsEmpty())
						{
							csp.bAllowAnyPathwayNames = true;
							for(int32 pwIdx=0; pwIdx<MultiVariantPlaylist->PathwayStreamInfs.Num(); ++pwIdx)
							{
								if (pwIdx)
								{
									csp.InitialDefaultCDN.Append(TEXT(" "));
								}
								csp.InitialDefaultCDN.Append(MultiVariantPlaylist->PathwayStreamInfs[pwIdx]->PathwayID);
							}
						}
						bool bNeedFirstSteering = PlayerSessionServices->GetContentSteeringHandler()->InitialSetup(FContentSteeringHandler::EStreamingProtocol::HLS, csp) &&
												  PlayerSessionServices->GetContentSteeringHandler()->NeedToObtainNewSteeringManifestNow();
						if (!bNeedFirstSteering)
						{
							SetupActivePlaylist();
						}
						else
						{
							// Because there is no guarantee that the initial PATHWAY-ID (if it is given) is actually one that exists
							// we perform a first pathway selection right now.
							FString CurrentPathway = MultiVariantPlaylist->PathwayStreamInfs.Num() ? MultiVariantPlaylist->PathwayStreamInfs[0]->PathwayID : FString(TEXT("."));
							FString NewPathwayId;
							FActiveHLSPlaylist::DeterminePathwayToUse(PlayerSessionServices, NewPathwayId, CurrentPathway, MultiVariantPlaylist);
							PlayerSessionServices->GetContentSteeringHandler()->SetCurrentlyActivePathway(NewPathwayId);
							bIsPreStartSteering = true;
						}
					}
				}
				break;
			}
			case FLoadRequestHLSPlaylist::ELoadType::Steering:
			{
				if (MultiVariantPlaylist.IsValid())
				{
					// Update content steering. It does not matter if this was successful or not.
					HTTP::FConnectionInfo ci;
					if (const HTTP::FConnectionInfo* pci = FinishedLoadRequests[i]->ResourceRequest->GetConnectionInfo())
					{
						ci = *pci;
					}
					FString SteeringJSON;
					TSharedPtrTS<FWaitableBuffer> ResponseBuffer = FinishedLoadRequests[i]->ResourceRequest->GetResponseBuffer();
					if (ResponseBuffer.IsValid() && !StringHelpers::ArrayToString(SteeringJSON, MakeConstArrayView<const uint8>(ResponseBuffer->GetLinearReadData(), ResponseBuffer->GetLinearReadSize())))
					{
						SteeringJSON.Empty();
					}
					PlayerSessionServices->GetContentSteeringHandler()->UpdateWithSteeringServerResponse(SteeringJSON, ci.StatusInfo.HTTPStatus, ci.ResponseHeaders);
					PerformContentSteeringCloning(FinishedLoadRequests[i]->bIsPreStartSteering);

					// If this was the initial steering request we now need to continue with the selection of the first playlists.
					if (FinishedLoadRequests[i]->bIsPreStartSteering)
					{
						SetupActivePlaylist();
					}
					else if (ActivePlaylist.IsValid())
					{
						ActivePlaylist->CheckForPathwaySwitch();
					}
				}
				break;
			}
			case FLoadRequestHLSPlaylist::ELoadType::InitialVariant:
			{
				--NumPendingInitialVariantRequest;
				MakeVariantMediaPlaylist(FinishedLoadRequests[i], false);

				if (NumPendingInitialVariantRequest == 0)
				{
					// Every initially failed variant we report to the ABR to not use.
					TSharedPtrTS<IAdaptiveStreamSelector> ABR = PlayerSessionServices->GetStreamSelector();
					check(ABR.IsValid());
					if (ABR.IsValid())
					{
						for(auto& It : CurrentlyFailedPlaylistRequests)
						{
							IAdaptiveStreamSelector::FDenylistedStream ds;
							ds.AssetUniqueID = It->PlaylistInfo.AssetID;
							ds.AdaptationSetUniqueID = It->PlaylistInfo.AdaptationSetID;
							ds.RepresentationUniqueID = It->PlaylistInfo.RepresentationID;
							ds.CDN = It->PlaylistInfo.PathwayID;
							ABR->MarkStreamAsUnavailable(ds);
						}
					}
					PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastError, FinishedLoadRequests[i]->ResourceRequest->GetConnectionInfo(), Playlist::EListType::Variant, Playlist::ELoadType::Initial, FinishedLoadRequests[i]->Attempt));
				}
				if (!LastError.IsOK())
				{
					PostError(LastError);
				}
				break;
			}
			case FLoadRequestHLSPlaylist::ELoadType::Variant:
			{
				MakeVariantMediaPlaylist(FinishedLoadRequests[i], true);
				if (!LastError.IsOK())
				{
					PostError(LastError);
				}

				break;
			}
			default:
			{
				break;
			}
		}
	}


	// Any new playlist failures?
	if (NewlyFailedPlaylistRequests.Num())
	{
		for(int32 i=0; i<NewlyFailedPlaylistRequests.Num(); ++i)
		{
			if (TSharedPtrTS<IAdaptiveStreamSelector> ABR = PlayerSessionServices->GetStreamSelector())
			{
				IAdaptiveStreamSelector::FDenylistedStream ds;
				ds.AssetUniqueID = NewlyFailedPlaylistRequests[i]->PlaylistInfo.AssetID;
				ds.AdaptationSetUniqueID = NewlyFailedPlaylistRequests[i]->PlaylistInfo.AdaptationSetID;
				ds.RepresentationUniqueID = NewlyFailedPlaylistRequests[i]->PlaylistInfo.RepresentationID;
				ds.CDN = NewlyFailedPlaylistRequests[i]->PlaylistInfo.PathwayID;
				ABR->MarkStreamAsUnavailable(ds);
			}
			CurrentlyFailedPlaylistRequests.Emplace(MoveTemp(NewlyFailedPlaylistRequests[i]));
		}
		NewlyFailedPlaylistRequests.Empty();
	}
	// Go over the blocked playlists and enable them again if they are allowed.
	for(int32 i=0; i<CurrentlyFailedPlaylistRequests.Num(); ++i)
	{
		if (CurrentlyFailedPlaylistRequests[i]->ExecuteAtUTC.IsValid() && Now > CurrentlyFailedPlaylistRequests[i]->ExecuteAtUTC)
		{
			// Remember that this had failed before, so we can track repeated failures over time.
			FFailedPlaylist* FailedBefore = RepeatedlyFailedPlaylistRequests.FindByPredicate([InFailed=CurrentlyFailedPlaylistRequests[i]](const FFailedPlaylist& InElem)
			{
				return InElem.Info.Equals(InFailed->PlaylistInfo);
			});
			bool bDeadForGood = false;
			if (FailedBefore)
			{
				++FailedBefore->NumFailures;
				bDeadForGood = FailedBefore->NumFailures >= 3;
			}
			else
			{
				RepeatedlyFailedPlaylistRequests.Emplace_GetRef().Info = CurrentlyFailedPlaylistRequests[i]->PlaylistInfo;
			}
			if (!bDeadForGood)
			{
				if (TSharedPtrTS<IAdaptiveStreamSelector> ABR = PlayerSessionServices->GetStreamSelector())
				{
					IAdaptiveStreamSelector::FDenylistedStream ds;
					ds.AssetUniqueID = CurrentlyFailedPlaylistRequests[i]->PlaylistInfo.AssetID;
					ds.AdaptationSetUniqueID = CurrentlyFailedPlaylistRequests[i]->PlaylistInfo.AdaptationSetID;
					ds.RepresentationUniqueID = CurrentlyFailedPlaylistRequests[i]->PlaylistInfo.RepresentationID;
					ds.CDN = CurrentlyFailedPlaylistRequests[i]->PlaylistInfo.PathwayID;
					ABR->MarkStreamAsAvailable(ds);
				}
			}
			CurrentlyFailedPlaylistRequests.RemoveAt(i);
			--i;
		}
	}

	// Check if a new steering manifest is needed
	if (PlayerSessionServices->GetContentSteeringHandler()->NeedToObtainNewSteeringManifestNow())
	{
		FString SteeringURL = PlayerSessionServices->GetContentSteeringHandler()->GetFinalSteeringServerRequestURL(PlayerSessionServices->GetContentSteeringHandler()->GetBaseSteeringServerRequestURL());
		PlayerSessionServices->GetContentSteeringHandler()->SetSteeringServerRequestIsPending();
		TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> lr = MakeShared<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>();
		lr->LoadType = FLoadRequestHLSPlaylist::ELoadType::Steering;
		lr->bIsPreStartSteering = bIsPreStartSteering;
		lr->ResourceRequest = MakeShared<FHTTPResourceRequest, ESPMode::ThreadSafe>();
		lr->ResourceRequest->Verb(TEXT("GET")).URL(SteeringURL)
			.ConnectionTimeout(GetTimeoutValue(HLS::OptionKeyPlaylistLoadConnectTimeout, 5000))
			.NoDataTimeout(GetTimeoutValue(HLS::OptionKeyPlaylistLoadNoDataTimeout, 2000));
			//.AllowStaticQuery(IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Playlist);
		PendingLoadRequests.Emplace(MoveTemp(lr));
	}
}

void FPlaylistHandlerHLS::PerformContentSteeringCloning(bool bIsPreStartFetch)
{
	// See if there are clones that need to be created.
	TArray<FContentSteeringHandler::FPathwayCloneEntry> NewClones(PlayerSessionServices->GetContentSteeringHandler()->GetCurrentCloneEntries());
	if (NewClones.Num())
	{
		auto UpdateURL = [&](const FString& InURL, const FContentSteeringHandler::FPathwayCloneEntry& InCloneParam) -> FString
		{
			FURL_RFC3986 updatedURL;
			updatedURL.Parse(InURL);
			if (InCloneParam.Host.Len())
			{
				updatedURL.SetHost(InCloneParam.Host);
			}
			updatedURL.AddOrUpdateQueryParams(InCloneParam.Params);
			FString NewURL = updatedURL.Get();

			// Perform variable substitution on the new URL, just like it would be done
			// on a URL that was originally appeared in the playlist.
			// It does not state in the RFC that this should be done though.
			FRegexMatcher Matcher(VariableSubstitutionPattern(), NewURL);
			TArray<FString> Substitutions;
			while(Matcher.FindNext())
			{
				Substitutions.Emplace(Matcher.GetCaptureGroup(1));
			}
			for(auto& subs : Substitutions)
			{
				for(auto& rpl : MultiVariantPlaylist->VariableSubstitutions)
				{
					if (subs.Equals(rpl.Name))
					{
						NewURL.ReplaceInline(*subs, *rpl.Value, ESearchCase::CaseSensitive);
					}
				}
			}
			return NewURL;
		};

		for(int32 cloneIdx=0; cloneIdx<NewClones.Num(); ++cloneIdx)
		{
			for(int32 pwIdx=0; pwIdx<MultiVariantPlaylist->PathwayStreamInfs.Num(); ++pwIdx)
			{
				auto src = MultiVariantPlaylist->PathwayStreamInfs[pwIdx];
				if (src->PathwayID.Equals(NewClones[cloneIdx].BaseId))
				{
					TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> cln = MakeSharedTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>();
					cln->PathwayID = NewClones[cloneIdx].Id;
					cln->StreamInfs = src->StreamInfs;
					cln->VideoVariantGroups = src->VideoVariantGroups;
					cln->AudioOnlyVariantGroups = src->AudioOnlyVariantGroups;

					TArray<FString> GroupsToClone[4];
					FString CloneSuffix(TEXT("@clone"));
					// Replace the pathway on the cloned stream-inf and collect groups that are referenced.
					for(int32 nSInf=0; nSInf<cln->StreamInfs.Num(); ++nSInf)
					{
						cln->StreamInfs[nSInf].PathwayId = cln->PathwayID;
						if (cln->StreamInfs[nSInf].VideoGroup.Len())
						{
							GroupsToClone[0].AddUnique(cln->StreamInfs[nSInf].VideoGroup);
							cln->StreamInfs[nSInf].VideoGroup.Append(CloneSuffix);
						}
						if (cln->StreamInfs[nSInf].AudioGroup.Len())
						{
							GroupsToClone[1].AddUnique(cln->StreamInfs[nSInf].AudioGroup);
							cln->StreamInfs[nSInf].AudioGroup.Append(CloneSuffix);
						}
						if (cln->StreamInfs[nSInf].SubtitleGroup.Len())
						{
							GroupsToClone[2].AddUnique(cln->StreamInfs[nSInf].SubtitleGroup);
							cln->StreamInfs[nSInf].SubtitleGroup.Append(CloneSuffix);
						}
						if (cln->StreamInfs[nSInf].ClosedCaptionGroup.Len())
						{
							GroupsToClone[3].AddUnique(cln->StreamInfs[nSInf].ClosedCaptionGroup);
							cln->StreamInfs[nSInf].ClosedCaptionGroup.Append(CloneSuffix);
						}
					}
					// Are there HOST and/or PARAMS in the clone description?
					if (NewClones[cloneIdx].Host.Len() || NewClones[cloneIdx].Params.Num())
					{
						for(int32 nSInf=0; nSInf<cln->StreamInfs.Num(); ++nSInf)
						{
							cln->StreamInfs[nSInf].URI = UpdateURL(cln->StreamInfs[nSInf].URI, NewClones[cloneIdx]);
						}
					}
					// Check if there are PER-VARIANT-URIS in the clone description
					for(int32 nSInf=0; nSInf<cln->StreamInfs.Num(); ++nSInf)
					{
						if (cln->StreamInfs[nSInf].StableVariantId.Len())
						{
							for(auto& variantURI : NewClones[cloneIdx].PerVariantURIs)
							{
								if (variantURI.Key.Equals(cln->StreamInfs[nSInf].StableVariantId))
								{
									cln->StreamInfs[nSInf].URI = variantURI.Value;
								}
							}
						}
					}

					// Need to clone groups?
					for(int32 nCloneGrpIdx=0; nCloneGrpIdx<UE_ARRAY_COUNT(GroupsToClone); ++nCloneGrpIdx)
					{
						for(auto& OrgGrpName : GroupsToClone[nCloneGrpIdx])
						{
							// Locate the group requiring cloning
							for(int32 nOrgGrpIdx=0; nOrgGrpIdx<MultiVariantPlaylist->RenditionGroupsOfType[nCloneGrpIdx].Num(); ++nOrgGrpIdx)
							{
								if (MultiVariantPlaylist->RenditionGroupsOfType[nCloneGrpIdx][nOrgGrpIdx].GroupID.Equals(OrgGrpName))
								{
									// Clone the group
									FMultiVariantPlaylistHLS::FRenditionGroup ClonedGroup(MultiVariantPlaylist->RenditionGroupsOfType[nCloneGrpIdx][nOrgGrpIdx]);
									ClonedGroup.GroupID.Append(CloneSuffix);
									// Go over the renditions and update their URLs
									for(auto& ClonedRendition : ClonedGroup.Renditions)
									{
										if (ClonedRendition.URI.Len())
										{
											ClonedRendition.URI = UpdateURL(ClonedRendition.URI, NewClones[cloneIdx]);
										}
										// Check if the rendition has a stable rendition id for which there is a new dedicated URL in the clone parameters
										if (ClonedRendition.StableRenditionId.Len())
										{
											for(auto& renditionURI : NewClones[cloneIdx].PerRenditionURIs)
											{
												if (renditionURI.Key.Equals(ClonedRendition.StableRenditionId))
												{
													ClonedRendition.URI = renditionURI.Value;
												}
											}
										}
									}
									// Add the cloned group to the list.
									MultiVariantPlaylist->RenditionGroupsOfType[nCloneGrpIdx].Emplace(MoveTemp(ClonedGroup));
									break;
								}
							}
						}
					}

					// The metadata is more involved and cloning with patching is too much hassle.
					// If this is the very first application of a clone on pre-start steering then
					// we do not need to do anything since the metadata will be created in the next step.
					// Otherwise we create the metadata for the clone now.
					if (!bIsPreStartFetch && ActivePlaylist.IsValid())
					{
						FErrorDetail pwErr = ActivePlaylist->PreparePathway(cln, MultiVariantPlaylist);
						check(pwErr.IsOK());(void)pwErr;
					}

					// Take note that we have created the clone, so we do not do this repeatedly.
					PlayerSessionServices->GetContentSteeringHandler()->CreatedClone(cln->PathwayID);

					// Add the clone to the pathway list
					MultiVariantPlaylist->PathwayStreamInfs.Emplace(MoveTemp(cln));
					break;
				}
			}
		}
	}
}


void FPlaylistHandlerHLS::LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams)
{
	FURL_RFC3986 UrlParser;
	UrlParser.Parse(InURL);
	MultiVariantPlaylistEffectiveURL = UrlParser.Get(true, false);
	ReplayEventParams = InReplayEventParams;
	FURL_RFC3986::GetQueryParams(MultiVariantURLFragmentComponents, UrlParser.GetFragment(), false);
	LoadMainPlaylist(MultiVariantPlaylistEffectiveURL);
}

void FPlaylistHandlerHLS::LoadMainPlaylist(const FString& InUrl)
{
	TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> lr = MakeShared<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>();
	lr->LoadType = FLoadRequestHLSPlaylist::ELoadType::Main;
	lr->ResourceRequest = MakeShared<FHTTPResourceRequest, ESPMode::ThreadSafe>();
	lr->ResourceRequest->Verb(TEXT("GET")).URL(InUrl)
		.ConnectionTimeout(GetTimeoutValue(HLS::OptionKeyPlaylistLoadConnectTimeout, 5000))
		.NoDataTimeout(GetTimeoutValue(HLS::OptionKeyPlaylistLoadNoDataTimeout, 2000))
		.AllowStaticQuery(IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Playlist);
	PendingLoadRequests.Emplace(MoveTemp(lr));
}

TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> FPlaylistHandlerHLS::CreateMediaPlaylistLoadRequest(const TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe>& InPlaylist)
{
	FString URL(InPlaylist->URL);
	// Reload from the effective URL (after possible redirections) we got from the previous load, if available.
	if (auto mp = InPlaylist->GetPlaylist())
	{
		URL = mp->URL;
	}

	TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> lr = MakeShared<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>();
	lr->LoadType = FLoadRequestHLSPlaylist::ELoadType::Variant;
	lr->bIsPrimaryPlaylist = InPlaylist->bIsPrimaryPlaylist;
	lr->PlaylistInfo = InPlaylist->PlaylistInfo;
	lr->ResourceRequest = MakeShared<FHTTPResourceRequest, ESPMode::ThreadSafe>();
	lr->ResourceRequest->Verb(TEXT("GET")).URL(URL)
		.ConnectionTimeout(GetTimeoutValue(HLS::OptionKeyPlaylistLoadConnectTimeout, 5000))
		.NoDataTimeout(GetTimeoutValue(HLS::OptionKeyPlaylistLoadNoDataTimeout, 2000))
		.AllowStaticQuery(IAdaptiveStreamingPlayerResourceRequest::EPlaybackResourceType::Playlist);
	return lr;
}

void FPlaylistHandlerHLS::RepeatPlaylistLoad(TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InRequestToRepeat, const FTimeValue& InAtUTC)
{
	TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> lr = MakeShared<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>();
	lr->LoadType = InRequestToRepeat->LoadType;
	lr->bIsPrimaryPlaylist = InRequestToRepeat->bIsPrimaryPlaylist;
	lr->PlaylistInfo = InRequestToRepeat->PlaylistInfo;
	lr->ResourceRequest = MakeShared<FHTTPResourceRequest, ESPMode::ThreadSafe>();
	lr->ResourceRequest->Verb(TEXT("GET")).URL(InRequestToRepeat->ResourceRequest->GetURL())
		.ConnectionTimeout(InRequestToRepeat->ResourceRequest->GetConnectionTimeout())
		.NoDataTimeout(InRequestToRepeat->ResourceRequest->GetNoDataTimeout())
		.AllowStaticQuery(InRequestToRepeat->ResourceRequest->GetStaticQuery());
	lr->ExecuteAtUTC = InAtUTC;
	lr->Attempt = InRequestToRepeat->Attempt + 1;
	PendingLoadRequests.Emplace(MoveTemp(lr));
}

void FPlaylistHandlerHLS::RetryInitialPlaylistWith(TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InRequestToRepeat, bool bIsPrimary)
{
	TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> lr = MakeShared<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>();
	lr->LoadType = FLoadRequestHLSPlaylist::ELoadType::InitialVariant;
	lr->bIsPrimaryPlaylist = bIsPrimary;
	lr->PlaylistInfo = InRequestToRepeat->PlaylistInfo;
	lr->ResourceRequest = MakeShared<FHTTPResourceRequest, ESPMode::ThreadSafe>();
	lr->ResourceRequest->Verb(TEXT("GET")).URL(InRequestToRepeat->ResourceRequest->GetURL())
		.ConnectionTimeout(InRequestToRepeat->ResourceRequest->GetConnectionTimeout())
		.NoDataTimeout(InRequestToRepeat->ResourceRequest->GetNoDataTimeout())
		.AllowStaticQuery(InRequestToRepeat->ResourceRequest->GetStaticQuery());
	PendingLoadRequests.Emplace(MoveTemp(lr));
}


FString FPlaylistHandlerHLS::GetURL() const
{
	return MultiVariantPlaylistEffectiveURL;
}

TSharedPtrTS<IManifest> FPlaylistHandlerHLS::GetManifest()
{
	return ActivePlaylist;
}


FTimeValue FPlaylistHandlerHLS::GetTimeoutValue(const FName& InOptionName, int32 InDefaultValueMillisec)
{
	return PlayerSessionServices->GetOptionValue(InOptionName).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(InDefaultValueMillisec));
}

TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> FPlaylistHandlerHLS::HandleMainPlaylist(const FTimeValue& Now, TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InRequest)
{
	// Notify download completion.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistDownloadMessage::Create(InRequest->ResourceRequest->GetConnectionInfo(), Playlist::EListType::Main, Playlist::ELoadType::Initial, InRequest->Attempt));

	FString PlaylistURL = InRequest->ResourceRequest->GetURL();
	// Failure?
	int32 ErrorCode = InRequest->ResourceRequest->GetError();
	if (ErrorCode)
	{
		int32 RetryDelayMS = -1;
		// Whether or not and how often we retry depends on the type of error.
		if (ErrorCode < 100 && InRequest->Attempt < 3)
		{
			RetryDelayMS = 500 * (1 << InRequest->Attempt);
		}
		else if (ErrorCode >= 502 && ErrorCode <= 504 && InRequest->Attempt < 2)
		{
			RetryDelayMS = 1000 * (1 << InRequest->Attempt);
		}
		if (RetryDelayMS < 0)
		{
			LastError = PostError(FString::Printf(TEXT("Failed to download playlist \"%s\" (%s)"), *PlaylistURL, *InRequest->ResourceRequest->GetErrorString()), HLS::ERRCODE_MAIN_PLAYLIST_DOWNLOAD_FAILED);
			return nullptr;
		}
		LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Failed to download playlist \"%s\" (%s), retrying..."), *PlaylistURL, *InRequest->ResourceRequest->GetErrorString()));
		RepeatPlaylistLoad(InRequest, Now + FTimeValue().SetFromMilliseconds(RetryDelayMS));
		return nullptr;
	}
	TSharedPtrTS<FWaitableBuffer> ResponseBuffer = InRequest->ResourceRequest->GetResponseBuffer();
	check(ResponseBuffer.IsValid());
	if (!ResponseBuffer.IsValid())
	{
		LastError = PostError(FString::Printf(TEXT("Failed to download playlist \"%s\""), *PlaylistURL), HLS::ERRCODE_MAIN_PLAYLIST_DOWNLOAD_FAILED);
		return nullptr;
	}
	FString M3U8;
	if (!StringHelpers::ArrayToString(M3U8,	MakeConstArrayView<const uint8>(ResponseBuffer->GetLinearReadData(), ResponseBuffer->GetLinearReadSize())))
	{
		LastError = PostError(FString::Printf(TEXT("Failed to parse playlist \"%s\""), *PlaylistURL), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
		return nullptr;
	}

	// Remember the effective URL after redirections.
	MultiVariantPlaylistEffectiveURL = InRequest->ResourceRequest->GetConnectionInfo()->EffectiveURL;
	TArray<FElectraHTTPStreamHeader> Headers;
	for(auto &hdr : InRequest->ResourceRequest->GetConnectionInfo()->ResponseHeaders)
	{
		Headers.Add(hdr);
	}
	// Create a parser and parse the response.
	TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> MainPlaylist = MakeShared<FPlaylistParserHLS, ESPMode::ThreadSafe>();
	LastError = MainPlaylist->Parse(M3U8, MultiVariantPlaylistEffectiveURL, Headers);
	if (LastError.IsError())
	{
		MainPlaylist.Reset();
	}
	return MainPlaylist;
}

TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> FPlaylistHandlerHLS::HandleVariantPlaylist(const FTimeValue& Now, TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe> InRequest)
{
	// Notify download completion.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistDownloadMessage::Create(InRequest->ResourceRequest->GetConnectionInfo(), Playlist::EListType::Variant, InRequest->UpdateRequestFor.IsValid() ? Playlist::ELoadType::Update : Playlist::ELoadType::Initial, InRequest->Attempt));

	FString PlaylistURL = InRequest->ResourceRequest->GetURL();
	// Failure?
	int32 ErrorCode = InRequest->ResourceRequest->GetError();
	if (ErrorCode)
	{
		// Get all variant load requests for this type.
		TArray<TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>> AllLoadRequests;
		if (ActivePlaylist.IsValid())
		{
			ActivePlaylist->GetAllMediaPlaylistLoadRequests(AllLoadRequests, InRequest->PlaylistInfo.StreamType);
		}
		// Remove our failed request from the list.
		AllLoadRequests.RemoveAll([&](const TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>& InElement)
		{
			return InElement->PlaylistInfo.RepresentationID.Equals(InRequest->PlaylistInfo.RepresentationID) && InElement->PlaylistInfo.PathwayID.Equals(InRequest->PlaylistInfo.PathwayID);
		});
		// Then remove all that already failed.
		AllLoadRequests.RemoveAll([&](const TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>& InElement)
		{
			for(auto& It : CurrentlyFailedPlaylistRequests)
			{
				if (InElement->PlaylistInfo.RepresentationID.Equals(It->PlaylistInfo.RepresentationID) && InElement->PlaylistInfo.PathwayID.Equals(It->PlaylistInfo.PathwayID))
				{
					return true;
				}
			}
			return false;
		});
		// Sort the remaining ones by descending bandwidth.
		AllLoadRequests.Sort([](const TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>& a, const TSharedPtr<FLoadRequestHLSPlaylist, ESPMode::ThreadSafe>& b)
		{
			return a->PlaylistInfo.RepresentationBandwidth > b->PlaylistInfo.RepresentationBandwidth;
		});
		// Find one with a smaller bandwidth than the one that failed.
		int32 Idx=-1;
		for(int32 i=0; i<AllLoadRequests.Num(); ++i)
		{
			if (AllLoadRequests[i]->PlaylistInfo.RepresentationBandwidth < InRequest->PlaylistInfo.RepresentationBandwidth)
			{
				Idx = i;
				break;
			}
		}
		// If there is none, go one step up.
		for(int32 i=AllLoadRequests.Num()-1; Idx<0 && i>=0; --i)
		{
			if (AllLoadRequests[i]->PlaylistInfo.RepresentationBandwidth > InRequest->PlaylistInfo.RepresentationBandwidth)
			{
				Idx = i;
				break;
			}
		}

		// If this was an initial variant request we will try to switch to a different variant if there is one.
		if (InRequest->LoadType == FLoadRequestHLSPlaylist::ELoadType::InitialVariant && Idx >= 0)
		{
			// Allow this playlist again in 10 seconds
			InRequest->ExecuteAtUTC = Now + FTimeValue().SetFromSeconds(10.0);
			// Add the current failed request to the list now.
			CurrentlyFailedPlaylistRequests.Emplace(InRequest);
			LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Failed to download playlist \"%s\" (%s), switching to \"%s\""), *PlaylistURL, *InRequest->ResourceRequest->GetErrorString(), *AllLoadRequests[Idx]->ResourceRequest->GetURL()));
			RetryInitialPlaylistWith(AllLoadRequests[Idx], InRequest->bIsPrimaryPlaylist);
			++NumPendingInitialVariantRequest;
			return nullptr;
		}


		int32 RetryDelayMS = -1;
		double BlockForSeconds = -1.0;
		// If this is a Live playlist update we don't have any time to spend on retries.
		if (InRequest->UpdateRequestFor.IsValid())
		{
			InRequest->UpdateRequestFor->LoadFailed();
			// If it's a recoverable error like a connection failure, or an intermittent server failure
			// we block this playlist for a while. Otherwise it gets blocked for good (eg. with a 404 response)
			bool bMaybeRetry = ErrorCode < 100 || (ErrorCode >= 502 && ErrorCode <= 504);
			if (bMaybeRetry)
			{
				BlockForSeconds = 20.0;
			}
		}
		else
		{
			// Whether or not and how often we retry depends on the type of error.
			if (ErrorCode < 100 && InRequest->Attempt < 3)
			{
				RetryDelayMS = 500 * (1 << InRequest->Attempt);
			}
			else if (ErrorCode >= 502 && ErrorCode <= 504 && InRequest->Attempt < 2)
			{
				RetryDelayMS = 1000 * (1 << InRequest->Attempt);
			}
			BlockForSeconds = 60.0;
		}
		if (RetryDelayMS < 0)
		{
			// Block this playlist for a while.
			if (BlockForSeconds > 0.0)
			{
				InRequest->ExecuteAtUTC = Now + FTimeValue().SetFromSeconds(BlockForSeconds);
			}
			else
			{
				InRequest->ExecuteAtUTC.SetToInvalid();
			}
			NewlyFailedPlaylistRequests.Emplace(InRequest);
			// If that was the last remaining candidate we fail.
			if (Idx < 0)
			{
				LastError = PostError(FString::Printf(TEXT("Failed to download playlist \"%s\" (%s)"), *PlaylistURL, *InRequest->ResourceRequest->GetErrorString()), HLS::ERRCODE_MAIN_PLAYLIST_DOWNLOAD_FAILED);
			}
			return nullptr;
		}
		LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Failed to download playlist \"%s\" (%s), retrying..."), *PlaylistURL, *InRequest->ResourceRequest->GetErrorString()));
		RepeatPlaylistLoad(InRequest, Now + FTimeValue().SetFromMilliseconds(RetryDelayMS));
		NumPendingInitialVariantRequest += InRequest->LoadType == FLoadRequestHLSPlaylist::ELoadType::InitialVariant ? 1 : 0;
		return nullptr;
	}
	TSharedPtrTS<FWaitableBuffer> ResponseBuffer = InRequest->ResourceRequest->GetResponseBuffer();
	check(ResponseBuffer.IsValid());
	if (!ResponseBuffer.IsValid())
	{
		LastError = PostError(FString::Printf(TEXT("Failed to download playlist \"%s\""), *PlaylistURL), HLS::ERRCODE_MAIN_PLAYLIST_DOWNLOAD_FAILED);
		return nullptr;
	}
	FString M3U8;
	if (!StringHelpers::ArrayToString(M3U8,	MakeConstArrayView<const uint8>(ResponseBuffer->GetLinearReadData(), ResponseBuffer->GetLinearReadSize())))
	{
		LastError = PostError(FString::Printf(TEXT("Failed to parse playlist \"%s\""), *PlaylistURL), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
		return nullptr;
	}

	// Create a parser and parse the response.
	TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> VariantPlaylist;
	VariantPlaylist = MakeShared<FPlaylistParserHLS, ESPMode::ThreadSafe>();
	TArray<FElectraHTTPStreamHeader> Headers;
	for(auto &hdr : InRequest->ResourceRequest->GetConnectionInfo()->ResponseHeaders)
	{
		Headers.Add(hdr);
	}
	LastError = VariantPlaylist->Parse(M3U8, InRequest->ResourceRequest->GetConnectionInfo()->EffectiveURL, Headers);
	if (LastError.IsError())
	{
		VariantPlaylist.Reset();
	}
	return VariantPlaylist;
}

bool FPlaylistHandlerHLS::BuildPlaylist(TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> InPlaylist, bool bIsMainPlaylist)
{
	// A playlist can be a multivariant playlist or a variant playlist, but not both at the same time.
	if (InPlaylist->IsMultiVariantPlaylist() && InPlaylist->IsVariantPlaylist())
	{
		LastError = PostError(FString::Printf(TEXT("Playlist contains both variant and multivariant tags!")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
		return false;
	}
	if (InPlaylist->IsMultiVariantPlaylist())
	{
		if (!bIsMainPlaylist)
		{
			LastError = PostError(FString::Printf(TEXT("Only the first loaded playlist is expected to be a multivariant playlist!")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
			return false;
		}
		if (BuildMultiVariantPlaylist(MultiVariantPlaylist, InPlaylist))
		{
			FillInMissingInformation(MultiVariantPlaylist, EFillInOptions::All);
			GroupVariantStreamsByPathways(MultiVariantPlaylist);
			AssignInternalVariantStreamIDs(MultiVariantPlaylist);
			GroupVariantStreamsByVideoProperties(MultiVariantPlaylist);
			GroupAudioOnlyVariantStreams(MultiVariantPlaylist);
			return true;
		}
	}
	else if (bIsMainPlaylist)
	{
		// This is the first loaded playlist and it is not a multivariant playlist.
		// In order to handle everything the same way we now construct a basic multivariant playlist
		// as a placeholder.
		LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("The playlist \"%s\" is not a multivariant playlist. Vital information is missing and playback is not guaranteed to work!"), *InPlaylist->GetURL()));

		TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> NewVariantPlaylist = MakeShared<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe>();
		NewVariantPlaylist->URL = InPlaylist->GetURL();
		FMultiVariantPlaylistHLS::FStreamInf& si = NewVariantPlaylist->InitialStreamInfs.Emplace_GetRef();
		si.URI = InPlaylist->GetURL();
		si.Bandwidth = 500000;

		MultiVariantPlaylist = MoveTemp(NewVariantPlaylist);
		FillInMissingInformation(MultiVariantPlaylist, EFillInOptions::All);
		GroupVariantStreamsByPathways(MultiVariantPlaylist);
		AssignInternalVariantStreamIDs(MultiVariantPlaylist);
		GroupVariantStreamsByVideoProperties(MultiVariantPlaylist);
		GroupAudioOnlyVariantStreams(MultiVariantPlaylist);
		return true;
	}
	return false;
}

bool FPlaylistHandlerHLS::PrepareSubstitutionVariables(TArray<FPlaylistParserHLS::FVariableSubstitution>& OutVariableSubstitutions, TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> InPlaylist, const TArray<FPlaylistParserHLS::FVariableSubstitution>& InParentSubstitutions)
{
	// We put the variable name back in the `{$ }` bracket because this makes it easier and
	// faster in the actual substitution later.
	auto AsBracketedSubstitution = [](const FString& InString) -> FString
	{
		return FString::Printf(TEXT("{$%s}"), *InString);
	};
	const TArray<TUniquePtr<FPlaylistParserHLS::FElement>>& Elements = InPlaylist->GetElements();
	for(int32 i=0; i<Elements.Num(); ++i)
	{
		if (Elements[i]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_DEFINE)
		{
			// First validate
			if (!InPlaylist->Validate_EXT_X_DEFINE(LastError, Elements[i]))
			{
				return false;
			}

			// `NAME`?
			if (const FPlaylistParserHLS::FAttribute* NameAttr = Elements[i]->GetAttribute(TEXT("NAME")))
			{
				// Same variable is not allowed twice.
				FString foo(AsBracketedSubstitution(NameAttr->Value));
				if (OutVariableSubstitutions.ContainsByPredicate([&foo](const FPlaylistParserHLS::FVariableSubstitution& e){return e.Name.Equals(foo);}))
				{
					LastError = PostError(FString::Printf(TEXT("Encountered EXT-X-DEFINE for same variable")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
					return false;
				}
				if (const FPlaylistParserHLS::FAttribute* Value = Elements[i]->GetAttribute(TEXT("VALUE")))
				{
					FPlaylistParserHLS::FVariableSubstitution vs {AsBracketedSubstitution(NameAttr->Value), Value->Value};
					OutVariableSubstitutions.Emplace(MoveTemp(vs));
				}
				else
				{
					LastError = PostError(FString::Printf(TEXT("EXT-X-DEFINE has no VALUE")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
					return false;
				}
			}
			// `IMPORT`?
			else if (const FPlaylistParserHLS::FAttribute* ImportAttr = Elements[i]->GetAttribute(TEXT("IMPORT")))
			{
				const FPlaylistParserHLS::FVariableSubstitution* ParentVariable = InParentSubstitutions.FindByKey(AsBracketedSubstitution(ImportAttr->Value));
				if (ParentVariable)
				{
					FPlaylistParserHLS::FVariableSubstitution vs {AsBracketedSubstitution(ImportAttr->Value), ParentVariable->Value};
					OutVariableSubstitutions.Emplace(MoveTemp(vs));
				}
				else
				{
					LastError = PostError(FString::Printf(TEXT("EXT-X-DEFINE references non-existing IMPORT parameter")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
					return false;
				}
			}
			// `QUERYPARAM`?
			else if (const FPlaylistParserHLS::FAttribute* QueryParamAttr = Elements[i]->GetAttribute(TEXT("QUERYPARAM")))
			{
				FString qp;
				if (InPlaylist->GetQueryParam(qp, QueryParamAttr->Value) && !qp.IsEmpty())
				{
					FPlaylistParserHLS::FVariableSubstitution vs {AsBracketedSubstitution(QueryParamAttr->Value), MoveTemp(qp)};
					OutVariableSubstitutions.Emplace(MoveTemp(vs));
				}
				else
				{
					LastError = PostError(FString::Printf(TEXT("EXT-X-DEFINE references non-existing query parameter")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
					return false;
				}
			}
		}
	}
	return true;
}

bool FPlaylistHandlerHLS::ParseStartTime(FStartTimeHLS& InOutStartTime, const TUniquePtr<FPlaylistParserHLS::FElement>& InElement)
{
	const TArray<FPlaylistParserHLS::FAttribute>& Attributes = InElement->AttributeList;
	bool bGotOffset = false;
	for(int32 i=0,iMax=Attributes.Num(); i<iMax; ++i)
	{
		const FString& AttrName(Attributes[i].Name);
		// TIME-OFFSET ?
		if (AttrName.Equals(TEXT("TIME-OFFSET")))
		{
			bGotOffset = true;
			InOutStartTime.Offset.SetFromTimeFraction(FTimeFraction().SetFromFloatString(Attributes[i].GetValue()));
		}
		// PRECISE ?
		else if (AttrName.Equals(TEXT("PRECISE")))
		{
			InOutStartTime.bPrecise = Attributes[i].GetValue().Equals(TEXT("YES"));
		}
	}
	if (!bGotOffset)
	{
		LastError = PostError(FString::Printf(TEXT("EXT-X-START is missing required TIME-OFFSET attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
		return false;
	}
	return true;
}

bool FPlaylistHandlerHLS::ParseServerControl(FServerControlHLS& InOutServerControl, const TUniquePtr<FPlaylistParserHLS::FElement>& InElement)
{
	const TArray<FPlaylistParserHLS::FAttribute>& Attributes = InElement->AttributeList;
	for(int32 i=0,iMax=Attributes.Num(); i<iMax; ++i)
	{
		const FString& AttrName(Attributes[i].Name);
		// CAN-SKIP-UNTIL ?
		if (AttrName.Equals(TEXT("CAN-SKIP-UNTIL")))
		{
			InOutServerControl.CanSkipUntil.SetFromTimeFraction(FTimeFraction().SetFromFloatString(Attributes[i].GetValue()));
		}
		// CAN-SKIP-DATERANGES ?
		else if (AttrName.Equals(TEXT("CAN-SKIP-DATERANGES")))
		{
			InOutServerControl.bCanSkipDateRanges = Attributes[i].GetValue().Equals(TEXT("YES"));
		}
		// HOLD-BACK ?
		else if (AttrName.Equals(TEXT("HOLD-BACK")))
		{
			InOutServerControl.HoldBack.SetFromTimeFraction(FTimeFraction().SetFromFloatString(Attributes[i].GetValue()));
		}
		// PART-HOLD-BACK ?
		else if (AttrName.Equals(TEXT("PART-HOLD-BACK")))
		{
			InOutServerControl.PartHoldBack.SetFromTimeFraction(FTimeFraction().SetFromFloatString(Attributes[i].GetValue()));
		}
		// CAN-BLOCK-RELOAD ?
		else if (AttrName.Equals(TEXT("CAN-BLOCK-RELOAD")))
		{
			InOutServerControl.bCanBlockReload = Attributes[i].GetValue().Equals(TEXT("YES"));
		}
	}
	return true;
}

bool FPlaylistHandlerHLS::BuildMediaPlaylist(TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe>& OutMediaPlaylist, TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> InPlaylist, TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	TSharedPtr<FMediaPlaylistHLS, ESPMode::ThreadSafe> NewMediaPlaylist = MakeShared<FMediaPlaylistHLS, ESPMode::ThreadSafe>();
	NewMediaPlaylist->URL = InPlaylist->GetURL();
	NewMediaPlaylist->ParsedURL.Parse(InPlaylist->GetURL());
	// Inherit server control from the multi variant playlist
	NewMediaPlaylist->ServerControl = InMultiVariantPlaylist->ServerControl;
	// Inherit start time offset from the multi variant playlist
	NewMediaPlaylist->StartTime = InMultiVariantPlaylist->StartTime;

	// First process all EXT-X-DEFINE elements to set up the values for the variable substitutions.
	TArray<FPlaylistParserHLS::FVariableSubstitution> EmptySubstitutions;
	if (!PrepareSubstitutionVariables(NewMediaPlaylist->VariableSubstitutions, InPlaylist, InMultiVariantPlaylist->VariableSubstitutions))
	{
		return false;
	}

	NewMediaPlaylist->Duration.SetToZero();
	NewMediaPlaylist->bHasEndList = InPlaylist->HasEndList();
	NewMediaPlaylist->bHasProgramDateTime = InPlaylist->HasProgramDateTime();
	bool bHasValidDateTime = false;

	TSharedPtr<FMediaEncryptionHLS, ESPMode::ThreadSafe> CurrentKeys, ActiveKeys, ActivatedKeysFrom;
	TSharedPtr<FMediaInitSegment, ESPMode::ThreadSafe> CurrentInitSegment, ActiveInitSegment;
	TOptional<FMediaByteRangeHLS> CurrentByteRange;
	TOptional<FTimeValue> CurrentProgramDateTime;
	FTimeValue LastKnownProgramDateTime;

	bool bNextIsDiscontinuity = false;
	bool bNextIsGap = false;

	auto ParseByteRange = [](const FString& InRange) -> FMediaByteRangeHLS
	{
		FMediaByteRangeHLS br;
		int32 AtIndex;
		// Offset given?
		if (InRange.FindChar(TCHAR('@'), AtIndex))
		{
			LexFromString(br.NumBytes, *(InRange.Mid(0, AtIndex)));
			LexFromString(br.Offset, *(InRange.Mid(AtIndex + 1)));
		}
		else
		{
			// No offset.
			LexFromString(br.NumBytes, *InRange);
		}
		return br;
	};

	auto ActivateCurrentKeys = [&]() -> void
	{
		if (CurrentKeys.IsValid() && CurrentKeys != ActivatedKeysFrom)
		{
			ActivatedKeysFrom = CurrentKeys;
			ActiveKeys = MakeShared<FMediaEncryptionHLS, ESPMode::ThreadSafe>();
			*ActiveKeys = *CurrentKeys;
			CurrentKeys.Reset();
		}
	};

	FTimeValue LongestMediaSegmentDuration;
	LongestMediaSegmentDuration.SetToZero();
	// Then process all other elements.
	const TArray<TUniquePtr<FPlaylistParserHLS::FElement>>& Elements = InPlaylist->GetElements();
	for(int32 nElem=0; nElem<Elements.Num(); ++nElem)
	{
		// EXTINF ?
		if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXTINF)
		{
			FMediaSegmentHLS& Segment = NewMediaPlaylist->MediaSegments.Emplace_GetRef();

			ActivateCurrentKeys();
			Segment.InitSegment = ActiveInitSegment;
			Segment.Encryption = ActiveKeys;
			if (!Elements[nElem]->URI.GetValue(Segment.URL, NewMediaPlaylist->VariableSubstitutions))
			{
				LastError = PostError(FString::Printf(TEXT("EXTINF has a bad URI")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}
			if (CurrentByteRange.IsSet())
			{
				/*
					If o is not present, a previous Media Segment MUST appear in the
					Playlist file and MUST be a sub-range of the same media resource, or
					the Media Segment is undefined and the client MUST fail to parse the
					Playlist.
				*/
				if (CurrentByteRange.GetValue().Offset < 0)
				{
					for(int32 prvIdx=NewMediaPlaylist->MediaSegments.Num()-2; prvIdx>=0; --prvIdx)
					{
						if (NewMediaPlaylist->MediaSegments[prvIdx].URL.Equals(Segment.URL))
						{
							if (NewMediaPlaylist->MediaSegments[prvIdx].ByteRange.NumBytes >= 0 && NewMediaPlaylist->MediaSegments[prvIdx].ByteRange.Offset >= 0)
							{
								CurrentByteRange.GetValue().Offset = NewMediaPlaylist->MediaSegments[prvIdx].ByteRange.Offset + NewMediaPlaylist->MediaSegments[prvIdx].ByteRange.NumBytes;
							}
							break;
						}
					}
					// No earlier segment found?
					if (NewMediaPlaylist->MediaSegments.Num() > 1)
					{
						if (CurrentByteRange.GetValue().Offset < 0)
						{
							LastError = PostError(FString::Printf(TEXT("EXT-X-BYTERANGE has a bad BYTERANGE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}
					}
					else
					{
						CurrentByteRange.GetValue().Offset = 0;
					}
				}
				Segment.ByteRange = CurrentByteRange.GetValue();
			}
			Segment.Duration.SetFromTimeFraction(FTimeFraction().SetFromFloatString(Elements[nElem]->ElementValue.GetValue()));

			// Track the longest segment duration to see if it aligns with the #EXT-X-TARGET-DURATION
			if (Segment.Duration > LongestMediaSegmentDuration)
			{
				LongestMediaSegmentDuration = Segment.Duration;
			}

			if (NewMediaPlaylist->bHasProgramDateTime)
			{
				/*
					If the first EXT-X-PROGRAM-DATE-TIME tag in a Playlist appears after
					one or more Media Segment URIs, the client SHOULD extrapolate
					backward from that tag (using EXTINF durations and/or media
					timestamps) to associate dates with those segments.  To associate a
					date with any other Media Segment that does not have an EXT-X-
					PROGRAM-DATE-TIME tag applied to it directly, the client SHOULD
					extrapolate forward from the last EXT-X-PROGRAM-DATE-TIME tag
					appearing before that segment in the Playlist.
				*/
				if (CurrentProgramDateTime.IsSet())
				{
					Segment.ProgramDateTime = CurrentProgramDateTime.GetValue();
					// Check backwards
					for(int32 prvIdx=NewMediaPlaylist->MediaSegments.Num()-2; prvIdx>=0; --prvIdx)
					{
						if (NewMediaPlaylist->MediaSegments[prvIdx].ProgramDateTime.IsValid())
						{
							FTimeValue NextExpectedFromPrevious = NewMediaPlaylist->MediaSegments[prvIdx].ProgramDateTime + NewMediaPlaylist->MediaSegments[prvIdx].Duration;
							FTimeValue Diff = Segment.ProgramDateTime - NextExpectedFromPrevious;
							// As a special case check if the same EXT-X-PROGRAM-DATE-TIME from the previous segment is given again.
							// This is of course incorrect but may happen with bad tools.
							if (LastKnownProgramDateTime.IsValid() && Segment.ProgramDateTime == LastKnownProgramDateTime)
							{
								// Extrapolate this segment forward from the previous.
								Segment.ProgramDateTime = NewMediaPlaylist->MediaSegments[NewMediaPlaylist->MediaSegments.Num()-2].ProgramDateTime + NewMediaPlaylist->MediaSegments[NewMediaPlaylist->MediaSegments.Num()-2].Duration;
							}
							else if (Diff.Abs() > HLS::kProgramDateTimeGapThreshold)
							{
								if (!bNextIsDiscontinuity)
								{
									LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("%s detected on timeline. %#.3f seconds between segments\n"), Diff<FTimeValue::GetZero()?TEXT("Overlap"):TEXT("Gap"), Diff.GetAsSeconds()));
								}
							}
							break;
						}
						else
						{
							// Extrapolate backwards.
							NewMediaPlaylist->MediaSegments[prvIdx].ProgramDateTime = NewMediaPlaylist->MediaSegments[prvIdx+1].ProgramDateTime - NewMediaPlaylist->MediaSegments[prvIdx].Duration;
						}
					}
					LastKnownProgramDateTime = CurrentProgramDateTime.GetValue();
				}
				else if (NewMediaPlaylist->MediaSegments.Num() > 1)
				{
					// We have to start somewhere. If the previous segment was the first and did not have a valid time, set it to zero.
					if (!NewMediaPlaylist->MediaSegments[NewMediaPlaylist->MediaSegments.Num()-2].ProgramDateTime.IsValid() &&
						 NewMediaPlaylist->MediaSegments.Num() == 2)
					{
						NewMediaPlaylist->MediaSegments[NewMediaPlaylist->MediaSegments.Num()-2].ProgramDateTime.SetToZero();
					}
					// Extrapolate forward.
					if (NewMediaPlaylist->MediaSegments[NewMediaPlaylist->MediaSegments.Num()-2].ProgramDateTime.IsValid())
					{
						Segment.ProgramDateTime = NewMediaPlaylist->MediaSegments[NewMediaPlaylist->MediaSegments.Num()-2].ProgramDateTime + NewMediaPlaylist->MediaSegments[NewMediaPlaylist->MediaSegments.Num()-2].Duration;
					}
				}
			}
			else
			{
				// When program-date-time is not used we store the accumulated duration in the member
				Segment.ProgramDateTime = NewMediaPlaylist->Duration;
			}
			Segment.MediaSequence = NewMediaPlaylist->NextMediaSequence;
			Segment.DiscontinuitySequence = NewMediaPlaylist->NextDiscontinuitySequence;
			Segment.bDiscontinuity = bNextIsDiscontinuity ? 1 : 0;
			Segment.bGap = bNextIsGap ? 1 : 0;
			if (bNextIsDiscontinuity)
			{
				++NewMediaPlaylist->NextDiscontinuitySequence;
				LastKnownProgramDateTime.SetToInvalid();
			}
			++NewMediaPlaylist->NextMediaSequence;
			NewMediaPlaylist->Duration += Segment.Duration;
			CurrentByteRange.Reset();
			CurrentProgramDateTime.Reset();
			bNextIsDiscontinuity = false;
			bNextIsGap = false;
		}
		// EXT-X-START ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_START)
		{
			if (!ParseStartTime(NewMediaPlaylist->StartTime, Elements[nElem]))
			{
				return false;
			}
		}
		// EXT-X-PLAYLIST-TYPE ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_PLAYLIST_TYPE)
		{
			NewMediaPlaylist->PlaylistType = InPlaylist->GetPlaylistType();
		}
		// EXT-X-TARGET-DURATION ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_TARGETDURATION)
		{
			double Value = 0.0;
			LexFromString(Value, *Elements[nElem]->ElementValue.GetValue());
			NewMediaPlaylist->TargetDuration.SetFromSeconds(Value);
		}
		// EXT-X-MEDIA-SEQUENCE ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_MEDIA_SEQUENCE)
		{
			NewMediaPlaylist->NextMediaSequence = 0;
			LexFromString(NewMediaPlaylist->NextMediaSequence, *Elements[nElem]->ElementValue.GetValue());
		}
		// EXT-X-DISCONTINUITY-SEQUENCE ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_DISCONTINUITY_SEQUENCE)
		{
			NewMediaPlaylist->NextDiscontinuitySequence = 0;
			LexFromString(NewMediaPlaylist->NextDiscontinuitySequence, *Elements[nElem]->ElementValue.GetValue());
		}
		// EXT-X-MAP ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_MAP)
		{
			if (!CurrentInitSegment.IsValid())
			{
				CurrentInitSegment = MakeShared<FMediaInitSegment, ESPMode::ThreadSafe>();
			}
			ActivateCurrentKeys();
			CurrentInitSegment->Encryption = ActiveKeys;

			const TArray<FPlaylistParserHLS::FAttribute>& Attributes = Elements[nElem]->AttributeList;
			for(int32 i=0,iMax=Attributes.Num(); i<iMax; ++i)
			{
				const FString& AttrName(Attributes[i].Name);
				if (AttrName.Equals(TEXT("URI")))
				{
					if (!Attributes[i].GetValue(CurrentInitSegment->URL, NewMediaPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-MAP has a bad URI value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}
				else if (AttrName.Equals(TEXT("BYTERANGE")))
				{
					FString ByteRange;
					if (!Attributes[i].GetValue(ByteRange, NewMediaPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-MAP has a bad BYTERANGE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}

					FMediaByteRangeHLS br = ParseByteRange(ByteRange);
					if (br.Offset < 0)
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-MAP has a bad BYTERANGE value (offset is required)")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					CurrentInitSegment->ByteRange = MoveTemp(br);
				}
			}
			if (CurrentInitSegment->URL.IsEmpty())
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-MAP is missing required URI attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}

			ActiveInitSegment = CurrentInitSegment;
			CurrentInitSegment.Reset();
		}
		// EXT-X-PROGRAM-DATE-TIME ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_PROGRAM_DATE_TIME)
		{
			FTimeValue DateTime;
			if (ISO8601::ParseDateTime(DateTime, Elements[nElem]->ElementValue.GetValue()))
			{
				CurrentProgramDateTime = DateTime;
				bHasValidDateTime = true;
			}
			/*
				PDT is optional, so if whatever is given here fails to parse we ignore it.

				else
				{
					LastError = PostError(FString::Printf(TEXT("EXT-X-PROGRAM-DATE-TIME has a bad value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
					return false;
				}
			*/
		}
		// EXT-X-DISCONTINUITY ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_DISCONTINUITY)
		{
			bNextIsDiscontinuity = true;
		}
		// EXT-X-BYTERANGE ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_BYTERANGE)
		{
			FMediaByteRangeHLS br = ParseByteRange(Elements[nElem]->ElementValue.GetValue());
			if (br.NumBytes < 0)
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-MAP has a bad BYTERANGE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}
			CurrentByteRange = br;
		}
		// EXT-X-KEY ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_KEY)
		{
			if (!CurrentKeys.IsValid())
			{
				CurrentKeys = MakeShared<FMediaEncryptionHLS, ESPMode::ThreadSafe>();
			}
			FMediaEncryptionHLS::FKeyInfo& KeyInfo = CurrentKeys->KeyInfos.Emplace_GetRef();

			const TArray<FPlaylistParserHLS::FAttribute>& Attributes = Elements[nElem]->AttributeList;
			for(int32 i=0,iMax=Attributes.Num(); i<iMax; ++i)
			{
				const FString& AttrName(Attributes[i].Name);
				// METHOD ?
				if (AttrName.Equals(TEXT("METHOD")))
				{
					KeyInfo.Method = Attributes[i].GetValue();
				}
				// URI ?
				else if (AttrName.Equals(TEXT("URI")))
				{
					if (!Attributes[i].GetValue(KeyInfo.URI, NewMediaPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-KEY has a bad URI value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}
				// IV ?
				else if (AttrName.Equals(TEXT("IV")))
				{
					if (!Attributes[i].GetValue(KeyInfo.IV, NewMediaPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-KEY has a bad IV value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}
				// KEYFORMAT ?
				else if (AttrName.Equals(TEXT("KEYFORMAT")))
				{
					if (!Attributes[i].GetValue(KeyInfo.KeyFormat, NewMediaPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-KEY has a bad KEYFORMAT value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}
				// KEYFORMATVERSIONS ?
				else if (AttrName.Equals(TEXT("KEYFORMATVERSIONS")))
				{
					if (!Attributes[i].GetValue(KeyInfo.KeyFormatVersions, NewMediaPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-KEY has a bad KEYFORMATVERSIONS value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}
			}
			if (KeyInfo.Method.IsEmpty())
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-KEY is missing required METHOD attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}
			if (KeyInfo.Method.Equals(TEXT("NONE")))
			{
				CurrentKeys.Reset();
				ActiveKeys.Reset();
				ActivatedKeysFrom.Reset();
			}
			else if (KeyInfo.URI.IsEmpty())
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-KEY is missing required URI attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}
		}
		// EXT-X-GAP ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_GAP)
		{
			bNextIsGap = true;
		}
		// EXT-X-SERVER-CONTROL ?
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_SERVER_CONTROL)
		{
			// Note: The HLS RFC does not state if all options need to be conveyed on a single EXT-X-SERVER-CONTROL
			//       or if there could be several and if so, how they complement each other. We take all of these
			//       as they appear and update a single option structure.
			ParseServerControl(NewMediaPlaylist->ServerControl, Elements[nElem]);
		}
	}

	// Check that the segments are within the target duration
	int64 MaxSegDurationInt = FMath::RoundToInt(LongestMediaSegmentDuration.GetAsSeconds());
	int64 TargetDuration = FMath::CeilToInt(NewMediaPlaylist->TargetDuration.GetAsSeconds());
	if (MaxSegDurationInt > TargetDuration)
	{
		LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("HLS (RFC-8216) violation: Longest playlist #EXTINF segment duration of %.4f is longer than the specified #EXT-TARGET-DURATION of %i, adjusting."), LongestMediaSegmentDuration.GetAsSeconds(), (int32)TargetDuration));
		NewMediaPlaylist->TargetDuration = LongestMediaSegmentDuration;
	}

	// If PDT is given in the playlist, but none of its values are valid, treat it as absent PDTs.
	if (NewMediaPlaylist->bHasProgramDateTime && !bHasValidDateTime)
	{
		NewMediaPlaylist->bHasProgramDateTime = false;
	}
	// Set the first segment program date time if present.
	if (NewMediaPlaylist->bHasProgramDateTime && NewMediaPlaylist->MediaSegments.Num())
	{
		NewMediaPlaylist->FirstProgramDateTime = NewMediaPlaylist->MediaSegments[0].ProgramDateTime;
	}
	OutMediaPlaylist = MoveTemp(NewMediaPlaylist);
	return true;
}

bool FPlaylistHandlerHLS::BuildMultiVariantPlaylist(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe>& OutMultiVariantPlaylist, TSharedPtr<FPlaylistParserHLS, ESPMode::ThreadSafe> InPlaylist)
{
	TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> NewVariantPlaylist = MakeShared<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe>();
	NewVariantPlaylist->URL = InPlaylist->GetURL();
	NewVariantPlaylist->ParsedURL.Parse(InPlaylist->GetURL());
	NewVariantPlaylist->ContentSteeringParams.bHaveContentSteering = InPlaylist->UsesContentSteering();

	// First process all EXT-X-DEFINE elements to set up the values for the variable substitutions.
	TArray<FPlaylistParserHLS::FVariableSubstitution> EmptySubstitutions;
	if (!PrepareSubstitutionVariables(NewVariantPlaylist->VariableSubstitutions, InPlaylist, EmptySubstitutions))
	{
		return false;
	}

	// Then process all EXT-X-MEDIA elements to set up the rendition groups and set up content steering and server control.
	const TArray<TUniquePtr<FPlaylistParserHLS::FElement>>& Elements = InPlaylist->GetElements();
	for(int32 nElem=0; nElem<Elements.Num(); ++nElem)
	{
		if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_MEDIA)
		{
			if (const FPlaylistParserHLS::FAttribute* Type = Elements[nElem]->GetAttribute(TEXT("TYPE")))
			{
				FString TypeValue = Type->GetValue();
				FMultiVariantPlaylistHLS::ERenditionGroupType TypeIndex = FMultiVariantPlaylistHLS::ERenditionGroupType::Invalid;
				if (TypeValue.Equals(TEXT("VIDEO"))) TypeIndex = FMultiVariantPlaylistHLS::ERenditionGroupType::Video;
				else if (TypeValue.Equals(TEXT("AUDIO"))) TypeIndex = FMultiVariantPlaylistHLS::ERenditionGroupType::Audio;
				else if (TypeValue.Equals(TEXT("SUBTITLES"))) TypeIndex = FMultiVariantPlaylistHLS::ERenditionGroupType::Subtitles;
				else if (TypeValue.Equals(TEXT("CLOSED-CAPTIONS"))) TypeIndex = FMultiVariantPlaylistHLS::ERenditionGroupType::ClosedCaptions;
				// If this is a TYPE that is not defined yet we ignore this element.
				if (TypeIndex != FMultiVariantPlaylistHLS::ERenditionGroupType::Invalid)
				{
					if (const FPlaylistParserHLS::FAttribute* GroupId = Elements[nElem]->GetAttribute(TEXT("GROUP-ID")))
					{
						FString GroupIdValue;
						if (!GroupId->GetValue(GroupIdValue, NewVariantPlaylist->VariableSubstitutions))
						{
							LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad GROUP-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}
						// Create a new group if one doesn't exist yet.
						TArray<FMultiVariantPlaylistHLS::FRenditionGroup>& Group(NewVariantPlaylist->RenditionGroupsOfType[(int32)TypeIndex]);
						if (!Group.Contains(GroupIdValue))
						{
							FMultiVariantPlaylistHLS::FRenditionGroup& NewGroup = Group.Emplace_GetRef();
							NewGroup.GroupID = GroupIdValue;
						}
						// Get the group to which to add the rendition.
						FMultiVariantPlaylistHLS::FRenditionGroup* rg = Group.FindByKey(GroupIdValue);
						check(rg);

						FMultiVariantPlaylistHLS::FRendition NewRendition;
						const TArray<FPlaylistParserHLS::FAttribute>& Attributes = Elements[nElem]->AttributeList;
						bool bGotName = false;
						for(int32 i=0,iMax=Attributes.Num(); i<iMax; ++i)
						{
							const FString& AttrName(Attributes[i].Name);
							// URI ?
							if (AttrName.Equals(TEXT("URI")))
							{
								if (TypeIndex != FMultiVariantPlaylistHLS::ERenditionGroupType::ClosedCaptions)
								{
									if (!Attributes[i].GetValue(NewRendition.URI, NewVariantPlaylist->VariableSubstitutions))
									{
										LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad URI value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
										return false;
									}
								}
								// Make an absolute URL
								FURL_RFC3986 UrlParser(NewVariantPlaylist->ParsedURL);
								UrlParser.ResolveWith(NewRendition.URI);
								NewRendition.URI = UrlParser.Get(true, true);
							}
							// LANGUAGE ?
							else if (AttrName.Equals(TEXT("LANGUAGE")))
							{
								FString Lang5646;
								if (!Attributes[i].GetValue(Lang5646, NewVariantPlaylist->VariableSubstitutions))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad LANGUAGE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
								if (!BCP47::ParseRFC5646Tag(NewRendition.LanguageRFC5646, Lang5646))
								{
									LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("LANGUAGE \"%s\" is not a valid BCP-47 tag!"), *Lang5646));
								}
							}
							// ASSOC-LANGUAGE ?
							else if (AttrName.Equals(TEXT("ASSOC-LANGUAGE")))
							{
								FString Lang5646;
								if (!Attributes[i].GetValue(Lang5646, NewVariantPlaylist->VariableSubstitutions))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad ASSOC-LANGUAGE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
								if (!BCP47::ParseRFC5646Tag(NewRendition.AssocLanguageRFC5646, Lang5646))
								{
									LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("ASSOC-LANGUAGE \"%s\" is not a valid BCP-47 tag!"), *Lang5646));
								}
							}
							// NAME ?
							else if (AttrName.Equals(TEXT("NAME")))
							{
								bGotName = true;
								if (!Attributes[i].GetValue(NewRendition.Name, NewVariantPlaylist->VariableSubstitutions))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad NAME value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
								// Check that there is no entry with the same NAME yet.
								for(auto& It : rg->Renditions)
								{
									if (It.Name.Equals(NewRendition.Name))
									{
										LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has same NAME value as another element")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
										return false;
									}
								}
							}
							// STABLE-RENDITION-ID ?
							else if (AttrName.Equals(TEXT("STABLE-RENDITION-ID")))
							{
								if (!Attributes[i].GetValue(NewRendition.StableRenditionId, NewVariantPlaylist->VariableSubstitutions))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad STABLE-RENDITION-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
								// Validate alphabet
								for(StringHelpers::FStringIterator It(NewRendition.StableRenditionId); It; ++It)
								{
									if (!((*It >= TCHAR('a') && *It <= TCHAR('z')) || (*It >= TCHAR('A') && *It <= TCHAR('Z')) || (*It >= TCHAR('0') && *It <= TCHAR('9')) ||
										  *It == TCHAR('+') || *It == TCHAR('/') || *It == TCHAR('=') || *It == TCHAR('.') || *It == TCHAR('-') || *It == TCHAR('_')))
									{
										LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has invalid characters in STABLE-RENDITION-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
										return false;
									}
								}
							}
							// DEFAULT ?
							else if (AttrName.Equals(TEXT("DEFAULT")))
							{
								FString Temp;
								if (!Attributes[i].GetValue(Temp, NewVariantPlaylist->VariableSubstitutions) && !(Temp.Equals(TEXT("YES")) || Temp.Equals(TEXT("NO"))))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad DEFAULT value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
								NewRendition.bDefault = Temp.Equals(TEXT("YES"));
							}
							// AUTOSELECT ?
							else if (AttrName.Equals(TEXT("AUTOSELECT")))
							{
								FString Temp;
								if (!Attributes[i].GetValue(Temp, NewVariantPlaylist->VariableSubstitutions) && !(Temp.Equals(TEXT("YES")) || Temp.Equals(TEXT("NO"))))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad AUTOSELECT value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
								NewRendition.bAutoSelect = Temp.Equals(TEXT("YES"));
							}
							// FORCED ?
							else if (AttrName.Equals(TEXT("FORCED")))
							{
								FString Temp;
								if (!Attributes[i].GetValue(Temp, NewVariantPlaylist->VariableSubstitutions) && !(Temp.Equals(TEXT("YES")) || Temp.Equals(TEXT("NO"))))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad FORCED value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
								NewRendition.bForced = Temp.Equals(TEXT("YES"));
							}
							// INSTREAM-ID ?
							else if (AttrName.Equals(TEXT("INSTREAM-ID")))
							{
								if (!Attributes[i].GetValue(NewRendition.InstreamId, NewVariantPlaylist->VariableSubstitutions))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad INSTREAM-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
								// Validate
								if (NewRendition.InstreamId.StartsWith(TEXT("CC"), ESearchCase::CaseSensitive))
								{
									if (NewRendition.InstreamId.Len() != 3 || !(NewRendition.InstreamId[2] == TCHAR('1') || NewRendition.InstreamId[2] == TCHAR('2') ||
										NewRendition.InstreamId[2] == TCHAR('3') || NewRendition.InstreamId[2] == TCHAR('4')))
									{
										LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad INSTREAM-ID value (not CC1 through CC4)")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
										return false;
									}
								}
								else if (NewRendition.InstreamId.StartsWith(TEXT("SERVICE"), ESearchCase::CaseSensitive))
								{
									bool bOk = NewRendition.InstreamId.Len() <= 9;
									if (bOk)
									{
										bOk = FChar::IsDigit(NewRendition.InstreamId[7]) && NewRendition.InstreamId.Len() == 9 ? FChar::IsDigit(NewRendition.InstreamId[8]) : true;
										if (bOk)
										{
											int32 ServiceNum = 0;
											LexFromString(ServiceNum, *NewRendition.InstreamId + 7);
											bOk = ServiceNum >= 1 && ServiceNum <= 63;
										}
									}
									if (!bOk)
									{
										LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad INSTREAM-ID value (not SERVICE1 through SERVICE63)")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
										return false;
									}
								}
								else
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad INSTREAM-ID value (not CCx or SERVICExx)")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
							}
							// BIT-DEPTH ?
							else if (AttrName.Equals(TEXT("BIT-DEPTH")))
							{
								if (TypeIndex == FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)
								{
									FString Temp = Attributes[i].GetValue();
									// Validate that this consists of digits only (must be a positive integer, so no sign or fractions)
									for(StringHelpers::FStringIterator It(Temp); It; ++It)
									{
										if (!(*It >= TCHAR('0') && *It <= TCHAR('9')))
										{
											LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has invalid characters in BIT-DEPTH value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
											return false;
										}
									}
									LexFromString(NewRendition.BitDepth, *Temp);
								}
							}
							// SAMPLE-RATE ?
							else if (AttrName.Equals(TEXT("SAMPLE-RATE")))
							{
								if (TypeIndex == FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)
								{
									FString Temp = Attributes[i].GetValue();
									// Validate that this consists of digits only (must be a positive integer, so no sign or fractions)
									for(StringHelpers::FStringIterator It(Temp); It; ++It)
									{
										if (!(*It >= TCHAR('0') && *It <= TCHAR('9')))
										{
											LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has invalid characters in SAMPLE-RATE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
											return false;
										}
									}
									LexFromString(NewRendition.SampleRate, *Temp);
								}
							}
							// CHANNELS ?
							else if (AttrName.Equals(TEXT("CHANNELS")))
							{
								if (TypeIndex == FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)
								{
									FString Temp;
									if (!Attributes[i].GetValue(Temp, NewVariantPlaylist->VariableSubstitutions))
									{
										LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad CHANNELS value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
										return false;
									}
									// We only want the first numeric part of the channel information.
									// The additional information is ignored for now.
									LexFromString(NewRendition.Channels, *Temp);
									if (NewRendition.Channels <= 0 || NewRendition.Channels > 32)
									{
										LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad CHANNELS value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
										return false;
									}
								}
							}
							// CHARACTERISTICS ?
							else if (AttrName.Equals(TEXT("CHARACTERISTICS")))
							{
								if (!Attributes[i].GetValue(NewRendition.Characteristics, NewVariantPlaylist->VariableSubstitutions))
								{
									LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA has a bad CHARACTERISTICS value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
									return false;
								}
							}
						}

						// The `NAME` attribute is required
						if (!bGotName)
						{
							LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA is missing required NAME attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}
						// With CLOSED-CAPTIONS the `INSTREAM-ID` is required
						if (TypeIndex == FMultiVariantPlaylistHLS::ERenditionGroupType::ClosedCaptions && NewRendition.InstreamId.IsEmpty())
						{
							LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA is missing required INSTREAM-ID attribute for CLOSED-CAPTIONS")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}

						// Add the new rendition to the group.
						rg->Renditions.Emplace(MoveTemp(NewRendition));
					}
					else
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA is missing required GROUP-ID attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}
			}
			else
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-MEDIA is missing required TYPE attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}
		}
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_CONTENT_STEERING)
		{
			if (const FPlaylistParserHLS::FAttribute* ServerURI = Elements[nElem]->GetAttribute(TEXT("SERVER-URI")))
			{
				if (!ServerURI->GetValue(NewVariantPlaylist->ContentSteeringParams.SteeringURI, NewVariantPlaylist->VariableSubstitutions))
				{
					LastError = PostError(FString::Printf(TEXT("EXT-X-CONTENT-STEERING has a bad SERVER-URI value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
					return false;
				}

				if (const FPlaylistParserHLS::FAttribute* PathwayId = Elements[nElem]->GetAttribute(TEXT("PATHWAY-ID")))
				{
					if (!PathwayId->GetValue(NewVariantPlaylist->ContentSteeringParams.PrimaryPathwayId, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-CONTENT-STEERING has a bad PATHWAY-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}

				// Check for our custom attributes
				if (const FPlaylistParserHLS::FAttribute* QueryBeforeStart = Elements[nElem]->GetAttribute(TEXT("EPIC-QUERY-BEFORE-START")))
				{
					// This is the equivalent of the DASH <ContentSteering@queryBeforeStart>
					FString qbsValue;
					// We use a string value here to allow it to be controlled by variable substitution.
					if (!QueryBeforeStart->GetValue(qbsValue, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-CONTENT-STEERING has a bad EPIC-QUERY-BEFORE-START value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					TArray<FString> qbfTrueValues { TEXT("true"), TEXT("yes"), TEXT("1") };
					NewVariantPlaylist->ContentSteeringParams.bQueryBeforeStart = qbfTrueValues.ContainsByPredicate([&](const FString& InMaybe){ return InMaybe.Equals(qbsValue, ESearchCase::IgnoreCase); });
				}
				if (const FPlaylistParserHLS::FAttribute* InitialSelectionPriority = Elements[nElem]->GetAttribute(TEXT("EPIC-INITIAL-SELECTION-PRIORITY")))
				{
					if (!InitialSelectionPriority->GetValue(NewVariantPlaylist->ContentSteeringParams.CustomInitialSelectionPriority, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-CONTENT-STEERING has a bad EPIC-INITIAL-SELECTION-PRIORITY value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}
			}
			else
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-CONTENT-STEERING is missing required SERVER-URI attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}
		}
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_SERVER_CONTROL)
		{
			// Note: The HLS RFC does not state if all options need to be conveyed on a single EXT-X-SERVER-CONTROL
			//       or if there could be several and if so, how they complement each other. We take all of these
			//       as they appear and update a single option structure.
			ParseServerControl(NewVariantPlaylist->ServerControl, Elements[nElem]);
		}
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_START)
		{
			ParseStartTime(NewVariantPlaylist->StartTime, Elements[nElem]);
		}
		else if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_SESSION_DATA)
		{
			if (const FPlaylistParserHLS::FAttribute* DataID = Elements[nElem]->GetAttribute(TEXT("DATA-ID")))
			{
				const FPlaylistParserHLS::FAttribute* SessDataValue = Elements[nElem]->GetAttribute(TEXT("VALUE"));
				const FPlaylistParserHLS::FAttribute* SessDataURI = Elements[nElem]->GetAttribute(TEXT("URI"));
				if (!SessDataValue && !SessDataURI)
				{
					LastError = PostError(FString::Printf(TEXT("EXT-X-SESSION-DATA has neither VALUE nor URI attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
					return false;
				}
				if (SessDataValue && SessDataURI)
				{
					LastError = PostError(FString::Printf(TEXT("EXT-X-SESSION-DATA has both VALUE or URI attributes")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
					return false;
				}
				// We do not resolve any session data pointing to another URL resource.
				if (SessDataValue)
				{
					IPlayerSessionServices::FPlaylistProperty Prop;
					if (!DataID->GetValue(Prop.Tag, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-SESSION-DATA has a bad DATA-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					if (!SessDataValue->GetValue(Prop.Value, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-SESSION-DATA has a bad VALUE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}

					// Check if the DATA-ID is one to configure CMCD with.
					if (PlayerSessionServices->GetCMCDHandler()->UseParametersFromPlaylist() && PlayerSessionServices->GetCMCDHandler()->UsePlaylistParametersFromKey().Equals(Prop.Tag))
					{
						FString CMCDConfig;
						if (FBase64::Decode(Prop.Value, CMCDConfig))
						{
							PlayerSessionServices->GetCMCDHandler()->UpdateParameters(CMCDConfig);
						}
					}
					else
					{
						// Send this session data property up to registered listeners.
						IPlayerSessionServices::ECustomPropertyResult Result = PlayerSessionServices->ValidateMainPlaylistCustomProperty(TEXT("hls"), InPlaylist->GetURL(), InPlaylist->GetResponseHeaders(), Prop);
						if (Result == IPlayerSessionServices::ECustomPropertyResult::Reject)
						{
							LastError = PostError(FString::Printf(TEXT("Playlist has been rejected by application due to bad EXT-X-SESSION-DATA")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}
					}
				}
			}
			else
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-SESSION-DATA is missing required DATA-ID attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}
		}
	}

	// Now process all EXT-X-STREAM-INF elements. These may reference any of the rendition groups.
	for(int32 nElem=0; nElem<Elements.Num(); ++nElem)
	{
		if (Elements[nElem]->Tag == FPlaylistParserHLS::EExtTag::EXT_X_STREAM_INF)
		{
			/*
				We are very strict here with duplicate attributes.
				The RFC says:
					"A given AttributeName MUST NOT appear more than once in a given
					attribute-list.  Clients SHOULD refuse to parse such Playlists."
			*/
			if (Elements[nElem]->bHaveDuplicateAttribute)
			{
				LastError = PostError(FString::Printf(TEXT("EXT_X_STREAM_INF gives same attribute more than once")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}

			FMultiVariantPlaylistHLS::FStreamInf NewStreamInf;

			const TArray<FPlaylistParserHLS::FAttribute>& Attributes = Elements[nElem]->AttributeList;
			for(int32 i=0,iMax=Attributes.Num(); i<iMax; ++i)
			{
				const FString& AttrName(Attributes[i].Name);
				// BANDWIDTH ?
				if (AttrName.Equals(TEXT("BANDWIDTH")))
				{
					FString Temp = Attributes[i].GetValue();
					// Validate that this consists of digits only (must be a positive integer, so no sign or fractions)
					if (!ValidateNumbersOnly(Temp))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has invalid characters in BANDWIDTH value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					LexFromString(NewStreamInf.Bandwidth, *Temp);
					if (NewStreamInf.Bandwidth <= 0)
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has bad BANDWIDTH value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
				}
				// CODECS ?
				else if (AttrName.Equals(TEXT("CODECS")))
				{
					FString CodecLine;
					if (!Attributes[i].GetValue(CodecLine, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad CODECS value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					// Split the codec line by comma
					const TCHAR* const CommaDelimiter = TEXT(",");
					CodecLine.ParseIntoArray(NewStreamInf.Codecs, CommaDelimiter, true);
					// Remove any surrounding whitespaces
					for(int32 nc=0; nc<NewStreamInf.Codecs.Num(); ++nc)
					{
						NewStreamInf.Codecs[nc].TrimStartAndEndInline();
						// Try to identify the codec
						NewStreamInf.ParsedCodecs.Emplace_GetRef().ParseFromRFC6381(NewStreamInf.Codecs[nc]);
						NewStreamInf.NumVideoCodec += NewStreamInf.ParsedCodecs[nc].IsVideoCodec() ? 1 : 0;
						NewStreamInf.NumAudioCodec += NewStreamInf.ParsedCodecs[nc].IsAudioCodec() ? 1 : 0;
						NewStreamInf.NumSubtitleCodec += NewStreamInf.ParsedCodecs[nc].IsSubtitleCodec() ? 1 : 0;
					}
				}
				// SUPPLEMENTAL-CODECS ?
				else if (AttrName.Equals(TEXT("SUPPLEMENTAL-CODECS")))
				{
					FString CodecLine;
					if (!Attributes[i].GetValue(CodecLine, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad SUPPLEMENTAL-CODECS value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					// Split the codec line by forward slash
					const TCHAR* const SlashDelimiter = TEXT("/");
					CodecLine.ParseIntoArray(NewStreamInf.SupplementalCodecs, SlashDelimiter, true);
					// Remove any surrounding whitespaces
					for(int32 nc=0; nc<NewStreamInf.SupplementalCodecs.Num(); ++nc)
					{
						NewStreamInf.SupplementalCodecs[nc].TrimStartAndEndInline();
					}
				}
				// VIDEO-RANGE ?
				else if (AttrName.Equals(TEXT("VIDEO-RANGE")))
				{
					NewStreamInf.VideoRange = Attributes[i].GetValue();
				}
				// RESOLUTION ?
				else if (AttrName.Equals(TEXT("RESOLUTION")))
				{
					FString Temp(Attributes[i].GetValue());
					TArray<FString> Resolution;
					const TCHAR* const ByDelimiter = TEXT("x");
					Temp.ParseIntoArray(Resolution, ByDelimiter, true);
					if (Resolution.Num() != 2)
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has bad RESOLUTION value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					// Remove any surrounding whitespaces
					for(auto& It : Resolution)
					{
						It.TrimStartAndEndInline();
					}
					if (!ValidateNumbersOnly(Resolution[0]) || !ValidateNumbersOnly(Resolution[1]))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has bad RESOLUTION value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					LexFromString(NewStreamInf.ResolutionW, *Resolution[0]);
					LexFromString(NewStreamInf.ResolutionH, *Resolution[1]);
				}
				// FRAME-RATE ?
				else if (AttrName.Equals(TEXT("FRAME-RATE")))
				{
					FString Temp(Attributes[i].GetValue());
					if (!ValidatePositiveFloatOnly(Temp))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has bad FRAME-RATE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					NewStreamInf.FrameRate.SetFromFloatString(Temp);
				}
				// STABLE-VARIANT-ID ?
				else if (AttrName.Equals(TEXT("STABLE-VARIANT-ID")))
				{
					if (!Attributes[i].GetValue(NewStreamInf.StableVariantId, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad STABLE-VARIANT-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					// Validate alphabet
					for(StringHelpers::FStringIterator It(NewStreamInf.StableVariantId); It; ++It)
					{
						if (!((*It >= TCHAR('a') && *It <= TCHAR('z')) || (*It >= TCHAR('A') && *It <= TCHAR('Z')) || (*It >= TCHAR('0') && *It <= TCHAR('9')) ||
								*It == TCHAR('+') || *It == TCHAR('/') || *It == TCHAR('=') || *It == TCHAR('.') || *It == TCHAR('-') || *It == TCHAR('_')))
						{
							LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has invalid characters in STABLE-VARIANT-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}
					}
				}
				// AUDIO ?
				else if (AttrName.Equals(TEXT("AUDIO")))
				{
					if (!Attributes[i].GetValue(NewStreamInf.AudioGroup, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad value for the AUDIO attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					// Check that the referenced group exists.
					FMultiVariantPlaylistHLS::FRenditionGroup* rg = NewVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio)].FindByKey(NewStreamInf.AudioGroup);
					if (!rg)
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF references AUDIO group \"%s\" that has not been declared"), *NewStreamInf.AudioGroup), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					rg->bIsReferenced = true;
				}
				// VIDEO ?
				else if (AttrName.Equals(TEXT("VIDEO")))
				{
					if (!Attributes[i].GetValue(NewStreamInf.VideoGroup, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad value for the VIDEO attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					// Check that the referenced group exists.
					FMultiVariantPlaylistHLS::FRenditionGroup* rg = NewVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Video)].FindByKey(NewStreamInf.VideoGroup);
					if (!rg)
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF references VIDEO group \"%s\" that has not been declared"), *NewStreamInf.VideoGroup), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					rg->bIsReferenced = true;
				}
				// SUBTITLES ?
				else if (AttrName.Equals(TEXT("SUBTITLES")))
				{
					if (!Attributes[i].GetValue(NewStreamInf.SubtitleGroup, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad value for the SUBTITLES attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					// Check that the referenced group exists.
					FMultiVariantPlaylistHLS::FRenditionGroup* rg = NewVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::Subtitles)].FindByKey(NewStreamInf.SubtitleGroup);
					if (!rg)
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF references SUBTITLES group \"%s\" that has not been declared"), *NewStreamInf.SubtitleGroup), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					rg->bIsReferenced = true;
				}
				// CLOSED-CAPTIONS ?
				else if (AttrName.Equals(TEXT("CLOSED-CAPTIONS")))
				{
					// There is a distinction made just for closed captions that there can be a value of NONE to indicate the absence of captions.
					// Because `NONE` could also be the name of a group we have to check if the attribute was a quoted string.
					if (Attributes[i].bWasQuoted)
					{
						if (!Attributes[i].GetValue(NewStreamInf.ClosedCaptionGroup, NewVariantPlaylist->VariableSubstitutions))
						{
							LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad value for the CLOSED-CAPTIONS attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}
						// Check that the referenced group exists.
						FMultiVariantPlaylistHLS::FRenditionGroup* rg = NewVariantPlaylist->RenditionGroupsOfType[static_cast<int32>(FMultiVariantPlaylistHLS::ERenditionGroupType::ClosedCaptions)].FindByKey(NewStreamInf.ClosedCaptionGroup);
						if (!rg)
						{
							LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF references CLOSED-CAPTIONS group \"%s\" that has not been declared"), *NewStreamInf.ClosedCaptionGroup), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}
						rg->bIsReferenced = true;
					}
				}
				// PATHWAY-ID ?
				else if (AttrName.Equals(TEXT("PATHWAY-ID")))
				{
					if (!Attributes[i].GetValue(NewStreamInf.PathwayId, NewVariantPlaylist->VariableSubstitutions))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad PATHWAY-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					// Validate alphabet
					for(StringHelpers::FStringIterator It(NewStreamInf.PathwayId); It; ++It)
					{
						if (!((*It >= TCHAR('a') && *It <= TCHAR('z')) || (*It >= TCHAR('A') && *It <= TCHAR('Z')) || (*It >= TCHAR('0') && *It <= TCHAR('9')) ||
								*It == TCHAR('.') || *It == TCHAR('-') || *It == TCHAR('_')))
						{
							LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has invalid characters in PATHWAY-ID value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
							return false;
						}
					}
				}
				// SCORE ?
				else if (AttrName.Equals(TEXT("SCORE")))
				{
					FString Temp(Attributes[i].GetValue());
					if (!ValidatePositiveFloatOnly(Temp))
					{
						LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has bad SCORE value")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
						return false;
					}
					LexFromString(NewStreamInf.Score, *Temp);
				}
				/*
				else if (AttrName.Equals(TEXT("AVERAGE-BANDWIDTH")))
				else if (AttrName.Equals(TEXT("HDCP-LEVEL")))
				else if (AttrName.Equals(TEXT("ALLOWED-CPC")))
				else if (AttrName.Equals(TEXT("REQ-VIDEO-LAYOUT")))
				else if (AttrName.Equals(TEXT("PROGRAM-ID")))			// PROGRAM-ID was deprecated with version 6
				*/
			}

			// BANDWIDTH is (sadly) the only required attribute
			if (NewStreamInf.Bandwidth <= 0)
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF is missing required BANDWIDTH attribute")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}
			// Well, and the URI of course
			if (!Elements[nElem]->URI.GetValue(NewStreamInf.URI, NewVariantPlaylist->VariableSubstitutions))
			{
				LastError = PostError(FString::Printf(TEXT("EXT-X-STREAM-INF has a bad URI")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
				return false;
			}

			// Make an absolute URL
			FURL_RFC3986 UrlParser(NewVariantPlaylist->ParsedURL);
			UrlParser.ResolveWith(NewStreamInf.URI);
			NewStreamInf.URI = UrlParser.Get(true, true);

			// Add the new stream inf to the list.
			NewVariantPlaylist->InitialStreamInfs.Emplace(MoveTemp(NewStreamInf));
		}
	}

	// Call external registry with an end-of-properties call.
	if (PlayerSessionServices->ValidateMainPlaylistCustomProperty(TEXT("hls"), InPlaylist->GetURL(), InPlaylist->GetResponseHeaders(), IPlayerSessionServices::FPlaylistProperty()) == IPlayerSessionServices::ECustomPropertyResult::Reject)
	{
		LastError = PostError(FString::Printf(TEXT("Playlist has been rejected by application")), HLS::ERRCODE_PLAYLIST_PARSING_FAILED);
		return false;
	}

	OutMultiVariantPlaylist = MoveTemp(NewVariantPlaylist);
	return true;
}

void FPlaylistHandlerHLS::GroupVariantStreamsByPathways(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	for(int32 ns=0; ns<InMultiVariantPlaylist->InitialStreamInfs.Num(); ++ns)
	{
		// If there is no PATHWAY-ID set on the stream inf yet, set the default pathway of "."
		if (InMultiVariantPlaylist->InitialStreamInfs[ns].PathwayId.IsEmpty())
		{
			InMultiVariantPlaylist->InitialStreamInfs[ns].PathwayId = TEXT(".");
		}
		// Assign to the pathway bucket, creating it if necessary.
		TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>* pwEntry = InMultiVariantPlaylist->PathwayStreamInfs.FindByPredicate([pwId=InMultiVariantPlaylist->InitialStreamInfs[ns].PathwayId](const TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>& e){return e->PathwayID.Equals(pwId);});
		if (!pwEntry)
		{
			TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> newPw(MakeSharedTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs>());
			InMultiVariantPlaylist->PathwayStreamInfs.Emplace(newPw);
			newPw->PathwayID = InMultiVariantPlaylist->InitialStreamInfs[ns].PathwayId;
			pwEntry = &InMultiVariantPlaylist->PathwayStreamInfs.Last();
		}
		(*pwEntry)->StreamInfs.Emplace(InMultiVariantPlaylist->InitialStreamInfs[ns]);
		(*pwEntry)->StreamInfs.Last().IndexOfSelfInArray = (*pwEntry)->StreamInfs.Num() - 1;
	}
	// From this point on the initial collection list of the stream infs must not be used any more.
	InMultiVariantPlaylist->InitialStreamInfs.Empty();
}

void FPlaylistHandlerHLS::AssignInternalVariantStreamIDs(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	for(auto& pwIt : InMultiVariantPlaylist->PathwayStreamInfs)
	{
		for(int32 i=0,iMax=pwIt->StreamInfs.Num(); i<iMax; ++i)
		{
			pwIt->StreamInfs[i].ID = FString::Printf(TEXT("%d"), i);
		}
	}
}


void FPlaylistHandlerHLS::GroupVariantStreamsByVideoProperties(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	auto UpdateHashStr = [](FSHA1& InOutHash, const FString& InString)	{ if (InString.Len()) InOutHash.UpdateWithString(*InString, InString.Len()); };
	auto GetBaseCodec = [](const FString& InCodec) -> FString
	{
		int32 DotPos;
		if (InCodec.FindChar(TCHAR('.'), DotPos))
		{
			return InCodec.Left(DotPos);
		}
		return InCodec;
	};
	// Group video streams by property for each pathway, since pathways may have different variant streams.
	for(int32 pwIdx=0; pwIdx<InMultiVariantPlaylist->PathwayStreamInfs.Num(); ++pwIdx)
	{
		TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> pwSinf(InMultiVariantPlaylist->PathwayStreamInfs[pwIdx]);
		TArray<FMultiVariantPlaylistHLS::FStreamInf>& sInfs(pwSinf->StreamInfs);

		TMultiMap<FString, int32> SamePropertyMap;
		for(int32 ns=0,nsMax=sInfs.Num(); ns<nsMax; ++ns)
		{
			FMultiVariantPlaylistHLS::FStreamInf& si = sInfs[ns];

			if (si.NumVideoCodec > 1)
			{
				LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Variant stream gives more than one video codec, ignoring this variant.")));
				continue;
			}

			// Generate a (hopefully) unique hash over all the variant stream attributes in an ordered and consistent way.
			FSHA1 Hash;

			TArray<FString> BaseCodecs;
			FString SupplementalCodec;
			bool bHasVideo = false;
			// Add all unique base codecs in sorted ascending order (so they same ones give the same hash)
			for(int32 i=0; i<si.ParsedCodecs.Num(); ++i)
			{
				if (si.ParsedCodecs[i].IsVideoCodec())
				{
					bHasVideo = true;
				}
				BaseCodecs.AddUnique(GetBaseCodec(si.Codecs[i]));
			}
			// No recognized video codec, no need to create a group.
			if (!bHasVideo)
			{
				continue;
			}
			if (si.SupplementalCodecs.Num())
			{
				SupplementalCodec = GetBaseCodec(si.SupplementalCodecs[0]);
			}
			BaseCodecs.Sort();
			for(auto& Codec : BaseCodecs)
			{
				UpdateHashStr(Hash, Codec);
			}
			UpdateHashStr(Hash, SupplementalCodec);
			UpdateHashStr(Hash, si.VideoRange);
			/*
				Do NOT add any of the video/audio/etc. groups as each variant may reference a different group
				(a variant is the bandwidth-switchable entity, the groups are things like different angles)
				If the groups use different codecs, like in
					#EXT-X-STREAM-INF:AVERAGE-BANDWIDTH=2168183,BANDWIDTH=2177116,CODECS="avc1.640020,mp4a.40.2",RESOLUTION=960x540,FRAME-RATE=60.000,CLOSED-CAPTIONS="cc1",AUDIO="aud1",SUBTITLES="sub1"
				and
					#EXT-X-STREAM-INF:AVERAGE-BANDWIDTH=2390686,BANDWIDTH=2399619,CODECS="avc1.640020,ac-3",RESOLUTION=960x540,FRAME-RATE=60.000,CLOSED-CAPTIONS="cc1",AUDIO="aud2",SUBTITLES="sub1"
				with
					#EXT-X-MEDIA:TYPE=AUDIO,GROUP-ID="aud1",LANGUAGE="en",NAME="English",AUTOSELECT=YES,DEFAULT=YES,CHANNELS="2",URI="a1/prog_index.m3u8"
					#EXT-X-MEDIA:TYPE=AUDIO,GROUP-ID="aud2",LANGUAGE="en",NAME="English",AUTOSELECT=YES,DEFAULT=YES,CHANNELS="6",URI="a2/prog_index.m3u8"
				we will have covered this by considering all base codecs in the hash at the top.

				Do not
					UpdateHashStr(Hash, si.AudioGroup);
					UpdateHashStr(Hash, si.SubtitleGroup);
					UpdateHashStr(Hash, si.ClosedCaptionGroup);
			*/
			FString HashValue = Hash.Finalize().ToString();
			SamePropertyMap.Emplace(MoveTemp(HashValue), ns);
		}

		// Go over the tuples of same hash values and create groups
		TMap<FString, int32> SameGroupURLHashMap;
		TArray<FString> KeyArray;
		SamePropertyMap.GetKeys(KeyArray);
		for(auto& kIt : KeyArray)
		{
			FMultiVariantPlaylistHLS::FVideoVariantGroup& vg = pwSinf->VideoVariantGroups.Emplace_GetRef();

			// Get the indices of the variant streams in the StreamInfs and sort them
			// back ascending as the ordering in the map is not guaranteed.
			for(auto it=SamePropertyMap.CreateConstKeyIterator(kIt); it; ++it)
			{
				vg.StreamInfIndices.Add((*it).Value);
			}
			vg.StreamInfIndices.Sort();

			// Gather the individual bandwidths to create a list of "qualities" and assign quality indices.
			TArray<int64> SortedBandwidths;
			for(int32 j=0; j<vg.StreamInfIndices.Num(); ++j)
			{
				SortedBandwidths.AddUnique(sInfs[vg.StreamInfIndices[j]].Bandwidth);
			}
			SortedBandwidths.Sort();

			// Add the URL of the media playlist in bandwidth order to the group hash.
			// The idea is that groups that use the same media playlist as a variant but differ in the use
			// of the VIDEO, AUDIO, SUBTITLES or CLOSED-CAPTIONS groups (and hence have a different combined BANDWIDTH
			// so we can't use that as an identifier) can be identified by a hash over all the playlists URLs.
			FSHA1 GroupURLHash;
			TArray<FMultiVariantPlaylistHLS::FStreamInf> SortedStreamInfs;
			for(int32 j=0; j<vg.StreamInfIndices.Num(); ++j)
			{
				SortedStreamInfs.Emplace(sInfs[vg.StreamInfIndices[j]]);
			}
			SortedStreamInfs.StableSort([](const FMultiVariantPlaylistHLS::FStreamInf& a, const FMultiVariantPlaylistHLS::FStreamInf& b){return a.Bandwidth < b.Bandwidth;});
			for(int32 j=0; j<SortedStreamInfs.Num(); ++j)
			{
				UpdateHashStr(GroupURLHash, SortedStreamInfs[j].URI);
			}
			FString HashValue = GroupURLHash.Finalize().ToString();
			if (SameGroupURLHashMap.Contains(HashValue))
			{
				vg.SameAsVideoVariantGroupIndex.Emplace(SameGroupURLHashMap[HashValue]);
			}
			else
			{
				SameGroupURLHashMap.Emplace(HashValue, pwSinf->VideoVariantGroups.Num()-1);
			}

			for(int32 j=0; j<vg.StreamInfIndices.Num(); ++j)
			{
				FMultiVariantPlaylistHLS::FStreamInf& si = sInfs[vg.StreamInfIndices[j]];
				si.QualityIndex = SortedBandwidths.Find(si.Bandwidth);
				for(int32 i=0; i<si.ParsedCodecs.Num(); ++i)
				{
					if (si.ParsedCodecs[i].IsVideoCodec())
					{
						vg.ParsedCodecs.Emplace(si.ParsedCodecs[i]);
						vg.BaseSupplementalCodecs.Append(si.SupplementalCodecs);
					}
				}
				check(vg.VideoRange.IsEmpty() || vg.VideoRange.Equals(si.VideoRange));
				vg.VideoRange = si.VideoRange;
			}
		}
	}
}

void FPlaylistHandlerHLS::GroupAudioOnlyVariantStreams(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	auto UpdateHashStr = [](FSHA1& InOutHash, const FString& InString)	{ if (InString.Len()) InOutHash.UpdateWithString(*InString, InString.Len()); };
	auto GetBaseCodec = [](const FString& InCodec) -> FString
	{
		int32 DotPos;
		if (InCodec.FindChar(TCHAR('.'), DotPos))
		{
			return InCodec.Left(DotPos);
		}
		return InCodec;
	};

	// Group audio streams by property for each pathway, since pathways may have different variant streams.
	for(int32 pwIdx=0; pwIdx<InMultiVariantPlaylist->PathwayStreamInfs.Num(); ++pwIdx)
	{
		TSharedPtrTS<FMultiVariantPlaylistHLS::FPathwayStreamInfs> pwSinf(InMultiVariantPlaylist->PathwayStreamInfs[pwIdx]);
		TArray<FMultiVariantPlaylistHLS::FStreamInf>& sInfs(pwSinf->StreamInfs);

		TMultiMap<FString, int32> SamePropertyMap;
		bool bHasAnyVariantMissingAudioCodec = false;
		for(int32 ns=0,nsMax=sInfs.Num(); ns<nsMax; ++ns)
		{
			FMultiVariantPlaylistHLS::FStreamInf& si = sInfs[ns];
			bHasAnyVariantMissingAudioCodec |= si.bReferencesAudioRenditionWithoutCodec;

			// Generate a (hopefully) unique hash over all the variant stream attributes in an ordered and consistent way.
			FSHA1 Hash;

			bool bHasVideo = false;
			for(int32 i=0; i<si.ParsedCodecs.Num(); ++i)
			{
				// When video is present this will not be an audio-only group.
				// We do not look at subtitles because an audio-only group is allowed to have subtitles (for Karaoke, for example).
				if (si.ParsedCodecs[i].IsVideoCodec())
				{
					bHasVideo = true;
					break;
				}
				else if (si.ParsedCodecs[i].IsAudioCodec())
				{
					UpdateHashStr(Hash, GetBaseCodec(si.Codecs[i]));
				}
			}
			if (!bHasVideo)
			{
				// Do NOT add the audio group as each variant may reference a different group
				// (a variant is the bandwidth-switchable entity, the groups are things like different languages, commentary, etc.)
				UpdateHashStr(Hash, si.SubtitleGroup);
				FString HashValue = Hash.Finalize().ToString();
				SamePropertyMap.Emplace(MoveTemp(HashValue), ns);
			}
		}

		// Go over the tuples of same hash values and create groups
		TArray<FString> KeyArray;
		SamePropertyMap.GetKeys(KeyArray);
		for(auto& kIt : KeyArray)
		{
			FMultiVariantPlaylistHLS::FAudioVariantGroup& ag = pwSinf->AudioOnlyVariantGroups.Emplace_GetRef();

			// Get the indices of the variant streams in the StreamInfs and sort them
			// back ascending as the ordering in the map is not guaranteed.
			for(auto it=SamePropertyMap.CreateConstKeyIterator(kIt); it; ++it)
			{
				ag.StreamInfIndices.Add((*it).Value);
			}
			ag.StreamInfIndices.Sort();

			const FMultiVariantPlaylistHLS::FStreamInf& si = sInfs[ag.StreamInfIndices[0]];
			ag.ParsedCodecs.Append(si.ParsedCodecs);

			// If any variant has missing audio codecs for a rendition group it references
			// and this audio-only variant is in the same group, then add the codec.
			if (bHasAnyVariantMissingAudioCodec && si.AudioGroup.Len())
			{
				for(int32 k=0,nsMax=sInfs.Num(); k<nsMax; ++k)
				{
					FMultiVariantPlaylistHLS::FStreamInf& si2 = sInfs[k];
					if (si.IndexOfSelfInArray != si2.IndexOfSelfInArray && si2.bReferencesAudioRenditionWithoutCodec && si2.AudioGroup.Equals(si2.AudioGroup))
					{
						si2.Codecs.Append(si.Codecs);
						si2.ParsedCodecs.Append(si.ParsedCodecs);
						si2.NumAudioCodec += si.ParsedCodecs.Num();
					}
				}
			}
		}
	}
}

void FPlaylistHandlerHLS::FillInMissingInformation(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist, EFillInOptions InFillInOpts)
{
	/*
		We try to fill in information on renditions that is not provided by the playlist, like the CODECS.
		The CODECS is unfortunately optional and apparently it is not anticipated for a device to not support all the codecs Apple envisions to be used with HLS.
		Historically the only video codec that was used was H.264/AVC and audio for the most part AAC with MPEG audio layer 3 also being a possibility.
		Since we are supporting these on all current devices we pretend that those are used if no CODECS is given.
		This may still fail if any stream is exceeding device capabilities, like 1080p60 but it can't be helped.

		Even if CODECS are provided they do not necessarily extend to alternate renditions in a rendition group.
		Suppose a rendition group contains 4 audio streams in the formats: AAC, AC3, AAC, MP3.
		Since the renditions do NOT support CODECS on their own, we need to make an educated guess as to their respective codecs from those
		codecs specified in the EXT-X-STREAM-INF, hoping that it gives 4 audio codecs we can apply in the order they are given to the alternate renditions.
		If that is not the case we will very likely assign the wrong codec since we use the last audio-type codec for the remaining alternate renditions.

		Likewise, for subtitles a CODEC is only given IF the format is IMSC/TTML. Any other format is not listed.
		This is however a somewhat lesser issue as the only other format in HLS is WebVTT, so we can set that codec (`wvtt`) ourselves.

		Historically HLS seems to have implied there to be audio as the recommendation was to include an audio-only variant, so if no CODECS are
		provided we will assume that audio is implied. If that is then not the case that is too bad.

		Similarly RESOLUTION is optional and sometimes missing. This value is mostly informative, but can be used to apply stream resolution limits.
		If not present we synthesize common values based on BANDWIDTH. Obviously these values will not be correct at all, and if used in the upper
		layer handler to filter streams by resolution will result in anything but proper filtering.

		Finally, if the same BANDWIDTH appears more than once with also matching resolution, codec and framerate (when specified, not synthesized!)
		we assume that this variant is available on a different CDN (if the URL is different).
		To avoid issues we will insert synthesized PATHWAY-ID as if this was a content-steering playlist unless already present in an actual steering
		playlist.
	*/


	// Check fallback CDN before synthesizing any other missing parameters.
	if ((InFillInOpts & EFillInOptions::FallbackCDNs) != EFillInOptions::None)
	{
		CheckForFallbackStreams(InMultiVariantPlaylist);
	}

	// Check for generally missing CODECS
	if ((InFillInOpts & EFillInOptions::Codecs) != EFillInOptions::None)
	{
		CheckForMissingCodecs(InMultiVariantPlaylist);
	}

	// Check for missing RESOLUTION. We do this after potentially adding the codecs
	// because we will be looking at them to see if there is a video codec specified so we do not assign
	// resolution to audio-only variants. This will still be wrong when we generated CODECS but this way
	// we won't add anyting if the codecs are present.
	if ((InFillInOpts & EFillInOptions::Resolution) != EFillInOptions::None)
	{
		CheckForMissingResolution(InMultiVariantPlaylist);
	}

	// Now that we have CODECS and RESOLUTION, assign the resolution to the parsed codec info.
	AssignResolutionAndFrameRateToCodecs(InMultiVariantPlaylist);

	// Apply the CODECS to the rendition groups
	if ((InFillInOpts & EFillInOptions::RenditionCodecs) != EFillInOptions::None)
	{
		AssignCodecsToRenditions(InMultiVariantPlaylist);
	}

	// Check that `SCORE` values are either given for all variants or none of them.
	if ((InFillInOpts & EFillInOptions::Scores) != EFillInOptions::None)
	{
		CheckForScore(InMultiVariantPlaylist);
	}
}

void FPlaylistHandlerHLS::AssignResolutionAndFrameRateToCodecs(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax; ++ns)
	{
		FMultiVariantPlaylistHLS::FStreamInf& si = InMultiVariantPlaylist->InitialStreamInfs[ns];
		if (si.NumVideoCodec)
		{
			for(int32 j=0; j<si.ParsedCodecs.Num(); ++j)
			{
				if (si.ParsedCodecs[j].IsVideoCodec())
				{
					si.ParsedCodecs[j].SetResolution(FStreamCodecInformation::FResolution(si.ResolutionW, si.ResolutionH));
					si.ParsedCodecs[j].SetFrameRate(si.FrameRate);
				}
			}
		}
	}
}

void FPlaylistHandlerHLS::AssignCodecsToRenditions(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	auto FindGroup = [mvp=InMultiVariantPlaylist](FMultiVariantPlaylistHLS::ERenditionGroupType InType, const FString& InName) -> FMultiVariantPlaylistHLS::FRenditionGroup*
	{
		TArray<FMultiVariantPlaylistHLS::FRenditionGroup>& RenditionGroups(mvp->RenditionGroupsOfType[static_cast<int32>(InType)]);
		for(int32 i=0; i<RenditionGroups.Num(); ++i)
		{
			if (RenditionGroups[i].GroupID.Equals(InName))
			{
				return &RenditionGroups[i];
			}
		}
		return nullptr;
	};

	// Go over each variant stream and see what groups it references
	for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax; ++ns)
	{
		FMultiVariantPlaylistHLS::FStreamInf& si = InMultiVariantPlaylist->InitialStreamInfs[ns];
		for(int32 ng=0; ng<3; ++ng)
		{
			FMultiVariantPlaylistHLS::FRenditionGroup* rg = nullptr;
			// Video
			if (ng == 0)
			{
				rg = FindGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Video, si.VideoGroup);
			}
			// Audio
			else if (ng == 1)
			{
				rg = FindGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Audio, si.AudioGroup);
			}
			// Subtitle
			else
			{
				rg = FindGroup(FMultiVariantPlaylistHLS::ERenditionGroupType::Subtitles, si.SubtitleGroup);
			}
			if (!rg)
			{
				continue;
			}
			TArray<FStreamCodecInformation> TypeCodecs;
			TArray<FString> TypeCodecNames;
			for(int32 i=0; i<si.ParsedCodecs.Num(); ++i)
			{
				if ((ng==0 && si.ParsedCodecs[i].IsVideoCodec()) ||
					(ng==1 && si.ParsedCodecs[i].IsAudioCodec()) ||
					(ng==2 && si.ParsedCodecs[i].IsSubtitleCodec()))
				{
					TypeCodecs.Emplace(si.ParsedCodecs[i]);
					TypeCodecNames.Emplace(si.Codecs[i]);
				}
			}

			// Check if an audio group is being referenced for which there is not codec provided.
			// This typically means there is a dedicated audio-only variant stream which itself references the same audio group.
			// We need to check for that later.
			if (ng==1 && TypeCodecNames.IsEmpty())
			{
				si.bReferencesAudioRenditionWithoutCodec = true;
			}

			// Check if a subtitle group is being referenced for which there are no codecs provided.
			// These should be WebVTT subtitles, so for simplicities sake we assign this now.
			if (ng==2 && TypeCodecNames.IsEmpty())
			{
				TypeCodecNames.Emplace(FString(TEXT("wvtt")));
				TypeCodecs.Emplace_GetRef().ParseFromRFC6381(TypeCodecNames.Last());
				si.NumSubtitleCodec = 1;
				si.ParsedCodecs.Append(TypeCodecs);
				si.Codecs.Append(TypeCodecNames);
			}
			// Check if the variant group already has codecs assigned
			if (rg->CodecNamesFromStreamInf.Num())
			{
				// If so, check that what we have here matches the current one
				if (TypeCodecNames != rg->CodecNamesFromStreamInf)
				{
					LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Variant stream references a rendition group with a different list of codecs than a preceeding variant stream did. Playback may not work correctly.")));
				}
			}
			else
			{
				rg->ParsedCodecsFromStreamInf = MoveTemp(TypeCodecs);
				rg->CodecNamesFromStreamInf = MoveTemp(TypeCodecNames);
			}

			// Assign/distribute the codecs onto the renditions in the group
			if (rg->Renditions.Num() > rg->CodecNamesFromStreamInf.Num() && rg->CodecNamesFromStreamInf.Num() > 1)
			{
				LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Rendition group contains more entries than the variant gives codecs for. Playback may not work correctly.")));
			}
			for(int32 i=0; i<rg->Renditions.Num(); ++i)
			{
				const int32 j = i<rg->CodecNamesFromStreamInf.Num() ? i : rg->CodecNamesFromStreamInf.Num()-1;
				if (j < 0)
				{
					LogMessage(IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Variant gives no codecs for rendition group. Playback may not work correctly.")));
					break;
				}
				rg->Renditions[i].ParsedCodecFromStreamInf = rg->ParsedCodecsFromStreamInf[j];
				rg->Renditions[i].CodecNameFromStreamInf = rg->CodecNamesFromStreamInf[j];
				// For audio groups, set the CHANNELS and SAMPLE-RATE
				if (ng == 1)
				{
					rg->Renditions[i].ParsedCodecFromStreamInf.SetNumberOfChannels(rg->Renditions[i].Channels);
					rg->Renditions[i].ParsedCodecFromStreamInf.SetSamplingRate(rg->Renditions[i].SampleRate);
				}
			}
		}
	}
}

void FPlaylistHandlerHLS::CheckForMissingCodecs(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax; ++ns)
	{
		FMultiVariantPlaylistHLS::FStreamInf& si = InMultiVariantPlaylist->InitialStreamInfs[ns];
		if (si.Codecs.IsEmpty())
		{
			LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Variant stream is missing the CODECS attribute. We pretend it to be \"avc1.640028,mp4a.40.2\". Playback may not work correctly.")));

			si.Codecs.Emplace(TEXT("avc1.640028"));
			si.ParsedCodecs.Emplace_GetRef().ParseFromRFC6381(si.Codecs.Last());
			si.Codecs.Emplace(TEXT("mp4a.40.2"));
			si.ParsedCodecs.Emplace_GetRef().ParseFromRFC6381(si.Codecs.Last());
			si.NumVideoCodec = 1;
			si.NumAudioCodec = 1;
		}
	}
}

void FPlaylistHandlerHLS::CheckForMissingResolution(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	// See if any variant stream containing video is missing the RESOLUTION attribute.
	bool bHasMissingResolution = false;
	for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax && !bHasMissingResolution; ++ns)
	{
		FMultiVariantPlaylistHLS::FStreamInf& si = InMultiVariantPlaylist->InitialStreamInfs[ns];
		bHasMissingResolution |= si.NumVideoCodec && si.ResolutionH <= 0;
	}
	if (bHasMissingResolution)
	{
		// If missing we now take the variant streams and sort them by BANDWIDTH,
		// generating a fake resolution for it.
		TArray<FMultiVariantPlaylistHLS::FStreamInf> VideoVariantStreams;
		for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax; ++ns)
		{
			FMultiVariantPlaylistHLS::FStreamInf& si = InMultiVariantPlaylist->InitialStreamInfs[ns];
			if (si.NumVideoCodec && !VideoVariantStreams.ContainsByPredicate([bw=si.Bandwidth](const FMultiVariantPlaylistHLS::FStreamInf& e){return e.Bandwidth == bw;}))
			{
				VideoVariantStreams.Emplace(si);
			}
		}
		// Sort by descending bandwidth
		VideoVariantStreams.Sort([](const FMultiVariantPlaylistHLS::FStreamInf& a, const FMultiVariantPlaylistHLS::FStreamInf& b) { return a.Bandwidth > b.Bandwidth; });
		static const int32 kCommonHeights[] =
		{
			1080, 960, 720, 648, 540, 480, 360, 270
		};
		// Go over the variants in descending bandwidth order and assign a typical resolution for its
		// position if the resolution is missing.
		for(int32 vsi=0; vsi<VideoVariantStreams.Num(); ++vsi)
		{
			const int32 Height = vsi < UE_ARRAY_COUNT(kCommonHeights) ? kCommonHeights[vsi] : kCommonHeights[UE_ARRAY_COUNT(kCommonHeights)-1];
			for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax; ++ns)
			{
				FMultiVariantPlaylistHLS::FStreamInf& si = InMultiVariantPlaylist->InitialStreamInfs[ns];
				if (si.NumVideoCodec && si.Bandwidth == VideoVariantStreams[vsi].Bandwidth && si.ResolutionH <= 0)
				{
					si.ResolutionH = Height;
					si.ResolutionW = Align(Height * 16 / 9, 2);
				}
			}
		}
	}
}

void FPlaylistHandlerHLS::CheckForScore(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	/*
		According to:
			"The SCORE attribute is OPTIONAL, but if any Variant Stream
			contains the SCORE attribute, then all Variant Streams in the
			Multivariant Playlist SHOULD have a SCORE attribute."


		and additionally, according to Apple's HLS authoring specification for Apple devices
		( https://developer.apple.com/documentation/http-live-streaming/hls-authoring-specification-for-apple-devices )

			"The SCORE attribute (if present) MUST be on every variant. Otherwise, the SCORE attribute will be ignored."
	*/
	int32 NumScores = 0;
	for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax; ++ns)
	{
		NumScores += InMultiVariantPlaylist->InitialStreamInfs[ns].Score >= 0.0f ? 1 : 0;
	}
	// If only some variants have a score value, reset it on all of them
	if (NumScores && NumScores != InMultiVariantPlaylist->InitialStreamInfs.Num())
	{
		LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Not all variant streams have a defined SCORE attribute. Ignoring the SCORE on all of them")));
		for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax; ++ns)
		{
			InMultiVariantPlaylist->InitialStreamInfs[ns].Score = -1.0f;
		}
	}
}

void FPlaylistHandlerHLS::CheckForFallbackStreams(TSharedPtr<FMultiVariantPlaylistHLS, ESPMode::ThreadSafe> InMultiVariantPlaylist)
{
	auto UpdateHashStr = [](FSHA1& InOutHash, const FString& InString)	{ if (InString.Len()) InOutHash.UpdateWithString(*InString, InString.Len()); };
	auto UpdateHashI64 = [](FSHA1& InOutHash, const int64 InValue)		{  InOutHash.Update(reinterpret_cast<const uint8*>(&InValue), sizeof(InValue)); };
	TMultiMap<FString, int32> IdenticalAttributeMap;
	for(int32 ns=0,nsMax=InMultiVariantPlaylist->InitialStreamInfs.Num(); ns<nsMax; ++ns)
	{
		const FMultiVariantPlaylistHLS::FStreamInf& si = InMultiVariantPlaylist->InitialStreamInfs[ns];

		// Generate a (hopefully) unique hash over all the variant stream attributes in an ordered and consistent way.
		FSHA1 Hash;
		TArray<FString> SortedCodecsList(si.Codecs);
		SortedCodecsList.Sort();
		for(auto& It : SortedCodecsList)
		{
			UpdateHashStr(Hash, It);
		}
		/*
		Do not add the groups!
			UpdateHashStr(Hash, si.VideoGroup);
			UpdateHashStr(Hash, si.AudioGroup);
			UpdateHashStr(Hash, si.SubtitleGroup);
			UpdateHashStr(Hash, si.ClosedCaptionGroup);
		*/
		UpdateHashStr(Hash, si.PathwayId);			// Use it if it has been set.
		UpdateHashI64(Hash, si.FrameRate.GetNumerator());
		UpdateHashI64(Hash, (int64)si.FrameRate.GetDenominator());
		UpdateHashI64(Hash, (int64)si.Bandwidth);
		UpdateHashI64(Hash, (int64)si.ResolutionW);
		UpdateHashI64(Hash, (int64)si.ResolutionH);
		FString HashValue = Hash.Finalize().ToString();
		IdenticalAttributeMap.Emplace(MoveTemp(HashValue), ns);
	}
	// How many groups with identical hashes were created?
	TArray<FString> KeyArray;
	IdenticalAttributeMap.GetKeys(KeyArray);
	// If there are no duplicates then we do not need to continue any further.
	if (KeyArray.Num() == IdenticalAttributeMap.Num())
	{
		return;
	}
	TMap<FString, int32> CountPerHash;
	for(auto& kIt : KeyArray)
	{
		CountPerHash.Emplace(kIt, IdenticalAttributeMap.Num(kIt));
	}
	TArray<int32> DifferentCounts;
	for(auto& kIt : CountPerHash)
	{
		DifferentCounts.AddUnique(kIt.Value);
	}
	// Are there groups with different counts?
	if (DifferentCounts.Num() > 1)
	{
		// Uneven distribution. Some variants have fallbacks and some do not.
		LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Some variant streams appear to have CDN fallbacks, but not all of them.")));
	}
	// If content steering is used the variants are expected to have pathway ids already assigned.
	// We took those into account up above in the hash, so anything that still comes out as duplicates would also
	// have to have the same id already, which is suspicious as content steering is intended for explicit CDN
	// management and thus duplicate variants should not be listed.
	if (InMultiVariantPlaylist->ContentSteeringParams.bHaveContentSteering)
	{
		LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Assigning generated PATHWAY-ID to like variants in a playlist that uses content steering. This may have undesirable effects.")));
	}
	// Go over the tuples of same hash values and assign generated PATHWAY-ID's to the members.
	TArray<int32> VariantIndicesToRemove;
	for(int32 j=0; j<KeyArray.Num(); ++j)
	{
		// Get the indices of the variant streams in the StreamInfs
		TArray<int32> VariantIndices;
		for(auto it=IdenticalAttributeMap.CreateConstKeyIterator(KeyArray[j]); it; ++it)
		{
			VariantIndices.Add((*it).Value);
		}
		// Sort them ascending as the ordering in the map is not guaranteed.
		VariantIndices.Sort();

		// Check if the URLs are identical for some reason. If so, this is not a fallback.
		// However, there could be variants that refer to different rendition groups.
		TArray<int32> DuplicatesToRemove;
		for(int32 i=1; i<VariantIndices.Num(); ++i)
		{
			const FMultiVariantPlaylistHLS::FStreamInf& si0 = InMultiVariantPlaylist->InitialStreamInfs[VariantIndices[i-1]];
			const FMultiVariantPlaylistHLS::FStreamInf& si1 = InMultiVariantPlaylist->InitialStreamInfs[VariantIndices[i]];
			if (si1.URI.Equals(si0.URI))
			{
				// If the groups are identical as well, this really is a duplicate.
				if (si1.VideoGroup.Equals(si0.VideoGroup) && si1.AudioGroup.Equals(si0.AudioGroup) && si1.SubtitleGroup.Equals(si0.SubtitleGroup))
				{
					DuplicatesToRemove.AddUnique(i);
					VariantIndicesToRemove.AddUnique(VariantIndices[i]);
				}
			}
		}

		// Then assign CDN values
		int32 CDN=0;
		for(int32 i=0; i<VariantIndices.Num(); ++i)
		{
			if (DuplicatesToRemove.Contains(i))
			{
				continue;
			}
			FMultiVariantPlaylistHLS::FStreamInf& si = InMultiVariantPlaylist->InitialStreamInfs[VariantIndices[i]];
			// Enclose the generated name with brackets, which are normally invalid characters for `PATHWAY-ID`
			// to indicate that this is a generated ID.
			++CDN;
			si.PathwayId = FString::Printf(TEXT("[CDN-%02d]"), CDN);
		}
	}
	// Finally, remove all the duplicate variants for good.
	VariantIndicesToRemove.Sort();
	for(int32 i=VariantIndicesToRemove.Num()-1; i>=0; --i)
	{
		InMultiVariantPlaylist->InitialStreamInfs.RemoveAt(VariantIndicesToRemove[i]);
		//LogMessage(IInfoLog::ELevel::Warning, FString::Printf(TEXT("Removing duplicate variant stream.")));
	}
}

} // namespace Electra
