// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlaylistReader.h"
#include "Player/PlayerSessionServices.h"


namespace Electra
{

class IPlaylistHandlerHLS : public IPlaylistReader
{
public:
	static TSharedPtrTS<IPlaylistReader> Create(IPlayerSessionServices* PlayerSessionServices);

	virtual ~IPlaylistHandlerHLS() = default;
};

} // namespace Electra
