// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlaylistParserHLS.h"

namespace Electra
{

FErrorDetail FPlaylistParserHLS::Parse(const FString& InM3U8, const FString& InEffectiveURL, const TArray<FElectraHTTPStreamHeader>& InResponseHeaders)
{
	PlaylistURL = InEffectiveURL;
	ResponseHeaders = InResponseHeaders;
	FURL_RFC3986 UrlParser;
	UrlParser.Parse(InEffectiveURL);
	UrlParser.GetQueryParams(QueryParameters, true);

	FErrorDetail Error;
	StringHelpers::FStringIterator It(InM3U8);
	EParseState State = EParseState::Begin;
	while(SkipWhitespaceAndEOL(It))
	{
		const TCHAR* const Remainder = It.GetRemainder();
		switch(State)
		{
			// Look for the starting `#EXTM3U` tag.
			case EParseState::Begin:
			{
				if (It.GetRemainingLength() > 7)
				{
					if (FCString::Strncmp(Remainder, TEXT("#EXTM3U"), 7) != 0)
					{
						Error.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("HLS playlist does not start with #EXTM3U")));
						return Error;
					}
					else
					{
						State = EParseState::SearchEXT;
						SkipUntilEOL(It);
					}
				}
				break;
			}
			// Search for the next `#EXT` tag.
			case EParseState::SearchEXT:
			{
				// Is this a comment?
				if (*Remainder == TCHAR('#'))
				{
					if (It.GetRemainingLength() >= 4 && FCString::Strncmp(Remainder, TEXT("#EXT"), 4) != 0)
					{
						// Yes, skip it.
						SkipUntilEOL(It);
					}
					else
					{
						EParseState NextState = ParseEXT(Error, It);
						if (NextState == EParseState::Failed)
						{
							return Error;
						}
						State = NextState;
					}
				}
				else
				{
					// Check if there is an `#EXTINF` tag in the list to which this URI applies.
					// If so, and it is not the most recent element, we move it down the list in order
					// to make it easier to apply and preceding tags like EXT_X_BYTERANGE, EXT_X_PROGRAM_DATE_TIME, etc.
					bool bExpecingURI = false;
					if (Elements.Num())
					{
						const int32 LastIdx = Elements.Num() - 1;
						for(int32 ne=LastIdx; ne>=0; --ne)
						{
							if (Elements[ne]->Tag == EExtTag::EXTINF || Elements[ne]->Tag == EExtTag::EXT_X_STREAM_INF)
							{
								if (Elements[ne]->URI.Value.IsEmpty())
								{
									bExpecingURI = true;
									if (ne != LastIdx)
									{
										auto URIElement(MoveTemp(Elements[ne]));
										Elements.RemoveAt(ne);
										Elements.Emplace(MoveTemp(URIElement));
									}
								}
								break;
							}
						}
					}
					if (bExpecingURI)
					{
						int32 RemainingNow = It.GetRemainingLength();

						// Check the URI line for variable substitutions
						const TCHAR* URIStr = It.GetRemainder();
						int32 VarSubstDepth = 0;
						const TCHAR* SubstStart = nullptr;
						TArray<FString> Substitutions;
						while(It && !IsNewline(It))
						{
							// Check for variable substitution in quoted strings.
							if (*URIStr == TCHAR('{') && It.GetRemainingLength() && URIStr[1] == TCHAR('$'))
							{
								if (VarSubstDepth)
								{
									Error.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Found nested variable substitution in URI")));
									return Error;
								}
								++VarSubstDepth;
								SubstStart = URIStr;
							}
							else if (VarSubstDepth && *URIStr == TCHAR('}'))
							{
								Substitutions.Emplace(FString::ConstructFromPtrSize(SubstStart, URIStr+1 - SubstStart));
								--VarSubstDepth;
								SubstStart = nullptr;
							}
							++URIStr;
							++It;
						}
						FString URL = FString::ConstructFromPtrSize(Remainder, RemainingNow - It.GetRemainingLength());
						if (URL.IsEmpty())
						{
							Error.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Empty URI line")));
							return Error;
						}
						Elements.Last()->URI.Value = MoveTemp(URL);
						Elements.Last()->URI.VariableSubstitutions = MoveTemp(Substitutions);
					}
					else
					{
						Error.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Found URI line where it was not expected")));
						return Error;
					}
				}
				break;
			}
		}
	}

	return Error;
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	auto IsTag = [](StringHelpers::FStringIterator& InOutIt, const TCHAR* InTag, int32 InTagLen) -> bool
	{
		bool bMatches = InOutIt.GetRemainingLength() >= InTagLen && FCString::Strncmp(InOutIt.GetRemainder() + 4, InTag + 4, InTagLen - 4) == 0;
		if (bMatches)
		{
			InOutIt += InTagLen;
		}
		return bMatches;
	};

	// The iterator still points to the `#EXT` part. There is no point in comparing against this common prefix all the
	// time, so we skip over it in the IsTag() lambda.
	#define TAGC(Tag) TEXT(Tag":"), sizeof(Tag)
	#define TAGN(Tag) TEXT(Tag), sizeof(Tag)-1
		 if (IsTag(InOutIt, TAGC("#EXTINF")))						return ParseEXTINF(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-PLAYLIST-TYPE")))			return ParseEXT_X_PLAYLIST_TYPE(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGN("#EXT-X-ENDLIST")))				return ParseEXT_X_ENDLIST(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-MEDIA")))					return ParseEXT_X_MEDIA(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-TARGETDURATION")))			return ParseEXT_X_TARGETDURATION(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-MEDIA-SEQUENCE")))			return ParseEXT_X_MEDIA_SEQUENCE(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-DISCONTINUITY-SEQUENCE")))	return ParseEXT_X_DISCONTINUITY_SEQUENCE(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-MAP")))					return ParseEXT_X_MAP(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-PROGRAM-DATE-TIME")))		return ParseEXT_X_PROGRAM_DATE_TIME(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-STREAM-INF")))				return ParseEXT_X_STREAM_INF(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGN("#EXT-X-DISCONTINUITY")))			return ParseEXT_X_DISCONTINUITY(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-BYTERANGE")))				return ParseEXT_X_BYTERANGE(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-KEY")))					return ParseEXT_X_KEY(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-START")))					return ParseEXT_X_START(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-DEFINE")))					return ParseEXT_X_DEFINE(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGN("#EXT-X-GAP")))					return ParseEXT_X_GAP(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-SESSION-KEY")))			return ParseEXT_X_SESSION_KEY(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-CONTENT-STEERING")))		return ParseEXT_X_CONTENT_STEERING(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-SERVER-CONTROL")))			return ParseEXT_X_SERVER_CONTROL(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-SESSION-DATA")))			return ParseEXT_X_SESSION_DATA(OutErr, InOutIt);

	/*
	else if (IsTag(InOutIt, TAGC("#EXT-X-I-FRAME-STREAM-INF")))		return ParseEXT_X_I_FRAME_STREAM_INF(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGN("#EXT-X-INDEPENDENT-SEGMENTS")))	return ParseEXT_X_INDEPENDENT_SEGMENTS(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGN("#EXT-X-I-FRAMES-ONLY")))			return ParseEXTOther(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-PART-INF")))				return ParseEXTOther(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-BITRATE")))				return ParseEXTOther(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-PART")))					return ParseEXTOther(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-DATERANGE")))				return ParseEXTOther(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-SKIP")))					return ParseEXTOther(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-PRELOAD-HINT")))			return ParseEXTOther(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-RENDITION-REPORT")))		return ParseEXTOther(OutErr, InOutIt);
	else if (IsTag(InOutIt, TAGC("#EXT-X-ALLOW-CACHE")))			return ParseEXTOther(OutErr, InOutIt);	// was removed with HLS version 7
	*/
	else															return ParseEXTOther(OutErr, InOutIt);
	#undef TAGN
	#undef TAGC
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_Common(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt, EAttrType AttributeType, EExtTag InTag, EPlaylistTagType InTagType)
{
	TUniquePtr<FElement> Element = MakeUnique<FElement>(InTag);
	bHaveMultiVariantTag |= InTagType == EPlaylistTagType::MultiVariantOnly;
	bHaveVariantTag |= InTagType == EPlaylistTagType::VariantOnly;
	if (AttributeType == EAttrType::List)
	{
		if (!ParseAttributes(OutErr, Element, InOutIt))
		{
			return EParseState::Failed;
		}
	}
	else if (AttributeType == EAttrType::Element)
	{
		if (!ParseElementValue(OutErr, Element, InOutIt))
		{
			return EParseState::Failed;
		}
	}
	Elements.Emplace(MoveTemp(Element));
	return EParseState::SearchEXT;
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXTOther(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	SkipUntilEOL(InOutIt);
	return EParseState::SearchEXT;
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXTINF(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	FAttribute Duration, Title;
	if (!ParseOneAttribute(OutErr, Duration.Value, Duration.bWasQuoted, nullptr, InOutIt))
	{
		return EParseState::Failed;
	}
	const TCHAR* Remainder = InOutIt.GetRemainder();
	int32 RemainingLength = InOutIt.GetRemainingLength();
	SkipUntilEOL(InOutIt);
	Title.Value = FString::ConstructFromPtrSize(Remainder, RemainingLength - InOutIt.GetRemainingLength());

	TUniquePtr<FElement> Element = MakeUnique<FElement>(EExtTag::EXTINF);
	bHaveVariantTag = true;
	Element->ElementValue = MoveTemp(Duration);
	if (!Title.Value.IsEmpty())
	{
		Element->AttributeList.Emplace(MoveTemp(Title));
	}
	Elements.Emplace(MoveTemp(Element));
	return EParseState::SearchEXT;
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_PLAYLIST_TYPE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	EParseState State = ParseEXT_X_Common(OutErr, InOutIt, EAttrType::Element, EExtTag::EXT_X_PLAYLIST_TYPE, EPlaylistTagType::VariantOnly);
	if (State != EParseState::Failed)
	{
		if (Elements.Last()->ElementValue.GetValue().Equals(TEXT("VOD")))
		{
			PlaylistType = EPlaylistType::VOD;
		}
		else if (Elements.Last()->ElementValue.GetValue().Equals(TEXT("EVENT")))
		{
			PlaylistType = EPlaylistType::Event;
		}
		else
		{
			PlaylistType = EPlaylistType::Live;
		}
	}
	return State;
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_ENDLIST(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	bHaveEndList = true;
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::None, EExtTag::EXT_X_ENDLIST, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_MEDIA(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_MEDIA, EPlaylistTagType::MultiVariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_TARGETDURATION(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::Element, EExtTag::EXT_X_TARGETDURATION, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_MEDIA_SEQUENCE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::Element, EExtTag::EXT_X_MEDIA_SEQUENCE, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_DISCONTINUITY_SEQUENCE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::Element, EExtTag::EXT_X_DISCONTINUITY_SEQUENCE, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_MAP(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_MAP, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_PROGRAM_DATE_TIME(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	bHaveProgramDateTime = true;
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::Element, EExtTag::EXT_X_PROGRAM_DATE_TIME, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_STREAM_INF(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	const TCHAR * const LineStart = InOutIt.GetRemainder();
	EParseState State = ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_STREAM_INF, EPlaylistTagType::MultiVariantOnly);
	if (State != EParseState::Failed)
	{
		// For the `EXT-X-STREAM-INF` tag we also store the full line (sans tag). This helps in identifying some older multivariant
		// playlists giving the same stream inf repeatedly with different URIs, which was probably intended to indicate different CDNs.
		const TCHAR * const LineEnd = InOutIt.GetRemainder();
		Elements.Last()->FullLineAfterTag = FString::ConstructFromPtrSize(LineStart, LineEnd-LineStart);
	}
	return State;
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_DISCONTINUITY(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::None, EExtTag::EXT_X_DISCONTINUITY, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_BYTERANGE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::Element, EExtTag::EXT_X_BYTERANGE, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_KEY(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_KEY, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_I_FRAME_STREAM_INF(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_I_FRAME_STREAM_INF, EPlaylistTagType::MultiVariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_START(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_START, EPlaylistTagType::Either);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_DEFINE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	bHaveDefine = true;
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_DEFINE, EPlaylistTagType::Either);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_GAP(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::None, EExtTag::EXT_X_GAP, EPlaylistTagType::VariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_SESSION_KEY(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_SESSION_KEY, EPlaylistTagType::MultiVariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_CONTENT_STEERING(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	bHaveContentSteering = true;
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_CONTENT_STEERING, EPlaylistTagType::MultiVariantOnly);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_SERVER_CONTROL(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_SERVER_CONTROL, EPlaylistTagType::Either);
}

FPlaylistParserHLS::EParseState FPlaylistParserHLS::ParseEXT_X_SESSION_DATA(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt)
{
	return ParseEXT_X_Common(OutErr, InOutIt, EAttrType::List, EExtTag::EXT_X_SESSION_DATA, EPlaylistTagType::MultiVariantOnly);
}


bool FPlaylistParserHLS::ParseOneAttribute(FErrorDetail& OutErr, FString& OutValue, bool& bOutWasQuoted, TArray<FString>* OutSubstitutions, StringHelpers::FStringIterator& InOutIt)
{
	if (InOutIt)
	{
		const TCHAR* Remainder = InOutIt.GetRemainder();
		int32 RemainingLength = InOutIt.GetRemainingLength();
		if (*Remainder != TCHAR('"'))
		{
			bOutWasQuoted = false;
			bool bIsHexValue = RemainingLength > 2 && Remainder[0] == TCHAR('0') && (Remainder[1] == TCHAR('x') || Remainder[1] == TCHAR('X'));
			const TCHAR* Str = InOutIt.GetRemainder();
			int32 VarSubstDepth = 0;
			const TCHAR* SubstStart = nullptr;
			while(InOutIt && !FChar::IsWhitespace(*InOutIt) && *InOutIt != TCHAR(','))
			{
				// Check for variable substitution in hexadecimal strings.
				if (bIsHexValue && *Str == TCHAR('{') && InOutIt.GetRemainingLength() && Str[1] == TCHAR('$'))
				{
					if (VarSubstDepth)
					{
						OutErr.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Found nested variable substitution in hex value")));
						return false;
					}
					++VarSubstDepth;
					SubstStart = Str;
				}
				else if (VarSubstDepth && *Str == TCHAR('}'))
				{
					FString SubstName = FString::ConstructFromPtrSize(SubstStart, Str+1 - SubstStart);
					--VarSubstDepth;
					SubstStart = nullptr;
					if (OutSubstitutions)
					{
						OutSubstitutions->Emplace(MoveTemp(SubstName));
					}
				}
				++Str;

				++InOutIt;
			}
			if (InOutIt && !IsNewline(InOutIt) && *InOutIt != TCHAR(','))
			{
				OutErr.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Failed to parse unquoted attribute value")));
				return false;
			}
			OutValue = FString::ConstructFromPtrSize(Remainder, RemainingLength - InOutIt.GetRemainingLength());
			if (*InOutIt == TCHAR(','))
			{
				++InOutIt;
			}
		}
		else
		{
			bOutWasQuoted = true;
			--RemainingLength;
			++Remainder;
			++InOutIt;
			const TCHAR* Str = InOutIt.GetRemainder();
			int32 VarSubstDepth = 0;
			const TCHAR* SubstStart = nullptr;
			while(InOutIt && !IsNewline(InOutIt) && *InOutIt != TCHAR('"'))
			{
				// Check for variable substitution in quoted strings.
				if (*Str == TCHAR('{') && InOutIt.GetRemainingLength() && Str[1] == TCHAR('$'))
				{
					if (VarSubstDepth)
					{
						OutErr.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Found nested variable substitution in quoted string")));
						return false;
					}
					++VarSubstDepth;
					SubstStart = Str;
				}
				else if (VarSubstDepth && *Str == TCHAR('}'))
				{
					FString SubstName = FString::ConstructFromPtrSize(SubstStart, Str+1 - SubstStart);
					--VarSubstDepth;
					SubstStart = nullptr;
					if (OutSubstitutions)
					{
						OutSubstitutions->Emplace(MoveTemp(SubstName));
					}
				}
				++Str;

				++InOutIt;
			}

			if (InOutIt && *InOutIt != TCHAR('"'))
			{
				OutErr.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Failed to parse quoted attribute value")));
				return false;
			}
			OutValue = FString::ConstructFromPtrSize(Remainder, RemainingLength - InOutIt.GetRemainingLength());
			if (*InOutIt == TCHAR('"'))
			{
				++InOutIt;
			}
			SkipWhitespace(InOutIt);
			if (*InOutIt == TCHAR(','))
			{
				++InOutIt;
			}
		}
		SkipWhitespace(InOutIt);
	}
	return true;
}


bool FPlaylistParserHLS::ParseElementValue(FErrorDetail& OutErr, TUniquePtr<FElement>& InOutElement, StringHelpers::FStringIterator& InOutIt)
{
	if (InOutIt)
	{
		FString Value;
		bool bQuoted = false;
		if (!ParseOneAttribute(OutErr, Value, bQuoted, nullptr, InOutIt))
		{
			return false;
		}
		InOutElement->ElementValue.Value = MoveTemp(Value);
		InOutElement->ElementValue.bWasQuoted = bQuoted;
	}
	return true;
}

bool FPlaylistParserHLS::ParseAttributes(FErrorDetail& OutErr, TUniquePtr<FElement>& InOutElement, StringHelpers::FStringIterator& InOutIt)
{
	enum class EWhat
	{
		Name,
		Value
	};
	EWhat What = EWhat::Name;
	FAttribute Attribute;
	while(InOutIt)
	{
		const TCHAR* const Remainder = InOutIt.GetRemainder();
		int32 RemainingLength = InOutIt.GetRemainingLength();
		if (What == EWhat::Name)
		{
			if (IsNewline(InOutIt))
			{
				return true;
			}
			while(InOutIt && ((*InOutIt >= TCHAR('A') && *InOutIt <= TCHAR('Z')) || (*InOutIt >= TCHAR('0') && *InOutIt <= TCHAR('9')) || *InOutIt == TCHAR('-')))
			{
				++InOutIt;
			}
			Attribute.Name = FString::ConstructFromPtrSize(Remainder, RemainingLength - InOutIt.GetRemainingLength());
			SkipWhitespace(InOutIt);
			if (*InOutIt != TCHAR('='))
			{
				OutErr.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Failed to parse attribute name")));
				return false;
			}
			++InOutIt;
			SkipWhitespace(InOutIt);
			if (!InOutIt || (InOutIt && IsNewline(InOutIt)))
			{
				OutErr.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Unexpected line end after attribute name")));
				return false;
			}
			What = EWhat::Value;
		}
		else
		{
			if (!ParseOneAttribute(OutErr, Attribute.Value, Attribute.bWasQuoted, &Attribute.VariableSubstitutions, InOutIt))
			{
				return false;
			}
			InOutElement->bHaveDuplicateAttribute |= InOutElement->AttributeList.ContainsByPredicate([AttrName=Attribute.Name](const FAttribute& e){ return e.Name.Equals(AttrName); });
			InOutElement->AttributeList.Emplace(MoveTemp(Attribute));
			Attribute.bWasQuoted = false;
			What = EWhat::Name;
		}
	}
	return true;
}


bool FPlaylistParserHLS::Validate_EXT_X_DEFINE(FErrorDetail& OutError, const TUniquePtr<FElement>& InElement)
{
	auto CheckValidName = [](const FString& InString) -> bool
	{
		for(StringHelpers::FStringIterator It(InString); It; ++It)
		{
			if (!((*It >= TCHAR('a') && *It <= TCHAR('z')) ||
				  (*It >= TCHAR('A') && *It <= TCHAR('Z')) ||
				  (*It >= TCHAR('0') && *It <= TCHAR('9')) ||
				   *It == TCHAR('-') ||
				   *It == TCHAR('_')))
			{
				return false;
			}
		}
		return true;
	};
	// Check that we have either `NAME`, `IMPORT` or `QUERYPARAM` but no combinations.
	int32 NumName=0, NumImport=0, NumQueryParam=0, NumValue=0;
	const TArray<FAttribute>& Attributes = InElement->AttributeList;
	for(int32 i=0; i<Attributes.Num(); ++i)
	{
		if (Attributes[i].Name.Equals(TEXT("NAME")))
		{
			++NumName;
			if (!CheckValidName(Attributes[i].Value))
			{
				// Set to some value to trigger the if() below.
				NumName = 100;
				break;
			}
		}
		else if (Attributes[i].Name.Equals(TEXT("VALUE")))
		{
			++NumValue;
		}
		else if (Attributes[i].Name.Equals(TEXT("IMPORT")))
		{
			++NumImport;
			if (!CheckValidName(Attributes[i].Value))
			{
				// Set to some value to trigger the if() below.
				NumImport = 100;
				break;
			}
		}
		else if (Attributes[i].Name.Equals(TEXT("QUERYPARAM")))
		{
			++NumQueryParam;
			if (!CheckValidName(Attributes[i].Value))
			{
				// Set to some value to trigger the if() below.
				NumQueryParam = 100;
				break;
			}
		}
		if (Attributes[i].VariableSubstitutions.Num())
		{
			OutError.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("An EXT-X-DEFINE tag cannot use a variable substitution itself")));
			return false;
		}
	}
	if (NumName + NumImport + NumQueryParam != 1)
	{
		OutError.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Invalid EXT-X-DEFINE tag")));
		return false;
	}
	if (NumImport && bHaveMultiVariantTag)
	{
		OutError.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Invalid EXT-X-DEFINE tag, IMPORT cannot appear in multi variant playlist")));
		return false;
	}
	if (NumName && !NumValue)
	{
		OutError.SetFacility(Facility::EFacility::HLSParser).SetCode(1).SetMessage(FString::Printf(TEXT("Invalid EXT-X-DEFINE tag, NAME also requires VALUE")));
		return false;
	}
	return true;
}


} // namespace Electra
