// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "ErrorDetail.h"
#include "Utilities/StringHelpers.h"
#include "Utilities/URLParser.h"
#include "ElectraHTTPStream.h"

namespace Electra
{

class FPlaylistParserHLS
{
public:
	FPlaylistParserHLS() = default;
	FErrorDetail Parse(const FString& InM3U8, const FString& InEffectiveURL, const TArray<FElectraHTTPStreamHeader>& InResponseHeaders);

	enum class EPlaylistType
	{
		VOD,
		Event,
		Live
	};

	const FString& GetURL() const
	{ return PlaylistURL; }

	const TArray<FElectraHTTPStreamHeader>& GetResponseHeaders() const
	{ return ResponseHeaders; }

	bool IsMultiVariantPlaylist() const
	{ return bHaveMultiVariantTag; }

	bool IsVariantPlaylist() const
	{ return bHaveVariantTag; }

	EPlaylistType GetPlaylistType() const
	{ return PlaylistType; }

	bool HasEndList() const
	{ return bHaveEndList; }

	bool HasProgramDateTime() const
	{ return bHaveProgramDateTime; }

	bool UsesContentSteering() const
	{ return bHaveContentSteering; }

	enum class EExtTag
	{
		EXTINF,
		EXT_X_PLAYLIST_TYPE,
		EXT_X_ENDLIST,
		EXT_X_MEDIA,
		EXT_X_TARGETDURATION,
		EXT_X_MEDIA_SEQUENCE,
		EXT_X_DISCONTINUITY_SEQUENCE,
		EXT_X_MAP,
		EXT_X_PROGRAM_DATE_TIME,
		EXT_X_STREAM_INF,
		EXT_X_DISCONTINUITY,
		EXT_X_BYTERANGE,
		EXT_X_KEY,
		EXT_X_I_FRAME_STREAM_INF,
		EXT_X_INDEPENDENT_SEGMENTS,
		EXT_X_START,
		EXT_X_DEFINE,
		EXT_X_GAP,
		EXT_X_I_FRAMES_ONLY,
		EXT_X_PART_INF,
		EXT_X_SERVER_CONTROL,
		EXT_X_BITRATE,
		EXT_X_PART,
		EXT_X_DATERANGE,
		EXT_X_SKIP,
		EXT_X_PRELOAD_HINT,
		EXT_X_RENDITION_REPORT,
		EXT_X_SESSION_DATA,
		EXT_X_SESSION_KEY,
		EXT_X_CONTENT_STEERING
	};

	struct FVariableSubstitution
	{
		FString Name;
		FString Value;
		bool operator == (const FString& InName) const
		{ return Name.Equals(InName); }
	};

	struct FAttribute
	{
		FString Name;
		FString Value;
		bool bWasQuoted = false;
		TArray<FString> VariableSubstitutions;

		const FString& GetValue() const
		{
			return Value;
		}
		bool GetValue(FString& OutValue, const TArray<FPlaylistParserHLS::FVariableSubstitution>& InVariableValues) const
		{
			if (VariableSubstitutions.IsEmpty())
			{
				OutValue = Value;
				return true;
			}
			else
			{
				FString v(Value);
				for(auto& It : VariableSubstitutions)
				{
					const FPlaylistParserHLS::FVariableSubstitution* SourceValue = InVariableValues.FindByKey(It);
					if (!SourceValue)
					{
						return false;
					}
					v.ReplaceInline(*It, *SourceValue->Value, ESearchCase::CaseSensitive);
				}
				OutValue = MoveTemp(v);
				return true;
			}
		}
	};

	struct FElement
	{
		FElement(EExtTag InTag) : Tag(InTag) {}
		EExtTag Tag;
		TArray<FAttribute> AttributeList;
		FAttribute ElementValue;
		FAttribute URI;
		FString FullLineAfterTag;
		bool bHaveDuplicateAttribute = false;
		const FAttribute* GetAttribute(const TCHAR* const InAttributeName)
		{
			for(int32 i=0,iMax=AttributeList.Num(); i<iMax; ++i)
			{
				if (AttributeList[i].Name.Equals(InAttributeName))
				{
					return &AttributeList[i];
				}
			}
			return nullptr;
		}
	};

	const TArray<TUniquePtr<FElement>>& GetElements() const
	{ return Elements; }

	bool GetQueryParam(FString& OutValue, const FString& InParam) const
	{
		for(auto &It : QueryParameters)
		{
			if (It.Name.Equals(InParam))
			{
				OutValue = It.Value;
				return true;
			}
		}
		return false;
	}

	// Validation methods
	bool Validate_EXT_X_DEFINE(FErrorDetail& OutError, const TUniquePtr<FElement>& InElement);

private:
	static bool IsNewline(const StringHelpers::FStringIterator& InOutIt)
	{
		return *InOutIt == TCHAR('\n') || *InOutIt == TCHAR('\r');
	}
	static bool SkipWhitespace(StringHelpers::FStringIterator& InOutIt)
	{
		while(InOutIt && FChar::IsWhitespace(*InOutIt) && !IsNewline(InOutIt))
		{
			++InOutIt;
		}
		return !!InOutIt;
	}
	static bool SkipWhitespaceAndEOL(StringHelpers::FStringIterator& InOutIt)
	{
		while(InOutIt && FChar::IsWhitespace(*InOutIt))
		{
			++InOutIt;
		}
		return !!InOutIt;
	}
	static bool SkipUntilEOL(StringHelpers::FStringIterator& InOutIt)
	{
		while(InOutIt && !IsNewline(InOutIt))
		{
			++InOutIt;
		}
		return !!InOutIt;
	};

	enum class EPlaylistTagType
	{
		MultiVariantOnly,
		VariantOnly,
		Either
	};

	enum class EParseState
	{
		Begin,
		SearchEXT,
		Failed
	};

	EParseState ParseEXT(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXTINF(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_PLAYLIST_TYPE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_ENDLIST(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_MEDIA(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_TARGETDURATION(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_MEDIA_SEQUENCE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_DISCONTINUITY_SEQUENCE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_MAP(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_PROGRAM_DATE_TIME(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_STREAM_INF(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_DISCONTINUITY(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_BYTERANGE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_KEY(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_I_FRAME_STREAM_INF(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_START(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_DEFINE(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_GAP(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_SESSION_KEY(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_CONTENT_STEERING(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_SERVER_CONTROL(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXT_X_SESSION_DATA(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	EParseState ParseEXTOther(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt);
	enum class EAttrType
	{
		List,
		Element,
		None
	};
	EParseState ParseEXT_X_Common(FErrorDetail& OutErr, StringHelpers::FStringIterator& InOutIt, EAttrType AttributeType, EExtTag InTag, EPlaylistTagType InTagType);

	bool ParseOneAttribute(FErrorDetail& OutErr, FString& OutValue, bool& bOutWasQuoted, TArray<FString>* OutSubstitutions, StringHelpers::FStringIterator& InOutIt);
	bool ParseAttributes(FErrorDetail& OutErr, TUniquePtr<FElement>& InOutElement, StringHelpers::FStringIterator& InOutIt);
	bool ParseElementValue(FErrorDetail& OutErr, TUniquePtr<FElement>& InOutElement, StringHelpers::FStringIterator& InOutIt);

	// The query parameters from the playlist URL. Use an array over a map to maintain case!
	TArray<Electra::FURL_RFC3986::FQueryParam> QueryParameters;
	TArray<FElectraHTTPStreamHeader> ResponseHeaders;
	FString PlaylistURL;

	TArray<TUniquePtr<FElement>> Elements;
	bool bHaveMultiVariantTag = false;
	bool bHaveVariantTag = false;
	bool bHaveEndList = false;
	bool bHaveProgramDateTime = false;
	bool bHaveDefine = false;
	bool bHaveContentSteering = false;
	EPlaylistType PlaylistType = EPlaylistType::Live;
};

} // namespace Electra
