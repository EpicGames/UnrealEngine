// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "StreamTypes.h"
#include "StreamAccessUnitBuffer.h"
#include "HTTP/HTTPManager.h"
#include "Player/PlaybackTimeline.h"
#include "Player/AdaptiveStreamingPlayerMetrics.h"
#include "Player/PlayerSessionServices.h"
#include "ParameterDictionary.h"
#include "MediaStreamMetadata.h"

namespace Electra
{

	class IStreamReader;

	enum class EMediaFormatType
	{
		Unknown,
		ISOBMFF,					// mp4
		HLS,						// Apple HLS (HTTP Live Streaming)
		DASH,						// MPEG DASH
		MKV,						// Matroska / WebM
		MPEGAudio					// MPEG audio (eg .mp3)
	};


	struct FPlayStartOptions
	{
		FPlayStartOptions()
		{
			PlaybackRange.Start = FTimeValue::GetZero();
			PlaybackRange.End = FTimeValue::GetPositiveInfinity();
		}
		FTimeRange		PlaybackRange;
		bool			bFrameAccuracy = false;
	};

	struct FPlayStartPosition
	{
		FTimeValue			Time;
		FPlayStartOptions	Options;
	};

	struct FLowLatencyDescriptor
	{
		struct FLatency
		{
			int64 ReferenceID = -1;
			FTimeValue Target;
			FTimeValue Min;
			FTimeValue Max;
		};
		struct FPlayRate
		{
			FTimeValue Min;
			FTimeValue Max;
		};
		FLatency Latency;
		FPlayRate PlayRate;

		FTimeValue GetLatencyMin() const
		{ return Latency.Min; }
		FTimeValue GetLatencyMax() const
		{ return Latency.Max; }
		FTimeValue GetLatencyTarget() const
		{ return Latency.Target; }
		FTimeValue GetPlayrateMin() const
		{ return PlayRate.Min; }
		FTimeValue GetPlayrateMax() const
		{ return PlayRate.Max; }
	};

	struct IProducerReferenceTimeInfo
	{
		enum class EType
		{
			Encoder,
			Captured
		};
		virtual FTimeValue GetWallclockTime() const = 0;
		virtual uint64 GetPresentationTime() const = 0;
		virtual uint32 GetID() const = 0;
		virtual EType GetType() const = 0;
		virtual bool GetIsInband() const = 0;
	};



	class IStreamSegment : public TSharedFromThis<IStreamSegment, ESPMode::ThreadSafe>
	{
	public:
		virtual ~IStreamSegment() = default;

		/**
		 * Sets a playback sequence ID. This is useful to check the ID later in asynchronously received messages
		 * to validate the request still being valid or having become outdated.
		 */
		virtual void SetPlaybackSequenceID(uint32 PlaybackSequenceID) = 0;
		/**
		 * Returns the playback sequence ID. See SetPlaybackSequenceID()
		 */
		virtual uint32 GetPlaybackSequenceID() const = 0;

		virtual void SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay) = 0;

		virtual FTimeValue GetExecuteAtUTCTime() const = 0;

		virtual EStreamType GetType() const = 0;

		/**
		 * Returns a list of dependent streams.
		 */
		virtual void GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const = 0;

		/**
		 * Returns a list of requested streams. This must be the initial request that has a list of
		 * all the segments of the streams selected for playback.
		 */
		virtual void GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams) = 0;

		/**
		 * Returns a list of streams, primary or dependent, that are already at EOS in this request.
		 */
		virtual void GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams) = 0;

		/**
		 * Returns the first PTS value as indicated by the media timeline. This should correspond to the actual absolute PTS of the sample.
		 */
		virtual FTimeValue GetFirstPTS() const = 0;

		/**
		 * Returns the time range this request covers.
		 */
		virtual FTimeRange GetTimeRange() const = 0;

		virtual int32 GetQualityIndex() const = 0;
		virtual int32 GetBitrate() const = 0;

		virtual void GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const = 0;

		virtual bool GetStartupDelay(FTimeValue& OutStartTime, FTimeValue& OutTimeIntoSegment, FTimeValue& OutSegmentDuration) const = 0;
	};



	class IManifest : public TMediaNoncopyable<IManifest>
	{
	public:
		enum class EType
		{
			OnDemand,						//!< An on-demand presentation
			Live,							//!< A live presentation
		};

		enum class EReplayEventType
		{
			NoReplay,
			IsReplay
		};

		enum class ESearchType
		{
			Closest,				//!< Find closest match
			After,					//!< Find match only for fragment times >= target time
			Before,					//!< Find match only for fragment times <= target time
			StrictlyAfter,			//!< Match must be strictly after (>). Used to locate the next segment.
			StrictlyBefore,			//!< Match must be strictly before (<). Used to locate the previous segment.
			Same,					//!< Match must be for the same fragment
		};

		class FResult
		{
		public:
			enum class EType
			{
				Found,				//!< Found
				NotFound,			//!< Not found
				PastEOS,			//!< Time is beyond the duration
				BeforeStart,		//!< Time is before the start time
				TryAgainLater,		//!< Not found at the moment. Playlist load may be pending
				NotLoaded,			//!< Not loaded (playlist has not been requested)
				Error				//!< An unrecoverable error
			};
			FResult(EType InType = EType::NotFound)
				: Type(InType)
			{
			}
			FResult& RetryAfterMilliseconds(int32 Milliseconds)
			{
				Type = EType::TryAgainLater;
				RetryAgainAtTime = MEDIAutcTime::Current() + FTimeValue().SetFromMilliseconds(Milliseconds);
				return *this;
			}
			FResult& RetryAfter(const FTimeValue& InAfter)
			{
				if (InAfter.IsValid())
				{
					Type = EType::TryAgainLater;
					RetryAgainAtTime = MEDIAutcTime::Current() + InAfter;
				}
				return *this;
			}
			FResult& SetErrorDetail(const FErrorDetail& InErrorDetail)
			{
				ErrorDetail = InErrorDetail;
				return *this;
			}
			EType GetType() const
			{
				return Type;
			}
			bool IsSuccess() const
			{
				return GetType() == EType::Found;
			}
			const FTimeValue& GetRetryAgainAtTime() const
			{
				return RetryAgainAtTime;
			}
			const FErrorDetail& GetErrorDetail() const
			{
				return ErrorDetail;
			}
			static const TCHAR* GetTypeName(EType s)
			{
				switch(s)
				{
					case EType::Found:
					{
						return TEXT("Found");
					}
					case EType::NotFound:
					{
						return TEXT("Not found");
					}
					case EType::PastEOS:
					{
						return TEXT("Behind EOF");
					}
					case EType::BeforeStart:
					{
						return TEXT("Before start");
					}
					case EType::TryAgainLater:
					{
						return TEXT("Try again later");
					}
					case EType::NotLoaded:
					{
						return TEXT("Not loaded");
					}
					case EType::Error:
					{
						return TEXT("Error");
					}
					default:
					{
						return TEXT("undefined");
					}
				}
			}
		protected:
			EType			Type;
			FTimeValue		RetryAgainAtTime;
			FErrorDetail	ErrorDetail;
		};



		virtual ~IManifest() = default;

		//-------------------------------------------------------------------------
		// Presentation related functions
		//

		//! Returns the media format of this presentation.
		virtual EMediaFormatType GetMediaFormatType() const = 0;
		//! Returns true if the media playlist will be updated periodically, false if not.
		virtual bool ReloadsPlaylistPeriodically() const = 0;
		//! Returns true if there could be more than one track per kind to switch between.
		virtual bool SupportsMultipleTracksPerKind() const = 0;
		//! Returns true if during playback the order of tracks can change (like in DASH period transitions).
		virtual bool OrderOfTracksCanChangeDynamically() const = 0;

		//! Returns the type of this presentation, either on-demand or live.
		virtual EType GetPresentationType() const = 0;

		//! Returns the type of replay event.
		virtual EReplayEventType GetReplayType() const = 0;

		//! Returns the low-latency descriptor, if any. May return nullptr if there is none.
		virtual TSharedPtrTS<const FLowLatencyDescriptor> GetLowLatencyDescriptor() const = 0;

		//! Calculates the live latency given the current playback position and optional encoder latency.
		virtual FTimeValue CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const = 0;


		//-------------------------------------------------------------------------
		// Presentation and play time related functions.
		//

		/**
		 * Returns the time value the timeline is anchored at.
		 * Usually this is a UTC timestamp at which the presentation began but it could
		 * literally be anything.
		 * All time values are absolute and have this anchor added in.
		 *
		 * @return The time this presentation timeline is anchored at.
		 */
		virtual FTimeValue GetAnchorTime() const = 0;

		/**
		 * Returns the total time range of the timeline, including the end time of the last sample
		 * to which the player will not be able to seek to.
		 *
		 * @return Time range of assets on the timeline.
		 */
		virtual FTimeRange GetTotalTimeRange() const = 0;

		/**
		 * Returns the seekable range of the timeline, which is a subset of the total
		 * time range. Playback can start at any point in the seekable range with
		 * additional constraints imposed by the format.
		 *
		 * @return Time range in which playback can start.
		 */
		virtual FTimeRange GetSeekableTimeRange() const = 0;

		enum class EPlaybackRangeType
		{
			/**
			 * Initial playback range as may be defined using `#t=s,e` URL fragment parameter.
			 * This is used only on first playstart and is canceled when a Seek() is performed.
			 */
			TemporaryPlaystartRange,
			/**
			 * Fixed playback range that may be defined using `#r=s,e` URL fragment parameter.
			 * This is a non-standard parameter. The specified range will be locked in place
			 * and any Seek() can only be performed inside this range.
			 */
			LockedPlaybackRange
		};
		/**
		 * Returns the playback range on the timeline, which is a subset of the total
		 * time range. This may be set through manifest internal means or by URL fragment
		 * parameters where permissable (eg. example.mp4#t=22,50).
		 * If start or end are not specified they will be set to invalid.
		 *
		 * @return Optionally set time range to which playback is restricted.
		 */
		virtual FTimeRange GetPlaybackRange(EPlaybackRangeType InRangeType) const = 0;

		/**
		 * Returns the duration of the assets on the timeline.
		 * Typically this is the difference of the end and start values of the total time range
		 * unless this is a Live presentation for which the duration will be set to infinite.
		 *
		 * @return Duration of the timeline.
		 */
		virtual FTimeValue GetDuration() const = 0;

		/**
		 * Returns the playback start time as defined by the presentation itself.
		 * If the presentation has no preferred start time an invalid value is returned.
		 */
		virtual FTimeValue GetDefaultStartTime() const = 0;

		/**
		 * Clears the internal default start time so it will not be used again.
		 */
		virtual void ClearDefaultStartTime() = 0;

		/**
		 * Returns the playback end time as defined by the presentation itself.
		 * If the presentation has no preferred end time an invalid value is returned.
		 */
		virtual FTimeValue GetDefaultEndTime() const = 0;

		/**
		 * Clears the internal default end time so it will not be used again.
		 */
		virtual void ClearDefaultEndTime() = 0;

		//! Returns track metadata. For period based presentations the streams can be different per period in which case the metadata of the first period is returned.
		virtual void GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const = 0;

		//! Updates long term playback metadata (mostly for audio casts)
		virtual void UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData) = 0;

		//
		virtual FTimeValue GetMinBufferTime() const = 0;

		//
		virtual TSharedPtrTS<IProducerReferenceTimeInfo> GetProducerReferenceTimeInfo(int64 ID) const = 0;

		virtual FTimeValue GetDesiredLiveLatency() const = 0;

		enum class ELiveEdgePlayMode
		{
			// Never play on the Live edge.
			Never,
			// Play on Live edge on start, disable when a Pause() or Seek() is issued unless it's a seek to Live edge.
			Default,
			// Always play on the Live edge (stream can't or should not be paused)
			Always,
		};
		virtual ELiveEdgePlayMode GetLiveEdgePlayMode() const = 0;


		enum class EPlayRateType
		{
			UnthinnedRate,
			ThinnedRate
		};
		virtual TRangeSet<double> GetPossiblePlaybackRates(EPlayRateType InForType) const = 0;

		//! Needs to be called when the user has explicitly triggered a seek, including a programmatic loop back to the beginning.
		//! For presentations with dynamic content changes (eg. DASH xlink:onRequest Periods) the content may need to be updated
		//! again. This is different to internal seeking for retry purposes where content will not be re-resolved.
		virtual void UpdateDynamicRefetchCounter() = 0;

		/**
		 * Prepares the manifest for looping. This does not normally do anything except for when this is a replay,
		 * in which case the internal timestamps may be modified such that the entire presentation is shifted into
		 * the future by one presentation duration.
		 */
		virtual void PrepareForLooping(int32 InNumLoopsToAdd) = 0;

		enum class EClockSyncType
		{
			Recommended,
			Required
		};
		virtual void TriggerClockSync(EClockSyncType InClockSyncType) = 0;

		virtual void TriggerPlaylistRefresh() = 0;

		virtual void ReachedStableBuffer() = 0;

		//-------------------------------------------------------------------------
		// Stream fragment reader
		//
		//! Create a stream reader handler suitable for streams from this manifest.
		virtual IStreamReader* CreateStreamReaderHandler() = 0;



		/**
		 * Interface to a playback period.
		 */
		class IPlayPeriod : private TMediaNoncopyable<IPlayPeriod>
		{
		public:
			virtual ~IPlayPeriod() = default;

			virtual void SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) = 0;

			enum class EReadyState
			{
				NotLoaded,
				Loading,
				Loaded,
				Preparing,
				IsReady,
			};
			virtual EReadyState GetReadyState() = 0;
			virtual void Load() = 0;
			virtual void PrepareForPlay() = 0;

			//! Returns the bitrate of the default stream (usually the first one specified).
			virtual int64 GetDefaultStartingBitrate() const = 0;

			/**
			 * Returns the selected stream's buffer source information for the specified stream type
			 * after setting the stream preferences via SetStreamPreferences() and a following PrepareToPlay().
			 * Valid only when GetReadyState() returns IsReady.
			 * If a nullptr is returned there is no stream for this type.
			 */
			virtual TSharedPtrTS<FBufferSourceInfo> GetSelectedStreamBufferSourceInfo(EStreamType StreamType) = 0;

			virtual FString GetSelectedAdaptationSetID(EStreamType StreamType) = 0;

			enum class ETrackChangeResult
			{
				Changed,
				NotChanged,
				NewPeriodNeeded,
				StartOver
			};

			/**
			 * Changes the specified track over to one of the passed attributes.
			 * If Changed is returned the new selection has been made active and the new FBufferSourceInfo can be fetched
			 * with GetSelectedStreamBufferSourceInfo(). If NotChanged is returned no change has been made, either because
			 * the track is already selected or no (better) match could be made.
			 * If NewPeriodNeeded is returned this play period cannot perform track changing on the fly and a new period
			 * with new stream preferences must be created instead.
			 */
			virtual ETrackChangeResult ChangeTrackStreamPreference(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) = 0;

			/**
			 * Returns the media asset for this play period.
			 *
			 * @return Pointer to the media asset.
			 */
			virtual TSharedPtrTS<ITimelineMediaAsset> GetMediaAsset() const = 0;

			/**
			 * Selects a specific stream to be used for all next segment downloads.
			 *
			 * @param AdaptationSetID
			 * @param RepresentationID
			 */
			virtual void SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex) = 0;

			struct FInitSegmentPreload
			{
				FString AdaptationSetID;
				FString RepresentationID;
			};
			/**
			 * Triggers pre-loading of initialization segments.
			 * This may be called multiple times with different streams. The implementation needs to keep track
			 * of which init segments have already been loaded.
			 */
			virtual void TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload) = 0;

			/**
			 * Sets up a starting segment request to begin playback at the specified time.
			 * The streams selected through SelectStream() will be used.
			 *
			 * @param OutSegment
			 * @param InSequenceState
			 * @param StartPosition
			 * @param SearchType
			 *
			 * @return
			 */
			virtual FResult GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) = 0;


			/**
			 * Same as GetStartingSegment() except this is for a specific stream (video, audio, ...) only.
			 * To be used when a track (language) change is made and a new segment is needed at the current playback position.
			 */
			virtual FResult GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType StreamType, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) = 0;

			/**
			 * Sets up a starting segment request to loop playback to.
			 * The streams selected through SelectStream() will be used.
			 *
			 * @param OutSegment
			 * @param InSequenceState
			 * @param StartPosition
			 * @param SearchType
			 *
			 * @return
			 */
			virtual FResult GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) = 0;

			/**
			 * Gets the segment request for the segment following the specified earlier request.
			 *
			 * @param OutSegment
			 * @param CurrentSegment
			 * @param Options
			 *
			 * @return
			 */
			virtual FResult GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options) = 0;

			/**
			 * Gets the segment request for the same segment on a different quality level or CDN.
			 *
			 * @param OutSegment
			 * @param CurrentSegment
			 * @param Options
			 * @param bReplaceWithFillerData
			 *
			 * @return
			 */
			virtual FResult GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData) = 0;

			/**
			 * Called by the ABR to increase the delay in fetching the next segment in case the segment returned a 404 when fetched at
			 * the announced availability time. This may reduce 404's on the next segment fetches.
			 *
			 * @param IncreaseAmount
			 */
			virtual void IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount) = 0;

			/**
			 * Returns the average segment duration.
			 */
			virtual void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID) = 0;
		};

		//! Finds the playback period the specified start time falls into.
		virtual FResult FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType) = 0;

		//! Locates the period following the given segment.
		virtual FResult FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment) = 0;
	};


	/**
	 * Playlist metadata changed.
	 */
	class FPlaylistMetadataUpdateMessage : public IPlayerMessage
	{
	public:
		static TSharedPtrTS<IPlayerMessage> Create(FTimeValue InValidFrom, TSharedPtrTS<FMetadataParser> InMetadata, bool bInTriggerInternalRefresh)
		{
			TSharedPtrTS<FPlaylistMetadataUpdateMessage> p(new FPlaylistMetadataUpdateMessage(InValidFrom, InMetadata, bInTriggerInternalRefresh));
			return p;
		}

		static const FString& Type()
		{
			static FString TypeName("PlaylistMetadataUpdate");
			return TypeName;
		}

		virtual const FString& GetType() const
		{
			return Type();
		}

		FTimeValue GetValidFrom() const
		{
			return ValidFrom;
		}

		TSharedPtrTS<FMetadataParser> GetMetadata() const
		{
			return Metadata;
		}

		bool GetTriggerInternalRefresh() const
		{
			return bTriggerInternalRefresh;
		}

	private:
		FPlaylistMetadataUpdateMessage(FTimeValue InValidFrom, TSharedPtrTS<FMetadataParser> InMetadata, bool bInTriggerInternalRefresh)
			: ValidFrom(InValidFrom)
			, Metadata(MoveTemp(InMetadata))
			, bTriggerInternalRefresh(bInTriggerInternalRefresh)
		{ }
		FTimeValue ValidFrom;
		TSharedPtrTS<FMetadataParser> Metadata;
		bool bTriggerInternalRefresh;
	};


} // namespace Electra


