// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PlayerCore.h"
#include "PlayerTime.h"
#include "StreamTypes.h"


namespace Electra
{
	class IPlaybackAssetRepresentation
	{
	public:
		virtual ~IPlaybackAssetRepresentation() = default;

		virtual FString GetUniqueIdentifier() const = 0;

		virtual const FStreamCodecInformation& GetCodecInformation() const = 0;

		virtual int32 GetBitrate() const = 0;

		virtual int32 GetQualityIndex() const = 0;

		virtual bool CanBePlayed() const = 0;
	};

	class IPlaybackAssetAdaptationSet
	{
	public:
		virtual ~IPlaybackAssetAdaptationSet() = default;

		/**
		 * Returns a unique identifier for this adaptation set.
		 * This may be a value from the manifest or an internally generated one.
		 * NOTE: The identifier is unique only within the owning media asset!
		 *
		 * @return Unique identifier for this adaptation set with the owning media asset.
		 */
		virtual FString GetUniqueIdentifier() const = 0;

		virtual FString GetListOfCodecs() const = 0;

		virtual const BCP47::FLanguageTag& GetLanguageTag() const = 0;

		virtual int32 GetNumberOfRepresentations() const = 0;

		virtual bool IsLowLatencyEnabled() const = 0;

		virtual TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByIndex(int32 RepresentationIndex) const = 0;

		virtual TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByUniqueIdentifier(const FString& UniqueIdentifier) const = 0;
	};


	class ITimelineMediaAsset
	{
	public:
		virtual ~ITimelineMediaAsset() = default;

		/**
		 * Returns the time range of this asset on the playback timeline.
		 * The timeline anchor time is included in this range.
		 *
		 * @return Time range of this asset on the playback timeline.
		 */
		virtual FTimeRange GetTimeRange() const = 0;

		/**
		 * Returns the duration of this asset, which is typically the difference between
		 * the end and start value of GetTimeRange() unless it is last asset of a Live
		 * presentation timeline for which the duration is infinite.
		 *
		 * @return Duration of this asset.
		 */
		virtual FTimeValue GetDuration() const = 0;

		/**
		 * Returns the asset identifier, if present in the manifest, for this asset.
		 *
		 * @return Asset identifier as it appears in the manifest.
		 */
		virtual FString GetAssetIdentifier() const = 0;


		/**
		 * Returns a unique identifier for this asset.
		 * This may be a value from the manifest or an internally generated one.
		 *
		 * @return Unique identifier for this asset.
		 */
		virtual FString GetUniqueIdentifier() const = 0;


		/**
		 * Returns the number of "adaptation sets" for a particular type of stream.
		 * An adaptation set is defined as a group of streams representing the same content at different
		 * quality levels to dynamically switch between.
		 * Streams are further grouped by language and codec (there will not be a mix of either in a
		 * single adaptation set).
		 *
		 * @param OfStreamType
		 *               Type of elementary stream for which to the the number of available adaptation sets.
		 *
		 * @return Number of adaptation sets for the specified type.
		 */
		virtual int32 GetNumberOfAdaptationSets(EStreamType OfStreamType) const = 0;

		/**
		 * Returns the media asset's adaptation set by index.
		 *
		 * @param OfStreamType
		 *               Type of elementary stream for which to get the adaptation set.
		 * @param AdaptationSetIndex
		 *               Index of the adaptation set to return.
		 *
		 * @return Shared pointer to the requested adaptation set.
		 */
		virtual TSharedPtrTS<IPlaybackAssetAdaptationSet> GetAdaptationSetByTypeAndIndex(EStreamType OfStreamType, int32 AdaptationSetIndex) const = 0;


		/**
		 * Returns "track" metadata for the specified type of stream. This is essentially the adaptation sets per type with their
		 * individual representations.
		 */
		virtual void GetMetaData(TArray<FTrackMetadata>& OutMetadata, EStreamType OfStreamType) const = 0;

		/**
		 * Updates asset metadata on long running Livestreams.
		 */
		virtual void UpdateRunningMetaData(const FString& InKindOfValue, const FVariant& InNewValue) = 0;
	};


} // namespace Electra


