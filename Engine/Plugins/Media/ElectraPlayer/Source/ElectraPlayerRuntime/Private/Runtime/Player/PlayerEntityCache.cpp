// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerEntityCache.h"
#include "Player/PlayerSessionServices.h"
#include "Containers/LruCache.h"

namespace Electra
{

class FPlayerEntityCache : public IPlayerEntityCache
{
public:
	FPlayerEntityCache(IPlayerSessionServices* SessionServices);
	virtual ~FPlayerEntityCache();

	virtual void HandleEntityExpiration() override;
	virtual void SetRecentResponseHeaders(EEntityType InForEntity, const FString& InURL, const TArray<HTTP::FHTTPHeader>& InResponseHeaders) override;
	virtual void GetRecentResponseHeaders(FString& OutURL, TArray<HTTP::FHTTPHeader>& OutResponseHeaders, EEntityType InForEntity) const override;
	virtual void CacheEntity(const FCacheItem& EntityToAdd) override;
	virtual bool GetCachedEntity(FCacheItem& OutCachedEntity, const FString& URL, const FString& Range) override;

private:
	struct FEntityResponse
	{
		FString URL;
		TArray<HTTP::FHTTPHeader> Headers;
	};

	static const uint32 kMaxCacheEntries = 256;

	IPlayerSessionServices* SessionServices;

	mutable FCriticalSection Lock;
	TLruCache<FString, FCacheItem> 	Cache;
	TMap<EEntityType, FEntityResponse> RecentResponseHeaders;
};

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtrTS<IPlayerEntityCache> IPlayerEntityCache::Create(IPlayerSessionServices* SessionServices)
{
	return MakeSharedTS<FPlayerEntityCache>(SessionServices);
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

FPlayerEntityCache::FPlayerEntityCache(IPlayerSessionServices* InSessionServices)
	: SessionServices(InSessionServices)
	, Cache(kMaxCacheEntries)
{
}

FPlayerEntityCache::~FPlayerEntityCache()
{
	FScopeLock ScopeLock(&Lock);
	Cache.Empty();
	RecentResponseHeaders.Empty();
}


void FPlayerEntityCache::HandleEntityExpiration()
{
}

void FPlayerEntityCache::SetRecentResponseHeaders(EEntityType InForEntity, const FString& InURL, const TArray<HTTP::FHTTPHeader>& InResponseHeaders)
{
	FScopeLock ScopeLock(&Lock);
	RecentResponseHeaders.Emplace(InForEntity, FEntityResponse({InURL, InResponseHeaders}));
}

void FPlayerEntityCache::GetRecentResponseHeaders(FString& OutURL, TArray<HTTP::FHTTPHeader>& OutResponseHeaders, EEntityType InForEntity) const
{
	FScopeLock ScopeLock(&Lock);
	if (RecentResponseHeaders.Contains(InForEntity))
	{
		OutURL = RecentResponseHeaders[InForEntity].URL;
		OutResponseHeaders = RecentResponseHeaders[InForEntity].Headers;
	}
	else
	{
		OutURL.Empty();
		OutResponseHeaders.Empty();
	}
}

void FPlayerEntityCache::CacheEntity(const FCacheItem& EntityToAdd)
{
	FString CachedURL = EntityToAdd.Range + EntityToAdd.URL;
	FScopeLock ScopeLock(&Lock);
	Cache.Remove(CachedURL);
	Cache.Add(CachedURL, EntityToAdd);
}

bool FPlayerEntityCache::GetCachedEntity(FCacheItem& OutCachedEntity, const FString& URL, const FString& Range)
{
	FString CachedURL = Range + URL;
	FScopeLock ScopeLock(&Lock);
	const FCacheItem* Item = Cache.FindAndTouch(CachedURL);
	if (Item)
	{
		OutCachedEntity = *Item;
		return true;
	}
	return false;
}




} // namespace Electra

