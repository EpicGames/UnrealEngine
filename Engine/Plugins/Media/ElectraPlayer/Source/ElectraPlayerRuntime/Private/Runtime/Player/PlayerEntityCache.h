// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "ParameterDictionary.h"
#include "HTTP/HTTPManager.h"
#include "MP4BoxBase.h"

namespace Electra
{
class IPlayerSessionServices;
class IParserISO14496_12;
class FParsedMOOVCommon;
class IParserMKV;



class IPlayerEntityCache
{
public:
	enum class EEntityType
	{
		Document,
		Segment,
		XLink,
		Callback,
		Other
	};

	struct FCacheItem
	{
		FString URL;
		FString Range;
		FTimeValue ExpiresAtUTC;
		TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe> ISOBMFFBox;
		TSharedPtr<FParsedMOOVCommon, ESPMode::ThreadSafe> ParsedMOOV;
		TSharedPtrTS<const IParserISO14496_12>	Parsed14496_12Data;
		TSharedPtrTS<const IParserMKV>	ParsedMatroskaData;
		TSharedPtrTS<const TArray<uint8>> RawPayloadData;
	};

	static TSharedPtrTS<IPlayerEntityCache> Create(IPlayerSessionServices* SessionServices);

	virtual ~IPlayerEntityCache() = default;

	/**
	 * Call this periodically to handle expiration times of cached entities.
	 */
	virtual void HandleEntityExpiration() = 0;

	/**
	 * Sets the HTTP response headers of the most recently made entity request.
	 */
	virtual void SetRecentResponseHeaders(EEntityType InForEntity, const FString& InURL, const TArray<HTTP::FHTTPHeader>& InResponseHeaders) = 0;

	/**
	 * Returns the HTTP response headers of the most recently made entity request.
	 */
	virtual void GetRecentResponseHeaders(FString& OutURL, TArray<HTTP::FHTTPHeader>& OutResponseHeaders, EEntityType InForEntity) const = 0;

	/**
	 * Add an entity to the cache.
	 */
	virtual void CacheEntity(const FCacheItem& EntityToAdd) = 0;
	/**
	 * Return a cached entity.
	 * Returns true if found, false if not.
	 */
	virtual bool GetCachedEntity(FCacheItem& OutCachedEntity, const FString& URL, const FString& Range) = 0;

protected:
	IPlayerEntityCache() = default;
	IPlayerEntityCache(const IPlayerEntityCache& other) = delete;
	IPlayerEntityCache& operator = (const IPlayerEntityCache& other) = delete;
};


} // namespace Electra

