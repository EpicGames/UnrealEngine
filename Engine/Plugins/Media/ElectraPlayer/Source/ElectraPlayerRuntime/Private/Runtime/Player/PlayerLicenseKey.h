// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlayerSessionServices.h"
#include "ParameterDictionary.h"



namespace Electra
{

	class FLicenseKeyMessage : public IPlayerMessage
	{
	public:
		enum class EReason
		{
			LicenseKeyDownload,
			LicenseKeyData
		};

		static TSharedPtrTS<IPlayerMessage> Create(EReason Reason, const FErrorDetail& PlayerResult, const HTTP::FConnectionInfo* ConnectionInfo)
		{
			TSharedPtrTS<FLicenseKeyMessage> p(new FLicenseKeyMessage(Reason, PlayerResult, ConnectionInfo));
			return p;
		}

		static const FString& Type()
		{
			static FString TypeName("LicenseKey");
			return TypeName;
		}

		virtual const FString& GetType() const
		{
			return Type();
		}

		EReason GetReason() const
		{
			return Reason;
		}

		const FErrorDetail& GetResult() const
		{
			return Result;
		}

		const HTTP::FConnectionInfo& GetConnectionInfo() const
		{
			return ConnectionInfo;
		}

	private:
		FLicenseKeyMessage(EReason InReason, const FErrorDetail& PlayerResult, const HTTP::FConnectionInfo* InConnectionInfo)
			: Result(PlayerResult)
			, Reason(InReason)
		{
			if (InConnectionInfo)
			{
				// Have to make a dedicated copy of the connection info in order to get a copy of the retry info at this point in time.
				ConnectionInfo = *InConnectionInfo;
			}
		}
		HTTP::FConnectionInfo	ConnectionInfo;
		FErrorDetail			Result;
		EReason					Reason;
	};

} // namespace Electra


