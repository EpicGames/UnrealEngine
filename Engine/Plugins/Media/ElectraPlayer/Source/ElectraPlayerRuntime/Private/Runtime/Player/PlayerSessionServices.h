// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "ErrorDetail.h"
#include "InfoLog.h"
#include "StreamTypes.h"

struct FElectraHTTPStreamHeader;

namespace Electra
{
	class ISynchronizedUTCTime;
	class IElectraHttpManager;
	class IAdaptiveStreamSelector;
	class IPlayerABRLiveControl;
	class IPlayerStreamFilter;
	struct FAccessUnitBufferInfo;
	class IAdaptiveStreamingPlayerResourceProvider;
	class IPlayerEntityCache;
	class IPlaylistReader;
	class IAdaptiveStreamingPlayerAEMSHandler;
	class FDRMManager;
	class FContentSteeringHandler;
	class FCMCDHandler;
	class IHTTPResponseCache;


	class IPlayerMessage
	{
	public:
		virtual ~IPlayerMessage() = default;
		virtual const FString& GetType() const = 0;
	};


	class IPlayerSessionServices
	{
	public:
		virtual ~IPlayerSessionServices() = default;

		/**
		 * Post an error. Playback will be halted.
		 *
		 * @param Error  Error that occurred.
		 */
		virtual void PostError(const FErrorDetail& Error) = 0;


		/**
		 * Posts a message to the log.
		 *
		 * @param FromFacility
		 * @param LogLevel
		 * @param Message
		 */
		virtual void PostLog(Facility::EFacility FromFacility, IInfoLog::ELevel LogLevel, const FString& Message) = 0;


		/**
		 * Sends a message to the player worker thread.
		 *
		 * @param PlayerMessage
		 */
		virtual void SendMessageToPlayer(TSharedPtrTS<IPlayerMessage> PlayerMessage) = 0;

		/**
		 * Returns the external GUID identifying this player and its associated external texture.
		 */
		virtual void GetExternalGuid(FGuid& OutExternalGuid) = 0;

		/**
		 * Returns the synchronized UTC clock instance associated with this player instance.
		 *
		 * @return Pointer to the synchronized clock.
		 */
		virtual ISynchronizedUTCTime* GetSynchronizedUTCTime() = 0;

		/**
		 * Returns the static resource provider, if any.
		 *
		 * @return Pointer to the static resource provider.
		 */
		virtual TSharedPtr<IAdaptiveStreamingPlayerResourceProvider, ESPMode::ThreadSafe> GetStaticResourceProvider() = 0;

		/**
		 * Returns the HTTP manager instance serving all HTTP requests of this player instance.
		 *
		 * @return Pointer to the HTTP manager.
		 */
		virtual TSharedPtrTS<IElectraHttpManager> GetHTTPManager() = 0;

		/**
		 * Returns the ABR stream selector instance.
		 *
		 * @return Pointer to the ABR stream selector.
		 */
		virtual TSharedPtrTS<IAdaptiveStreamSelector> GetStreamSelector() = 0;

		/**
		 * Returns the player ABR control interface.
		 */
		virtual IPlayerABRLiveControl* GetPlayerARBControl() = 0;

		/**
		 * Returns the stream filter interface used by playlist readers to determine whether or not a stream
		 * can be used on the platform.
		 */
		virtual IPlayerStreamFilter* GetStreamFilter() = 0;

		/**
		 * Returns user configured codec selection priorities.
		 */
		virtual const FCodecSelectionPriorities& GetCodecSelectionPriorities(EStreamType ForStream) = 0;

		/**
		 * Returns the entity cache of this player.
		 */
		virtual TSharedPtrTS<IPlayerEntityCache> GetEntityCache() = 0;

		/**
		 * Returns the HTTP response cache of this player.
		 */
		virtual TSharedPtrTS<IHTTPResponseCache> GetHTTPResponseCache() = 0;

		/**
		 * Returns the manifest reader instance. The reader is responsible for reading additionally required
		 * resources when needed. Requests can be enqueued through this.
		 */
		virtual	TSharedPtrTS<IPlaylistReader> GetManifestReader() = 0;

		/**
		 * Returns the content steering handler for this player instance.
		 */
		virtual TSharedPtrTS<FContentSteeringHandler> GetContentSteeringHandler() = 0;

		/**
		 * Returns the CMCD handler for this player instance.
		 */
		virtual TSharedPtrTS<FCMCDHandler> GetCMCDHandler() = 0;

		/**
		 * Returns the "Application Event or Metadata Streams" (AEMS) handler.
		 */
		virtual IAdaptiveStreamingPlayerAEMSHandler* GetAEMSEventHandler() = 0;

		/**
		 * Returns the dictionary used in queries via GetMediaInfo().
		 * This is used to expose metadata through the IMediaPlayer interface.
		 */
		virtual FParamDictTS& GetMediaInfoDictionary() = 0;

		/**
		 * Returns the mutable player option dictionary. Other than initial options this serves as an interface to
		 * pass values between internal sub systems.
		 */
		virtual FParamDictTS& GetMutableOptions() = 0;

		/**
		 * Checks if a certain player option has been set.
		 */
		virtual bool HaveOptionValue(const FName& InOption) = 0;

		/**
		 * Returns a player option value.
		 */
		virtual const FVariantValue GetOptionValue(const FName& InOption) = 0;

		 /**
		  * Returns the DRM manager, if any.
		  */
		virtual TSharedPtrTS<FDRMManager> GetDRMManager() = 0;

		enum class EPlayEndReason
		{
			EndAll,
			NextItem,
			ErrorCondition
		};
		class IPlayEndReason
		{
		public:
			virtual ~IPlayEndReason() = default;
		};
		virtual void SetPlaybackEnd(const FTimeValue& InEndAtTime, EPlayEndReason InEndingReason, TSharedPtrTS<IPlayEndReason> InCustomManifestObject) = 0;



		/**
		 * Called when a non-standard property is encountered in the main playlist (not a variant playlist!).
		 * This may be passed on to registered listeners to evaluate if the playlist meets certain custom requirements.
		 * Even if there are no such properties in the playlist this method is called at least once with an empty
		 * property to signal the end of the property list, which is useful if playback must be rejected when no
		 * custom properties are present (if they have been removed by some playlist transformation service).
		 */
		enum class ECustomPropertyResult
		{
			// Default handling, depending on where the property occurs.
			Default,
			// Property accepted, continue.
			Accept,
			// Property rejected, fail!
			Reject
		};
		struct FPlaylistProperty
		{
			FString Tag;
			FString Value;
		};
		virtual ECustomPropertyResult ValidateMainPlaylistCustomProperty(const FString& InProtocol, const FString& InPlaylistURL, const TArray<FElectraHTTPStreamHeader>& InPlaylistFetchResponseHeaders, const FPlaylistProperty& InProperty) = 0;
	};


} // namespace Electra



