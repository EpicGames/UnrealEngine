// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "StreamTypes.h"


namespace Electra
{

	class IPlayerStreamFilter
	{
	public:
		virtual ~IPlayerStreamFilter() = default;

		virtual bool CanDecodeStream(const FStreamCodecInformation& InStreamCodecInfo) const = 0;

		virtual bool CanDecodeSubtitle(const FString& MimeType, const FString& Codec) const = 0;
	};



} // namespace Electra


