// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "StreamTypes.h"
#include "Player/Manifest.h"
#include "Player/PlayerSessionServices.h"


namespace Electra
{
	class IAccessUnitMemoryProvider;
	struct FAccessUnit;
	struct FBufferSourceInfo;


	/**
	 * Base class for reading fragments from a stream.
	 * Specialized versions for particular container formats shall derive from this class.
	**/
	class IStreamReader : private TMediaNoncopyable<IStreamReader>
	{
	public:
		virtual ~IStreamReader() = default;

		class StreamReaderEventListener
		{
		public:
			virtual ~StreamReaderEventListener() = default;
			virtual bool OnFragmentAccessUnitReceived(FAccessUnit* pAccessUnit) = 0;
			virtual void OnFragmentReachedEOS(EStreamType InStreamType, TSharedPtr<const FBufferSourceInfo, ESPMode::ThreadSafe> InStreamSourceInfo) = 0;
			virtual void OnFragmentOpen(TSharedPtrTS<IStreamSegment> pRequest) = 0;
			virtual void OnFragmentClose(TSharedPtrTS<IStreamSegment> pRequest) = 0;
		};

		struct CreateParam
		{
			IAccessUnitMemoryProvider* MemoryProvider = nullptr;
			StreamReaderEventListener* EventListener = nullptr;
		};

		//! Creates the instance internally based on the creation parameters.
		virtual UEMediaError Create(IPlayerSessionServices* PlayerSessionService, const CreateParam& createParam) = 0;

		//! Returns true if data from all multiplexed tracks is read at the same time in a single sequential read.
		virtual bool LoadsMultiplexedTracksSequentially() = 0;

		//! Cancels all pending requests and stops processing. The instance can no longer be used after that!
		virtual void Close() = 0;

		enum class EAddResult
		{
			Added,
			TryAgainLater,
			Error
		};
		//! Adds a request to read from a stream
		virtual EAddResult AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> Request) = 0;

		//! Cancels any ongoing requests of the given stream type. Silent cancellation will not notify OnFragmentClose() or OnFragmentReachedEOS(). 
		virtual void CancelRequest(EStreamType StreamType, bool bSilent) = 0;

		//! Cancels all pending requests.
		virtual void CancelRequests() = 0;
	};

} // namespace Electra


