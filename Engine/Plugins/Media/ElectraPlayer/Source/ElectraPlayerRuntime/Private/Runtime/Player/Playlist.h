// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Containers/Array.h"
#include "Utilities/URLParser.h"
#include "Player/AdaptivePlayerOptionKeynames.h"

namespace Electra
{
	namespace Playlist
	{
		class FReplayEventParams
		{
		public:
			void SetLoopingAtStart(bool bInIsLoopEnabledAtStart)
			{ bIsLoopEnabledAtStart = bInIsLoopEnabledAtStart; }
			bool IsLoopingAtStartEnabled() const
			{ return bIsLoopEnabledAtStart; }

			bool HaveReplayParams() const
			{ return !UrlFragmentParams.IsEmpty(); }
			
			// Parse URL fragment parameters
			bool ParseFromURL(const FString& InURL)
			{
				static const TArray<FString> FragmentParamsToParse { CustomOptions::Custom_EpicStaticStart, CustomOptions::Custom_EpicDynamicStart, CustomOptions::Custom_EpicUTCUrl, CustomOptions::Custom_EpicUTCNow };
				SourceURL = CleanedURL = InURL;
				FURL_RFC3986 UrlParser;
				if (UrlParser.Parse(SourceURL))
				{
					FString NewFragment;
					FString URLFragment = UrlParser.GetFragment();
					TArray<FURL_RFC3986::FQueryParam> UrlFragments;
					FURL_RFC3986::GetQueryParams(UrlFragments, URLFragment, false);
					if (UrlFragments.Num())
					{
						for(int32 i=0; i<UrlFragments.Num(); ++i)
						{
							if (FragmentParamsToParse.Contains(UrlFragments[i].Name))
							{
								UrlFragmentParams.Emplace(UrlFragments[i]);
								UrlFragments.RemoveAt(i);
								--i;
							}
							else
							{
								if (!NewFragment.IsEmpty())
								{
									NewFragment.Append(TEXT("&"));
								}
								NewFragment.Append(UrlFragments[i].Name);
								if (!UrlFragments[i].Value.IsEmpty())
								{
									NewFragment.Append(TEXT("="));
									NewFragment.Append(UrlFragments[i].Value);
								}
							}
						}
						CleanedURL = UrlParser.Get(true, false);
						if (NewFragment.Len())
						{
							CleanedURL.Append(TEXT("#"));
							CleanedURL.Append(NewFragment);
						}
						return true;
					}
				}
				return false;
			}
			FString GetSourceURL() const
			{ return SourceURL; }
			FString GetCleanedURL() const
			{ return CleanedURL; }
			const TArray<FURL_RFC3986::FQueryParam>& GetUrlFragmentParams() const
			{ return UrlFragmentParams; }
		private:
			FString SourceURL;
			FString CleanedURL;
			TArray<FURL_RFC3986::FQueryParam> UrlFragmentParams;
			bool bIsLoopEnabledAtStart = false;
		};


		enum class EListType
		{
			Main,
			Variant
		};

		static const TCHAR* const GetPlaylistTypeString(EListType InListType)
		{
			switch(InListType)
			{
				case EListType::Main:
				{
					return TEXT("Main");
				}
				case EListType::Variant:
				{
					return TEXT("Variant");
				}
			}
			return TEXT("n/a");
		}


		enum class ELoadType
		{
			Initial,
			Update
		};

		static const TCHAR* const GetPlaylistLoadTypeString(ELoadType InLoadType)
		{
			switch (InLoadType)
			{
				case ELoadType::Initial:
				{
					return TEXT("Initial");
				}
				case ELoadType::Update:
				{
					return TEXT("Update");
				}
			}
			return TEXT("n/a");
		}
	}
} // namespace Electra
