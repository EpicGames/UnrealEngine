// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlayerSessionServices.h"
#include "StreamTypes.h"
#include "Player/Playlist.h"
#include "Player/Manifest.h"
#include "ParameterDictionary.h"
#include "HTTP/HTTPManager.h"



namespace Electra
{
	// Forward declarations
	class IManifest;

	class IPlaylistReader
	{
	public:
		virtual ~IPlaylistReader() = default;

		/**
		 * Returns the media format this reader handles.
		 */
		virtual EMediaFormatType GetMediaFormatType() const = 0;

		/**
		 * Must call Close() before dropping any TShared..<> of this to allow for any internally
		 * used TWeak..<> to still be valid if used by AsShared().
		 */
		virtual void Close() = 0;

		/**
		 * Called every so often by the player's worker thread to handle this class.
		 */
		virtual void HandleOnce() = 0;

		/**
		 * Returns the type of the playlist (eg. "hls", "dash", etc.)
		 */
		virtual const FString& GetPlaylistType() const = 0;

		/**
		 * Loads and parses the playlist.
		 *
		 * @param URL URL of the playlist to load
		 * @param InReplayEventParams URL fragments pertaining to replay event which have been removed from the source URL
		 */
		virtual void LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams) = 0;

		/**
		 * Returns the URL from which the playlist was loaded (or supposed to be loaded).
		 *
		 * @return The playlist URL
		 */
		virtual FString GetURL() const = 0;


		/**
		 * Returns the manifest interface to access the playlist in a uniform way.
		 *
		 * @return
		 */
		virtual TSharedPtrTS<IManifest> GetManifest() = 0;


		class PlaylistDownloadMessage : public IPlayerMessage
		{
		public:
			static TSharedPtrTS<IPlayerMessage> Create(const HTTP::FConnectionInfo* ConnectionInfo, Playlist::EListType InListType, Playlist::ELoadType InLoadType, int32 InAttempts)
			{
				TSharedPtrTS<PlaylistDownloadMessage> p(new PlaylistDownloadMessage(ConnectionInfo, InListType, InLoadType, InAttempts));
				return p;
			}

			static const FString& Type()
			{
				static FString TypeName(TEXT("PlaylistDownload"));
				return TypeName;
			}

			virtual const FString& GetType() const
			{ return Type(); }

			Playlist::EListType GetListType() const
			{ return ListType; }

			Playlist::ELoadType GetLoadType() const
			{ return LoadType; }

			int32 GetAttempts() const
			{ return Attempts; }

			const HTTP::FConnectionInfo& GetConnectionInfo() const
			{ return ConnectionInfo; }

		private:
			PlaylistDownloadMessage(const HTTP::FConnectionInfo* InConnectionInfo, Playlist::EListType InListType, Playlist::ELoadType InLoadType, int32 InAttempts)
				: ListType(InListType)
				, LoadType(InLoadType)
				, Attempts(InAttempts)
			{
				if (InConnectionInfo)
				{
					ConnectionInfo = *InConnectionInfo;
				}
			}
			HTTP::FConnectionInfo ConnectionInfo;
			Playlist::EListType ListType;
			Playlist::ELoadType LoadType;
			int32 Attempts;
		};


		class PlaylistLoadedMessage : public IPlayerMessage
		{
		public:
			static TSharedPtrTS<IPlayerMessage> Create(const FErrorDetail& PlayerResult, const HTTP::FConnectionInfo* ConnectionInfo, Playlist::EListType InListType, Playlist::ELoadType InLoadType, int32 InAttempts)
			{
				TSharedPtrTS<PlaylistLoadedMessage> p(new PlaylistLoadedMessage(PlayerResult, ConnectionInfo, InListType, InLoadType, InAttempts));
				return p;
			}

			static const FString& Type()
			{
				static FString TypeName(TEXT("PlaylistLoaded"));
				return TypeName;
			}

			virtual const FString& GetType() const
			{ return Type(); }

			const FErrorDetail& GetResult() const
			{ return Result; }

			Playlist::EListType GetListType() const
			{ return ListType; }

			Playlist::ELoadType GetLoadType() const
			{ return LoadType; }

			int32 GetAttempts() const
			{ return Attempts; }

			const HTTP::FConnectionInfo& GetConnectionInfo() const
			{ return ConnectionInfo; }

		private:
			PlaylistLoadedMessage(const FErrorDetail& PlayerResult, const HTTP::FConnectionInfo* InConnectionInfo, Playlist::EListType InListType, Playlist::ELoadType InLoadType, int32 InAttempts)
				: Result(PlayerResult)
				, ListType(InListType)
				, LoadType(InLoadType)
				, Attempts(InAttempts)
			{
				if (InConnectionInfo)
				{
					ConnectionInfo = *InConnectionInfo;
				}
			}
			HTTP::FConnectionInfo ConnectionInfo;
			FErrorDetail Result;
			Playlist::EListType ListType;
			Playlist::ELoadType LoadType;
			int32 Attempts;
		};

	};


} // namespace Electra
