// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamSegmentReaderCommon.h"
#include "PlayerCore.h"
#include "HTTP/HTTPManager.h"
#include "HTTP/HTTPResponseCache.h"
#include "MP4Utilities.h"
#include "MP4Track.h"
#include "Demuxer/ParserISO14496-12.h"
#include "Demuxer/ParserISO14496-12_Utils.h"
#include "Demuxer/ParserISO13818-1.h"
#include "Demuxer/ParserMKV.h"
#include "Demuxer/ParserMKV_Utils.h"
#include "StreamAccessUnitBuffer.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/PlayerEntityCache.h"
#include "Player/DASH/PlaylistReaderDASH.h"	// needed for inband events
#include "Player/DASH/OptionKeynamesDASH.h"
#include "Player/DASH/PlayerEventDASH.h"
#include "Player/DASH/PlayerEventDASH_Internal.h"
#include "Player/DRM/DRMManager.h"
#include "Player/CMCDHandler.h"
#include "Utilities/Utilities.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/UtilsMP4.h"
#include "Utilities/UtilsMPEG.h"
#include "Utilities/ElectraBitstream.h"
#include "Utilities/MP4Helpers.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "MediaStreamMetadata.h"
#include "Misc/TVariant.h"
#include "BufferedDataReader.h"


DECLARE_CYCLE_STAT(TEXT("FStreamSegmentReaderCommon::HandleRequest"), STAT_ElectraPlayer_Common_StreamReader, STATGROUP_ElectraPlayer);


/*********************************************************************************************************************/

namespace Electra { namespace Stream {
TAutoConsoleVariable<bool> CVarElectraStreamUseLegacyMP4(
	TEXT("Electra.UseLegacyMP4Stream"),
	false,
	TEXT("Use old mp4 parser for DASH and HLS streaming.\n")
	TEXT(" true: use old parser, false use new parser"),
	ECVF_Default);
}}
/*********************************************************************************************************************/


namespace Electra
{

const int32 INTERNAL_SEG_ERROR_INIT_SEGMENT_DOWNLOAD_ERROR = 1;
const int32 INTERNAL_SEG_ERROR_INIT_SEGMENT_NOTFOUND_ERROR = 2;
const int32 INTERNAL_SEG_ERROR_INIT_SEGMENT_PARSE_ERROR = 3;
const int32 INTERNAL_SEG_ERROR_INIT_SEGMENT_TOO_SHORT = 4;
const int32 INTERNAL_SEG_ERROR_INIT_SEGMENT_FORMAT_PROBE_ERROR = 5;
const int32 INTERNAL_SEG_ERROR_UNSUPPORTED_PROTOCOL = 6;
const int32 INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE = 7;
const int32 INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT = 8;
const int32 INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT = 9;
const int32 INTERNAL_SEG_ERROR_SIDELOAD_DOWNLOAD_ERROR = 10;



namespace
{
	class FGenericDataReader : public IGenericDataReader
	{
	public:
		FGenericDataReader() = default;
		virtual ~FGenericDataReader() = default;
		int64 ReadData(void* InDestinationBuffer, int64 InNumBytesToRead, int64 InFromOffset) override;
		int64 GetCurrentOffset() const override;
		int64 GetTotalSize() const override;
		bool HasReadBeenAborted() const override;
		bool HasReachedEOF() const override;

		void SetSourceBuffer(TSharedPtrTS<FWaitableBuffer> InDataBuffer);
		void SetSourceBuffer(TArray<uint8>&& InData);
		bool HaveSourceBuffer() const;
		void Abort();
		bool HasErrored() const;
		bool IsEndOfData() const;
		bool GetEOD() const;
		const uint8* GetBufferBaseAddress() const;

		TConstArrayView<uint8> GetConstArrayView() const
		{ return DataBuffer ? MakeConstArrayView<uint8>(DataBuffer->GetLinearReadData(), DataBuffer->GetLinearReadSize()) : Data; }
	private:
		TSharedPtrTS<FWaitableBuffer> DataBuffer;
		TArray<uint8> Data;
		std::atomic<int64> CurrentPos {0};
		volatile bool bAborted = false;
	};

	void FGenericDataReader::SetSourceBuffer(TSharedPtrTS<FWaitableBuffer> InDataBuffer)
	{
		DataBuffer = MoveTemp(InDataBuffer);
		CurrentPos = 0;
		bAborted = false;
	}

	void FGenericDataReader::SetSourceBuffer(TArray<uint8>&& InData)
	{
		DataBuffer.Reset();
		Data = MoveTemp(InData);
		CurrentPos = 0;
		bAborted = false;
	}

	bool FGenericDataReader::HaveSourceBuffer() const
	{
		return DataBuffer.IsValid();
	}

	void FGenericDataReader::Abort()
	{
		bAborted = true;
	}

	bool FGenericDataReader::HasErrored() const
	{
		return DataBuffer->HasErrored();
	}

	bool FGenericDataReader::IsEndOfData() const
	{
		return DataBuffer->IsEndOfData();
	}

	bool FGenericDataReader::GetEOD() const
	{
		return DataBuffer->GetEOD();
	}

	const uint8* FGenericDataReader::GetBufferBaseAddress() const
	{
		return DataBuffer->GetLinearReadData();
	}

	int64 FGenericDataReader::ReadData(void* InDestinationBuffer, int64 InNumBytesToRead, int64 InFromOffset)
	{
		check(InFromOffset == -1 || InFromOffset == CurrentPos);
		InFromOffset = CurrentPos;
		while(1)
		{
			if (DataBuffer->WaitUntilSizeAvailable(InFromOffset + InNumBytesToRead, 1000*20))
			{
				int64 NumAvail = DataBuffer->Num() - InFromOffset;
				int64 NumToCopy = Utils::Max((int64)0, Utils::Min(InNumBytesToRead, NumAvail));
				if (NumToCopy > 0)
				{
					if (InDestinationBuffer)
					{
						FScopeLock lock(DataBuffer->GetLock());
						FMemory::Memcpy(InDestinationBuffer, DataBuffer->GetLinearReadData() + InFromOffset, InNumBytesToRead);
					}
					CurrentPos += NumToCopy;
					return NumToCopy;
				}
				else
				{
					// End of file.
					return DataBuffer->HasErrored() ? -1 : 0;
				}
			}
			else if (bAborted)
			{
				break;
			}
		}
		return 0;
	}

	int64 FGenericDataReader::GetCurrentOffset() const
	{
		return CurrentPos;
	}

	int64 FGenericDataReader::GetTotalSize() const
	{
		return DataBuffer->GetLinearReadSize();
	}

	bool FGenericDataReader::HasReadBeenAborted() const
	{
		return bAborted;
	}

	bool FGenericDataReader::HasReachedEOF() const
	{
		bool bEOF = !HasErrored() && GetEOD() && GetCurrentOffset() >= GetTotalSize();
		return bEOF;
	}
}





FStreamSegmentRequestCommon::FStreamSegmentRequestCommon()
{
}

FStreamSegmentRequestCommon::~FStreamSegmentRequestCommon()
{
}

void FStreamSegmentRequestCommon::SetPlaybackSequenceID(uint32 PlaybackSequenceID)
{
	CurrentPlaybackSequenceID = PlaybackSequenceID;
}

uint32 FStreamSegmentRequestCommon::GetPlaybackSequenceID() const
{
	return CurrentPlaybackSequenceID;
}

void FStreamSegmentRequestCommon::SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay)
{
	// If there is a delay specified and the current time is already past the availability time
	// then this is an old segment before the Live edge since we had paused or seeked backwards.
	// In that case, or if there is no availability time due to VoD, set the availability time
	// as the provided current time to apply the delay to.
	if (UTCNow.IsValid() && ExecutionDelay > FTimeValue::GetZero())
	{
		if (!ASAST.IsValid() || UTCNow > ASAST)
		{
			ASAST = UTCNow;
		}
	}
	DownloadDelayTime = ExecutionDelay;
}

FTimeValue FStreamSegmentRequestCommon::GetExecuteAtUTCTime() const
{
	FTimeValue When = ASAST;
	if (DownloadDelayTime.IsValid())
	{
		When += DownloadDelayTime;
	}
	return When;
}

EStreamType FStreamSegmentRequestCommon::GetType() const
{
	return StreamType;
}

void FStreamSegmentRequestCommon::GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const
{
	OutDependentStreams.Empty();
	if (DependentStreams.Num())
	{
		for(auto& Stream : DependentStreams)
		{
			OutDependentStreams.Emplace(Stream);
		}
	}
	else if (bIsMultiplex)
	{
		TArray<EStreamType> DepTypes;
		if (!bIgnoreVideo)
		{
			DepTypes.Emplace(EStreamType::Video);
		}
		if (!bIgnoreAudio)
		{
			DepTypes.Emplace(EStreamType::Audio);
		}
		if (!bIgnoreSubtitles)
		{
			DepTypes.Emplace(EStreamType::Subtitle);
		}
		for(auto& dpTyp : DepTypes)
		{
			if (dpTyp != StreamType)
			{
				TSharedPtrTS<FStreamSegmentRequestCommon> Dep = MakeSharedTS<FStreamSegmentRequestCommon>();
				Dep->StreamType = dpTyp;
				OutDependentStreams.Emplace(Dep);
			}
		}
	}
}

void FStreamSegmentRequestCommon::GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams)
{
	OutRequestedStreams.Empty();
	for(auto& Stream : DependentStreams)
	{
		OutRequestedStreams.Emplace(Stream);
	}
}

void FStreamSegmentRequestCommon::GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams)
{
	OutAlreadyEndedStreams.Empty();
	if (bIsEOSSegment)
	{
		OutAlreadyEndedStreams.Push(SharedThis(this));
	}
	for(int32 i=0; i<DependentStreams.Num(); ++i)
	{
		if (DependentStreams[i]->bIsEOSSegment)
		{
			OutAlreadyEndedStreams.Push(DependentStreams[i]);
		}
	}
}

FTimeValue FStreamSegmentRequestCommon::GetFirstPTS() const
{
	FTimeValue First = AST + AdditionalAdjustmentTime + PeriodStart + FTimeValue((Segment.bFrameAccuracyRequired ? Segment.MediaLocalFirstPTS : Segment.Time) - Segment.PTO, Segment.Timescale);
	First.SetSequenceIndex(TimestampSequenceIndex);
	return First;
}

FTimeRange FStreamSegmentRequestCommon::GetTimeRange() const
{
	FTimeRange tr;
	tr.Start = AST + AdditionalAdjustmentTime + PeriodStart + FTimeValue(Segment.Time - Segment.PTO, Segment.Timescale);
	tr.End = AST + AdditionalAdjustmentTime + PeriodStart + FTimeValue(Segment.Time + Segment.Duration - Segment.PTO, Segment.Timescale);
	tr.Start.SetSequenceIndex(TimestampSequenceIndex);
	tr.End.SetSequenceIndex(TimestampSequenceIndex);
	return tr;
}

int32 FStreamSegmentRequestCommon::GetQualityIndex() const
{
	return Representation->GetQualityIndex();
}

int32 FStreamSegmentRequestCommon::GetBitrate() const
{
	return Representation->GetBitrate();
}

void FStreamSegmentRequestCommon::GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const
{
	OutStats = DownloadStats;
}

bool FStreamSegmentRequestCommon::GetStartupDelay(FTimeValue& OutStartTime, FTimeValue& OutTimeIntoSegment, FTimeValue& OutSegmentDuration) const
{
	check(DependentStreams.Num());
	if (DependentStreams.Num())
	{
		OutTimeIntoSegment.SetFromND(DependentStreams[0]->Segment.MediaLocalFirstAUTime - DependentStreams[0]->Segment.Time, DependentStreams[0]->Segment.Timescale, 0);
		OutSegmentDuration.SetFromND(DependentStreams[0]->Segment.Duration, DependentStreams[0]->Segment.Timescale, 0);
		OutStartTime = DependentStreams[0]->GetFirstPTS();
		return true;
	}
	return false;
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

namespace
{
	class FMediaSegmentBoxCallback : public IParserISO14496_12::IBoxCallback
	{
	public:
		IParserISO14496_12::IBoxCallback::EParseContinuation OnFoundBox(IParserISO14496_12::FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset) override
		{
			if (Box == IParserISO14496_12::BoxType_moof)
			{
				FMoofInfo& moof = Moofs.Emplace_GetRef();
				moof.MoofPos = FileDataOffset;
				moof.MoofSize = BoxSizeInBytes;
			}
			else if (Box == IParserISO14496_12::BoxType_mdat)
			{
				if (Moofs.Num())
				{
					FMoofInfo& moof = Moofs.Last();
					moof.MdatPos = FileDataOffset;
					moof.MdatSize = BoxSizeInBytes;
				}
			}
			return Box == IParserISO14496_12::BoxType_mdat ? IParserISO14496_12::IBoxCallback::EParseContinuation::Stop : IParserISO14496_12::IBoxCallback::EParseContinuation::Continue;
		}
		IParserISO14496_12::IBoxCallback::EParseContinuation OnEndOfBox(IParserISO14496_12::FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset) override
		{ return IParserISO14496_12::IBoxCallback::EParseContinuation::Continue; }

		struct FMoofInfo
		{
			int64 MoofPos = 0;
			int32 MoofSize = 0;
			int64 MdatPos = 0;
			int64 MdatSize = 0;
		};
		TArray<FMoofInfo> Moofs;
	};

}

uint32 FStreamSegmentReaderCommon::FStreamHandler::UniqueDownloadID = 1;

FStreamSegmentReaderCommon::FStreamSegmentReaderCommon()
{
}

FStreamSegmentReaderCommon::~FStreamSegmentReaderCommon()
{
	Close();
}

UEMediaError FStreamSegmentReaderCommon::Create(IPlayerSessionServices* InPlayerSessionService, const IStreamReader::CreateParam& InCreateParam)
{
	check(InPlayerSessionService);
	PlayerSessionService = InPlayerSessionService;

	if (!InCreateParam.MemoryProvider || !InCreateParam.EventListener)
	{
		return UEMEDIA_ERROR_BAD_ARGUMENTS;
	}

	bIsStarted = true;
	for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(StreamHandlers); ++i)
	{
		StreamHandlers[i].PlayerSessionService = PlayerSessionService;
		StreamHandlers[i].Parameters		   = InCreateParam;
		StreamHandlers[i].bTerminate		   = false;
		StreamHandlers[i].bWasStarted		   = false;
		StreamHandlers[i].bRequestCanceled     = false;
		StreamHandlers[i].bSilentCancellation  = false;
		StreamHandlers[i].bHasErrored   	   = false;
		StreamHandlers[i].IsIdleSignal.Signal();
		StreamHandlers[i].ThreadSetName(i==0 ? "Electra video segment loader" :
										i==1 ? "Electra audio segment loader" :
											   "Electra subtitle segment loader");
	}
	return UEMEDIA_ERROR_OK;
}

bool FStreamSegmentReaderCommon::LoadsMultiplexedTracksSequentially()
{
	return false;
}

void FStreamSegmentReaderCommon::Close()
{
	if (bIsStarted)
	{
		bIsStarted = false;
		// Signal the worker threads to end.
		for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(StreamHandlers); ++i)
		{
			StreamHandlers[i].bTerminate = true;
			StreamHandlers[i].Cancel(true);
			StreamHandlers[i].SignalWork();
		}
		// Wait until they finished.
		for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(StreamHandlers); ++i)
		{
			if (StreamHandlers[i].bWasStarted)
			{
				StreamHandlers[i].ThreadWaitDone();
				StreamHandlers[i].ThreadReset();
			}
		}
	}
}

IStreamReader::EAddResult FStreamSegmentReaderCommon::AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> InRequest)
{
	TSharedPtrTS<FStreamSegmentRequestCommon> Request = StaticCastSharedPtr<FStreamSegmentRequestCommon>(InRequest);

	if (Request->bIsInitialStartRequest)
	{
		PostError(PlayerSessionService, TEXT("Initial start request segments cannot be enqueued!"), 0);
		return IStreamReader::EAddResult::Error;
	}
	else
	{
		// Get the handler for the main request.
		FStreamHandler* Handler = nullptr;
		switch(Request->GetType())
		{
			case EStreamType::Video:
				Handler = &StreamHandlers[0];
				break;
			case EStreamType::Audio:
				Handler = &StreamHandlers[1];
				break;
			case EStreamType::Subtitle:
				Handler = &StreamHandlers[2];
				break;
			default:
				break;
		}
		if (!Handler)
		{
			ErrorDetail.SetMessage(FString::Printf(TEXT("No handler for stream type")));
			return IStreamReader::EAddResult::Error;
		}
		// Is the handler busy?
		bool bIsIdle = Handler->IsIdleSignal.WaitTimeout(1000 * 1000);
		if (!bIsIdle)
		{
			ErrorDetail.SetMessage(FString::Printf(TEXT("The handler for this stream type is busy!?")));
			return IStreamReader::EAddResult::Error;
		}

		Request->SetPlaybackSequenceID(CurrentPlaybackSequenceID);
		if (!Handler->bWasStarted)
		{
			Handler->ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(Handler, &FStreamHandler::WorkerThread));
			Handler->bWasStarted = true;
		}

		Handler->bRequestCanceled = false;
		Handler->bSilentCancellation = false;
		Handler->CurrentRequest = Request;
		Handler->SignalWork();
	}
	return IStreamReader::EAddResult::Added;
}

void FStreamSegmentReaderCommon::CancelRequest(EStreamType StreamType, bool bSilent)
{
	if (StreamType == EStreamType::Video)
	{
		StreamHandlers[0].Cancel(bSilent);
	}
	else if (StreamType == EStreamType::Audio)
	{
		StreamHandlers[1].Cancel(bSilent);
	}
	else if (StreamType == EStreamType::Subtitle)
	{
		StreamHandlers[2].Cancel(bSilent);
	}
}

void FStreamSegmentReaderCommon::CancelRequests()
{
	for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(StreamHandlers); ++i)
	{
		StreamHandlers[i].Cancel(false);
	}
}





FStreamSegmentReaderCommon::FStreamHandler::FStreamHandler()
{ }

FStreamSegmentReaderCommon::FStreamHandler::~FStreamHandler()
{
	// NOTE: The thread will have been terminated by the enclosing FStreamSegmentReaderCommon's Close() method!
	//       Also, this may have run on the thread pool instead of a dedicated worker thread.
}

void FStreamSegmentReaderCommon::FStreamHandler::Cancel(bool bSilent)
{
	bSilentCancellation = bSilent;
	bRequestCanceled = true;
}

void FStreamSegmentReaderCommon::FStreamHandler::SignalWork()
{
	WorkSignal.Release();
}

void FStreamSegmentReaderCommon::FStreamHandler::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	StreamSelector = PlayerSessionService->GetStreamSelector();
	if (StreamSelector.IsValid())
	{
		while(!bTerminate)
		{
			WorkSignal.Obtain();
			if (!bTerminate)
			{
				if (CurrentRequest.IsValid())
				{
					IsIdleSignal.Reset();
					if (!bRequestCanceled)
					{
						HandleRequest();
					}
					else
					{
						CurrentRequest.Reset();
					}
					IsIdleSignal.Signal();
				}
				bRequestCanceled = false;
				bSilentCancellation = false;
			}
		}
	}
	StreamSelector.Reset();
}

FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::SetError(const FString& InMessage, uint16 InCode)
{
	SegmentError.SetError(UEMEDIA_ERROR_FORMAT_ERROR);
	SegmentError.SetFacility(Facility::EFacility::CommonSegmentReader);
	SegmentError.SetCode(InCode);
	SegmentError.SetMessage(InMessage);
	bHasErrored = true;
	return EHandleResult::Failed;
}

bool FStreamSegmentReaderCommon::FStreamHandler::HasErrored() const
{
	return bHasErrored;
}

void FStreamSegmentReaderCommon::FStreamHandler::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	PlayerSessionService->PostLog(Facility::EFacility::DASHStreamReader, Level, Message);
}

int32 FStreamSegmentReaderCommon::FStreamHandler::HTTPProgressCallback(const IElectraHttpManager::FRequest* InHTTPRequest)
{
	HTTPUpdateStats(MEDIAutcTime::Current(), InHTTPRequest);
	++ProgressReportCount;

	// Aborted?
	return HasReadBeenAborted() ? 1 : 0;
}

void FStreamSegmentReaderCommon::FStreamHandler::HTTPCompletionCallback(const IElectraHttpManager::FRequest* InHTTPRequest)
{
	HTTPUpdateStats(FTimeValue::GetInvalid(), InHTTPRequest);
	bHasErrored = CurrentConnectionInfo.StatusInfo.ErrorDetail.IsError();
}

void FStreamSegmentReaderCommon::FStreamHandler::HTTPUpdateStats(const FTimeValue& CurrentTime, const IElectraHttpManager::FRequest* InHTTPRequest)
{
	// Only update elements that are needed by the ABR here.
	FScopeLock lock(&MetricUpdateLock);
	CurrentConnectionInfo = InHTTPRequest->ConnectionInfo;
	if (CurrentConnectionInfo.EffectiveURL.Len())
	{
		DownloadStats.URL.URL = CurrentConnectionInfo.EffectiveURL;
	}
	DownloadStats.HTTPStatusCode = CurrentConnectionInfo.StatusInfo.HTTPStatus;
	DownloadStats.TimeToFirstByte = CurrentConnectionInfo.TimeUntilFirstByte;
	DownloadStats.TimeToDownload = ((CurrentTime.IsValid() ? CurrentTime : CurrentConnectionInfo.RequestEndTime) - CurrentConnectionInfo.RequestStartTime).GetAsSeconds();
	DownloadStats.ByteSize = CurrentConnectionInfo.ContentLength;
	DownloadStats.NumBytesDownloaded = CurrentConnectionInfo.BytesReadSoFar;
}

void FStreamSegmentReaderCommon::FStreamHandler::HandleRequest()
{
	TSharedPtrTS<FStreamSegmentRequestCommon> Request = CurrentRequest;
	check(Request.IsValid());
	if (!Request.IsValid())
	{
		return;
	}
	// Needs to be DASH or HLS.
	check(Request->StreamingProtocol != FStreamSegmentRequestCommon::EStreamingProtocol::Undefined);

	// Set up the download stats values from the request.
	DownloadStats.ResetOutput();
	DownloadStats.StreamType = Request->GetType();
	DownloadStats.MediaAssetID = Request->Period.IsValid() ? Request->Period->GetUniqueIdentifier() : "";
	DownloadStats.AdaptationSetID = Request->AdaptationSet.IsValid() ? Request->AdaptationSet->GetUniqueIdentifier() : "";
	DownloadStats.RepresentationID = Request->Representation.IsValid() ? Request->Representation->GetUniqueIdentifier() : "";
	DownloadStats.PresentationTime = Request->GetFirstPTS().GetAsSeconds();
	FTimeValue SegmentDuration(Request->Segment.Duration, Request->Segment.Timescale, 0);
	DownloadStats.Duration = SegmentDuration.GetAsSeconds();
	DownloadStats.Bitrate = Request->GetBitrate();
	DownloadStats.QualityIndex = Request->QualityIndex;
	DownloadStats.HighestQualityIndex = Request->MaxQualityIndex;
	DownloadStats.RetryNumber = Request->NumOverallRetries;

	// Clear internal work variables.
	bHasErrored = false;
	bAbortedByABR = false;
	bAllowEarlyEmitting = false;
	bFillRemainingDuration = false;
	ABRAbortReason.Empty();
	ProgressReportCount = 0;
	SegmentError.Clear();
	CurrentConnectionInfo = HTTP::FConnectionInfo();

	// Fetch the initialization segment, if required.
	FetchInitSegment(Request);

	// Error?
	if (!SegmentError.IsOK())
	{
		bHasErrored = true;
		if (DownloadStats.FailureReason.IsEmpty())
		{
			DownloadStats.FailureReason = SegmentError.GetMessage();
		}
		Request->ConnectionInfo = CurrentConnectionInfo;
		Request->DownloadStats = DownloadStats;
		CurrentRequest.Reset();
		if (!bSilentCancellation)
		{
			StreamSelector->ReportDownloadEnd(DownloadStats);
			Parameters.EventListener->OnFragmentClose(Request);
		}
		return;
	}

	// Variable to track is something failed.
	bool bContinue = true;

	// Perform common initial handling regardless of EOS or filler segments.
	HandleCommonMediaBegin(Request);

	// Now start downloading the media segment.
	EHandleResult SegmentHandleResult = EHandleResult::Failed;
	EMediaSegmentTriggerResult DownloadTriggerResult = TriggerMediaSegmentDownload(Request);
	if (DownloadTriggerResult == EMediaSegmentTriggerResult::DontHandle)
	{
		SegmentHandleResult = EHandleResult::Finished;
	}
	else if (DownloadTriggerResult == EMediaSegmentTriggerResult::IsFiller)
	{
		bFillRemainingDuration = true;
		if (Request->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH)
		{
			if (Request->ExpectedContainerFormat == FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12)
			{
				SegmentHandleResult = Stream::CVarElectraStreamUseLegacyMP4.GetValueOnAnyThread() ? HandleMP4MediaLegacy(Request) : HandleMP4Media(Request);
			}
			else if (Request->ExpectedContainerFormat == FStreamSegmentRequestCommon::EContainerFormat::Matroska_WebM)
			{
				SegmentHandleResult = HandleMKVMedia(Request);
			}
			else
			{
				unimplemented();
			}
		}
		else
		{
			SegmentHandleResult = HandleFillerDataSetup(Request);
		}
	}
	else if (DownloadTriggerResult == EMediaSegmentTriggerResult::IsSideloaded)
	{
		SegmentHandleResult = HandleSideloadedMedia(Request);
	}
	else
	{
		// Check if for HLS full segment encyption is used. If so we need to decrypt the
		// segment data before be can probe its contents.
		if (Request->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::HLS && Request->DrmMedia.DrmClient.IsValid())
		{
			// We don't know upfront if this is full encryption or sample encryption.
			// We have to create a decrypter instance and check with it.
			if (Request->DrmMedia.DrmClient->CreateDecrypter(Decrypter, Request->DrmMedia.DrmMimeType) != ElectraCDM::ECDMError::Success)
			{
				SetError(FString::Printf(TEXT("Failed to create segment decrypter: \"%s\""), *Request->DrmMedia.DrmClient->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
				bHasErrored = true;
				bContinue = false;
			}
			else if (Decrypter->IsBlockStreamDecrypter())
			{
				ReadBuffer.BlockDecrypter = MoveTemp(Decrypter);
				ReadBuffer.BlockDecrypterIV = Request->DrmMedia.DrmIV;
				ReadBuffer.BlockDecrypterKID = Request->DrmMedia.DrmKID;
				ReadBuffer.bIsEncrypted = true;
				ReadBuffer.bDecrypterReady = false;
			}
			Decrypter.Reset();
		}

		// We now wait for the first few bytes to arrive. We will probe them to determine the
		// format of the media segment.
		const int32 kNumProbeBytesNeeded = 16;
		uint8 ProbeBytes[kNumProbeBytesNeeded] {};
		if (bContinue)
		{
			while(!ReadBuffer.WaitUntilSizeAvailable(kNumProbeBytesNeeded, 1000*20))
			{
				if (HasErrored() || HasReadBeenAborted())
				{
					bContinue = false;
					break;
				}
			}
			if (bContinue)
			{
				FScopeLock ProbeBufferLock(ReadBuffer.GetLock());
				const uint8* MediaDataPtr = ReadBuffer.GetLinearReadData();
				int32 NumProbeBytes = Utils::Min((int32) ReadBuffer.GetLinearReadSize(), (int32)kNumProbeBytesNeeded);
				FMemory::Memcpy(ProbeBytes, ReadBuffer.GetLinearReadData(), NumProbeBytes);
			}
		}

		// If failed or aborted we proceed no further.
		if (bContinue)
		{
			int32 ID3HeaderSize = 0;

			FStreamSegmentRequestCommon::EContainerFormat ContainerFormat = Request->ExpectedContainerFormat;
			auto GetUINT32BE = [](const uint8* InData) -> uint32
			{
				return (static_cast<uint32>(InData[0]) << 24) | (static_cast<uint32>(InData[1]) << 16) | (static_cast<uint32>(InData[2]) << 8) | static_cast<uint32>(InData[3]);
			};
			// ISO/IEC 14496-12 ?
			const uint32 boxName = GetUINT32BE(ProbeBytes + 4);
			if (boxName == MP4Utilities::MakeBoxAtom('s','t','y','p') || boxName == MP4Utilities::MakeBoxAtom('m','o','o','f') || boxName == MP4Utilities::MakeBoxAtom('s','i','d','x')
				|| boxName == MP4Utilities::MakeBoxAtom('f','t','y','p') || boxName == MP4Utilities::MakeBoxAtom('m','o','o','v') || boxName == MP4Utilities::MakeBoxAtom('e','m','s','g')
				|| boxName == MP4Utilities::MakeBoxAtom('f','r','e','e') || boxName == MP4Utilities::MakeBoxAtom('s','k','i','p') || boxName == MP4Utilities::MakeBoxAtom('s','s','i','x')
				|| boxName == MP4Utilities::MakeBoxAtom('p','r','f','t') || boxName == MP4Utilities::MakeBoxAtom('u','d','t','a'))
			{
				ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12;
			}
			// Matroska / WebM ?
			else if (GetUINT32BE(ProbeBytes) == 0x1a45dfa3 || GetUINT32BE(ProbeBytes) == 0x1f43b675)
			{
				ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::Matroska_WebM;
			}
			// .mp3 / .aac raw file with an ID3 header ?
			else if (ProbeBytes[0] == 'I' && ProbeBytes[1] == 'D' && ProbeBytes[2] == '3' &&
				ProbeBytes[3] != 0xff && ProbeBytes[4] != 0xff && ProbeBytes[6] < 0x80 && ProbeBytes[7] < 0x80 && ProbeBytes[8] < 0x80 && ProbeBytes[9] < 0x80)
			{
				ID3HeaderSize = (int32) (10U + (ProbeBytes[6] << 21U) + (ProbeBytes[7] << 14U) + (ProbeBytes[8] << 7U) + ProbeBytes[9]);
				ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::ID3_Raw;
			}
			// Raw WebVTT file ? (starts with "WEBVTT" or BOM+"WEBVTT")
			// Could also be any even empty file as long as the init segment indicates WebVTT
			else if (IsWebVTTHeader(ProbeBytes, 9) ||
				(InitSegmentData.IsType<TSharedPtrTS<const TArray<uint8>>>() && IsWebVTTHeader(InitSegmentData.Get<TSharedPtrTS<const TArray<uint8>>>()->GetData(), InitSegmentData.Get<TSharedPtrTS<const TArray<uint8>>>()->Num())))
			{
				ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::WebVTT_Raw;
			}
			// ISO/IEC 13818-1 ?
			else if (ProbeBytes[0] == 0x47)
			{
				ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::ISO_13818_1;
			}

			if (ContainerFormat == FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12)
			{
				SegmentHandleResult = Stream::CVarElectraStreamUseLegacyMP4.GetValueOnAnyThread() ? HandleMP4MediaLegacy(Request) : HandleMP4Media(Request);
			}
			else if (ContainerFormat == FStreamSegmentRequestCommon::EContainerFormat::Matroska_WebM)
			{
				SegmentHandleResult = HandleMKVMedia(Request);
			}
			else if (ContainerFormat == FStreamSegmentRequestCommon::EContainerFormat::ISO_13818_1)
			{
				SegmentHandleResult = HandleTSMedia(Request);
			}
			else if (ContainerFormat == FStreamSegmentRequestCommon::EContainerFormat::ID3_Raw)
			{
				SegmentHandleResult = HandleID3RawMedia(Request, ID3HeaderSize);
			}
			else if (ContainerFormat == FStreamSegmentRequestCommon::EContainerFormat::WebVTT_Raw)
			{
				SegmentHandleResult = HandleRawSubtitleMedia(Request);
			}
			else
			{
				if (ReadBuffer.DidDecryptionFail())
				{
					SetError(FString::Printf(TEXT("Failed to decrypt media segment")), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
				}
				else
				{
					SetError(FString::Printf(TEXT("Failed to determine format of media segment")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
				}
				bHasErrored = true;
			}
		}
	}

	// Handle common end of media segment download, successful or not.
	HandleCommonMediaEnd(SegmentHandleResult, Request);
}

void FStreamSegmentReaderCommon::FStreamHandler::SetupSegmentDownloadStatsFromConnectionInfo(const HTTP::FConnectionInfo& ci)
{
	DownloadStats.HTTPStatusCode = ci.StatusInfo.HTTPStatus;
	DownloadStats.TimeToFirstByte = ci.TimeUntilFirstByte;
	DownloadStats.TimeToDownload = (ci.RequestEndTime - ci.RequestStartTime).GetAsSeconds();
	DownloadStats.ByteSize = ci.ContentLength;
	DownloadStats.NumBytesDownloaded = ci.BytesReadSoFar;
	if (DownloadStats.FailureReason.IsEmpty())
	{
		DownloadStats.FailureReason = ci.StatusInfo.ErrorDetail.GetMessage();
	}
}

//---------------------------------------------------------------------------------------------------------------------
/**
 * Fetches the initialization segment necessary to parse the media segment, if one is needed.
 * If the init segment is encrypted it will be decrypted (HLS only).
 */
bool FStreamSegmentReaderCommon::FStreamHandler::FetchInitSegment(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	check(InRequest.IsValid());

	const FSegmentInformationCommon& segInfo = InRequest->Segment;
	FSegmentInformationCommon::FURL InitURLToUse(segInfo.InitializationURL);
	bool bFetchFromSelfInitializing = false;

	// Side-loaded segments do not have init data.
	if (segInfo.bIsSideload)
	{
		return true;
	}

	// Set the download stats with the information for downloading an init segment.
	DownloadStats.StatsID = FMediaInterlockedIncrement(UniqueDownloadID);
	DownloadStats.SegmentType = Metrics::ESegmentType::Init;
	DownloadStats.URL = InitURLToUse.Url;
	DownloadStats.Range = InitURLToUse.Range;
	DownloadStats.SteeringID = InitURLToUse.SteeringID;

	// The presence of an init segment is signalled differently for HLS and DASH.
	if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH)
	{
		// With DASH, if the init segment is not specified it is said that the media stream is self-initializing.
		// This means that the init segment is the 'moov' box located somewhere in the actual media stream (if this is an fmp4).
		if (InRequest->ExpectedContainerFormat == FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12)
		{
			if (InitURLToUse.Url.URL.IsEmpty() || (InitURLToUse.Range.IsEmpty() && InitURLToUse.Url.URL.Equals(segInfo.MediaURL.Url.URL)))
			{
				FString InitRange(InitURLToUse.Range);
				if (InitURLToUse.Url.URL.IsEmpty())
				{
					InitURLToUse = segInfo.MediaURL;
				}
				InitURLToUse.Range = MoveTemp(InitRange);
				if (InitURLToUse.Range.IsEmpty())
				{
					bFetchFromSelfInitializing = true;
					DownloadStats.URL = InitURLToUse.Url;
					DownloadStats.Range = InitURLToUse.Range;
				}
			}
		}
	}
	else if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::HLS)
	{
		// With HLS, if there is no init segment specified explicitly, then there is none.
		// For fmp4's it is required to be set and for transport streams it is then implied that
		// the PAT and PMT are present in every segment.
	}
	else
	{
		SetError(FString::Printf(TEXT("Unimplemented streaming protocol")), INTERNAL_SEG_ERROR_UNSUPPORTED_PROTOCOL);
		return false;
	}

	// When no init segment is needed, signal completion and return.
	if (InitURLToUse.Url.URL.IsEmpty())
	{
		return true;
	}

	// Otherwise, check with the entity cache if we already have it from earlier.
	TSharedPtrTS<IPlayerEntityCache> EntityCache = PlayerSessionService ? PlayerSessionService->GetEntityCache() : nullptr;
	if (EntityCache.IsValid())
	{
		IPlayerEntityCache::FCacheItem CachedItem;
		if (EntityCache->GetCachedEntity(CachedItem, InitURLToUse.Url.URL, InitURLToUse.Range))
		{
			if (CachedItem.ParsedMOOV.IsValid())
			{
				InitSegmentData.Emplace<TSharedPtrTS<FParsedMOOVCommon>>(CachedItem.ParsedMOOV);
				DownloadStats.bWasSuccessful = true;
			}
			else if (CachedItem.Parsed14496_12Data.IsValid())
			{
				InitSegmentData.Emplace<TSharedPtrTS<const IParserISO14496_12>>(CachedItem.Parsed14496_12Data);
				DownloadStats.bWasSuccessful = true;
			}
			else if (CachedItem.ParsedMatroskaData.IsValid())
			{
				InitSegmentData.Emplace<TSharedPtrTS<const IParserMKV>>(CachedItem.ParsedMatroskaData);
				DownloadStats.bWasSuccessful = true;
			}
			else if (CachedItem.RawPayloadData.IsValid())
			{
				InitSegmentData.Emplace<TSharedPtrTS<const TArray<uint8>>>(CachedItem.RawPayloadData);
				DownloadStats.bWasSuccessful = true;
			}
			return true;
		}
	}

	FStreamSegmentRequestCommon::EContainerFormat ContainerFormat = InRequest->ExpectedContainerFormat;
	// Not cached yet, need to fetch.
	CurrentConnectionInfo = HTTP::FConnectionInfo();
	FGenericDataReader StaticSegmentDataReader;
	TArray<HTTP::FHTTPHeader> ReqHeaders;
	if (InitURLToUse.CustomHeader.Len() && InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH)
	{
		ReqHeaders.Emplace(HTTP::FHTTPHeader({DASH::HTTPHeaderOptionName, InitURLToUse.CustomHeader}));
	}
	// Self-initializing media?
	if (!bFetchFromSelfInitializing)
	{
		TSharedPtrTS<FHTTPResourceRequest> rr = MakeSharedTS<FHTTPResourceRequest>();
		TSharedPtr<FHTTPResourceRequestCompletionSignal, ESPMode::ThreadSafe> rrsig = FHTTPResourceRequestCompletionSignal::Create();

		FString InitSegmentUrl(InitURLToUse.Url.URL);
		// Set up information for CMCD.
		FCMCDHandler::FRequestObjectInfo ri;
		ri.MediaStreamType = InRequest->StreamType;
		ri.EncodedBitrate = InRequest->GetBitrate();
		PlayerSessionService->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::InitSegment, FCMCDHandler::EObjectType::InitSegment, InitSegmentUrl, ReqHeaders, InitURLToUse.Url.CDN, ri);

		rr->Verb(TEXT("GET")).URL(InitSegmentUrl).Range(InitURLToUse.Range).Headers(ReqHeaders).AcceptEncoding(TEXT("identity"))
			.ConnectionTimeout(FTimeValue().SetFromMilliseconds(5000)).NoDataTimeout(FTimeValue().SetFromMilliseconds(2000))
			.StreamTypeAndQuality(InRequest->StreamType, InRequest->QualityIndex, InRequest->MaxQualityIndex)
			.CompletionSignal(rrsig)
			.StartGet(PlayerSessionService);
		while(!rrsig->WaitTimeout(1000 * 10))
		{
			if (HasReadBeenAborted())
			{
				rr->Cancel();
				break;
			}
		}
		if (HasReadBeenAborted())
		{
			return true;
		}
		CurrentConnectionInfo = *rr->GetConnectionInfo();
		// If CMCD added query parameters to the URL then we need to remove them from the effective URL.
		if (!InitSegmentUrl.Equals(InitURLToUse.Url.URL))
		{
			// Keep the initial URL though as it was since it was (a) not modified and (b) it is what the init segment gets cached under.
			// We do not want to use the redirected effective URL for the cache.
			CurrentConnectionInfo.EffectiveURL = PlayerSessionService->GetCMCDHandler()->RemoveParamsFromURL(CurrentConnectionInfo.EffectiveURL);
		}

		TSharedPtrTS<FWaitableBuffer> ResponseBuffer = rr->GetResponseBuffer();

		bool bSuccessful = !rr->GetError() && ResponseBuffer.IsValid();
		SetupSegmentDownloadStatsFromConnectionInfo(CurrentConnectionInfo);

		// Success?
		if (!bSuccessful)
		{
			// No.
			SetError(FString::Printf(TEXT("Init segment download error: %s"), *CurrentConnectionInfo.StatusInfo.ErrorDetail.GetMessage()), INTERNAL_SEG_ERROR_INIT_SEGMENT_DOWNLOAD_ERROR);
			return false;
		}
		// We need to have some amount of data that this could even be an init segment of sorts.
		if (ResponseBuffer->Num() < 6)
		{
			// Not enough data
			SetError(FString::Printf(TEXT("Init segment too small to contain relevant data")), INTERNAL_SEG_ERROR_INIT_SEGMENT_TOO_SHORT);
			return false;
		}

		// For HLS there is the possibility that the init segment is encrypted.
		if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::HLS && InRequest->DrmInit.DrmClient.IsValid())
		{
			// We don't know upfront if this is full encryption or sample encryption, so we create a decrypter instance and check with it.
			// NOTE: Technically this is true, but since there are no media samples inside an init segment the encryption can realistically
			//       only be full segment encryption.
			if (InRequest->DrmInit.DrmClient->CreateDecrypter(Decrypter, InRequest->DrmInit.DrmMimeType) != ElectraCDM::ECDMError::Success)
			{
				SetError(FString::Printf(TEXT("Failed to create segment decrypter: \"%s\""), *InRequest->DrmInit.DrmClient->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
				return false;
			}
			else if (Decrypter->IsBlockStreamDecrypter())
			{
				// Wait until the decrypter is ready
				while(!HasReadBeenAborted())
				{
					if (Decrypter->GetState() == ElectraCDM::ECDMState::WaitingForKey || Decrypter->GetState() == ElectraCDM::ECDMState::Idle)
					{
						FMediaRunnable::SleepMilliseconds(100);
					}
					else
					{
						break;
					}
				}
				if (HasReadBeenAborted())
				{
					Decrypter.Reset();
					return true;
				}
				if (Decrypter->GetState() != ElectraCDM::ECDMState::Ready)
				{
					SetError(FString::Printf(TEXT("Failed to create segment decrypter: \"%s\""), *InRequest->DrmInit.DrmClient->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
					Decrypter.Reset();
					DownloadStats.bParseFailure = true;
					return false;
				}
				// Decrypt the init segment in place.
				ElectraCDM::FMediaCDMSampleInfo si;
				si.IV = InRequest->DrmInit.DrmIV;
				si.DefaultKID = InRequest->DrmInit.DrmKID;
				ElectraCDM::IMediaCDMDecrypter::IStreamDecryptHandle* BlockDecrypterHandle = nullptr;
				ElectraCDM::ECDMError err = Decrypter->BlockStreamDecryptStart(BlockDecrypterHandle, si);
				int32 NumDecrypted = 0;
				if (err == ElectraCDM::ECDMError::Success)
				{
					err = Decrypter->BlockStreamDecryptInPlace(BlockDecrypterHandle, NumDecrypted, ResponseBuffer->GetLinearReadData(), (int32) ResponseBuffer->GetLinearReadSize(), true);
					Decrypter->BlockStreamDecryptEnd(BlockDecrypterHandle);
					BlockDecrypterHandle = nullptr;
				}
				if (err != ElectraCDM::ECDMError::Success)
				{
					SetError(FString::Printf(TEXT("Failed to decrypt init segment: \"%s\""), *InRequest->DrmInit.DrmClient->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
					Decrypter.Reset();
					DownloadStats.bParseFailure = true;
					return false;
				}
				ResponseBuffer->SetLinearReadSize(NumDecrypted);
			}
			Decrypter.Reset();
		}

		// What format did we get?
		const uint8* InitDataPtr = ResponseBuffer->GetLinearReadData();
		int64 InitDataSize = ResponseBuffer->GetLinearReadSize();
		auto GetUINT32BE = [](const uint8* InData) -> uint32
		{
			return (static_cast<uint32>(InData[0]) << 24) | (static_cast<uint32>(InData[1]) << 16) | (static_cast<uint32>(InData[2]) << 8) | static_cast<uint32>(InData[3]);
		};
		// ISO/IEC 14496-12 ?
		const uint32 boxName = InitDataSize >= 8 ? GetUINT32BE(InitDataPtr + 4) : 0;
		if (boxName == MP4Utilities::MakeBoxAtom('f','t','y','p') || boxName == MP4Utilities::MakeBoxAtom('f','r','e','e') || boxName == MP4Utilities::MakeBoxAtom('s','k','i','p')
			|| boxName == MP4Utilities::MakeBoxAtom('p','d','i','n') || boxName == MP4Utilities::MakeBoxAtom('u','d','t','a'))
		{
			StaticSegmentDataReader.SetSourceBuffer(ResponseBuffer);
			ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12;
		}
		// Matroska / WebM ?
		else if (GetUINT32BE(InitDataPtr) == 0x1a45dfa3)
		{
			StaticSegmentDataReader.SetSourceBuffer(ResponseBuffer);
			ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::Matroska_WebM;
		}
		// ISO/IEC 13818-1 ?
		else if (*InitDataPtr == 0x47)
		{
			check((ResponseBuffer->GetLinearReadSize() % 188) == 0);
			StaticSegmentDataReader.SetSourceBuffer(ResponseBuffer);
			ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::ISO_13818_1;
		}
		// Raw WebVTT? (starts with "WEBVTT" or BOM+"WEBVTT")
		else if (IsWebVTTHeader(InitDataPtr, InitDataSize))
		{
			StaticSegmentDataReader.SetSourceBuffer(ResponseBuffer);
			ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::WebVTT_Raw;
		}
		else
		{
			DownloadStats.bParseFailure = true;
			SetError(FString::Printf(TEXT("Init segment data does not seem to be any supported format")), INTERNAL_SEG_ERROR_INIT_SEGMENT_FORMAT_PROBE_ERROR);
			return false;
		}
	}
	else
	{
		// For self-initializing media we need to locate the `moov` box.
		FString InitSegmentUrl(InitURLToUse.Url.URL);
		// Set up information for CMCD.
		FCMCDHandler::FRequestObjectInfo ri;
		ri.MediaStreamType = InRequest->StreamType;
		ri.EncodedBitrate = InRequest->GetBitrate();
		PlayerSessionService->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::InitSegment, FCMCDHandler::EObjectType::InitSegment, InitSegmentUrl, ReqHeaders, InitURLToUse.Url.CDN, ri);

		if (!Stream::CVarElectraStreamUseLegacyMP4.GetValueOnAnyThread())
		{
  			TSharedPtr<FMP4DataLoader, ESPMode::ThreadSafe> DataLoader = FMP4DataLoader::Create();
			FMP4DataLoader::FHttpParams HttpParams;
			HttpParams.HTTPManager = PlayerSessionService->GetHTTPManager();
			HttpParams.HTTPResponseCache = IHTTPResponseCache::Create(1024*1024, 32, nullptr);
			HttpParams.RequestHeaders = ReqHeaders;
			HttpParams.ConnectTimeoutMillis = 5000;
			HttpParams.NoDataTimeoutMillis = 2000;
			DataLoader->Open(InitSegmentUrl, HttpParams);

			MP4Utilities::FMP4BoxLocator BoxLocator;
			TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> RootBoxes;
			const TArray<uint32> StopAfterBoxes { MP4Utilities::MakeBoxAtom('m','o','o','v') };
			bool bGotBoxes = BoxLocator.LocateAndReadRootBoxes(RootBoxes, DataLoader, TArray<uint32>(), StopAfterBoxes, false, TArray<uint32>(), FMP4DataLoader::FCancellationCheckDelegate::CreateLambda([&](){return HasReadBeenAborted();}));
			if (HasReadBeenAborted())
			{
				return true;
			}
			CurrentConnectionInfo = DataLoader->GetConnectionInfo();
			// If CMCD added query parameters to the URL then we need to remove them from the effective URL.
			if (!InitSegmentUrl.Equals(InitURLToUse.Url.URL))
			{
				// Keep the initial URL though as it was since it was (a) not modified and (b) it is what the init segment gets cached under.
				// We do not want to use the redirected effective URL for the cache.
				CurrentConnectionInfo.EffectiveURL = PlayerSessionService->GetCMCDHandler()->RemoveParamsFromURL(CurrentConnectionInfo.EffectiveURL);
			}
			SetupSegmentDownloadStatsFromConnectionInfo(CurrentConnectionInfo);
			if (!bGotBoxes)
			{
				FString ErrMsg = BoxLocator.GetLastError();
				if (ErrMsg.IsEmpty())
				{
					ErrMsg = CurrentConnectionInfo.StatusInfo.ErrorDetail.GetMessage();
				}
				SetError(FString::Printf(TEXT("Self-initializing media `moov` scan error: %s"),	*ErrMsg), INTERNAL_SEG_ERROR_INIT_SEGMENT_DOWNLOAD_ERROR);
				return false;
			}
			bool bHaveMoov = false;
			for(int32 i=0; i<RootBoxes.Num(); ++i)
			{
				if (RootBoxes[i]->Type == MP4Utilities::MakeBoxAtom('m','o','o','v'))
				{
					StaticSegmentDataReader.SetSourceBuffer(RootBoxes[i]->GetBoxDataRAW());
					ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12;
					bHaveMoov = true;
					break;
				}
			}
			if (!bHaveMoov)
			{
				SetError(FString::Printf(TEXT("Self-initializing media does not have a 'moov' box")), INTERNAL_SEG_ERROR_INIT_SEGMENT_NOTFOUND_ERROR);
				return false;
			}
		}
		else
		{
			UtilsMP4::FMP4RootBoxLocator BoxLocator;
			TArray<UtilsMP4::FMP4RootBoxLocator::FBoxInfo> Boxes;
			const TArray<uint32> FirstBox { MP4Utilities::MakeBoxAtom('f','t','y','p') };
			const TArray<uint32> MoovBox { MP4Utilities::MakeBoxAtom('m','o','o','v') };
			bool bSuccess = BoxLocator.LocateRootBoxes(Boxes, PlayerSessionService->GetHTTPManager(), InitSegmentUrl, ReqHeaders, FirstBox, MoovBox, MoovBox, UtilsMP4::FMP4RootBoxLocator::FCancellationCheckDelegate::CreateLambda([&]()
			{
				return HasReadBeenAborted();
			}));
			if (HasReadBeenAborted())
			{
				return true;
			}
			CurrentConnectionInfo = BoxLocator.GetConnectionInfo();
			// If CMCD added query parameters to the URL then we need to remove them from the effective URL.
			if (!InitSegmentUrl.Equals(InitURLToUse.Url.URL))
			{
				// Keep the initial URL though as it was since it was (a) not modified and (b) it is what the init segment gets cached under.
				// We do not want to use the redirected effective URL for the cache.
				CurrentConnectionInfo.EffectiveURL = PlayerSessionService->GetCMCDHandler()->RemoveParamsFromURL(CurrentConnectionInfo.EffectiveURL);
			}
			SetupSegmentDownloadStatsFromConnectionInfo(CurrentConnectionInfo);
			if (!bSuccess)
			{
				FString ErrMsg = BoxLocator.GetErrorMessage();
				if (ErrMsg.IsEmpty())
				{
					ErrMsg = CurrentConnectionInfo.StatusInfo.ErrorDetail.GetMessage();
				}
				SetError(FString::Printf(TEXT("Self-initializing media 'moov' scan error: %s"),	*ErrMsg), INTERNAL_SEG_ERROR_INIT_SEGMENT_DOWNLOAD_ERROR);
				return false;
			}
			const UtilsMP4::FMP4RootBoxLocator::FBoxInfo* Moov = Boxes.FindByPredicate([InType=MP4Utilities::MakeBoxAtom('m','o','o','v')](const UtilsMP4::FMP4RootBoxLocator::FBoxInfo& InBox){return InBox.Type == InType;});
			if (Moov)
			{
				StaticSegmentDataReader.SetSourceBuffer(Moov->DataBuffer);
				ContainerFormat = FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12;
			}
			else
			{
				SetError(FString::Printf(TEXT("Self-initializing media does not have a 'moov' box")), INTERNAL_SEG_ERROR_INIT_SEGMENT_NOTFOUND_ERROR);
				return false;
			}
		}
	}

	// Set the response headers with the entity cache.
	if (EntityCache.IsValid())
	{
		EntityCache->SetRecentResponseHeaders(IPlayerEntityCache::EEntityType::Segment, InitURLToUse.Url.URL, CurrentConnectionInfo.ResponseHeaders);
	}

	// Parse the init segment.
	UEMediaError parseError = UEMEDIA_ERROR_FORMAT_ERROR;
	switch(ContainerFormat)
	{
		case FStreamSegmentRequestCommon::EContainerFormat::ISO_14496_12:
		{
			if (!Stream::CVarElectraStreamUseLegacyMP4.GetValueOnAnyThread())
			{
				TSharedPtr<FParsedMOOVCommon, ESPMode::ThreadSafe> ParsedMOOV = MakeShared<FParsedMOOVCommon, ESPMode::ThreadSafe>();
				if (ParsedMOOV->Parse(StaticSegmentDataReader.GetConstArrayView()))
				{
					InitSegmentData.Emplace<TSharedPtrTS<FParsedMOOVCommon>>(ParsedMOOV);
					// Add this to the entity cache in case it needs to be retrieved again.
					if (EntityCache.IsValid())
					{
						IPlayerEntityCache::FCacheItem CacheItem;
						CacheItem.URL = InitURLToUse.Url.URL;
						CacheItem.Range = InitURLToUse.Range;
						CacheItem.ParsedMOOV = MoveTemp(ParsedMOOV);
						EntityCache->CacheEntity(CacheItem);
					}
					DownloadStats.bWasSuccessful = true;
					StreamSelector->ReportDownloadEnd(DownloadStats);
					return true;
				}
				else
				{
					DownloadStats.bParseFailure = true;
					StreamSelector->ReportDownloadEnd(DownloadStats);
					SetError(FString::Printf(TEXT("Parse error of init segment \"%s\""), *InitURLToUse.Url.URL), INTERNAL_SEG_ERROR_INIT_SEGMENT_PARSE_ERROR);
					return false;
				}
			}
			else
			{
				TSharedPtrTS<IParserISO14496_12> MP4InitSeg = IParserISO14496_12::CreateParser();
				FMediaSegmentBoxCallback ParseBoxCallback;
				parseError = MP4InitSeg->ParseHeader(&StaticSegmentDataReader, &ParseBoxCallback, PlayerSessionService, nullptr);
				if (parseError == UEMEDIA_ERROR_OK || parseError == UEMEDIA_ERROR_END_OF_STREAM)
				{
					// Parse the tracks of the init segment. We do this mainly to get to the CSD we might need should we have to insert filler data later.
					parseError = MP4InitSeg->PrepareTracks(PlayerSessionService, TSharedPtrTS<const IParserISO14496_12>());
					if (parseError == UEMEDIA_ERROR_OK)
					{
						InitSegmentData.Emplace<TSharedPtrTS<const IParserISO14496_12>>(MP4InitSeg);
						// Add this to the entity cache in case it needs to be retrieved again.
						if (EntityCache.IsValid())
						{
							IPlayerEntityCache::FCacheItem CacheItem;
							CacheItem.URL = InitURLToUse.Url.URL;
							CacheItem.Range = InitURLToUse.Range;
							CacheItem.Parsed14496_12Data = MP4InitSeg;
							EntityCache->CacheEntity(CacheItem);
						}
						DownloadStats.bWasSuccessful = true;
						StreamSelector->ReportDownloadEnd(DownloadStats);
						return true;
					}
					else
					{
						DownloadStats.bParseFailure = true;
						StreamSelector->ReportDownloadEnd(DownloadStats);
						SetError(FString::Printf(TEXT("Track preparation of init segment \"%s\" failed"), *InitURLToUse.Url.URL), INTERNAL_SEG_ERROR_INIT_SEGMENT_PARSE_ERROR);
						return false;
					}
				}
				else
				{
					DownloadStats.bParseFailure = true;
					StreamSelector->ReportDownloadEnd(DownloadStats);
					SetError(FString::Printf(TEXT("Parse error of init segment \"%s\""), *InitURLToUse.Url.URL), INTERNAL_SEG_ERROR_INIT_SEGMENT_PARSE_ERROR);
					return false;
				}
			}
		}
		case FStreamSegmentRequestCommon::EContainerFormat::Matroska_WebM:
		{
			TSharedPtrTS<IParserMKV> MKVInitSeg = IParserMKV::CreateParser(nullptr);
			FErrorDetail MKVParseError = MKVInitSeg->ParseHeader(&StaticSegmentDataReader, static_cast<Electra::IParserMKV::EParserFlags>(IParserMKV::EParserFlags::ParseFlag_OnlyTracks));
			if (MKVParseError.IsOK())
			{
				MKVParseError = MKVInitSeg->PrepareTracks();
				if (MKVParseError.IsOK())
				{
					InitSegmentData.Emplace<TSharedPtrTS<const IParserMKV>>(MKVInitSeg);
					// Add this to the entity cache in case it needs to be retrieved again.
					if (EntityCache.IsValid())
					{
						IPlayerEntityCache::FCacheItem CacheItem;
						CacheItem.URL = InitURLToUse.Url.URL;
						CacheItem.Range = InitURLToUse.Range;
						CacheItem.ParsedMatroskaData = MKVInitSeg;
						EntityCache->CacheEntity(CacheItem);
					}
					DownloadStats.bWasSuccessful = true;
					StreamSelector->ReportDownloadEnd(DownloadStats);
					return true;
				}
				else
				{
					DownloadStats.bParseFailure = true;
					StreamSelector->ReportDownloadEnd(DownloadStats);
					SetError(FString::Printf(TEXT("Track preparation of init segment \"%s\" failed"), *InitURLToUse.Url.URL), INTERNAL_SEG_ERROR_INIT_SEGMENT_PARSE_ERROR);
					return false;
				}
			}
			else
			{
				DownloadStats.bParseFailure = true;
				StreamSelector->ReportDownloadEnd(DownloadStats);
				SetError(FString::Printf(TEXT("Parse error of init segment \"%s\""), *InitURLToUse.Url.URL), INTERNAL_SEG_ERROR_INIT_SEGMENT_PARSE_ERROR);
				return false;
			}
		}
		case FStreamSegmentRequestCommon::EContainerFormat::ISO_13818_1:
		{
			TSharedPtrTS<IParserISO13818_1> TSParser = IParserISO13818_1::CreateParser();
			IParserISO13818_1::FSourceInfo TSSourceInfo;
			IParserISO13818_1::EParseState TSParseState = TSParser->BeginParsing(PlayerSessionService, &StaticSegmentDataReader, IParserISO13818_1::EParserFlags::ParseFlag_Default, TSSourceInfo);
			while(TSParseState != IParserISO13818_1::EParseState::Failed && TSParseState != IParserISO13818_1::EParseState::EOS)
			{
				TSParseState = TSParser->Parse(PlayerSessionService, &StaticSegmentDataReader);
				if (TSParseState == IParserISO13818_1::EParseState::NewProgram)
				{
					// We do not cache the parser state for TS files but the raw packets that led up to the new program.
					TSharedPtrTS<TArray<uint8>> TSPacketBlob = MakeSharedTS<TArray<uint8>>(StaticSegmentDataReader.GetBufferBaseAddress(), StaticSegmentDataReader.GetCurrentOffset());
					InitSegmentData.Emplace<TSharedPtrTS<const TArray<uint8>>>(TSPacketBlob);
					// Add this to the entity cache in case it needs to be retrieved again.
					if (EntityCache.IsValid())
					{
						IPlayerEntityCache::FCacheItem CacheItem;
						CacheItem.URL = InitURLToUse.Url.URL;
						CacheItem.Range = InitURLToUse.Range;
						CacheItem.RawPayloadData = TSPacketBlob;
						EntityCache->CacheEntity(CacheItem);
					}
					DownloadStats.bWasSuccessful = true;
					StreamSelector->ReportDownloadEnd(DownloadStats);
					return true;
				}
			}
			DownloadStats.bParseFailure = true;
			StreamSelector->ReportDownloadEnd(DownloadStats);
			SetError(FString::Printf(TEXT("Track preparation of init segment \"%s\" failed"), *InitURLToUse.Url.URL), INTERNAL_SEG_ERROR_INIT_SEGMENT_PARSE_ERROR);
			return false;
		}
		case FStreamSegmentRequestCommon::EContainerFormat::WebVTT_Raw:
		{
			TSharedPtrTS<TArray<uint8>> RawPacketBlob = MakeSharedTS<TArray<uint8>>(StaticSegmentDataReader.GetBufferBaseAddress(), StaticSegmentDataReader.GetTotalSize());
			InitSegmentData.Emplace<TSharedPtrTS<const TArray<uint8>>>(RawPacketBlob);
			// Add this to the entity cache in case it needs to be retrieved again.
			if (EntityCache.IsValid())
			{
				IPlayerEntityCache::FCacheItem CacheItem;
				CacheItem.URL = InitURLToUse.Url.URL;
				CacheItem.Range = InitURLToUse.Range;
				CacheItem.RawPayloadData = RawPacketBlob;
				EntityCache->CacheEntity(CacheItem);
			}
			DownloadStats.bWasSuccessful = true;
			StreamSelector->ReportDownloadEnd(DownloadStats);
			return true;
		}
		default:
		{
			break;
		}
	}
	return true;
}

FStreamSegmentReaderCommon::FStreamHandler::EMediaSegmentTriggerResult FStreamSegmentReaderCommon::FStreamHandler::TriggerMediaSegmentDownload(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	SegmentError.Clear();

	DownloadStats.ResetOutput();
	DownloadStats.StatsID = FMediaInterlockedIncrement(UniqueDownloadID);
	DownloadStats.SegmentType = Metrics::ESegmentType::Media;
	DownloadStats.URL = InRequest->Segment.MediaURL.Url;
	DownloadStats.Range = InRequest->Segment.MediaURL.Range;
	DownloadStats.SteeringID = InRequest->Segment.MediaURL.SteeringID;

	Parameters.EventListener->OnFragmentOpen(InRequest);

	// If this is an EOS segment we are not supposed to do anything with it.
	if (InRequest->bIsEOSSegment)
	{
		return EMediaSegmentTriggerResult::DontHandle;
	}
	// Also do nothing if this segment fell off the timeline.
	else if (InRequest->bIsFalloffSegment)
	{
		return EMediaSegmentTriggerResult::DontHandle;
	}
	// For empty filler data segments or gap segments we do not trigger the download.
	else if (InRequest->bInsertFillerData || InRequest->bIsGapSegment)
	{
		return EMediaSegmentTriggerResult::IsFiller;
	}
	// Sideloaded data?
	else if (InRequest->Segment.bIsSideload)
	{
		return EMediaSegmentTriggerResult::IsSideloaded;
	}

	ReadBuffer.Reset();
	ReadBuffer.ReceiveBuffer = MakeSharedTS<FWaitableBuffer>();

	ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->ProgressDelegate   = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateRaw(this, &FStreamHandler::HTTPProgressCallback);
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateRaw(this, &FStreamHandler::HTTPCompletionCallback);
	HTTPRequest = MakeSharedTS<IElectraHttpManager::FRequest>();
	HTTPRequest->ReceiveBuffer = ReadBuffer.ReceiveBuffer;
	HTTPRequest->ProgressListener = ProgressListener;
	HTTPRequest->ResponseCache = PlayerSessionService->GetHTTPResponseCache();
	HTTPRequest->Parameters.URL = InRequest->Segment.MediaURL.Url.URL;
	HTTPRequest->Parameters.Range.Set(InRequest->Segment.MediaURL.Range);
	HTTPRequest->Parameters.StreamType = InRequest->StreamType;
	HTTPRequest->Parameters.QualityIndex = InRequest->QualityIndex;
	HTTPRequest->Parameters.MaxQualityIndex = InRequest->MaxQualityIndex;
	HTTPRequest->Parameters.AcceptEncoding.Set(TEXT("identity"));
	if (InRequest->Segment.MediaURL.CustomHeader.Len())
	{
		HTTPRequest->Parameters.RequestHeaders.Emplace(HTTP::FHTTPHeader({DASH::HTTPHeaderOptionName, InRequest->Segment.MediaURL.CustomHeader}));
	}
	HTTPRequest->Parameters.bCollectTimingTraces = InRequest->Segment.bLowLatencyChunkedEncodingExpected;
	// Set timeouts for media segment retrieval
	HTTPRequest->Parameters.ConnectTimeout = PlayerSessionService->GetOptionValue(DASH::OptionKeyMediaSegmentConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 4));
	HTTPRequest->Parameters.NoDataTimeout = PlayerSessionService->GetOptionValue(DASH::OptionKeyMediaSegmentNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 4));

	// Set up information for CMCD.
	FTimeValue SegmentDuration(InRequest->Segment.Duration, InRequest->Segment.Timescale, 0);
	FCMCDHandler::FRequestObjectInfo ri;
	ri.MediaStreamType = InRequest->StreamType;
	ri.EncodedBitrate = InRequest->GetBitrate();
	ri.ObjectDuration = (int32) SegmentDuration.GetAsMilliseconds();
	for(auto& NextMediaUrl : InRequest->Segment.NextMediaURLS)
	{
		auto& nxt = ri.NextObjectRequest.Emplace_GetRef();
		nxt.URL = NextMediaUrl.Url.URL;
		nxt.Range = NextMediaUrl.Range;
	}
	FCMCDHandler::EObjectType CMCDot =
		InRequest->bIsMultiplex && !InRequest->bIgnoreVideo && !InRequest->bIgnoreAudio ? FCMCDHandler::EObjectType::MuxedAudioAndVideo :
		InRequest->StreamType == EStreamType::Video ? FCMCDHandler::EObjectType::VideoOnly :
		InRequest->StreamType == EStreamType::Audio ? FCMCDHandler::EObjectType::AudioOnly : FCMCDHandler::EObjectType::CaptionOrSubtitle;
	PlayerSessionService->GetCMCDHandler()->SetupRequestObject(FCMCDHandler::ERequestType::Segment, CMCDot, HTTPRequest->Parameters.URL, HTTPRequest->Parameters.RequestHeaders, InRequest->Segment.MediaURL.Url.CDN, ri);
	bAddedCMCDQueryToMediaURL = !HTTPRequest->Parameters.URL.Equals(InRequest->Segment.MediaURL.Url.URL);

	ProgressReportCount = 0;
	PlayerSessionService->GetHTTPManager()->AddRequest(HTTPRequest, false);

	return EMediaSegmentTriggerResult::Started;
}

void FStreamSegmentReaderCommon::FStreamHandler::HandleCommonMediaBegin(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH)
	{
		// Clear out the list of events found the last time.
		SegmentEventsFound.Empty();
		CheckForInbandDASHEvents(InRequest);
	}
}

void FStreamSegmentReaderCommon::FStreamHandler::HandleCommonMediaEnd(EHandleResult InSegmentResult, TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	Decrypter.Reset();

	ProgressListener.Reset();
	if (HTTPRequest.IsValid())
	{
		PlayerSessionService->GetHTTPManager()->RemoveRequest(HTTPRequest, false);
		CurrentConnectionInfo = HTTPRequest->ConnectionInfo;
		if (bAddedCMCDQueryToMediaURL)
		{
			CurrentConnectionInfo.EffectiveURL = PlayerSessionService->GetCMCDHandler()->RemoveParamsFromURL(CurrentConnectionInfo.EffectiveURL);
		}
		HTTPRequest.Reset();
	}

	if (InSegmentResult == EHandleResult::Failed && !HasReadBeenAborted())
	{
		bHasErrored = true;
	}

	// If the segment was skipped, we must neither insert filler data, not emit any access units
	// that may have been added to the track data maps already (from demuxed tracks that came
	// before the track that was checked).
	if (InSegmentResult == EHandleResult::Skipped)
	{
		TrackDataMap.Empty();
	}

	// Do we need to fill remaining duration with dummy data?
	if (bFillRemainingDuration)
	{
		for(auto& ItActiveTrackData : TrackDataMap)
		{
			InsertFillerData(ItActiveTrackData.Value, InRequest);
		}
		// TBD: Does that mean we need to clear errors as this means the download was actually successful?
	}

	// We now send all the pending AUs in stages until all tracks have sent all their AUs.
	// We need to do it that way to ensure that one buffer will not run dry while we are
	// stalled on feeding the other buffer.
	FTimeValue DurationDownloaded(FTimeValue::GetZero());
	FTimeValue DurationDelivered(FTimeValue::GetZero());
	if (bAbortedByABR && !bAllowEarlyEmitting)
	{
		// When asked to abort and we were not allowed to send any AUs yet then we must also
		// not send what we have accumulated now since the segment will be retried on another
		// quality level.
	}
	else
	{
		while(1)
		{
			bool bSentSomething = false;
			for(auto& ItActiveTrackData : TrackDataMap)
			{
				CurrentlyActiveTrackData = ItActiveTrackData.Value;
				CurrentlyActiveTrackData->bGotAllSamples = true;
				EEmitResult er = EmitSamples(EEmitType::UntilBlocked);
				if (CurrentlyActiveTrackData->DurationSuccessfullyRead > DurationDownloaded)
				{
					DurationDownloaded = CurrentlyActiveTrackData->DurationSuccessfullyRead;
				}
				if (CurrentlyActiveTrackData->DurationSuccessfullyDelivered > DurationDelivered)
				{
					DurationDelivered = CurrentlyActiveTrackData->DurationSuccessfullyDelivered;
				}
				bSentSomething |= er != EEmitResult::SentEverything;
			}
			if (!bSentSomething)
			{
				break;
			}
			FMediaRunnable::SleepMilliseconds(100);
		}
	}

	// Dump the active track map.
	TrackDataMap.Empty();
	CurrentlyActiveTrackData.Reset();
	PrimaryTrackData.Reset();

	if (DownloadStats.FailureReason.IsEmpty())
	{
		DownloadStats.FailureReason = SegmentError.GetMessage();
	}
	// If the ABR aborted this takes precedence in the failure message. Overwrite it.
	if (bAbortedByABR)
	{
		DownloadStats.FailureReason = ABRAbortReason;
	}
	// Set up remaining download stat fields.
	DownloadStats.bWasAborted = bAbortedByABR;
	DownloadStats.bWasSuccessful = !bHasErrored && !bAbortedByABR;
	DownloadStats.bDidTimeout = CurrentConnectionInfo.StatusInfo.ErrorCode == HTTP::EStatusErrorCode::ERRCODE_HTTP_CONNECTION_TIMEOUT;
	DownloadStats.URL.URL = CurrentConnectionInfo.EffectiveURL;
	DownloadStats.HTTPStatusCode = CurrentConnectionInfo.StatusInfo.HTTPStatus;
	DownloadStats.DurationDownloaded = DurationDownloaded.GetAsSeconds();
	DownloadStats.DurationDelivered = DurationDelivered.GetAsSeconds();
	DownloadStats.TimeToFirstByte = CurrentConnectionInfo.TimeUntilFirstByte;
	DownloadStats.TimeToDownload = (CurrentConnectionInfo.RequestEndTime - CurrentConnectionInfo.RequestStartTime).GetAsSeconds();
	DownloadStats.ByteSize = CurrentConnectionInfo.ContentLength;
	DownloadStats.NumBytesDownloaded = CurrentConnectionInfo.BytesReadSoFar;
	DownloadStats.bIsCachedResponse = CurrentConnectionInfo.bIsCachedResponse;
	DownloadStats.bWasSkipped = InSegmentResult == EHandleResult::Skipped;
	DownloadStats.bWasFalloffSegment = InRequest->bIsFalloffSegment;
	CurrentConnectionInfo.GetTimingTraces(DownloadStats.TimingTraces);

	// Was this request for a segment that might potentially be missing and it did?
	if (InRequest->Segment.bMayBeMissing && (DownloadStats.HTTPStatusCode == 404 || DownloadStats.HTTPStatusCode == 416))
	{
		// This is not an actual error then. Pretend all was well.
		DownloadStats.bWasSuccessful = true;
		DownloadStats.HTTPStatusCode = 200;
		DownloadStats.bIsMissingSegment = true;
		DownloadStats.FailureReason.Empty();
		CurrentConnectionInfo.StatusInfo.Empty();
		// Take note of the missing segment in the segment info as well so the search for the next segment
		// can return quicker.
		InRequest->Segment.bIsMissing = true;
	}

	// DASH specific
	if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH)
	{
		// If we had to wait for the segment to become available and we got a 404 back we might have been trying to fetch
		// the segment before the server made it available.
		if (InRequest->ASAST.IsValid() && (DownloadStats.HTTPStatusCode == 404 || DownloadStats.HTTPStatusCode == 416))
		{
			FTimeValue Now = PlayerSessionService->GetSynchronizedUTCTime()->GetTime();
			if ((DownloadStats.AvailibilityDelay = (InRequest->ASAST - Now).GetAsSeconds()) == 0.0)
			{
				// In the extremely unlikely event this comes out to zero exactly set a small value so the ABR knows there was a delay.
				DownloadStats.AvailibilityDelay = -0.01;
			}
		}

		// If we failed to get the segment and there is an inband DASH event stream which triggers MPD events and
		// we did not get such an event in the 'emsg' boxes, then we err on the safe side and assume this segment
		// would have carried an MPD update event and fire an artificial event.
		if (!DownloadStats.bWasSuccessful && InRequest->Segment.InbandEventStreams.FindByPredicate([](const FSegmentInformationCommon::FInbandEventStream& This){ return This.SchemeIdUri.Equals(DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_2012); }))
		{
			if (SegmentEventsFound.IndexOfByPredicate([](const TSharedPtrTS<DASH::FPlayerEvent>& This){return This->GetSchemeIdUri().Equals(DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_2012);}) == INDEX_NONE)
			{
				SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
				CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
				TSharedPtrTS<DASH::FPlayerEvent> NewEvent = MakeSharedTS<DASH::FPlayerEvent>();
				NewEvent->SetOrigin(IAdaptiveStreamingPlayerAEMSEvent::EOrigin::InbandEventStream);
				NewEvent->SetSchemeIdUri(DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_2012);
				NewEvent->SetValue(TEXT("1"));
				NewEvent->SetID("$missed$");
				FTimeValue EPT((int64)InRequest->Segment.Time, InRequest->Segment.Timescale);
				FTimeValue PTO(InRequest->Segment.PTO, InRequest->Segment.Timescale);
				FTimeValue TimeOffset = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
				NewEvent->SetPresentationTime(TimeOffset - PTO + EPT);
				NewEvent->SetPeriodID(InRequest->Period->GetUniqueIdentifier());
				PlayerSessionService->GetAEMSEventHandler()->AddEvent(NewEvent, InRequest->Period->GetUniqueIdentifier(), IAdaptiveStreamingPlayerAEMSHandler::EEventAddMode::AddIfNotExists);
			}
		}
	}

	InRequest->ConnectionInfo = CurrentConnectionInfo;
	InRequest->DownloadStats = DownloadStats;

	// Clean out everything before reporting OnFragmentClose().
	CurrentRequest.Reset();
	ReadBuffer.Reset();
	SegmentEventsFound.Empty();

	if (!bSilentCancellation)
	{
		StreamSelector->ReportDownloadEnd(DownloadStats);
		Parameters.EventListener->OnFragmentClose(InRequest);
	}
}

void FStreamSegmentReaderCommon::FStreamHandler::SelectPrimaryTrackData(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest)
{
	for(auto& It : TrackDataMap)
	{
		if (It.Value->StreamType == EStreamType::Video && !InRequest->bIgnoreVideo)
		{
			PrimaryTrackData = It.Value;
			return;
		}
		else if (It.Value->StreamType == EStreamType::Audio && !InRequest->bIgnoreAudio)
		{
			PrimaryTrackData = It.Value;
			return;
		}
		else if (It.Value->StreamType == EStreamType::Subtitle && !InRequest->bIgnoreSubtitles)
		{
			PrimaryTrackData = It.Value;
			return;
		}
	}
}


FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::HandleFillerDataSetup(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	// We do not necessarily have an existing init segment to look at (although for fmp4 and mkv we most likely will),
	// so we need to rely on the information in the request only.
	TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> NewTrackDataMap;
	auto AddTrack = [&](EStreamType InStreamType) -> void
	{
		int32 Idx = StreamTypeToArrayIndex(InStreamType);
		TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
		td->StreamType = InStreamType;
		td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[Idx]));
		td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
		NewTrackDataMap.Emplace(Idx, MoveTemp(td));
	};
	for(int32 stIdx=0; stIdx<3; ++stIdx)
	{
		if (stIdx == 0 && InRequest->SourceBufferInfo[stIdx].IsValid() && !InRequest->bIgnoreVideo)
		{
			AddTrack(EStreamType::Video);
		}
		else if (stIdx == 1 && InRequest->SourceBufferInfo[stIdx].IsValid() && !InRequest->bIgnoreAudio)
		{
			AddTrack(EStreamType::Audio);
		}
		else if (stIdx == 2 && InRequest->SourceBufferInfo[stIdx].IsValid() && !InRequest->bIgnoreSubtitles)
		{
			AddTrack(EStreamType::Subtitle);
		}
	}
	TrackDataMap = MoveTemp(NewTrackDataMap);
	return TrackDataMap.Num() ? EHandleResult::Finished : EHandleResult::Failed;
}


FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::HandleMP4Media(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	// We need to have an mp4 init segment.
	if (!InitSegmentData.IsType<TSharedPtrTS<FParsedMOOVCommon>>())
	{
		return SetError(FString::Printf(TEXT("Wrong init segment type for fmp4 media segment")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
	}
	TSharedPtrTS<FParsedMOOVCommon> InitSegment = InitSegmentData.Get<TSharedPtrTS<FParsedMOOVCommon>>();
	if (!InitSegment.IsValid())
	{
		return SetError(FString::Printf(TEXT("No init segment for fmp4 media segment")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
	}
	if (InitSegment->GetTrackMap().IsEmpty())
	{
		return SetError(FString::Printf(TEXT("Init segment contains no tracks")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
	}
	TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> NewTrackDataTypeMap[4];
	for(auto &tkmIt : InitSegment->GetTrackMap())
	{
		const FParsedMOOVCommon::FTrackInfo& tkInf(tkmIt.Value);
		if (tkInf.CodecFormat.Type == FCodecTypeFormat::EType::Video && InRequest->bIgnoreVideo)
		{
			continue;
		}
		else if (tkInf.CodecFormat.Type == FCodecTypeFormat::EType::Audio && InRequest->bIgnoreAudio)
		{
			continue;
		}
		else if (tkInf.CodecFormat.Type == FCodecTypeFormat::EType::Subtitle && InRequest->bIgnoreSubtitles)
		{
			continue;
		}

		TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
		td->StreamType = tkInf.CodecFormat.Type == FCodecTypeFormat::EType::Video ? EStreamType::Video :
						 tkInf.CodecFormat.Type == FCodecTypeFormat::EType::Audio ? EStreamType::Audio : EStreamType::Subtitle;
		const int32 stIdx = StreamTypeToArrayIndex(td->StreamType);
		// Copy the source buffer info into a new instance and set the playback sequence ID in it.
		check(InRequest->SourceBufferInfo[stIdx].IsValid());
		if (!InRequest->SourceBufferInfo[stIdx].IsValid())
		{
			continue;
		}
		td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[stIdx]));
		td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
		// Set the CSD
		td->CSD = MakeSharedTS<FAccessUnit::CodecData>();
		td->CSD->CodecSpecificData = tkInf.CodecFormat.CSD;
		td->CSD->RawCSD = tkInf.CodecFormat.DCR;
		td->CSD->ParsedInfo = FStreamCodecInformation(tkInf.CodecFormat);
		// Set information from the playlist codec information that may not be available or accurate in the init segment.
		td->CSD->ParsedInfo.SetBitrate(InRequest->CodecInfo[stIdx].GetBitrate());
		NewTrackDataTypeMap[stIdx].Emplace(tkInf.TrackID, MoveTemp(td));
	}
	// At present we only want to have a single track per type in the media segment. If there are more we use the one
	// with the smallest ID (a stipulation made by the HLS specification; we apply it regardless of protocol)
	for(int32 nt=0; nt<3; ++nt)
	{
		if (NewTrackDataTypeMap[nt].Num())
		{
			auto it(NewTrackDataTypeMap[nt].CreateConstIterator());
			TrackDataMap.Add(it.Key(), it.Value());
		}
	}
	if (TrackDataMap.Num() == 0 || (InRequest->bIsMultiplex && NewTrackDataTypeMap[StreamTypeToArrayIndex(InRequest->GetType())].Num() == 0))
	{
		SetError(FString::Printf(TEXT("Init segment contains no usable tracks")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}

	SelectPrimaryTrackData(InRequest);

	// If this is a filler segment then we are done at this point and let the end-of-segment handling
	// create the filler data for the active tracks we just set up.
	if (InRequest->bInsertFillerData)
	{
		return EHandleResult::Finished;
	}

	// See if the segment is encrypted (sample encryption, not whole segment)
	// The assumption is that all tracks in the segment are encrypted the same way and a single decrypter will do.
	if (!ReadBuffer.bIsEncrypted && InRequest->DrmMedia.DrmClient.IsValid())
	{
		check(!Decrypter.IsValid());
		if (InRequest->DrmMedia.DrmClient->CreateDecrypter(Decrypter, InRequest->DrmMedia.DrmMimeType) != ElectraCDM::ECDMError::Success)
		{
			SetError(FString::Printf(TEXT("Failed to create decrypter for segment, \"%s\""), *InRequest->DrmMedia.DrmClient->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
			return EHandleResult::Failed;
		}
	}

	// Create a box loader wrapper.
	TSharedPtr<FBoxDataLoader, ESPMode::ThreadSafe> BoxLoader = MakeShared<FBoxDataLoader, ESPMode::ThreadSafe>(this);

	// Enter the parsing loop.
	bool bDone = false;
	FTimeValue SegmentDuration(InRequest->Segment.Duration, InRequest->Segment.Timescale, 0);
	FTimeValue TimeOffset = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
	bool bDoNotTruncateAtPresentationEnd = PlayerSessionService->GetOptionValue(OptionKeyDoNotTruncateAtPresentationEnd).SafeGetBool(false);
	bool bIsLastSegment = InRequest->Segment.bIsLastInPeriod;
	const FString& RequestURL(InRequest->Segment.MediaURL.Url.URL);
	bool bSkippedBecauseOfTimestampCheck = false;
	int32 StreamTypeAUCount[4] { 0,0,0,0 };
	int64 LastByteOffsetOfMDAT = -1;
	bool bTimeCheckPassed = InRequest->TimestampVars.Next.bCheck == false;
	bool bSendMetaData = !!InitSegment->GetMETABox();
	bool bSendEventData = true;
	int32 NumMoofs = 0;
	while(!bDone && !HasErrored() && !HasReadBeenAborted())
	{
		if (HasReachedEOF())
		{
			bDone = true;
			break;
		}
		Metrics::FSegmentDownloadStats::FMovieChunkInfo MoofInfo;
		MoofInfo.HeaderOffset = GetCurrentOffset();
		LastByteOffsetOfMDAT = -1;

		// Read all the root boxes up to the next `mdat` box.
		MP4Utilities::FMP4BoxLocator BoxLocator;
		const TArray<uint32> UnspecBoxes;
		TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> RootBoxes;
		bool bGotBoxes = BoxLocator.LocateAndReadRootBoxes(RootBoxes, BoxLoader, UnspecBoxes, UnspecBoxes, true, UnspecBoxes, FMP4DataLoader::FCancellationCheckDelegate());
		// If we did not get the necessary boxes see if we hit EOF. This may happen if there are trailing boxes that we do
		// not need, like `mfra` or similar.
		if (!bGotBoxes)
		{
			bDone = true;
			if (HasReachedEOF())
			{
				break;
			}
			else if (!HasErrored() && !HasReadBeenAborted())
			{
				SetError(FString::Printf(TEXT("Failed to find movie boxes in segment \"%s\""), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			}
			break;
		}
		if (HasErrored() || HasReadBeenAborted())
		{
			break;
		}

		TArray<TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>> ParsedRootBoxes;
		ParsedRootBoxes.SetNum(RootBoxes.Num());
		TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe> MetaBox;
		TSharedPtr<MP4Boxes::FMP4BoxSIDX, ESPMode::ThreadSafe> SidxBox;
		TArray<TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe>> MoofMetaBoxes;
		TArray<TSharedPtr<MP4Boxes::FMP4BoxPRFT, ESPMode::ThreadSafe>> ProducerReferenceTimeBoxes;
		TArray<TSharedPtr<MP4Boxes::FMP4BoxEMSG, ESPMode::ThreadSafe>> EventMessageBoxes;
		TArray<TSharedPtr<MP4Boxes::FMP4BoxPSSH, ESPMode::ThreadSafe>> ProtectionSystemBoxes;
		for(int32 i=0; i<RootBoxes.Num(); ++i)
		{
			MP4Boxes::FMP4BoxTreeParser tp;
			if (tp.ParseBoxTree(RootBoxes[i]))
			{
				ParsedRootBoxes[i] = tp.GetBoxTree();
				switch(ParsedRootBoxes[i]->GetType())
				{
					case MP4Utilities::MakeBoxAtom('m','o','o','f'):
					{
						++NumMoofs;

						// Is there a udta/meta in this moof?
						auto UdtaBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxMOOF>(ParsedRootBoxes[i])->FindBoxRecursive<MP4Boxes::FMP4BoxUDTA>(MP4Utilities::MakeBoxAtom('u','d','t','a'), 0);
						if (UdtaBox)
						{
							auto UdtaMetaBox = UdtaBox->FindBoxRecursive<MP4Boxes::FMP4BoxMETA>(MP4Utilities::MakeBoxAtom('m','e','t','a'), 0);
							if (UdtaMetaBox)
							{
								MoofMetaBoxes.Emplace(MoveTemp(UdtaMetaBox));
								bSendMetaData = true;
							}
						}
						break;
					}
					case MP4Utilities::MakeBoxAtom('m','d','a','t'):
					{
						LastByteOffsetOfMDAT = ParsedRootBoxes[i]->GetBoxFileOffset() + ParsedRootBoxes[i]->GetBoxSize();
						break;
					}
					case MP4Utilities::MakeBoxAtom('s','t','y','p'):
					case MP4Utilities::MakeBoxAtom('f','t','y','p'):
					{
						auto XtypBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxFTYP>(ParsedRootBoxes[i]);
						if (XtypBox->GetAllBrands().Contains(MP4Utilities::MakeBoxAtom('l','m','s','g')))
						{
							InRequest->Segment.bSawLMSG = true;
						}
						break;
					}
					case MP4Utilities::MakeBoxAtom('u','d','t','a'):
					{
						MetaBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxUDTA>(ParsedRootBoxes[i])->FindBoxRecursive<MP4Boxes::FMP4BoxMETA>(MP4Utilities::MakeBoxAtom('m','e','t','a'), 0);
						bSendMetaData = true;
						break;
					}
					case MP4Utilities::MakeBoxAtom('s','i','d','x'):
					{
						SidxBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxSIDX>(ParsedRootBoxes[i]);
						break;
					}
					case MP4Utilities::MakeBoxAtom('p','r','f','t'):
					{
						ProducerReferenceTimeBoxes.Add(StaticCastSharedPtr<MP4Boxes::FMP4BoxPRFT>(ParsedRootBoxes[i]));
						break;
					}
					case MP4Utilities::MakeBoxAtom('e','m','s','g'):
					{
						EventMessageBoxes.Add(StaticCastSharedPtr<MP4Boxes::FMP4BoxEMSG>(ParsedRootBoxes[i]));
						break;
					}
					case MP4Utilities::MakeBoxAtom('p','s','s','h'):
					{
						ProtectionSystemBoxes.Add(StaticCastSharedPtr<MP4Boxes::FMP4BoxPSSH>(ParsedRootBoxes[i]));
						break;
					}
					default:
					{
						break;
					}
				}
			}
			else
			{
				return SetError(FString::Printf(TEXT("Failed to parse movie boxes in segment \"%s\""), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			}
		}
		if (LastByteOffsetOfMDAT < 0)
		{
			if (NumMoofs == 0)
			{
				return SetError(FString::Printf(TEXT("`mdat` box not found in segment \"%s\""), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			}
			else
			{
				// We probably encountered trailing boxes like `mfra` or similar.
				bDone = true;
				break;
			}
		}

		struct FSelectedTrackParams
		{
			FDRMTypeFormat DrmFormat;
			ElectraCDM::FMediaCDMSampleInfo SampleEncryptionInfo;
			TSharedPtr<MP4Boxes::FMP4Track, ESPMode::ThreadSafe> Track;
			TSharedPtr<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe> FragmentInfo;
			TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe> Iterator;
			TArray<TSharedPtr<MP4Boxes::FMP4BoxPRFT, ESPMode::ThreadSafe>> PRFTBoxes;
			uint32 Timescale = 0;
			bool bIsEncrypted = false;
			bool SetupBaseEncryption()
			{
				bIsEncrypted = DrmFormat.EncryptionInfo.IsType<FDRMTypeFormat::FISOEncryptionInfo>();
				if (bIsEncrypted)
				{
					const FDRMTypeFormat::FISOEncryptionInfo& encInf(DrmFormat.EncryptionInfo.Get<FDRMTypeFormat::FISOEncryptionInfo>());
					SampleEncryptionInfo.DefaultKID = encInf.DefaultKID;
					SampleEncryptionInfo.IV = encInf.DefaultIV;
					SampleEncryptionInfo.Scheme4CC = encInf.Scheme;
					if (encInf.BlockPattern.IsSet())
					{
						SampleEncryptionInfo.Pattern.PatternType = 1;
						SampleEncryptionInfo.Pattern.CryptByteBlock = encInf.BlockPattern.GetValue().CryptByteBlock;
						SampleEncryptionInfo.Pattern.SkipByteBlock = encInf.BlockPattern.GetValue().SkipByteBlock;
					}
					return true;
				}
				return false;
			}
			bool SetupSampleEncryption()
			{
				if (bIsEncrypted)
				{
					const MP4Boxes::FMP4BoxSENC::FEntry& enc = Iterator->GetEncryptionInfo();
					SampleEncryptionInfo.SubSamples.Empty();
					SampleEncryptionInfo.IV = enc.IV;
					for(auto &sub : enc.SubSamples)
					{
						SampleEncryptionInfo.SubSamples.Add({.NumClearBytes = sub.NumClearBytes, .NumEncryptedBytes = sub.NumEncryptedBytes});
					}
					return true;
				}
				return false;
			}
		};
		TMap<uint32, FSelectedTrackParams> SelectedTrackParamMap;

		// Prepare tracks
		for(auto &tkmIt : InitSegment->GetTrackMap())
		{
			const FParsedMOOVCommon::FTrackInfo& tkInf(tkmIt.Value);
			FSelectedTrackParams SelectedTrackParam;
			SelectedTrackParam.Timescale = tkInf.TrackTimescale;
			SelectedTrackParam.DrmFormat = tkInf.DrmFormat;
			SelectedTrackParam.SetupBaseEncryption();
			SelectedTrackParam.FragmentInfo = MakeShared<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe>();
			if (!SelectedTrackParam.FragmentInfo->Prepare(tkInf.TrakBox, InitSegment->GetMVEXBox(), ParsedRootBoxes))
			{
				return SetError(FString::Printf(TEXT("Failed to prepare fragment information in segment \"%s\""), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			}
			SelectedTrackParam.Track = MP4Boxes::FMP4Track::Create(tkInf.TrakBox, SelectedTrackParam.FragmentInfo);
			MP4Utilities::FFractionalTime Inf;
			if (!SelectedTrackParam.Track->Prepare(Inf, Inf))
			{
				return SetError(FString::Printf(TEXT("Failed to prepare track in segment \"%s\""), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			}
			SelectedTrackParam.Iterator = SelectedTrackParam.Track->CreateIterator();
			if (!SelectedTrackParam.Iterator)
			{
				return SetError(FString::Printf(TEXT("Failed to prepare track iterator in segment \"%s\""), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			}
			// Collect the `prft` boxes for this track
			for(auto &prft : ProducerReferenceTimeBoxes)
			{
				if (prft->GetReferenceTrackID() == tkInf.TrackID)
				{
					SelectedTrackParam.PRFTBoxes.Add(prft);
				}
			}
			SelectedTrackParamMap.Add(tkInf.TrackID, SelectedTrackParam);
		}

		// Send the metadata from the init segment or from this `moof`.
		// This is not per-track metadata, only segment-global metadata!
		if (bSendMetaData)
		{
			// Use the first meta data box contained in the moof, if it exists.
			// Otherwise use the root level metadata box of the fragment, which should not
			// exist because it is not permitted by ISO/IEC 14496:12 to be at the root level
			// but in case some tool puts it there we use it.
			if (MoofMetaBoxes.Num())
			{
				MetaBox = MoofMetaBoxes[0];
			}
			HandleMP4Metadata(InRequest, InitSegment->GetMETABox(), MetaBox);
			bSendMetaData = false;
		}

		// Send event data from `emsg` boxes. Do this only for the first round of boxes we encounter, not repeated ones.
		if (bSendEventData && EventMessageBoxes.Num())
		{
			HandleMP4EventMessages(InRequest, SidxBox, EventMessageBoxes);
			bSendEventData = false;
		}

		// Locate the track iterator that will handle the next incoming data.
		auto GetNextTrackIndexToIterate = [&]() -> int64
		{
			int64 NextIteratorFileDataOffset = TNumericLimits<int64>::Max();
			int64 ItIdx = -1;
			for(auto &stpIt : SelectedTrackParamMap)
			{
				if (stpIt.Value.Iterator && stpIt.Value.Iterator->GetSampleFileOffset() < NextIteratorFileDataOffset)
				{
					NextIteratorFileDataOffset = stpIt.Value.Iterator->GetSampleFileOffset();
					ItIdx = stpIt.Key;
				}
			}
			return ItIdx;
		};


		// Process consecutive samples from each track.
		int64 NextTrackIndex = -1;
		while(!bDone && !HasErrored() && !HasReadBeenAborted())
		{
			if (NextTrackIndex < 0)
			{
				NextTrackIndex = GetNextTrackIndexToIterate();
				// All tracks finished in this `moof`?
				if (NextTrackIndex < 0)
				{
					break;
				}
			}

			FSelectedTrackParams& stp(SelectedTrackParamMap[(uint32)NextTrackIndex]);

			MP4Boxes::FMP4Track::FIterator::FChunkInfo SampleChunkInfo;
			if (!stp.Iterator->GetCurrentChunkRemainingSampleInfo(SampleChunkInfo))
			{
				return SetError(FString::Printf(TEXT("Failed to get sample chunk information in segment \"%s\""), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			}

			// Set this as the currently active track.
			CurrentlyActiveTrackData = TrackDataMap[NextTrackIndex];
			const int32 stIdx = StreamTypeToArrayIndex(CurrentlyActiveTrackData->StreamType);
			const uint32 TrackTimescale = stp.Timescale;

			// Perform some checks and adjustments for the first access unit.
			// Note: this is not necessarily correct in that the times given in the segment are
			//       presently in media track local time and since we may be dealing with multiple
			//       tracks here this one value does not necessarily apply to all.
			if (StreamTypeAUCount[stIdx] == 0)
			{
				CurrentlyActiveTrackData->MediaLocalFirstAUTime = InRequest->Segment.MediaLocalFirstAUTime;
				CurrentlyActiveTrackData->MediaLocalLastAUTime = bDoNotTruncateAtPresentationEnd ? TNumericLimits<int64>::Max() : InRequest->Segment.MediaLocalLastAUTime;
				CurrentlyActiveTrackData->PTO = InRequest->Segment.PTO;

				if (TrackTimescale != InRequest->Segment.Timescale)
				{
					// Need to rescale the AU times from the MPD timescale to the media timescale.
					CurrentlyActiveTrackData->MediaLocalFirstAUTime = FTimeFraction(InRequest->Segment.MediaLocalFirstAUTime, InRequest->Segment.Timescale).GetAsTimebase(TrackTimescale);
					CurrentlyActiveTrackData->MediaLocalLastAUTime = CurrentlyActiveTrackData->MediaLocalLastAUTime == TNumericLimits<int64>::Max() ? CurrentlyActiveTrackData->MediaLocalLastAUTime : FTimeFraction(InRequest->Segment.MediaLocalLastAUTime, InRequest->Segment.Timescale).GetAsTimebase(TrackTimescale);
					CurrentlyActiveTrackData->PTO = FTimeFraction(InRequest->Segment.PTO, InRequest->Segment.Timescale).GetAsTimebase(TrackTimescale);
					/*
					Removed because this keeps irritating people who do not understand the problem and how to fix this in their MPD generator.

						if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH && !InRequest->bWarnedAboutTimescale)
						{
							InRequest->bWarnedAboutTimescale = true;
							LogMessage(IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Track timescale %u differs from timescale of %u in MPD or segment index. This may cause playback problems!"), TrackTimescale, InRequest->Segment.Timescale));
						}
					*/
				}
				// Set the PTO in the codec data extras. This is a rarely used value and constant for the segment.
				// The only use we have for it so far is to remap subtitle timestamps to split periods.
				CurrentlyActiveTrackData->CSD->ParsedInfo.GetExtras().Set(StreamCodecInformationOptions::PresentationTimeOffset, FVariantValue(FTimeValue(CurrentlyActiveTrackData->PTO, TrackTimescale)));

				// Producer reference time (DASH only)
				if (InRequest->Segment.MeasureLatencyViaReferenceTimeInfoID >= 0 && InRequest->Segment.ProducerReferenceTimeInfos.Num())
				{
					// We only look at inband 'prtf' boxes if the <ProducerReferenceTime> element in the MPD tells us to.
					// This is similar to events that are not to be considered any more if the MPD doesn't specify them.
					for(auto &MPDPrtf : InRequest->Segment.ProducerReferenceTimeInfos)
					{
						if (MPDPrtf.GetID() == (uint32)InRequest->Segment.MeasureLatencyViaReferenceTimeInfoID)
						{
							// Use the inband 'prft' boxes?
							if (MPDPrtf.bInband)
							{
								bool bFound = false;
								for(auto &MP4Prtf : stp.PRFTBoxes)
								{
									if ((MPDPrtf.Type == FProducerReferenceTimeInfo::EType::Encoder && MP4Prtf->GetReferenceType() == MP4Boxes::FMP4BoxPRFT::EReferenceType::Encoder) ||
										(MPDPrtf.Type == FProducerReferenceTimeInfo::EType::Captured && MP4Prtf->GetReferenceType() == MP4Boxes::FMP4BoxPRFT::EReferenceType::Captured))
									{
										RFC5905::ParseNTPTime(CurrentlyActiveTrackData->ProducerTime.Base, MP4Prtf->GetNTPTimestamp());
										CurrentlyActiveTrackData->ProducerTime.Media = MP4Prtf->GetMediaTime();
										bFound = true;
										break;
									}
								}
								// When the MPD says that there are inband prtf's then this has to be so. If for some reason this is not the case
								// then what are we to do?
								if (!bFound)
								{
									// We take the values from the MPD here, which may be better than nothing?!
									CurrentlyActiveTrackData->ProducerTime.Base = MPDPrtf.WallclockTime;
									CurrentlyActiveTrackData->ProducerTime.Media = MPDPrtf.PresentationTime;
								}
							}
							else
							{
								// Use values from MPD
								CurrentlyActiveTrackData->ProducerTime.Base = MPDPrtf.WallclockTime;
								CurrentlyActiveTrackData->ProducerTime.Media = MPDPrtf.PresentationTime;
							}
							break;
						}
					}
				}
			}

			// Establish time mapping for this track
			if (!CurrentlyActiveTrackData->TimeMappingOffset.IsValid())
			{
				if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::HLS && InRequest->HLS.bNoPDTMapping == false)
				{
					const int64 bmdt = stp.Iterator->GetBaseMediaDecodeTime().Get(MP4Utilities::FFractionalTime(0,1)).GetNumerator();
					FTimeValue BaseMediaDecodeTime(bmdt, TrackTimescale, 0);
					CurrentlyActiveTrackData->TimeMappingOffset = FTimeValue().SetFromHNS(InRequest->Segment.Time) - BaseMediaDecodeTime;
				}
				else
				{
					CurrentlyActiveTrackData->TimeMappingOffset.SetToZero();
				}
			}


			// Process all samples in the current chunk
			while(NextTrackIndex >= 0 && !bDone && !HasErrored() && !HasReadBeenAborted())
			{
				// Get the DTS and PTS. Those are 0-based in a fragment and offset by the base media decode time of the fragment.
				MP4Utilities::FFractionalTime AUDTS = stp.Iterator->GetEffectiveDTS();
				MP4Utilities::FFractionalTime AUPTS = stp.Iterator->GetEffectivePTS();
				MP4Utilities::FFractionalTime AUDuration = stp.Iterator->GetDuration();

				// Create access unit
				FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
				AccessUnit->ESType = CurrentlyActiveTrackData->StreamType;
				AccessUnit->AUSize = (uint32) stp.Iterator->GetSampleSize();
				AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
				AccessUnit->bIsFirstInSequence = CurrentlyActiveTrackData->bIsFirstInSequence;
				AccessUnit->bIsSyncSample = stp.Iterator->IsSyncOrRAPSample();
				AccessUnit->bIsDummyData = false;
				AccessUnit->AUCodecData = CurrentlyActiveTrackData->CSD;
				AccessUnit->BufferSourceInfo = CurrentlyActiveTrackData->BufferSourceInfo;

				FTimeValue Duration(AUDuration.GetNumerator(), AUDuration.GetDenominator(), 0);
				AccessUnit->Duration = Duration;

				// Offset the AU's DTS and PTS to the time mapping of the segment.
				AccessUnit->DTS.SetFromND(AUDTS.GetNumerator() - CurrentlyActiveTrackData->PTO, TrackTimescale);
				AccessUnit->DTS += TimeOffset + CurrentlyActiveTrackData->TimeMappingOffset;
				AccessUnit->PTS.SetFromND(AUPTS.GetNumerator() - CurrentlyActiveTrackData->PTO, TrackTimescale);
				AccessUnit->PTS += TimeOffset + CurrentlyActiveTrackData->TimeMappingOffset;

				// Remember the first AU's DTS for this stream type in the segment
				if (!InRequest->TimestampVars.Local.First[stIdx].IsValid())
				{
					InRequest->TimestampVars.Local.First[stIdx] = AccessUnit->DTS;
				}
				if (CurrentlyActiveTrackData == PrimaryTrackData)
				{
					if (StreamTypeAUCount[stIdx] == 0)
					{
						// Check that the timestamps are greater than what we are expecting.
						// This is to prevent reading the same segment data from a different stream again after a stream switch.
						if (InRequest->TimestampVars.Next.bCheck)
						{
							if (InRequest->TimestampVars.Next.ExpectedLargerThan[stIdx].IsValid() && InRequest->TimestampVars.Local.First[stIdx] <= InRequest->TimestampVars.Next.ExpectedLargerThan[stIdx])
							{
								FAccessUnit::Release(AccessUnit);
								AccessUnit = nullptr;
								bDone = true;
								InRequest->TimestampVars.Next.bFailed = true;
								bSkippedBecauseOfTimestampCheck = true;
								break;
							}
						}
						bTimeCheckPassed = true;

						InRequest->FirstTimestampReceivedDelegate.ExecuteIfBound(InRequest);
					}

					// If the track uses encryption we update the DRM system with the PSSH boxes that are currently in use.
					// NOTE: This works only if there is a single decrypter for any of the multiplexed tracks.
					//       If there is ever more than one this needs to be extended.
					if (Decrypter.IsValid() && ProtectionSystemBoxes.Num())
					{
						TArray<TArray<uint8>> PSSHBoxes;
						for(auto &psb : ProtectionSystemBoxes)
						{
							PSSHBoxes.Emplace(psb->GetBoxDataRAW());
						}
						Decrypter->UpdateInitDataFromMultiplePSSH(PSSHBoxes);
					}
				}

				// Set the sequence index member and update all timestamps with it as well.
				AccessUnit->SequenceIndex = InRequest->TimestampSequenceIndex;
				AccessUnit->DTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
				AccessUnit->PTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);

				AccessUnit->EarliestPTS.SetFromND(CurrentlyActiveTrackData->MediaLocalFirstAUTime - CurrentlyActiveTrackData->PTO, TrackTimescale);
				AccessUnit->EarliestPTS += TimeOffset;
				if (InRequest->Segment.bFrameAccuracyRequired && InRequest->FrameAccurateStartTime.IsValid())
				{
					AccessUnit->EarliestPTS = InRequest->FrameAccurateStartTime;
				}
				AccessUnit->EarliestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
				if (CurrentlyActiveTrackData->MediaLocalLastAUTime != TNumericLimits<int64>::Max())
				{
					AccessUnit->LatestPTS.SetFromND(CurrentlyActiveTrackData->MediaLocalLastAUTime - CurrentlyActiveTrackData->PTO, TrackTimescale);
					AccessUnit->LatestPTS += TimeOffset;
				}
				else
				{
					AccessUnit->LatestPTS.SetToPositiveInfinity();
				}
				AccessUnit->LatestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);

				AccessUnit->ProducerReferenceTime = CurrentlyActiveTrackData->ProducerTime.Base + FTimeValue(AUDTS.GetNumerator() - CurrentlyActiveTrackData->ProducerTime.Media, TrackTimescale);

				// Set up the sample encryption if any. This needs to be done prior to reading the sample data.
				bool bIsSampleEncrypted = stp.SetupSampleEncryption();
				if (bIsSampleEncrypted)
				{
					// If the sample is encrypted in a HLS stream we need to replace the default KID.
					// That's because when the .m3u8 playlist was parsed there is no KID in the `#EXT-X-KEY` tag (for KEYFORMAT identity)
					// and we had to create a "KID" by hashing the license key URL, and thus need to now set that hash as the KID.
					if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::HLS)
					{
						stp.SampleEncryptionInfo.DefaultKID = InRequest->DrmMedia.DrmKID;
					}
				}

				const int64 AUFileOffset = stp.Iterator->GetSampleFileOffset();
				const int64 NumBytesToSkip = AUFileOffset - GetCurrentOffset();
				if (NumBytesToSkip < 0)
				{
					// Current read position is already farther than where the data is supposed to be.
					FAccessUnit::Release(AccessUnit);
					AccessUnit = nullptr;
					SetError(FString::Printf(TEXT("Read position already at %lld but data starts at %lld in segment \"%s\""), (long long int)GetCurrentOffset(), (long long int)AUFileOffset, *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
					break;
				}
				else if (NumBytesToSkip > 0)
				{
					int64 NumSkipped = ReadData(nullptr, NumBytesToSkip, -1);
					if (NumSkipped != NumBytesToSkip)
					{
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;
						if (!HasReadBeenAborted())
						{
							SetError(FString::Printf(TEXT("Failed to skip over %lld bytes in segment \"%s\""), (long long int)NumBytesToSkip, *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
						}
						break;
					}
				}
				if (MoofInfo.PayloadStartOffset == 0)
				{
					MoofInfo.PayloadStartOffset = GetCurrentOffset();
				}

				int64 NumRead = ReadData(AccessUnit->AUData, AccessUnit->AUSize, -1);
				if (NumRead == AccessUnit->AUSize)
				{
					MoofInfo.NumKeyframeBytes += AccessUnit->bIsSyncSample ? AccessUnit->AUSize : 0;
					// Only update duration for the primary stream type.
					if (InRequest->GetType() == CurrentlyActiveTrackData->StreamType)
					{
						MoofInfo.ContentDuration += Duration;
					}
					CurrentlyActiveTrackData->DurationSuccessfullyRead += Duration;

					// If we need to decrypt we have to wait for the decrypter to become ready.
					if (bIsSampleEncrypted && Decrypter.IsValid())
					{
						while(!bTerminate && !HasReadBeenAborted() &&
								(Decrypter->GetState() == ElectraCDM::ECDMState::WaitingForKey || Decrypter->GetState() == ElectraCDM::ECDMState::Idle))
						{
							FMediaRunnable::SleepMilliseconds(100);
						}
						ElectraCDM::ECDMError DecryptResult = ElectraCDM::ECDMError::Failure;
						if (Decrypter->GetState() == ElectraCDM::ECDMState::Ready)
						{
							SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
							CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
							DecryptResult = Decrypter->DecryptInPlace((uint8*) AccessUnit->AUData, (int32) AccessUnit->AUSize, stp.SampleEncryptionInfo);
						}
						if (DecryptResult != ElectraCDM::ECDMError::Success)
						{
							FAccessUnit::Release(AccessUnit);
							AccessUnit = nullptr;
							SetError(FString::Printf(TEXT("Failed to decrypt segment \"%s\" with error %d (%s)"), *RequestURL, (int32)DecryptResult, *Decrypter->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
							break;
						}
					}
				}
				else
				{
					// Did not get the number of bytes we needed. Either because of a read error or because we got aborted.
					FAccessUnit::Release(AccessUnit);
					AccessUnit = nullptr;
					bHasErrored = true;
					break;
				}

				// Check if the AU is outside the time range we are allowed to read.
				// The last one (the one that is already outside the range, actually) is tagged as such and sent into the buffer.
				// The respective decoder has to handle this flag if necessary and/or drop the AU.
				// We need to send at least one AU down so the FMultiTrackAccessUnitBuffer does not stay empty for this period!
				if (AccessUnit)
				{
					// Already sent the last one?
					if (!CurrentlyActiveTrackData->bReadPastLastPTS)
					{
						CurrentlyActiveTrackData->AddAccessUnit(AccessUnit);
					}
					FAccessUnit::Release(AccessUnit);
					AccessUnit = nullptr;
				}

				// Shall we pass on any AUs we already read?
				if (bAllowEarlyEmitting && bTimeCheckPassed)
				{
					EmitSamples(EEmitType::UntilBlocked);
				}

				CurrentlyActiveTrackData->bIsFirstInSequence = false;
				++StreamTypeAUCount[stIdx];

				bool bHaveNextSample = stp.Iterator->Next();
				if (!bHaveNextSample)
				{
					NextTrackIndex = -1;
					stp.Iterator.Reset();
				}
				else if (--SampleChunkInfo.NumSamples == 0)
				{
					NextTrackIndex = -1;
				}
			}
		}

		// If we have not yet reached the end of the `mdat` because of tracks we are not interested in
		// we need to skip over the remaining data in order to reach the following `moof` if there is one.
		int64 NumExcessMdatBytes = LastByteOffsetOfMDAT - GetCurrentOffset();
		check(NumExcessMdatBytes >= 0);
		if (NumExcessMdatBytes > 0 && !bDone && !HasErrored() && !HasReadBeenAborted())
		{
			int64 NumSkipped = ReadData(nullptr, NumExcessMdatBytes, -1);
			if (NumSkipped != NumExcessMdatBytes)
			{
				if (!HasReadBeenAborted())
				{
					SetError(FString::Printf(TEXT("Failed to skip over %lld bytes in segment \"%s\" to reach next moof"), (long long int)NumExcessMdatBytes, *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				}
				break;
			}
		}
		MoofInfo.PayloadEndOffset = GetCurrentOffset();
		if (InRequest->Segment.bLowLatencyChunkedEncodingExpected)
		{
			DownloadStats.MovieChunkInfos.Emplace(MoveTemp(MoofInfo));
		}
	}
	BoxLoader.Reset();
	return HasReadBeenAborted() ? EHandleResult::Aborted :
		   HasErrored() ? EHandleResult::Failed :
		   bSkippedBecauseOfTimestampCheck ? EHandleResult::Skipped :
		   EHandleResult::Finished;
}

FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::HandleMP4MediaLegacy(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	// We need to have an mp4 init segment.
	if (!InitSegmentData.IsType<TSharedPtrTS<const IParserISO14496_12>>())
	{
		SetError(FString::Printf(TEXT("Wrong init segment type for fmp4 media segment")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}
	TSharedPtrTS<const IParserISO14496_12> InitSegment = InitSegmentData.Get<TSharedPtrTS<const IParserISO14496_12>>();
	if (!InitSegment.IsValid())
	{
		SetError(FString::Printf(TEXT("No init segment for fmp4 media segment")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}

	if (InitSegment->GetNumberOfTracks() <= 0)
	{
		SetError(FString::Printf(TEXT("Init segment contains no tracks")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}
	TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> NewTrackDataTypeMap[4];
	for(int32 nt=0,ntmax=InitSegment->GetNumberOfTracks(); nt<ntmax; ++nt)
	{
		const IParserISO14496_12::ITrack* Track = InitSegment->GetTrackByIndex(nt);
		if (!Track)
		{
			continue;
		}
		FStreamCodecInformation ci = Track->GetCodecInformation();
		if (ci.GetStreamType() == EStreamType::Video && InRequest->bIgnoreVideo)
		{
			continue;
		}
		else if (ci.GetStreamType() == EStreamType::Audio && InRequest->bIgnoreAudio)
		{
			continue;
		}
		else if (ci.GetStreamType() == EStreamType::Subtitle && InRequest->bIgnoreSubtitles)
		{
			continue;
		}

		TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
		td->StreamType = ci.GetStreamType();
		const int32 stIdx = StreamTypeToArrayIndex(ci.GetStreamType());
		// Copy the source buffer info into a new instance and set the playback sequence ID in it.
		check(InRequest->SourceBufferInfo[stIdx].IsValid());
		if (!InRequest->SourceBufferInfo[stIdx].IsValid())
		{
			continue;
		}
		td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[stIdx]));
		td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
		// Set the CSD
		td->CSD = MakeSharedTS<FAccessUnit::CodecData>();
		td->CSD->CodecSpecificData = Track->GetCodecSpecificData();
		td->CSD->RawCSD = Track->GetCodecSpecificDataRAW();
		td->CSD->ParsedInfo = MoveTemp(ci);
		// Set information from the playlist codec information that may not be available or accurate in the init segment.
		td->CSD->ParsedInfo.SetBitrate(InRequest->CodecInfo[stIdx].GetBitrate());
		NewTrackDataTypeMap[stIdx].Emplace(Track->GetID(), MoveTemp(td));
	}
	// At present we only want to have a single track per type in the media segment. If there are more we use the one
	// with the smallest ID (a stipulation made by the HLS specification; we apply it regardless of protocol)
	for(int32 nt=0; nt<3; ++nt)
	{
		if (NewTrackDataTypeMap[nt].Num())
		{
			auto it(NewTrackDataTypeMap[nt].CreateConstIterator());
			TrackDataMap.Add(it.Key(), it.Value());
		}
	}
	if (TrackDataMap.Num() == 0 || (InRequest->bIsMultiplex && NewTrackDataTypeMap[StreamTypeToArrayIndex(InRequest->GetType())].Num() == 0))
	{
		SetError(FString::Printf(TEXT("Init segment contains no usable tracks")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}
	SelectPrimaryTrackData(InRequest);

	// If this is a filler segment then we are done at this point and let the end-of-segment handling
	// create the filler data for the active tracks we just set up.
	if (InRequest->bInsertFillerData)
	{
		return EHandleResult::Finished;
	}

	// See if the segment is encrypted (sample encryption, not whole segment)
	// The assumption is that all tracks in the segment are encrypted the same way and a single decrypter will do.
	if (!ReadBuffer.bIsEncrypted && InRequest->DrmMedia.DrmClient.IsValid())
	{
		check(!Decrypter.IsValid());
		if (InRequest->DrmMedia.DrmClient->CreateDecrypter(Decrypter, InRequest->DrmMedia.DrmMimeType) != ElectraCDM::ECDMError::Success)
		{
			SetError(FString::Printf(TEXT("Failed to create decrypter for segment, \"%s\""), *InRequest->DrmMedia.DrmClient->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
			return EHandleResult::Failed;
		}
	}

	// Create the parser
	TSharedPtrTS<IParserISO14496_12> MP4Parser = IParserISO14496_12::CreateParser();

	// Enter the parsing loop.
	bool bDone = false;
	FMediaSegmentBoxCallback ParseBoxCallback;
	FTimeValue SegmentDuration(InRequest->Segment.Duration, InRequest->Segment.Timescale, 0);
	FTimeValue TimeOffset = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
	bool bDoNotTruncateAtPresentationEnd = PlayerSessionService->GetOptionValue(OptionKeyDoNotTruncateAtPresentationEnd).SafeGetBool(false);
	bool bIsLastSegment = InRequest->Segment.bIsLastInPeriod;
	const FString& RequestURL(InRequest->Segment.MediaURL.Url.URL);
	int64 LastSuccessfulFilePos = 0;
	bool bSkippedBecauseOfTimestampCheck = false;
	int32 StreamTypeAUCount[4] { 0,0,0,0 };
	int32 MoofIdx = 0;
	bool bTimeCheckPassed = InRequest->TimestampVars.Next.bCheck == false;
	while(!bDone && !HasErrored() && !HasReadBeenAborted())
	{
		Metrics::FSegmentDownloadStats::FMovieChunkInfo MoofInfo;
		MoofInfo.HeaderOffset = GetCurrentOffset();
		UEMediaError parseError = MP4Parser->ParseHeader(this, &ParseBoxCallback, PlayerSessionService, InitSegment.Get());
		if (parseError == UEMEDIA_ERROR_OK)
		{
			{
				SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
				CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
				InRequest->Segment.bSawLMSG = MP4Parser->HasBrand(IParserISO14496_12::BrandType_lmsg);
				parseError = MP4Parser->PrepareTracks(PlayerSessionService, InitSegment);
			}
			if (parseError == UEMEDIA_ERROR_OK)
			{
				// Validate that the track IDs as specified in the init segment exist in the fragment
				TArray<uint32> SelectedTrackIDList;
				for(auto& trkIt : TrackDataMap)
				{
					SelectedTrackIDList.Emplace(trkIt.Key);
					const int32 TrackID = (int32) trkIt.Key;
					const IParserISO14496_12::ITrack* Track = MP4Parser->GetTrackByTrackID(TrackID);
					if (!Track)
					{
						SetError(FString::Printf(TEXT("Track with ID %d as listed in the init segment was not found in the media segment"), TrackID), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
						return EHandleResult::Failed;
					}
				}

				// Create the multitrack iterator for the selected tracks.
				TSharedPtrTS<IParserISO14496_12::IAllTrackIterator> AllTrackIterator;
				AllTrackIterator = MP4Parser->CreateAllTrackIteratorForTrackIDs(SelectedTrackIDList);

				// Iterate the moof
				for(bool bMultiTrackEnd=false; !bMultiTrackEnd && !bDone && !HasErrored() && !HasReadBeenAborted(); bMultiTrackEnd=!AllTrackIterator->Next())
				{
					// Get the current track's iterator.
					const IParserISO14496_12::ITrackIterator* TrackIterator = AllTrackIterator->Current();
					check(TrackIterator);
					if (!TrackIterator)
					{
						continue;
					}
					// Get the track from the iterator.
					const IParserISO14496_12::ITrack* Track = TrackIterator->GetTrack();
					check(Track);

					// Set this as the currently active track.
					CurrentlyActiveTrackData = TrackDataMap[Track->GetID()];

					const int32 stIdx = StreamTypeToArrayIndex(CurrentlyActiveTrackData->StreamType);

					const uint32 TrackTimescale = TrackIterator->GetTimescale();

					// Perform some checks and adjustments for the first access unit.
					// Note: this is not necessarily correct in that the times given in the segment are
					//       presently in media track local time and since we may be dealing with multiple
					//       tracks here this one value does not necessarily apply to all.
					if (StreamTypeAUCount[stIdx] == 0)
					{
						CurrentlyActiveTrackData->MediaLocalFirstAUTime = InRequest->Segment.MediaLocalFirstAUTime;
						CurrentlyActiveTrackData->MediaLocalLastAUTime = bDoNotTruncateAtPresentationEnd ? TNumericLimits<int64>::Max() : InRequest->Segment.MediaLocalLastAUTime;
						CurrentlyActiveTrackData->PTO = InRequest->Segment.PTO;

						if (TrackTimescale != InRequest->Segment.Timescale)
						{
							// Need to rescale the AU times from the MPD timescale to the media timescale.
							CurrentlyActiveTrackData->MediaLocalFirstAUTime = FTimeFraction(InRequest->Segment.MediaLocalFirstAUTime, InRequest->Segment.Timescale).GetAsTimebase(TrackTimescale);
							CurrentlyActiveTrackData->MediaLocalLastAUTime = CurrentlyActiveTrackData->MediaLocalLastAUTime == TNumericLimits<int64>::Max() ? CurrentlyActiveTrackData->MediaLocalLastAUTime : FTimeFraction(InRequest->Segment.MediaLocalLastAUTime, InRequest->Segment.Timescale).GetAsTimebase(TrackTimescale);
							CurrentlyActiveTrackData->PTO = FTimeFraction(InRequest->Segment.PTO, InRequest->Segment.Timescale).GetAsTimebase(TrackTimescale);

							if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH && !InRequest->bWarnedAboutTimescale)
							{
								InRequest->bWarnedAboutTimescale = true;
								LogMessage(IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Track timescale %u differs from timescale of %u in MPD or segment index. This may cause playback problems!"), TrackTimescale, InRequest->Segment.Timescale));
							}
						}
						// Set the PTO in the codec data extras. This is a rarely used value and constant for the segment.
						// The only use we have for it so far is to remap subtitle timestamps to split periods.
						CurrentlyActiveTrackData->CSD->ParsedInfo.GetExtras().Set(StreamCodecInformationOptions::PresentationTimeOffset, FVariantValue(FTimeValue(CurrentlyActiveTrackData->PTO, TrackTimescale)));

						// Producer reference time (DASH only)
						if (InRequest->Segment.MeasureLatencyViaReferenceTimeInfoID >= 0 && InRequest->Segment.ProducerReferenceTimeInfos.Num())
						{
							// We only look at inband 'prtf' boxes if the <ProducerReferenceTime> element in the MPD tells us to.
							// This is similar to events that are not to be considered any more if the MPD doesn't specify them.
							for(auto &MPDPrtf : InRequest->Segment.ProducerReferenceTimeInfos)
							{
								if (MPDPrtf.GetID() == (uint32)InRequest->Segment.MeasureLatencyViaReferenceTimeInfoID)
								{
									// Use the inband 'prft' boxes?
									if (MPDPrtf.bInband)
									{
										TArray<IParserISO14496_12::ITrack::FProducerReferenceTime> PRFTBoxes;
										Track->GetPRFTBoxes(PRFTBoxes);
										bool bFound = false;
										for(auto &MP4Prtf : PRFTBoxes)
										{
											if ((MPDPrtf.Type == FProducerReferenceTimeInfo::EType::Encoder && MP4Prtf.Reference == IParserISO14496_12::ITrack::FProducerReferenceTime::EReferenceType::Encoder) ||
												(MPDPrtf.Type == FProducerReferenceTimeInfo::EType::Captured && MP4Prtf.Reference == IParserISO14496_12::ITrack::FProducerReferenceTime::EReferenceType::Captured))
											{
												RFC5905::ParseNTPTime(CurrentlyActiveTrackData->ProducerTime.Base, MP4Prtf.NtpTimestamp);
												CurrentlyActiveTrackData->ProducerTime.Media = MP4Prtf.MediaTime;
												bFound = true;
												break;
											}
										}
										// When the MPD says that there are inband prtf's then this has to be so. If for some reason this is not the case
										// then what are we to do?
										if (!bFound)
										{
											// We take the values from the MPD here, which may be better than nothing?!
											CurrentlyActiveTrackData->ProducerTime.Base = MPDPrtf.WallclockTime;
											CurrentlyActiveTrackData->ProducerTime.Media = MPDPrtf.PresentationTime;
										}
									}
									else
									{
										// Use values from MPD
										CurrentlyActiveTrackData->ProducerTime.Base = MPDPrtf.WallclockTime;
										CurrentlyActiveTrackData->ProducerTime.Media = MPDPrtf.PresentationTime;
									}
									break;
								}
							}
						}
					}

					// Establish time mapping for this track
					if (!CurrentlyActiveTrackData->TimeMappingOffset.IsValid())
					{
						if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::HLS && InRequest->HLS.bNoPDTMapping == false)
						{
							FTimeValue BaseMediaDecodeTime(TrackIterator->GetBaseMediaDecodeTime(), TrackTimescale, 0);
							CurrentlyActiveTrackData->TimeMappingOffset = FTimeValue().SetFromHNS(InRequest->Segment.Time) - BaseMediaDecodeTime;
						}
						else
						{
							CurrentlyActiveTrackData->TimeMappingOffset.SetToZero();
						}
					}

					// Get the DTS and PTS. Those are 0-based in a fragment and offset by the base media decode time of the fragment.
					const int64 AUDTS = TrackIterator->GetDTS();
					const int64 AUPTS = TrackIterator->GetPTS();
					const int64 AUDuration = TrackIterator->GetDuration();

					// Create access unit
					FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
					AccessUnit->ESType = CurrentlyActiveTrackData->StreamType;
					AccessUnit->AUSize = (uint32) TrackIterator->GetSampleSize();
					AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
					AccessUnit->bIsFirstInSequence = CurrentlyActiveTrackData->bIsFirstInSequence;
					AccessUnit->bIsSyncSample = TrackIterator->IsSyncSample();
					AccessUnit->bIsDummyData = false;
					AccessUnit->AUCodecData = CurrentlyActiveTrackData->CSD;
					AccessUnit->BufferSourceInfo = CurrentlyActiveTrackData->BufferSourceInfo;

					FTimeValue Duration(AUDuration, TrackTimescale);
					AccessUnit->Duration = Duration;

					// Offset the AU's DTS and PTS to the time mapping of the segment.
					AccessUnit->DTS.SetFromND(AUDTS - CurrentlyActiveTrackData->PTO, TrackTimescale);
					AccessUnit->DTS += TimeOffset + CurrentlyActiveTrackData->TimeMappingOffset;
					AccessUnit->PTS.SetFromND(AUPTS - CurrentlyActiveTrackData->PTO, TrackTimescale);
					AccessUnit->PTS += TimeOffset + CurrentlyActiveTrackData->TimeMappingOffset;

					// Remember the first AU's DTS for this stream type in the segment
					if (!InRequest->TimestampVars.Local.First[stIdx].IsValid())
					{
						InRequest->TimestampVars.Local.First[stIdx] = AccessUnit->DTS;
					}
					if (CurrentlyActiveTrackData == PrimaryTrackData)
					{
						if (StreamTypeAUCount[stIdx] == 0)
						{
							// Check that the timestamps are greater than what we are expecting.
							// This is to prevent reading the same segment data from a different stream again after a stream switch.
							if (InRequest->TimestampVars.Next.bCheck)
							{
								if (InRequest->TimestampVars.Next.ExpectedLargerThan[stIdx].IsValid() && InRequest->TimestampVars.Local.First[stIdx] <= InRequest->TimestampVars.Next.ExpectedLargerThan[stIdx])
								{
									FAccessUnit::Release(AccessUnit);
									AccessUnit = nullptr;
									bDone = true;
									InRequest->TimestampVars.Next.bFailed = true;
									bSkippedBecauseOfTimestampCheck = true;
									break;
								}
							}
							bTimeCheckPassed = true;

							InRequest->FirstTimestampReceivedDelegate.ExecuteIfBound(InRequest);
						}

						HandleMP4MetadataLegacy(InRequest, MP4Parser, InitSegment, TimeOffset);
						HandleMP4EventMessagesLegacy(InRequest, MP4Parser);
						// If the track uses encryption we update the DRM system with the PSSH boxes that are currently in use.
						// NOTE: This works only if there is a single decrypter for any of the multiplexed tracks.
						//       If there is ever more than one this needs to be extended.
						if (Decrypter.IsValid())
						{
							TArray<TArray<uint8>> PSSHBoxes;
							Track->GetPSSHBoxes(PSSHBoxes, true, true);
							Decrypter->UpdateInitDataFromMultiplePSSH(PSSHBoxes);
						}
					}

					// Set the sequence index member and update all timestamps with it as well.
					AccessUnit->SequenceIndex = InRequest->TimestampSequenceIndex;
					AccessUnit->DTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
					AccessUnit->PTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);

					AccessUnit->EarliestPTS.SetFromND(CurrentlyActiveTrackData->MediaLocalFirstAUTime - CurrentlyActiveTrackData->PTO, TrackTimescale);
					AccessUnit->EarliestPTS += TimeOffset;
					if (InRequest->Segment.bFrameAccuracyRequired && InRequest->FrameAccurateStartTime.IsValid())
					{
						AccessUnit->EarliestPTS = InRequest->FrameAccurateStartTime;
					}
					AccessUnit->EarliestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
					if (CurrentlyActiveTrackData->MediaLocalLastAUTime != TNumericLimits<int64>::Max())
					{
						AccessUnit->LatestPTS.SetFromND(CurrentlyActiveTrackData->MediaLocalLastAUTime - CurrentlyActiveTrackData->PTO, TrackTimescale);
						AccessUnit->LatestPTS += TimeOffset;
					}
					else
					{
						AccessUnit->LatestPTS.SetToPositiveInfinity();
					}
					AccessUnit->LatestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);

					AccessUnit->ProducerReferenceTime = CurrentlyActiveTrackData->ProducerTime.Base + FTimeValue(AUDTS - CurrentlyActiveTrackData->ProducerTime.Media, TrackTimescale);

					ElectraCDM::FMediaCDMSampleInfo SampleEncryptionInfo;
					bool bIsSampleEncrypted = TrackIterator->GetEncryptionInfo(SampleEncryptionInfo);
					// If the sample is encrypted in a HLS stream we need to replace the default KID.
					if (bIsSampleEncrypted && InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::HLS)
					{
						// That's because when the .m3u8 playlist was parsed there is no KID in the `#EXT-X-KEY` tag (for KEYFORMAT identity)
						// and we had to create a "KID" by hashing the license key URL, and thus need to now set that hash as the KID.
						SampleEncryptionInfo.DefaultKID = InRequest->DrmMedia.DrmKID;
					}

					// There should not be any gaps!
					int64 NumBytesToSkip = TrackIterator->GetSampleFileOffset() - GetCurrentOffset();
					if (NumBytesToSkip < 0)
					{
						// Current read position is already farther than where the data is supposed to be.
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;
						SetError(FString::Printf(TEXT("Read position already at %lld but data starts at %lld in segment \"%s\""), (long long int)GetCurrentOffset(), (long long int)TrackIterator->GetSampleFileOffset(), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
						LogMessage(IInfoLog::ELevel::Error, SegmentError.GetMessage());
						bHasErrored = true;
						bDone = true;
						break;
					}
					else if (NumBytesToSkip > 0)
					{
						int64 NumSkipped = ReadData(nullptr, NumBytesToSkip, -1);
						if (NumSkipped != NumBytesToSkip)
						{
							FAccessUnit::Release(AccessUnit);
							AccessUnit = nullptr;
							SetError(FString::Printf(TEXT("Failed to skip over %lld bytes in segment \"%s\""), (long long int)NumBytesToSkip, *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
							LogMessage(IInfoLog::ELevel::Error, SegmentError.GetMessage());
							bHasErrored = true;
							bDone = true;
							break;
						}
					}

					if (MoofInfo.PayloadStartOffset == 0)
					{
						MoofInfo.PayloadStartOffset = GetCurrentOffset();
					}

					int64 NumRead = ReadData(AccessUnit->AUData, AccessUnit->AUSize, -1);
					if (NumRead == AccessUnit->AUSize)
					{
						MoofInfo.NumKeyframeBytes += AccessUnit->bIsSyncSample ? AccessUnit->AUSize : 0;
						// Only update duration for the primary stream type.
						if (InRequest->GetType() == CurrentlyActiveTrackData->StreamType)
						{
							MoofInfo.ContentDuration += Duration;
						}
						CurrentlyActiveTrackData->DurationSuccessfullyRead += Duration;
						LastSuccessfulFilePos = GetCurrentOffset();

						// If we need to decrypt we have to wait for the decrypter to become ready.
						if (bIsSampleEncrypted && Decrypter.IsValid())
						{
							while(!bTerminate && !HasReadBeenAborted() &&
									(Decrypter->GetState() == ElectraCDM::ECDMState::WaitingForKey || Decrypter->GetState() == ElectraCDM::ECDMState::Idle))
							{
								FMediaRunnable::SleepMilliseconds(100);
							}
							ElectraCDM::ECDMError DecryptResult = ElectraCDM::ECDMError::Failure;
							if (Decrypter->GetState() == ElectraCDM::ECDMState::Ready)
							{
								SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
								CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
								DecryptResult = Decrypter->DecryptInPlace((uint8*) AccessUnit->AUData, (int32) AccessUnit->AUSize, SampleEncryptionInfo);
							}
							if (DecryptResult != ElectraCDM::ECDMError::Success)
							{
								FAccessUnit::Release(AccessUnit);
								AccessUnit = nullptr;
								SetError(FString::Printf(TEXT("Failed to decrypt segment \"%s\" with error %d (%s)"), *RequestURL, (int32)DecryptResult, *Decrypter->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
								LogMessage(IInfoLog::ELevel::Error, SegmentError.GetMessage());
								bHasErrored = true;
								bDone = true;
								break;
							}
						}
					}
					else
					{
						// Did not get the number of bytes we needed. Either because of a read error or because we got aborted.
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;
						bDone = true;
						break;
					}

					// Check if the AU is outside the time range we are allowed to read.
					// The last one (the one that is already outside the range, actually) is tagged as such and sent into the buffer.
					// The respective decoder has to handle this flag if necessary and/or drop the AU.
					// We need to send at least one AU down so the FMultiTrackAccessUnitBuffer does not stay empty for this period!
					if (AccessUnit)
					{
						// Already sent the last one?
						if (CurrentlyActiveTrackData->bReadPastLastPTS)
						{
							// Yes. Release this AU and do not forward it. Continue reading however.
							FAccessUnit::Release(AccessUnit);
							AccessUnit = nullptr;
						}
						/*
						else if ((AccessUnit->DropState & FAccessUnit::EDropState::TooLate) == FAccessUnit::EDropState::TooLate)
						{
							// Tag the last one and send it off, but stop doing so for the remainder of the segment.
							// Note: we continue reading this segment all the way to the end on purpose in case there are further 'emsg' boxes.
							AccessUnit->bIsLastInPeriod = true;
							CurrentlyActiveTrackData->bReadPastLastPTS = true;
						}
						*/
					}

					if (AccessUnit)
					{
						CurrentlyActiveTrackData->AddAccessUnit(AccessUnit);
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;
					}

					// Shall we pass on any AUs we already read?
					if (bAllowEarlyEmitting && bTimeCheckPassed)
					{
						EmitSamples(EEmitType::UntilBlocked);
					}

					CurrentlyActiveTrackData->bIsFirstInSequence = false;
					++StreamTypeAUCount[stIdx];
				}

				MoofInfo.PayloadEndOffset = LastSuccessfulFilePos;
				if (InRequest->Segment.bLowLatencyChunkedEncodingExpected)
				{
					DownloadStats.MovieChunkInfos.Emplace(MoveTemp(MoofInfo));
				}
				// Check if we are done or if there is additional data that needs parsing, like more moof boxes.
				if (HasReadBeenAborted() || HasReachedEOF())
				{
					bDone = true;
				}
				else
				{
					// How many bytes did we not process and need to skip to reach the next moof?
					if (ParseBoxCallback.Moofs[MoofIdx].MdatSize > 0)	// the last mdat may have 0 size when it extends to the end of the file.
					{
						int64 NumExcessMdatBytes = ParseBoxCallback.Moofs[MoofIdx].MdatPos + ParseBoxCallback.Moofs[MoofIdx].MdatSize - GetCurrentOffset();
						check(NumExcessMdatBytes >= 0);
						if (NumExcessMdatBytes > 0)
						{
							int64 NumSkipped = ReadData(nullptr, NumExcessMdatBytes, -1);
							if (NumSkipped != NumExcessMdatBytes)
							{
								SetError(FString::Printf(TEXT("Failed to skip over %lld bytes in segment \"%s\" to reach next moof"), (long long int)NumExcessMdatBytes, *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
								LogMessage(IInfoLog::ELevel::Error, SegmentError.GetMessage());
								bHasErrored = true;
								bDone = true;
								break;
							}
						}
					}
				}
				++MoofIdx;
			}
			else
			{
				// Error preparing track for iterating
				SetError(FString::Printf(TEXT("Failed to prepare segment \"%s\" for iterating"), *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				LogMessage(IInfoLog::ELevel::Error, SegmentError.GetMessage());
				bHasErrored = true;
			}
		}
		else if (parseError == UEMEDIA_ERROR_END_OF_STREAM)
		{
			bDone = true;
		}
		else
		{
			// Failed to parse the segment (in general)
			if (!HasReadBeenAborted() && !HasErrored())
			{
				SetError(FString::Printf(TEXT("Failed to parse segment \"%s\""), *RequestURL), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
				LogMessage(IInfoLog::ELevel::Error, SegmentError.GetMessage());
				bHasErrored = true;
			}
		}
	}
	return HasReadBeenAborted() ? EHandleResult::Aborted :
		   HasErrored() ? EHandleResult::Failed :
		   bSkippedBecauseOfTimestampCheck ? EHandleResult::Skipped : EHandleResult::Finished;
}

FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::HandleMKVMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	// We need to have an webm/mkv init segment.
	if (!InitSegmentData.IsType<TSharedPtrTS<const IParserMKV>>())
	{
		SetError(FString::Printf(TEXT("Wrong init segment type for webm/mkv media segment")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}
	TSharedPtrTS<const IParserMKV> InitSegment = InitSegmentData.Get<TSharedPtrTS<const IParserMKV>>();
	if (!InitSegment.IsValid())
	{
		SetError(FString::Printf(TEXT("No init segment for webm/mkv media segment")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}

	if (InitSegment->GetNumberOfTracks() <= 0)
	{
		SetError(FString::Printf(TEXT("Init segment contains no tracks")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}
	TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> NewTrackDataMap;
	for(int32 nt=0,ntmax=InitSegment->GetNumberOfTracks(); nt<ntmax; ++nt)
	{
		const IParserMKV::ITrack* Track = InitSegment->GetTrackByIndex(nt);
		if (!Track)
		{
			continue;
		}
		FStreamCodecInformation ci = Track->GetCodecInformation();
		if (ci.GetStreamType() != InRequest->GetType())
		{
			continue;
		}

		TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
		td->StreamType = InRequest->GetType();
		td->bNeedToRecalculateDurations = InRequest->GetType() == EStreamType::Video;

		// Copy the source buffer info into a new instance and set the playback sequence ID in it.
		check(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(InRequest->GetType())].IsValid());
		td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(InRequest->GetType())]));
		td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
		// Set the CSD
		td->CSD = MakeSharedTS<FAccessUnit::CodecData>();
		td->CSD->ParsedInfo = MoveTemp(ci);
		td->CSD->CodecSpecificData = Track->GetCodecSpecificData();
		FVariantValue dcr = td->CSD->ParsedInfo.GetExtras().GetValue(StreamCodecInformationOptions::DecoderConfigurationRecord);
		if (dcr.IsValid() && dcr.IsType(FVariantValue::EDataType::TypeU8Array))
		{
			td->CSD->RawCSD = dcr.GetArray();
		}
		// Set information from the playlist codec information that may not be available or accurate in the init segment.
		td->CSD->ParsedInfo.SetBitrate(InRequest->CodecInfo[StreamTypeToArrayIndex(InRequest->GetType())].GetBitrate());
		NewTrackDataMap.Emplace(Track->GetID(), MoveTemp(td));
	}
	if (NewTrackDataMap.Num() == 0)
	{
		SetError(FString::Printf(TEXT("Init segment contains no usable tracks")), INTERNAL_SEG_ERROR_BAD_SEGMENT_TYPE);
		return EHandleResult::Failed;
	}
	// At present we only want to have a single track in the media segment. If there are more we use the one
	// with the smallest ID (a stipulation made by the HLS specification; we apply it regardless of protocol)
	if (NewTrackDataMap.Num() > 1)
	{
		auto it(NewTrackDataMap.CreateConstIterator());
		TrackDataMap.Add(it.Key(), it.Value());
	}
	else
	{
		TrackDataMap = MoveTemp(NewTrackDataMap);
	}
	SelectPrimaryTrackData(InRequest);

	// See if the segment is encrypted (sample encryption, not whole segment)
	// The assumption is that all tracks in the segment are encrypted the same way and a single decrypter will do.
	if (!ReadBuffer.bIsEncrypted && InRequest->DrmMedia.DrmClient.IsValid())
	{
		check(!Decrypter.IsValid());
		if (InRequest->DrmMedia.DrmClient->CreateDecrypter(Decrypter, InRequest->DrmMedia.DrmMimeType) != ElectraCDM::ECDMError::Success)
		{
			SetError(FString::Printf(TEXT("Failed to create decrypter for segment, \"%s\""), *InRequest->DrmMedia.DrmClient->GetLastErrorMessage()), INTERNAL_SEG_ERROR_FAILED_TO_DECRYPT);
			return EHandleResult::Failed;
		}
	}

	// Prepare the array of tracks we want to demultiplex.
	// This is just one for the time being.
	TArray<uint64> TrackIDsToParse;
	TrackIDsToParse.Emplace(TrackDataMap.CreateConstIterator()->Key);

	// Create the parser
	TSharedPtrTS<IParserMKV::IClusterParser> MKVParser = InitSegment->CreateClusterParser(this, TrackIDsToParse, IParserMKV::EClusterParseFlags::ClusterParseFlag_AllowFullDocument);

	// Enter the parsing loop.
	bool bDone = false;
	FTimeValue TimeOffset = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
	bool bDoNotTruncateAtPresentationEnd = PlayerSessionService->GetOptionValue(OptionKeyDoNotTruncateAtPresentationEnd).SafeGetBool(false);
	const FString& RequestURL(InRequest->Segment.MediaURL.Url.URL);
	FTimeValue PTO(InRequest->Segment.PTO, InRequest->Segment.Timescale);
	FTimeValue EarliestPTS(InRequest->Segment.MediaLocalFirstAUTime, InRequest->Segment.Timescale, InRequest->TimestampSequenceIndex);
	EarliestPTS += TimeOffset - PTO;
	if (InRequest->Segment.bFrameAccuracyRequired && InRequest->FrameAccurateStartTime.IsValid())
	{
		EarliestPTS = InRequest->FrameAccurateStartTime;
	}
	EarliestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
	FTimeValue LastPTS;
	if (!bDoNotTruncateAtPresentationEnd && InRequest->Segment.MediaLocalLastAUTime != TNumericLimits<int64>::Max())
	{
		LastPTS.SetFromND(InRequest->Segment.MediaLocalLastAUTime, InRequest->Segment.Timescale, InRequest->TimestampSequenceIndex);
		LastPTS += TimeOffset - PTO;
	}
	else
	{
		LastPTS.SetToPositiveInfinity();
	}
	LastPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
	bool bIsFirstFrame = true;
	FAccessUnit *AccessUnit = nullptr;
	auto PrepareAccessUnit = [this, &AccessUnit](int64 NumToRead) -> void*
	{
		if (!AccessUnit)
		{
			AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
			AccessUnit->AUSize = NumToRead;
			AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
			return AccessUnit->AUData;
		}
		else
		{
			void* NewBuffer = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize + NumToRead);
			FMemory::Memcpy(NewBuffer, AccessUnit->AUData, AccessUnit->AUSize);
			void* ReadTo = AdvancePointer(NewBuffer, AccessUnit->AUSize);
			AccessUnit->AdoptNewPayloadBuffer(NewBuffer, AccessUnit->AUSize + NumToRead);
			return ReadTo;
		}
	};
	while(!bDone && !HasErrored() && !HasReadBeenAborted())
	{
		IParserMKV::IClusterParser::EParseAction ParseAction = MKVParser->NextParseAction();
		const IParserMKV::IClusterParser::IAction* NextAction = MKVParser->GetAction();
		if (NextAction)
		{
			// Get the track ID to which this action applies.
			// Since we have prepared an array of track IDs we want to process the track ID can't really be
			// something we are not expecting.
			uint64 TrackID = NextAction->GetTrackID();
			check(TrackDataMap.Contains(TrackID));
			// Set this as the currently active track.
			CurrentlyActiveTrackData = TrackDataMap[TrackID];
			check(CurrentlyActiveTrackData.IsValid());
		}

		switch(ParseAction)
		{
			case IParserMKV::IClusterParser::EParseAction::ReadFrameData:
			{
				const IParserMKV::IClusterParser::IActionReadFrameData* Action = static_cast<const IParserMKV::IClusterParser::IActionReadFrameData*>(NextAction);
				check(Action);
				int64 NumToRead = Action->GetNumBytesToRead();
				void* ReadTo = PrepareAccessUnit(NumToRead);
				int64 nr = ReadData(ReadTo, NumToRead, -1);
				if (nr != NumToRead)
				{
					bHasErrored = true;
				}
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::FrameDone:
			{
				const IParserMKV::IClusterParser::IActionFrameDone* Action = static_cast<const IParserMKV::IClusterParser::IActionFrameDone*>(NextAction);
				check(Action);
				if (AccessUnit)
				{
					// Do a test on the first frame's PTS to see if this is always zero, which
					// is indicative of a bad DASH segmenter for MKV/WEBM.
					if (bIsFirstFrame && InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH)
					{
						FTimeValue st(InRequest->Segment.Time, InRequest->Segment.Timescale, (int64)0);
						if (Action->GetPTS().IsZero() && !st.IsZero())
						{
							// If the delta is greater than 0.5 seconds
							if (Utils::AbsoluteValue(st.GetAsSeconds() - Action->GetPTS().GetAsSeconds()) >= 0.5 && !InRequest->bWarnedAboutTimescale)
							{
								InRequest->bWarnedAboutTimescale = true;
								LogMessage(IInfoLog::ELevel::Verbose, FString::Printf(TEXT("Cluster timestamp is zero while MPD time says it should be %#7.4f. Using MPD time as start value, but this may cause playback problems!"), st.GetAsSeconds()));
							}
							TimeOffset += st;
						}
					}
					bIsFirstFrame = false;

					AccessUnit->ESType = InRequest->GetType();
					AccessUnit->PTS = Action->GetPTS();
					AccessUnit->PTS += TimeOffset;
					AccessUnit->PTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
					AccessUnit->DTS = Action->GetDTS();
					AccessUnit->DTS += TimeOffset;
					AccessUnit->DTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
					AccessUnit->Duration = Action->GetDuration();
					AccessUnit->EarliestPTS = EarliestPTS;
					AccessUnit->LatestPTS = LastPTS;

					AccessUnit->bIsFirstInSequence = CurrentlyActiveTrackData->bIsFirstInSequence;
					AccessUnit->bIsSyncSample = Action->IsKeyFrame();
					AccessUnit->bIsDummyData = false;
					AccessUnit->AUCodecData = CurrentlyActiveTrackData->CSD;
					AccessUnit->BufferSourceInfo = CurrentlyActiveTrackData->BufferSourceInfo;
					AccessUnit->SequenceIndex = InRequest->TimestampSequenceIndex;

					// VP9 codec?
					if (CurrentlyActiveTrackData->CSD->ParsedInfo.GetCodec4CC() == Utils::Make4CC('v','p','0','9'))
					{
						// We cannot trust the keyframe indicator from the demuxer.
						ElectraDecodersUtil::VPxVideo::FVP9UncompressedHeader Header;
						if (ElectraDecodersUtil::VPxVideo::ParseVP9UncompressedHeader(Header, AccessUnit->AUData, AccessUnit->AUSize))
						{
							AccessUnit->bIsSyncSample = Header.IsKeyframe();
						}

						// Any additional data?
						const TMap<uint64, TArray<uint8>>& BlockAdditionalData = Action->GetBlockAdditionalData();
						for(auto& badIt : BlockAdditionalData)
						{
							auto AddDynamicBlockData = [&](const FName& InName, const TArray<uint8>& InData) -> void
							{
								if (!AccessUnit->DynamicSidebandData.IsValid())
								{
									AccessUnit->DynamicSidebandData = MakeUnique<TMap<FName, TArray<uint8>>>();
								}
								AccessUnit->DynamicSidebandData->Emplace(InName, InData);
							};

							// What type of additional data is there?
							if (BlockAdditionalData.Contains(1))		// VP9 alpha channel?
							{
								AddDynamicBlockData(DynamicSidebandData::VPxAlpha, BlockAdditionalData[1]);
							}
							else if (BlockAdditionalData.Contains(4))	// VP9 ITU T.35 metadata?
							{
								AddDynamicBlockData(DynamicSidebandData::ITU_T_35, BlockAdditionalData[4]);
							}
						}
					}
					// VP8 codec?
					else if (CurrentlyActiveTrackData->CSD->ParsedInfo.GetCodec4CC() == Utils::Make4CC('v','p','0','8'))
					{
						ElectraDecodersUtil::VPxVideo::FVP8UncompressedHeader Header;
						if (ElectraDecodersUtil::VPxVideo::ParseVP8UncompressedHeader(Header, AccessUnit->AUData, AccessUnit->AUSize))
						{
							AccessUnit->bIsSyncSample = Header.IsKeyframe();
						}
					}

					CurrentlyActiveTrackData->bIsFirstInSequence = false;

					// Add to the track AU FIFO unless we already reached the last sample of the time range.
					if (!CurrentlyActiveTrackData->bReadPastLastPTS)
					{
						CurrentlyActiveTrackData->AddAccessUnit(AccessUnit);
					}
					CurrentlyActiveTrackData->DurationSuccessfullyRead = CurrentlyActiveTrackData->LargestPTS - CurrentlyActiveTrackData->SmallestPTS + CurrentlyActiveTrackData->AverageDuration;
				}
				FAccessUnit::Release(AccessUnit);
				AccessUnit = nullptr;
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::SkipOver:
			{
				const IParserMKV::IClusterParser::IActionSkipOver* Action = static_cast<const IParserMKV::IClusterParser::IActionSkipOver*>(NextAction);
				check(Action);
				int64 NumBytesToSkip = Action->GetNumBytesToSkip();
				int64 nr = ReadData(nullptr, NumBytesToSkip, -1);
				if (nr != NumBytesToSkip)
				{
					FAccessUnit::Release(AccessUnit);
					AccessUnit = nullptr;
					SetError(FString::Printf(TEXT("Failed to skip over %lld bytes in segment \"%s\""), (long long int)NumBytesToSkip, *RequestURL), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
					LogMessage(IInfoLog::ELevel::Error, SegmentError.GetMessage());
					bHasErrored = true;
				}
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::PrependData:
			{
				const IParserMKV::IClusterParser::IActionPrependData* Action = static_cast<const IParserMKV::IClusterParser::IActionPrependData*>(NextAction);
				check(Action);
				int64 NumToRead = Action->GetPrependData().Num();
				void* ReadTo = PrepareAccessUnit(NumToRead);
				FMemory::Memcpy(ReadTo, Action->GetPrependData().GetData(), Action->GetPrependData().Num());
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::DecryptData:
			{
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::EndOfData:
			{
				bDone = true;
				break;
			}
			default:
			case IParserMKV::IClusterParser::EParseAction::Failure:
			{
				bDone = true;
				break;
			}
		}

		// Shall we pass on any AUs we already read?
		if (bAllowEarlyEmitting)
		{
			EmitSamples(EEmitType::UntilBlocked);
		}
	}

	return HasReadBeenAborted() ? EHandleResult::Aborted : HasErrored() ? EHandleResult::Failed : EHandleResult::Finished;
}

FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::HandleTSMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	// We do not normally need an init segment for transport streams, but if there is one it better be a TS, too.
	if (!InitSegmentData.IsType<FNoInitSegmentData>() && !InitSegmentData.IsType<TSharedPtrTS<const TArray<uint8>>>())
	{
		SetError(FString::Printf(TEXT("Wrong init segment type for TS media segment")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
		return EHandleResult::Failed;
	}

	// Prepare the source segment info for parsing.
	IParserISO13818_1::FSourceInfo SegSrcInfo;
	if (InitSegmentData.IsType<TSharedPtrTS<const TArray<uint8>>>())
	{
		TSharedPtrTS<const TArray<uint8>> InitData = InitSegmentData.Get<TSharedPtrTS<const TArray<uint8>>>();
		if (InitData.IsValid() && InitData->Num())
		{
			SegSrcInfo.InitSegmentData = InitData;
		}
	}

	// Create the parser
	TSharedPtrTS<IParserISO13818_1> TSParser = IParserISO13818_1::CreateParser();
	IParserISO13818_1::EParserFlags ParserFlags = IParserISO13818_1::EParserFlags::ParseFlag_Default;
	if (SegSrcInfo.InitSegmentData.IsValid())
	{
		/*
			The HLS standard isn't quite clear what should happen with the PAT/PMT in the transport stream.
			It only states:

				The Media Initialization Section of an MPEG-2 Transport Stream
				Segment is a Program Association Table (PAT) followed by a Program
				Map Table (PMT).
				Transport Stream Segments MUST contain a single MPEG-2 Program;
				playback of Multi-Program Transport Streams is not defined.  Each
				Transport Stream Segment MUST contain a PAT and a PMT, or have an
				EXT-X-MAP tag (Section 4.4.4.5) applied to it.  The first two
				Transport Stream packets in a Segment without an EXT-X-MAP tag SHOULD
				be a PAT and a PMT.
		*/
		// For the sake of argument we say that any PAT/PMT in the stream is to be ignored.
		ParserFlags |= IParserISO13818_1::EParserFlags::ParseFlag_IgnoreProgramStream;
	}
	IParserISO13818_1::EParseState TSParserState = TSParser->BeginParsing(PlayerSessionService, this, ParserFlags, SegSrcInfo);
	if (TSParserState != IParserISO13818_1::EParseState::Continue)
	{
		SetError(FString::Printf(TEXT("Failed to initialize MPEG TS parser")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
		return EHandleResult::Failed;
	}

	// We do not know what the transport stream contains.
	// This is only revealed when parsing it, so that's what we have to do now.
	bool bDone = false;
	bool bHaveProgram = false;
	bool bIsFirstTimestamp = true;
	int32 StreamTypeAUCount[4] { 0,0,0,0 };
	FStreamSegmentRequestCommon::FTimestampVars::FInternal::FRollover Rollover[4];
	Rollover[0] = InRequest->TimestampVars.Internal.Rollover[0];
	Rollover[1] = InRequest->TimestampVars.Internal.Rollover[1];
	Rollover[2] = InRequest->TimestampVars.Internal.Rollover[2];
	Rollover[3] = InRequest->TimestampVars.Internal.Rollover[3];

	FTimeValue TimeOffset = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
	uint64 RawAdjustmentValue = InRequest->TimestampVars.Internal.RawAdjustmentValue.Get((uint64)0);
	FTimeValue TimeMappingOffset = InRequest->TimestampVars.Internal.SegmentBaseTime;
	check(TimeMappingOffset.IsValid());

	bool bSkippedBecauseOfTimestampCheck = false;

	const EStreamType PrimaryStreamType = !InRequest->bIgnoreVideo ? EStreamType::Video :
										  !InRequest->bIgnoreAudio ? EStreamType::Audio :
										  !InRequest->bIgnoreSubtitles ? EStreamType::Subtitle : EStreamType::Unsupported;
	const int32 PrimaryStreamTypeIndex = StreamTypeToArrayIndex(PrimaryStreamType);

	bool bTimeCheckPassed = InRequest->TimestampVars.Next.bCheck == false;

	FTimeValue EarliestPTS;
	EarliestPTS.SetFromHNS(InRequest->Segment.MediaLocalFirstAUTime);
	if (InRequest->Segment.bFrameAccuracyRequired && InRequest->FrameAccurateStartTime.IsValid())
	{
		EarliestPTS = InRequest->FrameAccurateStartTime;
	}
	EarliestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
	FTimeValue LatestPTS;
	if (InRequest->Segment.MediaLocalLastAUTime != TNumericLimits<int64>::Max())
	{
		LatestPTS.SetFromHNS(InRequest->Segment.MediaLocalLastAUTime);
	}
	else
	{
		LatestPTS.SetToPositiveInfinity();
	}
	LatestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);

	while(!bDone && !HasErrored() && !HasReadBeenAborted())
	{
		TSParserState = TSParser->Parse(PlayerSessionService, this);
		if (TSParserState == IParserISO13818_1::EParseState::Continue)
		{
			// Have not read enough data yet, just continue.
		}
		else if (TSParserState == IParserISO13818_1::EParseState::NewProgram)
		{
			// Found a new program.
			// We do not allow for program changes here, so if we already have one we error out.
			if (bHaveProgram)
			{
				SetError(FString::Printf(TEXT("Found an unsupported mid-segment program change.")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				return EHandleResult::Failed;
			}
			bHaveProgram = true;

			TSharedPtrTS<const IParserISO13818_1::FProgramTable> PrgTbl = TSParser->GetCurrentProgramTable();
			if (!PrgTbl.IsValid())
			{
				SetError(FString::Printf(TEXT("Internal MPEG TS parser error. Supposed new program is empty.")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				return EHandleResult::Failed;
			}
			// There must only be a single program contained in the transport stream.
			if (PrgTbl->ProgramTable.Num() != 1)
			{
				SetError(FString::Printf(TEXT("The MPEG TS segment must carry a single program only, not %d."), PrgTbl->ProgramTable.Num()), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				return EHandleResult::Failed;
			}

			int32 ProgramId = PrgTbl->ProgramTable.CreateConstIterator()->Key;
			const IParserISO13818_1::FProgramStream& ProgramStream = PrgTbl->ProgramTable[ProgramId];
			TArray<int32> SelectedPIDs;
			TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> NewTrackDataMap;
			// We only want one type of track each. As per the HLS standard we choose the one with
			// the lowest PID. Since the map is sorted by PID this means we pick the first.
			bool bHaveVideo = false;
			bool bHaveAudio = false;
			for(auto& StreamIterator : ProgramStream.StreamTable)
			{
				switch(StreamIterator.Value.StreamType)
				{
					// Audio
					case 0x03:	// MPEG 1 (ISO/IEC 11172-3) Layer 1, 2 or 3
					case 0x0f:	// AAC (ISO/IEC 13818-7)
					{
						if (!InRequest->bIgnoreAudio && !bHaveAudio)
						{
							SelectedPIDs.Add(StreamIterator.Key);

							TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
							td->StreamType = EStreamType::Audio;
							td->bNeedToRecalculateDurations = true;
							check(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Audio)].IsValid());
							td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Audio)]));
							td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
							NewTrackDataMap.Emplace(StreamIterator.Key, MoveTemp(td));

							bHaveAudio = true;
						}
						break;
					}
					// Video
					case 0x1b:	// AVC/H.264
					case 0x24:	// HEVC/H.265
					{
						if (!InRequest->bIgnoreVideo && !bHaveVideo)
						{
							SelectedPIDs.Add(StreamIterator.Key);

							TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
							td->StreamType = EStreamType::Video;
							td->bNeedToRecalculateDurations = true;
							check(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Video)].IsValid());
							td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Video)]));
							td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
							NewTrackDataMap.Emplace(StreamIterator.Key, MoveTemp(td));

							bHaveVideo = true;
						}
						break;
					}
					default:
					{
						break;
					}
				}
			}
			// Select the PIDs to demultiplex.
			TSParser->SelectProgramStreams(ProgramId, SelectedPIDs);
			TrackDataMap = MoveTemp(NewTrackDataMap);
			SelectPrimaryTrackData(InRequest);
		}
		else if (TSParserState == IParserISO13818_1::EParseState::HavePESPacket)
		{
			TArray<IParserISO13818_1::FESPacket> PESPackets;
			TSharedPtrTS<IParserISO13818_1::FPESData> PESPacket = TSParser->GetPESPacket();
			if (!PESPacket.IsValid())
			{
				SetError(FString::Printf(TEXT("Internal MPEG TS parser error. Supposed PES packet is empty.")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				return EHandleResult::Failed;
			}
			const int32 PESPID = PESPacket->PID;
			if (!TrackDataMap.Contains(PESPID))
			{
				SetError(FString::Printf(TEXT("Internal MPEG TS parser error. Got PES packet for PID %d that is not selected."), PESPID), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				return EHandleResult::Failed;
			}

			// Set the track corresponding to this PID as the active one.
			CurrentlyActiveTrackData = TrackDataMap[PESPID];
			const int32 stIdx = StreamTypeToArrayIndex(CurrentlyActiveTrackData->StreamType);

			// Parse out the individual PES packets, which could be multiple.
			if (TSParser->ParsePESPacket(PESPackets, PESPacket) != IParserISO13818_1::EPESPacketResult::Ok)
			{
				SetError(FString::Printf(TEXT("PES packet error in PID %d"), PESPacket->PID), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				return EHandleResult::Failed;
			}

			// Handle the packets
			FTimeValue PTS;
			FTimeValue DTS;
			for(int32 nPkt=0; nPkt<PESPackets.Num(); ++nPkt)
			{
				const IParserISO13818_1::FESPacket& pp(PESPackets[nPkt]);
				if (nPkt == 0)
				{
					const uint64 MaxTimestamp = 1ULL << 33U;
					const uint64 HalfMaxTimestamp = 1ULL << 32U;
					check(pp.PTS.IsSet());
					uint64 PTS90k = pp.PTS.GetValue();
					uint64 DTS90k = pp.DTS.Get(PTS90k);

					// Add the rollover value we accumulated so far.
					PTS90k += Rollover[stIdx].RawPTSOffset;
					DTS90k += Rollover[stIdx].RawDTSOffset;

					int64 EffectivePTS90k = (int64) PTS90k;
					int64 EffectiveDTS90k = (int64) DTS90k;
					// The decode timstamp cannot be greater than the presentation timestamp.
					// If that is the case then the PTS has already rolled over while the DTS did not yet.
					if (EffectiveDTS90k > EffectivePTS90k && Utils::AbsoluteValue(EffectiveDTS90k - EffectivePTS90k) > (int64)HalfMaxTimestamp)
					{
						// If there was no accumulated rollover yet, this is the one time
						// where the DTS will become negative, which is ok.
						EffectiveDTS90k -= (int64)MaxTimestamp;
					}

					// Check if the PTS and DTS differ by an unusual large amount.
					if (Utils::AbsoluteValue(EffectiveDTS90k - EffectivePTS90k) > 90000)
					{
						UE_LOGF(LogElectraPlayer, Verbose, "Large DTS to PTS discrepancy of %.4f seconds detected", (EffectiveDTS90k - EffectivePTS90k)/90000.0);
					}

					// Remember the first AU DTS and PTS to detect rollover.
					if (StreamTypeAUCount[stIdx] == 0)
					{
						CurrentlyActiveTrackData->PrevPTS90k = EffectivePTS90k;
						CurrentlyActiveTrackData->PrevDTS90k = EffectiveDTS90k;
					}
					// Detect rollover on DTS and PTS
					if (EffectiveDTS90k + (int64)HalfMaxTimestamp < (int64)CurrentlyActiveTrackData->PrevDTS90k)
					{
						UE_LOGF(LogElectraPlayer, Log, "DTS rollover detected: %lld -> %lld", (long long int) EffectiveDTS90k-(int64)Rollover[stIdx].RawDTSOffset, (long long int) CurrentlyActiveTrackData->PrevDTS90k-(int64)Rollover[stIdx].RawDTSOffset);
						Rollover[stIdx].RawDTSOffset += MaxTimestamp;
						EffectiveDTS90k += (int64) MaxTimestamp;
					}
					if (EffectivePTS90k + (int64)HalfMaxTimestamp < (int64)CurrentlyActiveTrackData->PrevPTS90k)
					{
						UE_LOGF(LogElectraPlayer, Log, "PTS rollover detected: %lld -> %lld", (long long int) EffectivePTS90k-(int64)Rollover[stIdx].RawPTSOffset, (long long int) CurrentlyActiveTrackData->PrevPTS90k-(int64)Rollover[stIdx].RawPTSOffset);
						Rollover[stIdx].RawPTSOffset += MaxTimestamp;
						EffectivePTS90k += (int64) MaxTimestamp;
					}
					CurrentlyActiveTrackData->PrevPTS90k = EffectivePTS90k;
					CurrentlyActiveTrackData->PrevDTS90k = EffectiveDTS90k;

					// For the very first timestamp from any track we take the offset to subtract to make things relative to zero.
					if (bIsFirstTimestamp)
					{
						if (InRequest->TimestampVars.bGetAndAdjustByFirstTimestamp)
						{
							InRequest->TimestampVars.Internal.SegmentBaseTime.SetFromHNS(InRequest->Segment.Time);
							TimeMappingOffset = InRequest->TimestampVars.Internal.SegmentBaseTime;
							// Take the PTS since it is at least the DTS and also not negative.
							check(EffectivePTS90k >= 0);
							InRequest->TimestampVars.Internal.RawAdjustmentValue = EffectivePTS90k;
							RawAdjustmentValue = EffectivePTS90k;
						}
						bIsFirstTimestamp = false;
					}

					EffectivePTS90k -= RawAdjustmentValue;
					EffectiveDTS90k -= RawAdjustmentValue;

					PTS.SetFrom90kHz(EffectivePTS90k);
					DTS.SetFrom90kHz(EffectiveDTS90k);

					// Remember the first AU's DTS for this stream type in the segment
					if (!InRequest->TimestampVars.Local.First[stIdx].IsValid())
					{
						InRequest->TimestampVars.Local.First[stIdx] = DTS;
					}
					if (stIdx == PrimaryStreamTypeIndex)
					{
						if (StreamTypeAUCount[stIdx] == 0)
						{
							// Check that the timestamps are greater than what we are expecting.
							// This is to prevent reading the same segment data from a different stream again after a stream switch.
							if (InRequest->TimestampVars.Next.bCheck)
							{
								if (InRequest->TimestampVars.Next.ExpectedLargerThan[stIdx].IsValid() && InRequest->TimestampVars.Local.First[stIdx] <= InRequest->TimestampVars.Next.ExpectedLargerThan[stIdx])
								{
									bDone = true;
									InRequest->TimestampVars.Next.bFailed = true;
									bSkippedBecauseOfTimestampCheck = true;
									break;
								}
							}
							bTimeCheckPassed = true;

							InRequest->FirstTimestampReceivedDelegate.ExecuteIfBound(InRequest);
						}
					}

					PTS += TimeMappingOffset;
					DTS += TimeMappingOffset;
				}

				// We get the codec specific data only now with the payload, so set it up.
				if (!CurrentlyActiveTrackData->CSD.IsValid())
				{
					// Check if there is CSD available on this packet. In a bad mux where there is no keyframe at the beginning
					// the CSD may be missing altogether.
					if (!pp.CSD.IsValid() || pp.CSD->IsEmpty())
					{
						// Skip this packet
						continue;
					}
					CurrentlyActiveTrackData->CSD = MakeSharedTS<FAccessUnit::CodecData>();
					if (!TSParser->ParseCSD(CurrentlyActiveTrackData->CSD->ParsedInfo, pp))
					{
						SetError(FString::Printf(TEXT("Failed to parse the CSD for PES packet stream type %d in PID %d"), PESPacket->StreamType, PESPacket->PID), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
						return EHandleResult::Failed;
					}
					CurrentlyActiveTrackData->CSD->CodecSpecificData = CurrentlyActiveTrackData->CSD->ParsedInfo.GetCodecSpecificData();
					CurrentlyActiveTrackData->CSD->ParsedInfo.SetBitrate(InRequest->CodecInfo[StreamTypeToArrayIndex(CurrentlyActiveTrackData->StreamType)].GetBitrate());

					// Get a default duration for this type of sample.
					if (CurrentlyActiveTrackData->CSD->ParsedInfo.IsAudioCodec())
					{
						int32 ns = (int32) CurrentlyActiveTrackData->CSD->ParsedInfo.GetExtras().GetValue(StreamCodecInformationOptions::SamplesPerBlock).SafeGetInt64(0);
						uint32 sr = (uint32) CurrentlyActiveTrackData->CSD->ParsedInfo.GetSamplingRate();
						if (ns && sr)
						{
							CurrentlyActiveTrackData->DefaultDurationFromCSD.SetFromND(ns, sr);
						}
						else
						{
							CurrentlyActiveTrackData->DefaultDurationFromCSD.SetFromND(1024, 48000);
						}
					}
					else if (CurrentlyActiveTrackData->CSD->ParsedInfo.IsVideoCodec())
					{
						FTimeFraction fr(CurrentlyActiveTrackData->CSD->ParsedInfo.GetFrameRate());
						if (fr.IsValid() && fr.GetNumerator())
						{
							CurrentlyActiveTrackData->DefaultDurationFromCSD.SetFromND((int64)fr.GetDenominator(), (uint32)fr.GetNumerator());
						}
						else
						{
							CurrentlyActiveTrackData->DefaultDurationFromCSD.SetFromND(1, 30);
						}
						// Some decoders need the ISO/IEC 14496-15 decoder configuration record, which we need to construct.
						if (CurrentlyActiveTrackData->CSD->ParsedInfo.GetCodec() == FStreamCodecInformation::ECodec::H264 &&
							CurrentlyActiveTrackData->CSD->RawCSD.IsEmpty())
						{
							ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord dcr;
							if (dcr.CreateFromCodecSpecificData(CurrentlyActiveTrackData->CSD->CodecSpecificData))
							{
								CurrentlyActiveTrackData->CSD->RawCSD = dcr.GetRawData();
							}
							else
							{
								SetError(FString::Printf(TEXT("Failed to create the H.264 decoder configuration record from the inband CSD")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
								return EHandleResult::Failed;
							}
						}
						else if (CurrentlyActiveTrackData->CSD->ParsedInfo.GetCodec() == FStreamCodecInformation::ECodec::H265 &&
							CurrentlyActiveTrackData->CSD->RawCSD.IsEmpty())
						{
							ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord dcr;
							if (dcr.CreateFromCodecSpecificData(CurrentlyActiveTrackData->CSD->CodecSpecificData))
							{
								CurrentlyActiveTrackData->CSD->RawCSD = dcr.GetRawData();
							}
							else
							{
								SetError(FString::Printf(TEXT("Failed to create the H.265 decoder configuration record from the inband CSD")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
								return EHandleResult::Failed;
							}
						}
					}
					else
					{
						CurrentlyActiveTrackData->DefaultDurationFromCSD.SetFromND(1, 60);
					}
				}

				// Create the access unit.
				FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
				AccessUnit->AUSize = pp.Data->Num();
				AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
				FMemory::Memcpy(AccessUnit->AUData, pp.Data->GetData(), pp.Data->Num());
				AccessUnit->ESType = CurrentlyActiveTrackData->StreamType;
				AccessUnit->PTS = PTS;
				AccessUnit->PTS += TimeOffset;
				AccessUnit->PTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
				AccessUnit->DTS = DTS;
				AccessUnit->DTS += TimeOffset;
				AccessUnit->DTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
				AccessUnit->Duration = CurrentlyActiveTrackData->DefaultDurationFromCSD;
				AccessUnit->EarliestPTS = EarliestPTS;
				AccessUnit->LatestPTS = LatestPTS;
				AccessUnit->bIsFirstInSequence = CurrentlyActiveTrackData->bIsFirstInSequence;
				AccessUnit->bIsSyncSample = pp.bIsSyncFrame;
				AccessUnit->bIsDummyData = false;
				AccessUnit->AUCodecData = CurrentlyActiveTrackData->CSD;
				AccessUnit->BufferSourceInfo = CurrentlyActiveTrackData->BufferSourceInfo;
				AccessUnit->SequenceIndex = InRequest->TimestampSequenceIndex;

				PTS += AccessUnit->Duration;
				DTS += AccessUnit->Duration;
				CurrentlyActiveTrackData->bIsFirstInSequence = false;

				// Add to the track AU FIFO unless we already reached the last sample of the time range.
				if (!CurrentlyActiveTrackData->bReadPastLastPTS)
				{
					CurrentlyActiveTrackData->AddAccessUnit(AccessUnit);
				}
				CurrentlyActiveTrackData->DurationSuccessfullyRead = CurrentlyActiveTrackData->LargestPTS - CurrentlyActiveTrackData->SmallestPTS + CurrentlyActiveTrackData->AverageDuration;

				FAccessUnit::Release(AccessUnit);
				AccessUnit = nullptr;

				++StreamTypeAUCount[stIdx];
			}
		}
		else if (TSParserState == IParserISO13818_1::EParseState::EOS)
		{
			bDone = true;
		}
		else if (TSParserState == IParserISO13818_1::EParseState::ReadError)
		{
			// Either the read error was already logged, or if the download was canceled we don't need to
			// create an error here either. Just be done and exit.
			bDone = true;
		}
		else if (TSParserState == IParserISO13818_1::EParseState::Failed)
		{
			SegmentError = TSParser->GetLastError();
			bHasErrored = true;
			return EHandleResult::Failed;
		}
		else
		{
			unimplemented();
			SetError(FString::Printf(TEXT("Internal MPEG TS parser error %d"), (int32)TSParserState), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			return EHandleResult::Failed;
		}

		// Shall we pass on any AUs we already read?
		if (bAllowEarlyEmitting && bTimeCheckPassed)
		{
			EmitSamples(EEmitType::UntilBlocked);
		}
	}

	InRequest->TimestampVars.Internal.Rollover[0] = Rollover[0];
	InRequest->TimestampVars.Internal.Rollover[1] = Rollover[1];
	InRequest->TimestampVars.Internal.Rollover[2] = Rollover[2];
	InRequest->TimestampVars.Internal.Rollover[3] = Rollover[3];

	return HasReadBeenAborted() ? EHandleResult::Aborted :
		   HasErrored() ? EHandleResult::Failed :
		   bSkippedBecauseOfTimestampCheck ? EHandleResult::Skipped : EHandleResult::Finished;
}

FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::HandleID3RawMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest, int32 InID3HeaderSize)
{
	// This is for audio segments only
	if (InRequest->GetType() != EStreamType::Audio)
	{
		SetError(FString::Printf(TEXT("Raw ID3 streams are expected to be used for audio only")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
		bHasErrored = true;
		return EHandleResult::Failed;
	}

	bool bSkippedBecauseOfTimestampCheck = false;

	TArray<uint8> ID3HeaderData;
	ID3HeaderData.SetNumUninitialized(InID3HeaderSize);
	int64 NumRead = ReadData(ID3HeaderData.GetData(), InID3HeaderSize, -1);
	TSharedPtrTS<MPEG::FID3V2Metadata> ID3Header = MakeSharedTS<MPEG::FID3V2Metadata>();
	if (NumRead == InID3HeaderSize)
	{
		if (!ID3Header->Parse(ID3HeaderData.GetData(), InID3HeaderSize))
		{
			SetError(FString::Printf(TEXT("Failed to parse ID3 header")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			bHasErrored = true;
			return EHandleResult::Failed;
		}
		// In case of HLS there needs to be private data present specifying the first PTS.
		uint64 FirstPTS90k = 0;
		// For the time being we allow packed audio only with HLS.
		if (InRequest->StreamingProtocol != FStreamSegmentRequestCommon::EStreamingProtocol::HLS)
		{
			SetError(FString::Printf(TEXT("Packed audio is currently only permitted with HLS")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			bHasErrored = true;
			return EHandleResult::Failed;
		}
		else//if (InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::HLS)
		{
			const MPEG::FID3V2Metadata::FItem* ApplePrivate = ID3Header->GetPrivateItems().FindByPredicate([](const MPEG::FID3V2Metadata::FItem& Item) { return Item.MimeType.Equals(TEXT("com.apple.streaming.transportStreamTimestamp")); });
			if (!ApplePrivate)
			{
				SetError(FString::Printf(TEXT("HLS packed audio requires ID3 'PRIV' tag")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				bHasErrored = true;
				return EHandleResult::Failed;
			}
			if (ApplePrivate->Value.GetType() != EVariantTypes::ByteArray || ApplePrivate->Value.GetValue<TArray<uint8>>().Num() != 8)
			{
				SetError(FString::Printf(TEXT("Bad HLS packed audio ID3 'PRIV' tag content")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
				bHasErrored = true;
				return EHandleResult::Failed;
			}
			const TArray<uint8>& PTSData(ApplePrivate->Value.GetValue<TArray<uint8>>());
			for(auto& ptsIt : PTSData)
			{
				FirstPTS90k <<= 8U;
				FirstPTS90k |= ptsIt;
			}
		}
		// There is no knowing at this point what type of packed audio this is.
		// We rely on the server returning a supported MIME type
		FString ContentType = HTTPRequest->ConnectionInfo.ContentType;
		enum class EPackedAudioFormat
		{
			Unknown,
			Mpeg123,
			AAC
		};
		EPackedAudioFormat PackedAudioFormat = EPackedAudioFormat::Unknown;
		if (ContentType.Equals(TEXT("audio/mpeg"), ESearchCase::IgnoreCase))
		{
			PackedAudioFormat = EPackedAudioFormat::Mpeg123;
		}
		else if (ContentType.Equals(TEXT("audio/aac"), ESearchCase::IgnoreCase) ||
				 ContentType.Equals(TEXT("audio/x-aac"), ESearchCase::IgnoreCase))
		{
			PackedAudioFormat = EPackedAudioFormat::AAC;
		}
		/*
		else if (MimeTypeHeader->Value.Equals(TEXT(""), ESearchCase::IgnoreCase))
		{
		}
		*/
		if (PackedAudioFormat == EPackedAudioFormat::Unknown)
		{
			SetError(FString::Printf(TEXT("Unsupported packed audio type or unsupported MIME type returned by server ('%s')"), *ContentType), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
			bHasErrored = true;
			return EHandleResult::Failed;
		}

		// Create a track map with one entry as track #0
		TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> NewTrackDataMap;
		TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
		td->StreamType = EStreamType::Audio;
		td->bNeedToRecalculateDurations = false;
		check(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Audio)].IsValid());
		td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Audio)]));
		td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
		NewTrackDataMap.Emplace(0, MoveTemp(td));
		TrackDataMap = MoveTemp(NewTrackDataMap);
		CurrentlyActiveTrackData = TrackDataMap[0];
		SelectPrimaryTrackData(InRequest);


		auto GetUINT32BE = [](const uint8* InData) -> uint32
		{
			return (static_cast<uint32>(InData[0]) << 24) | (static_cast<uint32>(InData[1]) << 16) | (static_cast<uint32>(InData[2]) << 8) | static_cast<uint32>(InData[3]);
		};
		FTimeValue TimeOffset = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
		const int32 kNumProbeBytesNeeded = 16;
		uint8 ProbeBytes[kNumProbeBytesNeeded];
		bool bDone = false;
		int32 nPkt = 0;
		FTimeValue PTS;
		uint64 RawAdjustmentValue = InRequest->TimestampVars.Internal.RawAdjustmentValue.Get((uint64)0);
		FTimeValue TimeMappingOffset = InRequest->TimestampVars.Internal.SegmentBaseTime;
		check(TimeMappingOffset.IsValid());
		const int32 stIdx = StreamTypeToArrayIndex(CurrentlyActiveTrackData->StreamType);
		bool bTimeCheckPassed = InRequest->TimestampVars.Next.bCheck == false;

		FTimeValue EarliestPTS;
		EarliestPTS.SetFromHNS(InRequest->Segment.MediaLocalFirstAUTime);
		if (InRequest->Segment.bFrameAccuracyRequired && InRequest->FrameAccurateStartTime.IsValid())
		{
			EarliestPTS = InRequest->FrameAccurateStartTime;
		}
		EarliestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
		FTimeValue LatestPTS;
		if (InRequest->Segment.MediaLocalLastAUTime != TNumericLimits<int64>::Max())
		{
			LatestPTS.SetFromHNS(InRequest->Segment.MediaLocalLastAUTime);
		}
		else
		{
			LatestPTS.SetToPositiveInfinity();
		}
		LatestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);

		while(!bDone && !HasErrored() && !HasReadBeenAborted())
		{
			// We now wait for the next few bytes to arrive so we can get the packet size
			// from the header this should be representing.
			bool bContinue = true;
			while(!ReadBuffer.WaitUntilSizeAvailable(ReadBuffer.ParsePos + kNumProbeBytesNeeded, 1000*20))
			{
				if (HasErrored() || HasReadBeenAborted())
				{
					bContinue = false;
					bDone = true;
					break;
				}
			}
			if (bContinue)
			{
				FScopeLock ProbeBufferLock(ReadBuffer.GetLock());
				const uint8* MediaDataPtr = ReadBuffer.GetLinearReadData();
				if (ReadBuffer.GetLinearReadSize() >= ReadBuffer.ParsePos + kNumProbeBytesNeeded)
				{
					FMemory::Memcpy(ProbeBytes, ReadBuffer.GetLinearReadData() + ReadBuffer.ParsePos, kNumProbeBytesNeeded);
				}
				else
				{
					// Done reading.
					bContinue = false;
					bDone = true;
				}
			}
			if (bContinue)
			{
				int32 PayloadPacketSize = 0;
				int32 PayloadSkipSize = 0;
				if (PackedAudioFormat == EPackedAudioFormat::Mpeg123)
				{
					const uint32 HeaderValue = GetUINT32BE(ProbeBytes);
					if (!ElectraDecodersUtil::MPEG::UtilsMPEG123::HasValidSync(HeaderValue))
					{
						SetError(FString::Printf(TEXT("Packed MPEG audio data does not have expected sync word")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
						bHasErrored = true;
						return EHandleResult::Failed;
					}
					PayloadPacketSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetFrameSize(HeaderValue);

					// Create CSD
					if (!CurrentlyActiveTrackData->CSD.IsValid())
					{
						CurrentlyActiveTrackData->CSD = MakeSharedTS<FAccessUnit::CodecData>();
						FStreamCodecInformation& ci = CurrentlyActiveTrackData->CSD->ParsedInfo;

						const int32 FrameSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplesPerFrame(HeaderValue);
						const int32 SampleRate = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplingRate(HeaderValue);
						ci.SetStreamType(EStreamType::Audio);
						ci.SetMimeType(TEXT("audio/mpeg"));
						ci.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
						ci.SetCodec4CC(Utils::Make4CC('m','p','g','a'));
						ci.SetProfile(ElectraDecodersUtil::MPEG::UtilsMPEG123::GetVersion(HeaderValue));
						ci.SetProfileLevel(ElectraDecodersUtil::MPEG::UtilsMPEG123::GetLayer(HeaderValue));
						ci.SetCodecSpecifierRFC6381(TEXT("mp4a.6b"));	// alternatively "mp4a.40.34"
						ci.SetSamplingRate(SampleRate);
						ci.SetNumberOfChannels(ElectraDecodersUtil::MPEG::UtilsMPEG123::GetChannelCount(HeaderValue));
						ci.GetExtras().Set(StreamCodecInformationOptions::SamplesPerBlock, FVariantValue((int64)FrameSize));
						ci.SetBitrate(InRequest->CodecInfo[stIdx].GetBitrate());
						CurrentlyActiveTrackData->DefaultDurationFromCSD.SetFromND(FrameSize, SampleRate);
					}
				}
				else if (PackedAudioFormat == EPackedAudioFormat::AAC)
				{
					if (!(ProbeBytes[0] == 0xff && (ProbeBytes[1] & 0xf0) == 0xf0))
					{
						SetError(FString::Printf(TEXT("Packed MPEG audio data does not have expected sync word")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
						bHasErrored = true;
						return EHandleResult::Failed;
					}

					FBitstreamReader br(ProbeBytes, kNumProbeBytesNeeded);
					br.SkipBits(12);	// sync word, already checked for.
					const uint32 mpeg_version = br.GetBits(1);
					const uint32 layer = br.GetBits(2);
					const uint32 prot_absent = br.GetBits(1);
					const uint32 profile = br.GetBits(2);
					const uint32 sampling_frequency_index = br.GetBits(4);
					const uint32 private_bit = br.GetBits(1);
					const uint32 channel_configuration = br.GetBits(3);
					const uint32 originalty = br.GetBits(1);
					const uint32 home = br.GetBits(1);
					const uint32 copyright_id = br.GetBits(1);
					const uint32 copyright_id_start = br.GetBits(1);
					const uint32 frame_length = br.GetBits(13);
					const uint32 buffer_fullness = br.GetBits(11);
					const uint32 num_frames = br.GetBits(2);
					const uint32 crc = prot_absent ? 0 : br.GetBits(16);
					const int32 FrameSize = frame_length - (prot_absent ? 7 : 9);
					if (num_frames > 0)
					{
						SetError(FString::Printf(TEXT("Multiple RDBs in ADTS frame is not supported!")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
						bHasErrored = true;
						return EHandleResult::Failed;
					}
					if (channel_configuration == 0)
					{
						SetError(FString::Printf(TEXT("Channel configuration 0 is not supported!")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
						bHasErrored = true;
						return EHandleResult::Failed;
					}

					PayloadPacketSize = FrameSize;
					PayloadSkipSize = frame_length - FrameSize;

					// Create CSD
					if (!CurrentlyActiveTrackData->CSD.IsValid())
					{
						CurrentlyActiveTrackData->CSD = MakeSharedTS<FAccessUnit::CodecData>();
						FStreamCodecInformation& ci = CurrentlyActiveTrackData->CSD->ParsedInfo;

						CurrentlyActiveTrackData->CSD->CodecSpecificData.SetNumUninitialized(2);
						uint32 csd = (profile + 1) << 11;
						csd |= sampling_frequency_index << 7;
						csd |= channel_configuration << 3;
						CurrentlyActiveTrackData->CSD->CodecSpecificData[0] = (uint8)(csd >> 8);
						CurrentlyActiveTrackData->CSD->CodecSpecificData[1] = (uint8)(csd & 255);

						ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord AudioSpecificConfig;
						AudioSpecificConfig.ParseFrom(CurrentlyActiveTrackData->CSD->CodecSpecificData.GetData(), CurrentlyActiveTrackData->CSD->CodecSpecificData.Num());
						ci.SetStreamType(EStreamType::Audio);
						ci.SetMimeType(TEXT("audio/mp4"));
						ci.SetCodec(FStreamCodecInformation::ECodec::AAC);
						ci.SetCodecSpecificData(AudioSpecificConfig.GetCodecSpecificData());
						ci.SetCodecSpecifierRFC6381(FString::Printf(TEXT("mp4a.40.%d"), AudioSpecificConfig.ExtAOT ? AudioSpecificConfig.ExtAOT : AudioSpecificConfig.AOT));
						ci.SetSamplingRate(AudioSpecificConfig.ExtSamplingFrequency ? AudioSpecificConfig.ExtSamplingFrequency : AudioSpecificConfig.SamplingRate);
						ci.SetChannelConfiguration(AudioSpecificConfig.ChannelConfiguration);
						ci.SetNumberOfChannels(ElectraDecodersUtil::MPEG::AACUtils::GetNumberOfChannelsFromChannelConfiguration(AudioSpecificConfig.ChannelConfiguration));
						// We assume that all platforms can decode PS (parametric stereo). As such we change the channel count from mono to stereo
						// to convey the _decoded_ format, not the source format.
						if (AudioSpecificConfig.ChannelConfiguration == 1 && AudioSpecificConfig.PSSignal > 0)
						{
							ci.SetNumberOfChannels(2);
						}
						const int32 NumDecodedSamplesPerBlock = AudioSpecificConfig.SBRSignal > 0 ? 2048 : 1024;
						ci.GetExtras().Set(StreamCodecInformationOptions::SamplesPerBlock, FVariantValue((int64)NumDecodedSamplesPerBlock));

						ci.SetBitrate(InRequest->CodecInfo[stIdx].GetBitrate());
						CurrentlyActiveTrackData->DefaultDurationFromCSD.SetFromND(NumDecodedSamplesPerBlock, ci.GetSamplingRate());
					}
				}

				if (nPkt == 0)
				{
					const uint64 MaxTimestamp = 1ULL << 33U;
					const uint64 HalfMaxTimestamp = 1ULL << 32U;

					// Detect PTS rollover to previous segment.
					if (InRequest->TimestampVars.Internal.PrevRawID3StartPTS.IsSet())
					{
						uint64 PrevPTS = InRequest->TimestampVars.Internal.PrevRawID3StartPTS.GetValue();
						if (FirstPTS90k < PrevPTS)
						{
							UE_LOGF(LogElectraPlayer, Log, "PTS rollover detected: %lld -> %lld", (long long int) PrevPTS, (long long int) FirstPTS90k);
							InRequest->TimestampVars.Internal.Rollover[stIdx].RawPTSOffset += MaxTimestamp;
						}
					}
					InRequest->TimestampVars.Internal.PrevRawID3StartPTS = FirstPTS90k;

					// Add the rollover value we accumulated so far.
					uint64 PTS90k = FirstPTS90k + InRequest->TimestampVars.Internal.Rollover[stIdx].RawPTSOffset;

					int64 EffectivePTS90k = (int64) PTS90k;
					CurrentlyActiveTrackData->PrevPTS90k = EffectivePTS90k;

					// Take the offset to subtract to make things relative to zero.
					if (InRequest->TimestampVars.bGetAndAdjustByFirstTimestamp)
					{
						InRequest->TimestampVars.Internal.SegmentBaseTime.SetFromHNS(InRequest->Segment.Time);
						TimeMappingOffset = InRequest->TimestampVars.Internal.SegmentBaseTime;
						InRequest->TimestampVars.Internal.RawAdjustmentValue = EffectivePTS90k;
						RawAdjustmentValue = EffectivePTS90k;
					}
					EffectivePTS90k -= RawAdjustmentValue;

					PTS.SetFrom90kHz(EffectivePTS90k);

					// Remember the first AU's DTS for this stream type in the segment
					if (!InRequest->TimestampVars.Local.First[stIdx].IsValid())
					{
						InRequest->TimestampVars.Local.First[stIdx] = PTS;
					}

					// Check that the timestamps are greater than what we are expecting.
					// This is to prevent reading the same segment data from a different stream again after a stream switch.
					if (InRequest->TimestampVars.Next.bCheck)
					{
						if (InRequest->TimestampVars.Next.ExpectedLargerThan[stIdx].IsValid() && InRequest->TimestampVars.Local.First[stIdx] <= InRequest->TimestampVars.Next.ExpectedLargerThan[stIdx])
						{
							bDone = true;
							InRequest->TimestampVars.Next.bFailed = true;
							bSkippedBecauseOfTimestampCheck = true;
							break;
						}
					}
					bTimeCheckPassed = true;

					InRequest->FirstTimestampReceivedDelegate.ExecuteIfBound(InRequest);

					PTS += TimeMappingOffset;
				}

				// Create the access unit.
				FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
				AccessUnit->AUSize = PayloadPacketSize;
				AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
				if (PayloadSkipSize)
				{
					NumRead = ReadData(nullptr, PayloadSkipSize, -1);
					if (NumRead != PayloadSkipSize)
					{
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;
						bDone = true;
						break;
					}
				}

				NumRead = ReadData(AccessUnit->AUData, AccessUnit->AUSize, -1);
				if (NumRead != AccessUnit->AUSize)
				{
					// Did not get the number of bytes we needed. Either because of a read error or because we got aborted.
					FAccessUnit::Release(AccessUnit);
					AccessUnit = nullptr;
					bDone = true;
					break;
				}

				AccessUnit->ESType = CurrentlyActiveTrackData->StreamType;
				AccessUnit->PTS = PTS + TimeOffset;
				AccessUnit->PTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
				AccessUnit->DTS = AccessUnit->PTS;
				AccessUnit->Duration = CurrentlyActiveTrackData->DefaultDurationFromCSD;
				AccessUnit->EarliestPTS = EarliestPTS;
				AccessUnit->LatestPTS = LatestPTS;
				AccessUnit->bIsFirstInSequence = CurrentlyActiveTrackData->bIsFirstInSequence;
				AccessUnit->bIsSyncSample = true;
				AccessUnit->bIsDummyData = false;
				AccessUnit->AUCodecData = CurrentlyActiveTrackData->CSD;
				AccessUnit->BufferSourceInfo = CurrentlyActiveTrackData->BufferSourceInfo;
				AccessUnit->SequenceIndex = InRequest->TimestampSequenceIndex;

				PTS += AccessUnit->Duration;
				CurrentlyActiveTrackData->bIsFirstInSequence = false;

				// Add to the track AU FIFO unless we already reached the last sample of the time range.
				if (!CurrentlyActiveTrackData->bReadPastLastPTS)
				{
					CurrentlyActiveTrackData->AddAccessUnit(AccessUnit);
				}
				CurrentlyActiveTrackData->DurationSuccessfullyRead = CurrentlyActiveTrackData->LargestPTS - CurrentlyActiveTrackData->SmallestPTS + CurrentlyActiveTrackData->AverageDuration;

				FAccessUnit::Release(AccessUnit);
				AccessUnit = nullptr;

				++nPkt;

				// Shall we pass on any AUs we already read?
				if (bAllowEarlyEmitting && bTimeCheckPassed)
				{
					EmitSamples(EEmitType::UntilBlocked);
				}
			}
		}
	}
	else if (!HasReadBeenAborted())
	{
		SetError(FString::Printf(TEXT("Failed to read ID3 header")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
		bHasErrored = true;
		return EHandleResult::Failed;
	}
	return HasReadBeenAborted() ? EHandleResult::Aborted :
		   HasErrored() ? EHandleResult::Failed :
		   bSkippedBecauseOfTimestampCheck ? EHandleResult::Skipped : EHandleResult::Finished;
}

FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::HandleSideloadedMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	SegmentError.Clear();

	DownloadStats.ResetOutput();
	DownloadStats.StatsID = FMediaInterlockedIncrement(UniqueDownloadID);
	DownloadStats.SegmentType = Metrics::ESegmentType::Media;
	DownloadStats.URL = InRequest->Segment.MediaURL.Url;
	DownloadStats.Range = InRequest->Segment.MediaURL.Range;
	DownloadStats.SteeringID = InRequest->Segment.MediaURL.SteeringID;

	// Check with the entity cache if we already have it from earlier.
	TSharedPtrTS<IPlayerEntityCache> EntityCache = PlayerSessionService ? PlayerSessionService->GetEntityCache() : nullptr;
	TSharedPtrTS<const TArray<uint8>> SideloadedData;
	if (EntityCache.IsValid())
	{
		IPlayerEntityCache::FCacheItem CachedItem;
		if (EntityCache->GetCachedEntity(CachedItem, InRequest->Segment.MediaURL.Url.URL, InRequest->Segment.MediaURL.Range))
		{
			if (CachedItem.RawPayloadData.IsValid())
			{
				SideloadedData = CachedItem.RawPayloadData;
				DownloadStats.bWasSuccessful = true;
			}
		}
	}
	if (!SideloadedData.IsValid())
	{
		// Not cached yet, need to fetch.
		CurrentConnectionInfo = HTTP::FConnectionInfo();

		TArray<HTTP::FHTTPHeader> ReqHeaders;
		if (InRequest->Segment.MediaURL.CustomHeader.Len() && InRequest->StreamingProtocol == FStreamSegmentRequestCommon::EStreamingProtocol::DASH)
		{
			ReqHeaders.Emplace(HTTP::FHTTPHeader({DASH::HTTPHeaderOptionName, InRequest->Segment.MediaURL.CustomHeader}));
		}

		TSharedPtrTS<FHTTPResourceRequest> rr = MakeSharedTS<FHTTPResourceRequest>();
		TSharedPtr<FHTTPResourceRequestCompletionSignal, ESPMode::ThreadSafe> rrsig = FHTTPResourceRequestCompletionSignal::Create();

		rr->Verb(TEXT("GET")).URL(InRequest->Segment.MediaURL.Url.URL).Range(InRequest->Segment.MediaURL.Range).Headers(ReqHeaders)
			.ConnectionTimeout(FTimeValue().SetFromMilliseconds(5000)).NoDataTimeout(FTimeValue().SetFromMilliseconds(2000))
			.StreamTypeAndQuality(InRequest->StreamType, InRequest->QualityIndex, InRequest->MaxQualityIndex)
			.CompletionSignal(rrsig)
			.StartGet(PlayerSessionService);
		while(!rrsig->WaitTimeout(1000 * 10))
		{
			if (HasReadBeenAborted())
			{
				rr->Cancel();
				break;
			}
		}
		if (HasReadBeenAborted())
		{
			return EHandleResult::Aborted;
		}
		CurrentConnectionInfo = *rr->GetConnectionInfo();
		TSharedPtrTS<FWaitableBuffer> ResponseBuffer = rr->GetResponseBuffer();

		bool bSuccessful = !rr->GetError() && ResponseBuffer.IsValid();
		SetupSegmentDownloadStatsFromConnectionInfo(CurrentConnectionInfo);

		// Success?
		if (!bSuccessful)
		{
			// No.
			SetError(FString::Printf(TEXT("Sideloaded segment download error: %s"),	*CurrentConnectionInfo.StatusInfo.ErrorDetail.GetMessage()), INTERNAL_SEG_ERROR_SIDELOAD_DOWNLOAD_ERROR);
			bHasErrored = true;
			return EHandleResult::Failed;
		}

		SideloadedData = MakeSharedTS<TArray<uint8>>(ResponseBuffer->GetLinearReadData(), ResponseBuffer->GetLinearReadSize());

		if (EntityCache.IsValid())
		{
			// Set the response headers with the entity cache.
			EntityCache->SetRecentResponseHeaders(IPlayerEntityCache::EEntityType::Segment, InRequest->Segment.MediaURL.Url.URL, CurrentConnectionInfo.ResponseHeaders);

			// Cache the sideloaded file as well.
			IPlayerEntityCache::FCacheItem CacheItem;
			CacheItem.URL = InRequest->Segment.MediaURL.Url.URL;
			CacheItem.Range = InRequest->Segment.MediaURL.Range;
			CacheItem.RawPayloadData = SideloadedData;
			EntityCache->CacheEntity(CacheItem);
		}
		StreamSelector->ReportDownloadEnd(DownloadStats);
	}

	if (SideloadedData.IsValid())
	{
		TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> NewTrackDataMap;
		TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
		td->StreamType = InRequest->GetType();
		// Copy the source buffer info into a new instance and set the playback sequence ID in it.
		check(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(InRequest->GetType())].IsValid());
		td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(InRequest->GetType())]));
		td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
		// Set the CSD
		td->CSD = MakeSharedTS<FAccessUnit::CodecData>();
		td->CSD->ParsedInfo = InRequest->CodecInfo[StreamTypeToArrayIndex(InRequest->GetType())];
		NewTrackDataMap.Emplace(1, td);
		TrackDataMap = MoveTemp(NewTrackDataMap);
		CurrentlyActiveTrackData = MoveTemp(td);

		uint32 TrackTimescale = InRequest->Segment.Timescale;

		FTimeValue TimeOffset = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
		const bool bDoNotTruncateAtPresentationEnd = PlayerSessionService->GetOptionValue(OptionKeyDoNotTruncateAtPresentationEnd).SafeGetBool(false);
		FTimeValue PTO(InRequest->Segment.PTO, InRequest->Segment.Timescale);
		FTimeValue EarliestPTS(InRequest->Segment.MediaLocalFirstAUTime, InRequest->Segment.Timescale, InRequest->TimestampSequenceIndex);
		EarliestPTS += TimeOffset - PTO;
		if (InRequest->Segment.bFrameAccuracyRequired && InRequest->FrameAccurateStartTime.IsValid())
		{
			EarliestPTS = InRequest->FrameAccurateStartTime;
		}
		EarliestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
		FTimeValue LatestPTS;
		if (!bDoNotTruncateAtPresentationEnd && InRequest->Segment.MediaLocalLastAUTime != TNumericLimits<int64>::Max())
		{
			LatestPTS.SetFromND(InRequest->Segment.MediaLocalLastAUTime, InRequest->Segment.Timescale, InRequest->TimestampSequenceIndex);
			LatestPTS += TimeOffset - PTO;
		}
		else
		{
			LatestPTS.SetToPositiveInfinity();
		}
		LatestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);

		// Set the PTO in the codec data extras. This is a rarely used value and constant for the segment.
		// The only use we have for it so far is to remap subtitle timestamps to split periods.
		CurrentlyActiveTrackData->CSD->ParsedInfo.GetExtras().Set(StreamCodecInformationOptions::PresentationTimeOffset, FVariantValue(PTO));

		// Create an access unit
		FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
		AccessUnit->ESType = InRequest->GetType();
		AccessUnit->AUSize = (uint32) SideloadedData->Num();
		AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
		FMemory::Memcpy(AccessUnit->AUData, SideloadedData->GetData(), SideloadedData->Num());
		AccessUnit->AUCodecData = CurrentlyActiveTrackData->CSD;
		AccessUnit->bIsFirstInSequence = true;
		AccessUnit->bIsSyncSample = true;
		AccessUnit->bIsDummyData = false;
		AccessUnit->bIsSideloaded = true;
		AccessUnit->BufferSourceInfo = CurrentlyActiveTrackData->BufferSourceInfo;
		AccessUnit->SequenceIndex = InRequest->TimestampSequenceIndex;

		// Sideloaded files coincide with the period start
		AccessUnit->DTS = TimeOffset;
		AccessUnit->PTS = TimeOffset;
		AccessUnit->DTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
		AccessUnit->PTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
		AccessUnit->EarliestPTS = EarliestPTS;
		AccessUnit->LatestPTS = LatestPTS;

		FTimeValue Duration(InRequest->Segment.Duration, TrackTimescale);
		AccessUnit->Duration = Duration;

		CurrentlyActiveTrackData->DurationSuccessfullyRead += Duration;
		CurrentlyActiveTrackData->AddAccessUnit(AccessUnit);
		FAccessUnit::Release(AccessUnit);
		AccessUnit = nullptr;
	}

	return EHandleResult::Finished;
}

FStreamSegmentReaderCommon::FStreamHandler::EHandleResult FStreamSegmentReaderCommon::FStreamHandler::HandleRawSubtitleMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	// This is for subtitle segments only
	if (InRequest->GetType() != EStreamType::Subtitle)
	{
		SetError(FString::Printf(TEXT("Raw subtitle streams are expected to be used for subtitles only")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
		bHasErrored = true;
		return EHandleResult::Failed;
	}
	// And also only for WebVTT at the moment
	if (!InRequest->CodecInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)].GetCodecName().Equals(TEXT("wvtt"), ESearchCase::IgnoreCase))
	{
		SetError(FString::Printf(TEXT("Raw subtitle streams are currently supported fro WebVTT only")), INTERNAL_SEG_ERROR_BAD_MEDIA_SEGMENT);
		bHasErrored = true;
		return EHandleResult::Failed;
	}

	// Create a track map with one entry as track #0
	TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> NewTrackDataMap;
	TSharedPtrTS<FActiveTrackData> td = MakeSharedTS<FActiveTrackData>();
	td->StreamType = EStreamType::Subtitle;
	td->bNeedToRecalculateDurations = false;
	check(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)].IsValid());
	td->BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>(*(InRequest->SourceBufferInfo[StreamTypeToArrayIndex(EStreamType::Subtitle)]));
	td->BufferSourceInfo->PlaybackSequenceID = InRequest->GetPlaybackSequenceID();
	NewTrackDataMap.Emplace(0, MoveTemp(td));
	TrackDataMap = MoveTemp(NewTrackDataMap);
	CurrentlyActiveTrackData = TrackDataMap[0];
	SelectPrimaryTrackData(InRequest);

	// Read the data, but at most 2 MiB
	const int64 kMaxReadSize = 2 << 20;
	bool bContinue = true;
	while(!ReadBuffer.WaitUntilSizeAvailable(kMaxReadSize, 1000*20))
	{
		if (HasErrored() || HasReadBeenAborted())
		{
			bContinue = false;
			break;
		}
	}
	if (bContinue)
	{
		// Create CSD
		if (!CurrentlyActiveTrackData->CSD.IsValid())
		{
			CurrentlyActiveTrackData->CSD = MakeSharedTS<FAccessUnit::CodecData>();
			FStreamCodecInformation& ci = CurrentlyActiveTrackData->CSD->ParsedInfo;
			ci.SetStreamType(EStreamType::Subtitle);
			ci.SetMimeType(TEXT("text/vtt"));
			ci.SetCodec(FStreamCodecInformation::ECodec::WebVTT);
			ci.SetCodecSpecifierRFC6381(TEXT("wvtt"));
		}

		FTimeValue PTS, EarliestPTS;
		EarliestPTS.SetFromHNS(InRequest->Segment.MediaLocalFirstAUTime);
		PTS.SetFromHNS(InRequest->Segment.Time);
		if (InRequest->Segment.bFrameAccuracyRequired && InRequest->FrameAccurateStartTime.IsValid())
		{
			EarliestPTS = InRequest->FrameAccurateStartTime;
		}
		EarliestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
		FTimeValue LatestPTS;
		if (InRequest->Segment.MediaLocalLastAUTime != TNumericLimits<int64>::Max())
		{
			LatestPTS.SetFromHNS(InRequest->Segment.MediaLocalLastAUTime);
		}
		else
		{
			LatestPTS.SetToPositiveInfinity();
		}
		LatestPTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);

		CurrentlyActiveTrackData->DefaultDurationFromCSD.SetFromND(InRequest->Segment.Duration, InRequest->Segment.Timescale);

		// Check if there has been an init segment for this which we need to prepend to the data.
		TArray<uint8> PrependData;
		if (InitSegmentData.IsType<TSharedPtrTS<const TArray<uint8>>>())
		{
			TSharedPtrTS<const TArray<uint8>> InitData = InitSegmentData.Get<TSharedPtrTS<const TArray<uint8>>>();
			if (InitData.IsValid() && InitData->Num())
			{
				PrependData = *InitData;
			}
		}

		// Create the access unit.
		FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
		AccessUnit->AUSize = ReadBuffer.GetLinearReadSize() + PrependData.Num();
		AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
		if (PrependData.Num())
		{
			FMemory::Memcpy(AccessUnit->AUData, PrependData.GetData(), PrependData.Num());
		}
		FMemory::Memcpy(AdvancePointer(AccessUnit->AUData, PrependData.Num()), ReadBuffer.GetLinearReadData(), AccessUnit->AUSize - PrependData.Num());
		AccessUnit->ESType = CurrentlyActiveTrackData->StreamType;
		AccessUnit->PTS = PTS;
		AccessUnit->PTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
		AccessUnit->DTS = AccessUnit->PTS;
		AccessUnit->Duration = CurrentlyActiveTrackData->DefaultDurationFromCSD;
		AccessUnit->EarliestPTS = EarliestPTS;
		AccessUnit->LatestPTS = LatestPTS;
		AccessUnit->bIsFirstInSequence = CurrentlyActiveTrackData->bIsFirstInSequence;
		AccessUnit->bIsSyncSample = true;
		AccessUnit->bIsDummyData = false;
		AccessUnit->AUCodecData = CurrentlyActiveTrackData->CSD;
		AccessUnit->BufferSourceInfo = CurrentlyActiveTrackData->BufferSourceInfo;
		AccessUnit->SequenceIndex = InRequest->TimestampSequenceIndex;

		CurrentlyActiveTrackData->bIsFirstInSequence = false;
		CurrentlyActiveTrackData->AddAccessUnit(AccessUnit);

		CurrentlyActiveTrackData->DurationSuccessfullyRead = CurrentlyActiveTrackData->DefaultDurationFromCSD;

		FAccessUnit::Release(AccessUnit);
		AccessUnit = nullptr;
	}
	return HasReadBeenAborted() ? EHandleResult::Aborted :
		   HasErrored() ? EHandleResult::Failed : EHandleResult::Finished;
}

void FStreamSegmentReaderCommon::FStreamHandler::InsertFillerData(TSharedPtrTS<FActiveTrackData> InActiveTrackData, TSharedPtrTS<FStreamSegmentRequestCommon> InRequest)
{
	if (!InActiveTrackData.IsValid() || !InRequest.IsValid())
	{
		return;
	}

	FTimeValue TimeOffset = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
	FTimeValue SegmentDurationToGo(InRequest->Segment.Duration, InRequest->Segment.Timescale, 0);
	FTimeValue DefaultDuration;
	FTimeValue NextExpectedDTS;
	if (InActiveTrackData->NumAddedTotal)
	{
		SegmentDurationToGo -= InActiveTrackData->DurationSuccessfullyRead;
		DefaultDuration = InActiveTrackData->DurationSuccessfullyRead / InActiveTrackData->NumAddedTotal;
		NextExpectedDTS = InActiveTrackData->LargestDTS + DefaultDuration;
	}
	else
	{
		NextExpectedDTS.SetFromND(InRequest->Segment.Time - InRequest->Segment.PTO, InRequest->Segment.Timescale);
		NextExpectedDTS += TimeOffset;
	}
	NextExpectedDTS.SetSequenceIndex(InRequest->TimestampSequenceIndex);
	if (!DefaultDuration.IsValid())
	{
		switch(InActiveTrackData->StreamType)
		{
			case EStreamType::Video:
			{
				DefaultDuration.SetFromND(1, 30);
				break;
			}
			case EStreamType::Audio:
			{
				DefaultDuration.SetFromND(1, 20);
				break;
			}
			default:
			{
				DefaultDuration.SetFromND(1, 10);
				break;
			}
		}
	}

	// Too small a gap to bother to fill?
	if (SegmentDurationToGo < DefaultDuration)
	{
		return;
	}
	FTimeValue PTO(InRequest->Segment.PTO, InRequest->Segment.Timescale, 0);
	FTimeValue Earliest;
	Earliest.SetFromND(InRequest->Segment.MediaLocalFirstAUTime - InRequest->Segment.PTO, InRequest->Segment.Timescale);
	Earliest += TimeOffset;
	if (InRequest->Segment.bFrameAccuracyRequired)
	{
		if (InRequest->FrameAccurateStartTime.IsValid())
		{
			Earliest = InRequest->FrameAccurateStartTime;
		}
	}
	Earliest.SetSequenceIndex(InRequest->TimestampSequenceIndex);

	bool bDoNotTruncateAtPresentationEnd = PlayerSessionService->GetOptionValue(OptionKeyDoNotTruncateAtPresentationEnd).SafeGetBool(false);
	FTimeValue Latest;
	if (bDoNotTruncateAtPresentationEnd)
	{
		Latest.SetToPositiveInfinity();
	}
	else
	{
		Latest.SetFromND(InRequest->Segment.MediaLocalLastAUTime - InRequest->Segment.PTO, InRequest->Segment.Timescale);
		Latest += TimeOffset;
	}
	Latest.SetSequenceIndex(InRequest->TimestampSequenceIndex);

	DownloadStats.bInsertedFillerData = SegmentDurationToGo > FTimeValue::GetZero();
	while(SegmentDurationToGo > FTimeValue::GetZero())
	{
		FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
		if (!AccessUnit)
		{
			break;
		}
		if (DefaultDuration > SegmentDurationToGo)
		{
			DefaultDuration = SegmentDurationToGo;
		}

		AccessUnit->ESType = InActiveTrackData->StreamType;
		AccessUnit->BufferSourceInfo = InActiveTrackData->BufferSourceInfo;
		AccessUnit->Duration = DefaultDuration;
		AccessUnit->AUSize = 0;
		AccessUnit->AUData = nullptr;
		AccessUnit->bIsDummyData = true;
		if (InActiveTrackData->CSD.IsValid() && InActiveTrackData->CSD->CodecSpecificData.Num())
		{
			AccessUnit->AUCodecData = InActiveTrackData->CSD;
		}

		// Set the sequence index member and update all timestamps with it as well.
		AccessUnit->SequenceIndex = InRequest->TimestampSequenceIndex;
		AccessUnit->DTS = NextExpectedDTS;
		AccessUnit->PTS = NextExpectedDTS;
		AccessUnit->EarliestPTS = Earliest;
		AccessUnit->LatestPTS = Latest;

		bool bIsLast = false;
		if (NextExpectedDTS > Latest)
		{
			bIsLast = AccessUnit->bIsLastInPeriod = true;
		}

		NextExpectedDTS += DefaultDuration;
		SegmentDurationToGo -= DefaultDuration;
		InActiveTrackData->AddAccessUnit(AccessUnit);
		FAccessUnit::Release(AccessUnit);
		AccessUnit = nullptr;

		if (bIsLast)
		{
			break;
		}
	}
}

void FStreamSegmentReaderCommon::FStreamHandler::CheckForInbandDASHEvents(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
	bool bHasInbandEvent = false;
	if (!InRequest->bIsEOSSegment)
	{
		for(int32 i=0; i<InRequest->Segment.InbandEventStreams.Num(); ++i)
		{
			const FSegmentInformationCommon::FInbandEventStream& ibs = InRequest->Segment.InbandEventStreams[i];
			if (ibs.SchemeIdUri.Equals(DASH::Schemes::ManifestEvents::Scheme_urn_mpeg_dash_event_2012))
			{
				bHasInbandEvent = true;
				break;
			}
		}
	}
	TSharedPtrTS<IPlaylistReader> ManifestReader = PlayerSessionService->GetManifestReader();
	IPlaylistReaderDASH* Reader = static_cast<IPlaylistReaderDASH*>(ManifestReader.Get());
	if (Reader)
	{
		Reader->SetStreamInbandEventUsage(InRequest->GetType(), bHasInbandEvent);
	}
}

void FStreamSegmentReaderCommon::FStreamHandler::HandleMP4EventMessagesLegacy(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest, const TSharedPtrTS<IParserISO14496_12>& InMP4Parser)
{
	if (InRequest->StreamingProtocol != FStreamSegmentRequestCommon::EStreamingProtocol::DASH)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
	// Are there 'emsg' boxes we need to handle?
	if (InMP4Parser->GetNumberOfEventMessages())
	{
		FTimeValue AbsPeriodStart = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
		// We may need the EPT from the 'sidx' if there is one.
		const IParserISO14496_12::ISegmentIndex* Sidx = InMP4Parser->GetSegmentIndexByIndex(0);
		for(int32 nEmsg=0, nEmsgs=InMP4Parser->GetNumberOfEventMessages(); nEmsg<nEmsgs; ++nEmsg)
		{
			const IParserISO14496_12::IEventMessage* Emsg = InMP4Parser->GetEventMessageByIndex(nEmsg);
			// This event must be described by an <InbandEventStream> in order to be processed.
			for(int32 i=0; i<InRequest->Segment.InbandEventStreams.Num(); ++i)
			{
				const FSegmentInformationCommon::FInbandEventStream& ibs = InRequest->Segment.InbandEventStreams[i];
				if (ibs.SchemeIdUri.Equals(Emsg->GetSchemeIdUri()) &&
					(ibs.Value.IsEmpty() || ibs.Value.Equals(Emsg->GetValue())))
				{
					TSharedPtrTS<DASH::FPlayerEvent> NewEvent = MakeSharedTS<DASH::FPlayerEvent>();
					NewEvent->SetOrigin(IAdaptiveStreamingPlayerAEMSEvent::EOrigin::InbandEventStream);
					NewEvent->SetSchemeIdUri(Emsg->GetSchemeIdUri());
					NewEvent->SetValue(Emsg->GetValue());
					NewEvent->SetID(LexToString(Emsg->GetID()));
					uint32 Timescale = Emsg->GetTimescale();
					uint32 Duration = Emsg->GetEventDuration();
					FTimeValue PTS;
					if (Emsg->GetVersion() == 0)
					{
						// Version 0 uses a presentation time delta relative to the EPT of the SIDX, if it exists, or if not
						// to the PTS of the first AU, which should be the same as the segment media start time.
						FTimeValue PTD((int64)Emsg->GetPresentationTimeDelta(), Timescale);
						FTimeValue EPT;
						if (Sidx)
						{
							EPT.SetFromND((int64)Sidx->GetEarliestPresentationTime(), Sidx->GetTimescale());
						}
						else
						{
							EPT.SetFromND((int64)InRequest->Segment.Time, InRequest->Segment.Timescale);
						}
						FTimeValue PTO(InRequest->Segment.PTO, InRequest->Segment.Timescale);
						PTS = AbsPeriodStart - PTO + EPT + PTD;
					}
					else if (Emsg->GetVersion() == 1)
					{
						FTimeValue EventTime(Emsg->GetPresentationTime(), Timescale);
						FTimeValue PTO(FTimeValue::GetZero());
						PTS = AbsPeriodStart - PTO + EventTime;
					}
					NewEvent->SetPresentationTime(PTS);
					if (Duration != 0xffffffff)
					{
						NewEvent->SetDuration(FTimeValue((int64)Duration, Timescale));
					}
					NewEvent->SetMessageData(Emsg->GetMessageData());
					NewEvent->SetPeriodID(InRequest->Period->GetUniqueIdentifier());
					// Add the event to the handler.
					if (PTS.IsValid())
					{
						/*
							Check that we have no seen this event in this segment already. This is for the case where the 'emsg' appears
							inbetween multiple 'moof' boxes. As per ISO/IEC 23009-1:2019 Section 5.10.3.3.1 General:
								A Media Segment if based on the ISO BMFF container may contain one or more event message ('emsg') boxes. If present, any 'emsg' box shall be placed as follows:
								- It may be placed before the first 'moof' box of the segment.
								- It may be placed in between any 'mdat' and 'moof' box. In this case, an equivalent 'emsg' with the same id value shall be present before the first 'moof' box of any Segment.
						*/
						int32 EventIdx = SegmentEventsFound.IndexOfByPredicate([&NewEvent](const TSharedPtrTS<DASH::FPlayerEvent>& This) {
							return NewEvent->GetSchemeIdUri().Equals(This->GetSchemeIdUri()) &&
								   NewEvent->GetID().Equals(This->GetID()) &&
								   (NewEvent->GetValue().IsEmpty() || NewEvent->GetValue().Equals(This->GetValue()));
						});
						if (EventIdx == INDEX_NONE)
						{
							PlayerSessionService->GetAEMSEventHandler()->AddEvent(NewEvent, InRequest->Period->GetUniqueIdentifier(), IAdaptiveStreamingPlayerAEMSHandler::EEventAddMode::AddIfNotExists);
							SegmentEventsFound.Emplace(MoveTemp(NewEvent));
						}
					}
					break;
				}
			}
		}
	}
}

void FStreamSegmentReaderCommon::FStreamHandler::HandleMP4MetadataLegacy(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest, const TSharedPtrTS<IParserISO14496_12>& InMP4Parser, const TSharedPtrTS<const IParserISO14496_12>& InMP4InitSegment, const FTimeValue& InBaseTime)
{
	// Get the metadata from the movie fragment or the init segment.
	const IParserISO14496_12::IMetadata* MoofMetadata = InMP4Parser->GetMetadata(IParserISO14496_12::EBaseBoxType::Moof);
	const IParserISO14496_12::IMetadata* MoovMetadata = !MoofMetadata ? InMP4InitSegment.IsValid() ? InMP4InitSegment->GetMetadata(IParserISO14496_12::EBaseBoxType::Moov) : nullptr : nullptr;
	if (MoofMetadata || MoovMetadata)
	{
		const IParserISO14496_12::IMetadata* md = MoofMetadata ? MoofMetadata : MoovMetadata;
		uint32 hdlr = md->GetHandler();
		uint32 res0 = md->GetReserved(0);
		TArray<MP4Utilities::FMP4BoxInfo> Boxes;
		for(int32 i=0, iMax=md->GetNumChildBoxes(); i<iMax; ++i)
		{
			Boxes.Emplace(MP4Utilities::FMP4BoxInfo({.Data = MakeConstArrayView<uint8>(reinterpret_cast<const uint8*>(md->GetChildBoxData(i)), md->GetChildBoxDataSize(i)), .Type=md->GetChildBoxType(i)}));
		}
		TSharedPtr<MP4Utilities::FMP4MetadataParser, ESPMode::ThreadSafe> MediaMetadata = MakeShared<MP4Utilities::FMP4MetadataParser, ESPMode::ThreadSafe>();
		if (MediaMetadata->Parse(hdlr, res0, Boxes) == FMetadataParser::EResult::Success)
		{
			FTimeValue StartTime = md == MoofMetadata ? InRequest->GetFirstPTS() : InBaseTime;
			StartTime.SetSequenceIndex(InRequest->TimestampSequenceIndex);
  			PlayerSessionService->SendMessageToPlayer(FPlaylistMetadataUpdateMessage::Create(StartTime, MediaMetadata, false));
		}
	}
}

void FStreamSegmentReaderCommon::FStreamHandler::HandleMP4Metadata(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest, const TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe>& InMoovMetaBox, const TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe>& InMoofMetaBox)
{
	if (!InMoovMetaBox && !InMoofMetaBox)
	{
		return;
	}
	const TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe>& MetaBox = InMoofMetaBox ? InMoofMetaBox : InMoovMetaBox;
	if (auto HdlrBox = MetaBox->GetHandler())
	{
		uint32 hdlr = HdlrBox->GetHandlerType();
		uint32 res0 = HdlrBox->GetReservedValue(0);

		TArray<MP4Utilities::FMP4BoxInfo> Boxes;
		for(auto &ch : MetaBox->GetChildren())
		{
			Boxes.Emplace(ch->GetBoxInfo());
		}
		TSharedPtr<MP4Utilities::FMP4MetadataParser, ESPMode::ThreadSafe> MediaMetadata = MakeShared<MP4Utilities::FMP4MetadataParser, ESPMode::ThreadSafe>();
		if (MediaMetadata->Parse(hdlr, res0, Boxes) == FMetadataParser::EResult::Success)
		{
			FTimeValue StartTime = InRequest->GetFirstPTS();
			StartTime.SetSequenceIndex(InRequest->TimestampSequenceIndex);
  			PlayerSessionService->SendMessageToPlayer(FPlaylistMetadataUpdateMessage::Create(StartTime, MediaMetadata, false));
		}
	}
}

void FStreamSegmentReaderCommon::FStreamHandler::HandleMP4EventMessages(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest, const TSharedPtr<MP4Boxes::FMP4BoxSIDX, ESPMode::ThreadSafe>& InSidxBox, const TArray<TSharedPtr<MP4Boxes::FMP4BoxEMSG, ESPMode::ThreadSafe>>& InEventMessageBoxes)
{
	if (InRequest->StreamingProtocol != FStreamSegmentRequestCommon::EStreamingProtocol::DASH || InEventMessageBoxes.IsEmpty())
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
	// Are there `emsg` boxes we need to handle?
	FTimeValue AbsPeriodStart = InRequest->PeriodStart + InRequest->AST + InRequest->AdditionalAdjustmentTime;
	// We may need the EPT from the `sidx` if there is one.
	for(int32 nEmsg=0, nEmsgs=InEventMessageBoxes.Num(); nEmsg<nEmsgs; ++nEmsg)
	{
		const TSharedPtr<MP4Boxes::FMP4BoxEMSG, ESPMode::ThreadSafe>& Emsg = InEventMessageBoxes[nEmsg];
		// This event must be described by an <InbandEventStream> in order to be processed.
		for(int32 i=0; i<InRequest->Segment.InbandEventStreams.Num(); ++i)
		{
			const FSegmentInformationCommon::FInbandEventStream& ibs = InRequest->Segment.InbandEventStreams[i];
			if (ibs.SchemeIdUri.Equals(Emsg->GetSchemeIdUri()) &&
				(ibs.Value.IsEmpty() || ibs.Value.Equals(Emsg->GetValue())))
			{
				TSharedPtrTS<DASH::FPlayerEvent> NewEvent = MakeSharedTS<DASH::FPlayerEvent>();
				NewEvent->SetOrigin(IAdaptiveStreamingPlayerAEMSEvent::EOrigin::InbandEventStream);
				NewEvent->SetSchemeIdUri(Emsg->GetSchemeIdUri());
				NewEvent->SetValue(Emsg->GetValue());
				NewEvent->SetID(LexToString(Emsg->GetID()));
				uint32 Timescale = Emsg->GetTimescale();
				uint32 Duration = Emsg->GetEventDuration();
				FTimeValue PTS;
				if (Emsg->GetVersion() == 0)
				{
					// Version 0 uses a presentation time delta relative to the EPT of the SIDX, if it exists, or if not
					// to the PTS of the first AU, which should be the same as the segment media start time.
					FTimeValue PTD((int64)Emsg->GetPresentationTimeDelta(), Timescale);
					FTimeValue EPT;
					if (InSidxBox)
					{
						EPT.SetFromND((int64)InSidxBox->GetEarliestPresentationTime(), InSidxBox->GetTimescale());
					}
					else
					{
						EPT.SetFromND((int64)InRequest->Segment.Time, InRequest->Segment.Timescale);
					}
					FTimeValue PTO(InRequest->Segment.PTO, InRequest->Segment.Timescale);
					PTS = AbsPeriodStart - PTO + EPT + PTD;
				}
				else if (Emsg->GetVersion() == 1)
				{
					FTimeValue EventTime(Emsg->GetPresentationTime(), Timescale);
					FTimeValue PTO(FTimeValue::GetZero());
					PTS = AbsPeriodStart - PTO + EventTime;
				}
				NewEvent->SetPresentationTime(PTS);
				if (Duration != 0xffffffff)
				{
					NewEvent->SetDuration(FTimeValue((int64)Duration, Timescale));
				}
				NewEvent->SetMessageData(Emsg->GetMessageData());
				NewEvent->SetPeriodID(InRequest->Period->GetUniqueIdentifier());
				// Add the event to the handler.
				if (PTS.IsValid())
				{
					/*
						Check that we have no seen this event in this segment already. This is for the case where the 'emsg' appears
						inbetween multiple 'moof' boxes. As per ISO/IEC 23009-1:2019 Section 5.10.3.3.1 General:
							A Media Segment if based on the ISO BMFF container may contain one or more event message ('emsg') boxes. If present, any 'emsg' box shall be placed as follows:
							- It may be placed before the first 'moof' box of the segment.
							- It may be placed in between any 'mdat' and 'moof' box. In this case, an equivalent 'emsg' with the same id value shall be present before the first 'moof' box of any Segment.
					*/
					int32 EventIdx = SegmentEventsFound.IndexOfByPredicate([&NewEvent](const TSharedPtrTS<DASH::FPlayerEvent>& This) {
						return NewEvent->GetSchemeIdUri().Equals(This->GetSchemeIdUri()) &&
								NewEvent->GetID().Equals(This->GetID()) &&
								(NewEvent->GetValue().IsEmpty() || NewEvent->GetValue().Equals(This->GetValue()));
					});
					if (EventIdx == INDEX_NONE)
					{
						PlayerSessionService->GetAEMSEventHandler()->AddEvent(NewEvent, InRequest->Period->GetUniqueIdentifier(), IAdaptiveStreamingPlayerAEMSHandler::EEventAddMode::AddIfNotExists);
						SegmentEventsFound.Emplace(MoveTemp(NewEvent));
					}
				}
				break;
			}
		}
	}
}




FStreamSegmentReaderCommon::FStreamHandler::EEmitResult FStreamSegmentReaderCommon::FStreamHandler::EmitSamples(EEmitType InEmitType)
{
	EEmitResult Result = EEmitResult::SentEverything;
	if (!CurrentlyActiveTrackData.IsValid())
	{
		return Result;
	}

	// Note: Here we do NOT check for `HasReadBeenAborted()` because we have to send all AUs we have accumulated
	//       so far, even if the ABR asked to abort. Only if it is a real cancellation do we not deliver data.
	auto HasBeenCanceled = [&]() -> bool
	{
		return bTerminate || bRequestCanceled;
	};

	while(CurrentlyActiveTrackData->AccessUnitFIFO.Num() && !HasBeenCanceled())
	{
		if (CurrentlyActiveTrackData->bNeedToRecalculateDurations)
		{
			// Need to have a certain amount of upcoming samples to be able to
			// (more or less) safely calculate timestamp differences.
			// NOTE: min to check depends on codec and B frame distance
			int32 NumToCheck = !CurrentlyActiveTrackData->bGotAllSamples ? 10 : CurrentlyActiveTrackData->AccessUnitFIFO.Num();
			if (CurrentlyActiveTrackData->AccessUnitFIFO.Num() < NumToCheck)
			{
				break;
			}
			// Locate the sample in the time-sorted list
			for(int32 i=0; i<CurrentlyActiveTrackData->SortedAccessUnitFIFO.Num(); ++i)
			{
				if (CurrentlyActiveTrackData->SortedAccessUnitFIFO[i].PTS == CurrentlyActiveTrackData->AccessUnitFIFO[0].PTS)
				{
					if (i < CurrentlyActiveTrackData->SortedAccessUnitFIFO.Num()-1)
					{
						CurrentlyActiveTrackData->AccessUnitFIFO[0].AU->Duration = CurrentlyActiveTrackData->SortedAccessUnitFIFO[i+1].PTS - CurrentlyActiveTrackData->SortedAccessUnitFIFO[i].PTS;
						if (!CurrentlyActiveTrackData->AverageDuration.IsValid() || CurrentlyActiveTrackData->AverageDuration.IsZero())
						{
							CurrentlyActiveTrackData->AverageDuration = CurrentlyActiveTrackData->AccessUnitFIFO[0].AU->Duration;
						}
					}
					CurrentlyActiveTrackData->SortedAccessUnitFIFO[i].Release();
					break;
				}
			}
			// Reduce the sorted list
			for(int32 i=0; i<CurrentlyActiveTrackData->SortedAccessUnitFIFO.Num(); ++i)
			{
				if (CurrentlyActiveTrackData->SortedAccessUnitFIFO[i].AU)
				{
					if (i)
					{
						CurrentlyActiveTrackData->SortedAccessUnitFIFO.RemoveAt(0, i);
					}
					break;
				}
			}
		}

		FAccessUnit* pNext = CurrentlyActiveTrackData->AccessUnitFIFO[0].AU;
		// Check if this is the last access unit in the requested time range.
		if (!CurrentlyActiveTrackData->bTaggedLastSample && pNext->PTS >= pNext->LatestPTS)
		{
			/*
				Because of B frames the last frame that must be decoded could actually be
				a later frame in decode order.
				Suppose the sequence IPBB with timestamps 0,3,1,2 respectively. Even though the
				P frame with timestamp 3 is "the last" one in presentation order, it will enter
				the decoder before the B frames.
				As such we need to tag the last B frame (2) as "the last one" even though its timestamp
				is before the last time requested.
				This would be easy if we had access to reliable DTS, but Matroska files only provide PTS.
				Note: This may seem superfluous since we are tagging as "last" which happens to be the
				      actual last element in the list, but there could really be even later frames in
				      the list that we will then remove to avoid sending frames into the decoder that
				      will be discarded after decoding, which is a waste of decode cycles.
			*/

			const FTimeValue NextPTS(pNext->PTS);
			// Sort the remaining access units by ascending PTS
			CurrentlyActiveTrackData->AccessUnitFIFO.Sort([](const FActiveTrackData::FSample& a, const FActiveTrackData::FSample& b){return a.PTS < b.PTS;});
			// Go backwards over the list and drop all access units that _follow_ the next one.
			for(int32 i=CurrentlyActiveTrackData->AccessUnitFIFO.Num()-1; i>0; --i)
			{
				if (CurrentlyActiveTrackData->AccessUnitFIFO[i].PTS > NextPTS)
				{
					for(int32 j=0; j<CurrentlyActiveTrackData->SortedAccessUnitFIFO.Num(); ++j)
					{
						if (CurrentlyActiveTrackData->SortedAccessUnitFIFO[j].PTS == CurrentlyActiveTrackData->AccessUnitFIFO[i].PTS)
						{
							CurrentlyActiveTrackData->SortedAccessUnitFIFO.RemoveAt(j);
							break;
						}
					}
					CurrentlyActiveTrackData->AccessUnitFIFO.RemoveAt(i);
				}
				else
				{
					break;
				}
			}
			// Sort the list back to index order.
			CurrentlyActiveTrackData->AccessUnitFIFO.Sort([](const FActiveTrackData::FSample& a, const FActiveTrackData::FSample& b){return a.SequentialIndex < b.SequentialIndex;});
			// Whichever element is the last in the list now is the one that needs to be tagged as such.
			CurrentlyActiveTrackData->AccessUnitFIFO.Last().AU->bIsLastInPeriod = true;
			CurrentlyActiveTrackData->bReadPastLastPTS = true;
			CurrentlyActiveTrackData->bTaggedLastSample = true;

			check(pNext == CurrentlyActiveTrackData->AccessUnitFIFO[0].AU);
			pNext = CurrentlyActiveTrackData->AccessUnitFIFO[0].AU;
		}

		while(!HasBeenCanceled())
		{
			if (Parameters.EventListener->OnFragmentAccessUnitReceived(pNext))
			{
				CurrentlyActiveTrackData->DurationSuccessfullyDelivered += pNext->Duration;
				CurrentlyActiveTrackData->AccessUnitFIFO[0].AU = nullptr;
				CurrentlyActiveTrackData->AccessUnitFIFO.RemoveAt(0);
				break;
			}
			// If emitting as much as we can we leave this loop now that the receiver is blocked.
			else if (InEmitType == EEmitType::UntilBlocked)
			{

				Result = EEmitResult::HaveRemaining;
				return Result;
			}
			else
			{
				FMediaRunnable::SleepMilliseconds(100);
			}
		}
	}
	return Result;
}



/**
 * Read n bytes of data into the provided buffer.
 *
 * Reading must return the number of bytes asked to get, if necessary by blocking.
 * If a read error prevents reading the number of bytes -1 must be returned.
 *
 * @param IntoBuffer Buffer into which to store the data bytes. If nullptr is passed the data must be skipped over.
 * @param NumBytesToRead The number of bytes to read. Must not read more bytes and no less than requested.
 * @return The number of bytes read or -1 on a read error.
 */
int64 FStreamSegmentReaderCommon::FStreamHandler::ReadData(void* IntoBuffer, int64 NumBytesToRead, int64 InFromOffset)
{
	// Make sure the buffer will have the amount of data we need.
	while(1)
	{
		// Check if a HTTP reader progress event fired in the meantime.
		if (ProgressReportCount && PrimaryTrackData.IsValid())
		{
			ProgressReportCount = 0;
			MetricUpdateLock.Lock();
			Metrics::FSegmentDownloadStats currentDownloadStats = DownloadStats;
			MetricUpdateLock.Unlock();
			currentDownloadStats.DurationDelivered = PrimaryTrackData->DurationSuccessfullyDelivered.GetAsSeconds();
			currentDownloadStats.DurationDownloaded = PrimaryTrackData->DurationSuccessfullyRead.GetAsSeconds();
			currentDownloadStats.TimeToDownload = (MEDIAutcTime::Current() - CurrentConnectionInfo.RequestStartTime).GetAsSeconds();
			FABRDownloadProgressDecision StreamSelectorDecision = StreamSelector->ReportDownloadProgress(currentDownloadStats);

			if ((StreamSelectorDecision.Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_EmitPartialData) != 0)
			{
				SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
				CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
				bAllowEarlyEmitting = true;
				// Deliver all enqueued AUs right now. Unless the request also gets aborted we could be stuck
				// in here for a while longer.
				EmitSamples(EEmitType::UntilBlocked);
			}
			if ((StreamSelectorDecision.Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_InsertFillerData) != 0)
			{
				bFillRemainingDuration = true;
			}
			if ((StreamSelectorDecision.Flags & FABRDownloadProgressDecision::EDecisionFlags::eABR_AbortDownload) != 0)
			{
				// When aborted and early emitting did place something into the buffers we need to fill
				// the remainder no matter what.
				if (PrimaryTrackData->DurationSuccessfullyDelivered > FTimeValue::GetZero())
				{
					bFillRemainingDuration = true;
				}
				ABRAbortReason = StreamSelectorDecision.Reason;
				bAbortedByABR = true;
				return -1;
			}
		}

		if (!ReadBuffer.WaitUntilSizeAvailable(ReadBuffer.ParsePos + NumBytesToRead, 1000 * 100))
		{
			if (HasErrored() || HasReadBeenAborted() || ReadBuffer.WasAborted())
			{
				return -1;
			}
		}
		else
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_Common_StreamReader);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, Common_StreamReader);
			FScopeLock lock(ReadBuffer.GetLock());
			if (ReadBuffer.GetLinearReadSize() >= ReadBuffer.ParsePos + NumBytesToRead)
			{
				if (IntoBuffer)
				{
					FMemory::Memcpy(IntoBuffer, ReadBuffer.GetLinearReadData() + ReadBuffer.ParsePos, NumBytesToRead);
				}
				lock.Unlock();
				ReadBuffer.ParsePos += NumBytesToRead;
				return NumBytesToRead;
			}
			else
			{
				// Return 0 at EOF and -1 on error.
				lock.Unlock();
				return HasErrored() ? -1 : 0;
			}
		}
	}
}

/**
 * Checks if the data source has reached the End Of File (EOF) and cannot provide any additional data.
 *
 * @return If EOF has been reached returns true, otherwise false.
 */
bool FStreamSegmentReaderCommon::FStreamHandler::HasReachedEOF() const
{
	return !HasErrored() && ReadBuffer.GetEOD() && ReadBuffer.ParsePos >= ReadBuffer.GetLinearReadSize();
}

/**
 * Checks if reading of the file and therefor parsing has been aborted.
 *
 * @return true if reading/parsing has been aborted, false otherwise.
 */
bool FStreamSegmentReaderCommon::FStreamHandler::HasReadBeenAborted() const
{
	return bTerminate || bRequestCanceled || bAbortedByABR;
}

/**
 * Returns the current read offset.
 *
 * The first read offset is not necessarily zero. It could be anywhere inside the source.
 *
 * @return The current byte offset in the source.
 */
int64 FStreamSegmentReaderCommon::FStreamHandler::GetCurrentOffset() const
{
	return ReadBuffer.ParsePos;
}

int64 FStreamSegmentReaderCommon::FStreamHandler::GetTotalSize() const
{
	return HTTPRequest.IsValid() && HTTPRequest->ConnectionInfo.ContentLength > 0 ? HTTPRequest->ConnectionInfo.ContentLength : TNumericLimits<int64>::Max();
}

bool FStreamSegmentReaderCommon::FStreamHandler::FReadBuffer::WaitUntilSizeAvailable(int64 SizeNeeded, int32 TimeoutMicroseconds)
{
	if (!bIsEncrypted || SizeNeeded <= 0)
	{
		return ReceiveBuffer->WaitUntilSizeAvailable(SizeNeeded, TimeoutMicroseconds);
	}
	else
	{
		if (!bDecrypterReady)
		{
			if (BlockDecrypter->GetState() == ElectraCDM::ECDMState::WaitingForKey || BlockDecrypter->GetState() == ElectraCDM::ECDMState::Idle)
			{
				return false;
			}
			bIsDecrypterGood = BlockDecrypter->GetState() == ElectraCDM::ECDMState::Ready;
			if (bIsDecrypterGood && !BlockDecrypterHandle)
			{
				ElectraCDM::FMediaCDMSampleInfo si;
				si.IV = BlockDecrypterIV;
				si.DefaultKID = BlockDecrypterKID;
				ElectraCDM::ECDMError err = BlockDecrypter->BlockStreamDecryptStart(BlockDecrypterHandle, si);
				if (err != ElectraCDM::ECDMError::Success)
				{
					bIsDecrypterGood = false;
				}
				BlockDecrypterBlockSize = BlockDecrypterHandle ? BlockDecrypterHandle->BlockSize : 0;
				if (BlockDecrypterBlockSize <= 0)
				{
					bIsDecrypterGood = false;
				}
			}
			bDecrypterReady = true;
		}
		if (!bIsDecrypterGood)
		{
			DecryptedDataBuffer.Abort();
			DecryptedDataBuffer.SetHasErrored();
			return true;
		}

		// Due to the nature of a block cipher we may need additional bytes prior to decrypting.
		int64 Needed = Align(SizeNeeded, BlockDecrypterBlockSize);
		if (!ReceiveBuffer->WaitUntilSizeAvailable(Needed, TimeoutMicroseconds))
		{
			return false;
		}
		// Propagate error and abort states.
		if (ReceiveBuffer->HasErrored())
		{
			DecryptedDataBuffer.SetHasErrored();
			return true;
		}
		if (ReceiveBuffer->WasAborted())
		{
			DecryptedDataBuffer.Abort();
			return true;
		}
		// Have all data decrypted already?
		if (DecryptedDataBuffer.GetEOD() && !BlockDecrypterHandle)
		{
			return true;
		}

		// Decrypt
		FScopeLock lock(ReceiveBuffer->GetLock());
		int64 SourceSizeAvail = ReceiveBuffer->GetLinearReadSize();
		bool bIsSourceAtEOS = ReceiveBuffer->GetEOD();
		if (SourceSizeAvail >= Needed)
		{
			int64 DecryptedSizeAvail = DecryptedDataBuffer.GetLinearReadSize();
			int64 NumNew = Needed - DecryptedSizeAvail;
			DecryptedDataBuffer.EnlargeTo(SourceSizeAvail);
			if (NumNew > 0 && BlockDecrypterHandle)
			{
				uint8* NewDataToDecryptPtr = DecryptedDataBuffer.GetLinearWriteData(NumNew);
				FMemory::Memcpy(NewDataToDecryptPtr, ReceiveBuffer->GetLinearReadData() + DecryptedSizeAvail, NumNew);
				lock.Unlock();
				bool bIsLastBlock = bIsSourceAtEOS && Needed >= SourceSizeAvail;
				int32 NumDecrypted = 0;
				ElectraCDM::ECDMError Result = BlockDecrypter->BlockStreamDecryptInPlace(BlockDecrypterHandle, NumDecrypted, NewDataToDecryptPtr, (int32)NumNew, bIsLastBlock);
				DecryptedDataBuffer.AppendedNewData(NumDecrypted);
				check(Result == ElectraCDM::ECDMError::Success);
				check(bIsLastBlock || NumDecrypted == NumNew);
				if (bIsLastBlock)
				{
					DecryptedDataBuffer.SetEOD();
					Result = BlockDecrypter->BlockStreamDecryptEnd(BlockDecrypterHandle);
					BlockDecrypterHandle = nullptr;
				}
			}
		}
		else
		{
			// Must have reached EOS.
			DecryptedDataBuffer.SetEOD();
			BlockDecrypter->BlockStreamDecryptEnd(BlockDecrypterHandle);
			BlockDecrypterHandle = nullptr;
		}
		return true;
	}
}

FCriticalSection* FStreamSegmentReaderCommon::FStreamHandler::FReadBuffer::GetLock()
{
	check(ReceiveBuffer.IsValid());
	return !bIsEncrypted ? ReceiveBuffer->GetLock() : DecryptedDataBuffer.GetLock();
}

const uint8* FStreamSegmentReaderCommon::FStreamHandler::FReadBuffer::GetLinearReadData() const
{
	check(ReceiveBuffer.IsValid());
	return !bIsEncrypted ? ReceiveBuffer->GetLinearReadData() : DecryptedDataBuffer.GetLinearReadData();
}

uint8* FStreamSegmentReaderCommon::FStreamHandler::FReadBuffer::GetLinearReadData()
{
	check(ReceiveBuffer.IsValid());
	return !bIsEncrypted ? ReceiveBuffer->GetLinearReadData() : DecryptedDataBuffer.GetLinearReadData();
}

int64 FStreamSegmentReaderCommon::FStreamHandler::FReadBuffer::GetLinearReadSize() const
{
	check(ReceiveBuffer.IsValid());
	return !bIsEncrypted ? ReceiveBuffer->GetLinearReadSize() : DecryptedDataBuffer.GetLinearReadSize();
}

bool FStreamSegmentReaderCommon::FStreamHandler::FReadBuffer::GetEOD() const
{
	check(ReceiveBuffer.IsValid());
	return !bIsEncrypted ? ReceiveBuffer->GetEOD() : DecryptedDataBuffer.GetEOD();
}

bool FStreamSegmentReaderCommon::FStreamHandler::FReadBuffer::WasAborted() const
{
	check(ReceiveBuffer.IsValid());
	return !bIsEncrypted ? ReceiveBuffer->WasAborted() : DecryptedDataBuffer.WasAborted();
}


} // namespace Electra
