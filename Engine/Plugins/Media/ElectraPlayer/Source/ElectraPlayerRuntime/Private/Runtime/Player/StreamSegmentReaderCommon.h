// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Player/PlayerStreamReader.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/DASH/PlayerEventDASH.h"
#include "MP4Boxes.h"
#include "MP4Utilities.h"
#include "Utilities/MP4Helpers.h"
#include "Utils/MPEG/ElectraUtilsMP4.h"
#include "Demuxer/ParserISO14496-12.h"
#include "Demuxer/ParserMKV.h"
#include "HTTP/HTTPManager.h"
#include "Player/DRM/DRMManager.h"
#include "Misc/TVariant.h"
#include "HAL/IConsoleManager.h"



namespace Electra
{
namespace Stream
{
extern TAutoConsoleVariable<bool> CVarElectraStreamUseLegacyMP4;
}

struct FMediaPlaylistAndStateHLS;

class FParsedMOOVCommon
{
public:
	virtual ~FParsedMOOVCommon() = default;

	struct FTrackInfo
	{
		TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe> TrakBox;
		FCodecTypeFormat CodecFormat;
		FDRMTypeFormat DrmFormat;
		FDecoderInformation DecoderInformation;
		uint32 TrackID = 0;
		uint32 TrackTimescale = 0;
	};

	bool Parse(TConstArrayView<uint8> InInitSegmentData)
	{
		TSharedPtr<MP4Boxes::FMP4BoxMOOV, ESPMode::ThreadSafe> MOOV;
		TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> RootBoxes;
		if (MP4Utilities::FMP4BoxLocator::LocateAndReadRootBoxesFromBuffer(RootBoxes, InInitSegmentData))
		{
			TArray<TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>> ParsedRootBoxes;
			ParsedRootBoxes.SetNum(RootBoxes.Num());
			for(int32 i=0; i<RootBoxes.Num(); ++i)
			{
				MP4Boxes::FMP4BoxTreeParser tp;
				if (tp.ParseBoxTree(RootBoxes[i]))
				{
					ParsedRootBoxes[i] = tp.GetBoxTree();
					if (ParsedRootBoxes[i]->GetType() == MP4Utilities::MakeBoxAtom('m','o','o','v'))
					{
						MOOV = StaticCastSharedPtr<MP4Boxes::FMP4BoxMOOV>(ParsedRootBoxes[i]);
						break;
					}
				}
				else
				{
					return false;
				}
			}
			if (MOOV)
			{
				return Parse(MOOV);
			}
		}
		return false;
	}

	bool Parse(TSharedPtr<MP4Boxes::FMP4BoxMOOV, ESPMode::ThreadSafe> InMOOV)
	{
		if (!InMOOV)
		{
			return false;
		}
		MoovBox = InMOOV;
		MvexBox = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxMVEX>(MP4Utilities::MakeBoxAtom('m','v','e','x'), 0);
		if (!MvexBox)
		{
			return false;
		}
		auto Udta = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxUDTA>(MP4Utilities::MakeBoxAtom('u','d','t','a'), 0);
		MetaBox = Udta ? Udta->FindBoxRecursive<MP4Boxes::FMP4BoxMETA>(MP4Utilities::MakeBoxAtom('m','e','t','a'), 0) : nullptr;
		TArray<TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>> AllTracks;
		MoovBox->GetAllBoxInstances(AllTracks, MP4Utilities::MakeBoxAtom('t','r','a','k'));
		for(int32 nTk=0; nTk<AllTracks.Num(); ++nTk)
		{
			auto Tkhd = AllTracks[nTk]->FindBoxRecursive<MP4Boxes::FMP4BoxTKHD>(MP4Utilities::MakeBoxAtom('t','k','h','d'), 0);
			auto Mdhd = AllTracks[nTk]->FindBoxRecursive<MP4Boxes::FMP4BoxMDHD>(MP4Utilities::MakeBoxAtom('m','d','h','d'), 1);
			if (!Tkhd || !Mdhd)
			{
				continue;
			}
			FTrackInfo ti;
			ti.TrakBox = AllTracks[nTk];
			ti.TrackID = Tkhd->GetTrackID();
			ti.TrackTimescale = Mdhd->GetTimescale();

			FString Msg;
			if (!ElectraDecodersUtil::MP4::GetTrackFormatInfo(Msg, ti.CodecFormat, ti.DrmFormat, ti.DecoderInformation, AllTracks[nTk], ti.TrackID, true))
			{
				continue;
			}
			if (!ti.DecoderInformation.bIsDecodable)
			{
				continue;
			}
			if (ti.CodecFormat.Type != FCodecTypeFormat::EType::Video && ti.CodecFormat.Type != FCodecTypeFormat::EType::Audio && ti.CodecFormat.Type != FCodecTypeFormat::EType::Subtitle)
			{
				continue;
			}
			TrackMap.Add(ti.TrackID, ti);
		}
		MoovBox->GetAllBoxInstances(PsshBoxes, MP4Utilities::MakeBoxAtom('p','s','s','h'));
		return true;
	}

	const TMap<uint32, FTrackInfo>& GetTrackMap() const
	{ return TrackMap; }

	TSharedPtr<MP4Boxes::FMP4BoxMVEX, ESPMode::ThreadSafe> GetMVEXBox() const
	{ return MvexBox; }

	TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe> GetMETABox() const
	{ return MetaBox; }
	const TArray<TSharedPtr<MP4Boxes::FMP4BoxPSSH, ESPMode::ThreadSafe>>& GetPSSHBoxes() const
	{ return PsshBoxes; }
private:
	TSharedPtr<MP4Boxes::FMP4BoxMOOV, ESPMode::ThreadSafe> MoovBox;
	TSharedPtr<MP4Boxes::FMP4BoxMVEX, ESPMode::ThreadSafe> MvexBox;
	TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe> MetaBox;
	TArray<TSharedPtr<MP4Boxes::FMP4BoxPSSH, ESPMode::ThreadSafe>> PsshBoxes;
	TMap<uint32, FTrackInfo> TrackMap;
};



struct FProducerReferenceTimeInfo : public IProducerReferenceTimeInfo
{
	FTimeValue WallclockTime;
	uint64 PresentationTime = 0;
	uint32 ID = 0;
	EType Type = EType::Encoder;
	bool bInband = false;
	FTimeValue GetWallclockTime() const override
	{ return WallclockTime; }
	uint64 GetPresentationTime() const override
	{ return PresentationTime; }
	uint32 GetID() const override
	{ return ID; }
	EType GetType() const override
	{ return Type; }
	bool GetIsInband() const override
	{ return bInband; }
};

struct FSegmentInformationCommon
{
	struct FURL
	{
		FMediaURL Url;
		FString Range;
		FString CustomHeader;
		int64 SteeringID = 0;
	};
	struct FInbandEventStream
	{
		FString SchemeIdUri;
		FString Value;
		int64 PTO = 0;
		uint32 Timescale = 0;
	};
	enum class EContainerType
	{
		ISO14496_12,
		ISO13818_1,
		Matroska
	};
	struct FCMCDNextSegNumberTimes
	{
		int64 Number = -1;
		int64 Time = -1;
	};
	FURL InitializationURL;
	FURL MediaURL;
	TArray<FURL> NextMediaURLS;
	TArray<FCMCDNextSegNumberTimes> NextMediaCMCDNumberTimes;
	FTimeValue ATO;
	int64 Time = 0;								// Time value T in timescale units
	int64 PTO = 0;								// PresentationTimeOffset
	int64 EPTdelta = 0;
	int64 Duration = 0;							// Duration of the segment. Not necessarily exact if <SegmentTemplate> is used).
	int64 Number = 0;							// Index of the segment.
	int64 SubIndex = 0;							// Subsegment index
	int64 MediaLocalFirstAUTime = 0;			// Time of the first AU to use in this segment in media local time
	int64 MediaLocalLastAUTime = 0;				// Time at which the last AU to use in thie segment ends in media local time
	int64 MediaLocalFirstPTS = 0;
	uint32 Timescale = 0;						// Local media timescale
	bool bLowLatencyChunkedEncodingExpected = false;
	bool bFrameAccuracyRequired = false;		// true if the segment was located for frame accurate seeking.
	bool bIsSideload = false;					// true if this is a side-loaded resource to be fetched and cached.
	bool bIsLastInPeriod = false;				// true if known to be the last segment in the period.
	bool bMayBeMissing = false;					// true if the last segment in <SegmentTemplate> that might not exist.
	TArray<FInbandEventStream> InbandEventStreams;
	TArray<FProducerReferenceTimeInfo> ProducerReferenceTimeInfos;
	int64 MeasureLatencyViaReferenceTimeInfoID = -1;

	// Out
	bool bIsMissing = false;					// Set to true if known to be missing.
	bool bSawLMSG = false;						// Will be set to true by the stream reader if the 'lmsg' brand was found.

	// Misc
	EContainerType ContainerType = EContainerType::ISO14496_12;
	int64 NumberOfBytes = 0;
	int64 FirstByteOffset = 0;

	FTimeValue CalculateASAST(const FTimeValue& AST, const FTimeValue& PeriodStart, bool bIsStatic)
	{
		if (bIsStatic)
		{
			return AST;
		}
		else
		{
			if (ATO < FTimeValue::GetPositiveInfinity())
			{
				return AST + PeriodStart + FTimeValue(Time - PTO - EPTdelta + Duration, Timescale) - ATO;
			}
			// ATO of infinity means the segment is always available, so we return zero time as earliest UTC time.
			return FTimeValue::GetZero();
		}
	}
	FTimeValue CalculateSAET(const FTimeValue& AST, const FTimeValue& PeriodStart, const FTimeValue& MPDAET, const FTimeValue& TSB, bool bIsStatic)
	{
		if (bIsStatic)
		{
			// If the MPD has a global availabilityEndTime then that is the end time of the segment as well.
			if (MPDAET.IsValid())
			{
				return MPDAET;
			}
			return FTimeValue::GetPositiveInfinity();
		}
		else
		{
			// If the MPD has a global availabilityEndTime then that is the end time of the segment as well.
			if (MPDAET.IsValid())
			{
				return MPDAET;
			}
			return AST + PeriodStart + FTimeValue(Time - PTO - EPTdelta + Duration * 2, Timescale) + (TSB.IsValid() ? TSB : FTimeValue::GetPositiveInfinity());
		}
	}
};


class FStreamSegmentRequestCommon : public IStreamSegment
{
public:
	FStreamSegmentRequestCommon();
	virtual ~FStreamSegmentRequestCommon();

	void SetPlaybackSequenceID(uint32 PlaybackSequenceID) override;
	uint32 GetPlaybackSequenceID() const override;

	void SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay) override;
	FTimeValue GetExecuteAtUTCTime() const override;

	EStreamType GetType() const override;

	void GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const override;
	void GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams) override;
	void GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams) override;

	FTimeValue GetFirstPTS() const override;
	FTimeRange GetTimeRange() const override;

	int32 GetQualityIndex() const override;
	int32 GetBitrate() const override;

	void GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const override;
	bool GetStartupDelay(FTimeValue& OutStartTime, FTimeValue& OutTimeIntoSegment, FTimeValue& OutSegmentDuration) const override;


	enum class EStreamingProtocol
	{
		// Not an actual type, used to signify that the protocol has not been set.
		Undefined,
		// MPEG DASH, ISO 23009-1
		DASH,
		// Apple HLS, RFC 8216
		HLS
	};

	enum class EContainerFormat
	{
		// Not an actual type, used to signify that the format has not been set.
		Undefined,
		// ".mp4" container
		ISO_14496_12,
		// ".ts" container
		ISO_13818_1,
		// ".mkv" or ".webm" container
		Matroska_WebM,
		// ".mp3" or ".aac" raw file with an ID3 tag
		ID3_Raw,
		// ".vtt" Raw WebVTT
		WebVTT_Raw
	};

	// Encryption
	struct FEncryption
	{
		TSharedPtrTS<ElectraCDM::IMediaCDMClient> DrmClient;
		FString DrmMimeType;
		TArray<uint8> DrmIV;
		TArray<uint8> DrmKID;
	};

	// HLS specific information carried to locate the next segment.
	struct FHLSSpecific
	{
		FTimeValue DurationDistanceToEnd;
		FTimeValue TimeWhenLoaded;
		TSharedPtr<FMediaPlaylistAndStateHLS, ESPMode::ThreadSafe> Playlist;
		int64 DiscontinuitySequence = -1;
		int32 LocalIndex = -1;
		bool bNoPDTMapping = false;
		bool bHasDiscontinuity = false;
	};
	FHLSSpecific HLS;

	// Timestamp information and adjustment
	struct FTimestampVars
	{
		// If set the first timestamp (DTS) is stored for reference and subtracted from all DTS and PTS timestamps
		// in this and future requests (this structure is passed from request to request).
		bool bGetAndAdjustByFirstTimestamp = false;


		struct FSegmentTimes
		{
			// Per stream type first DTS values in the segment.
			FTimeValue First[4];
			void Reset()
			{
				for(int32 i=0; i<4; ++i)
				{
					First[i].SetToInvalid();
				}
			}
		};
		struct FNextExpectedStartTimes
		{
			FTimeValue ExpectedLargerThan[4];	// time must be larger than this
			bool bCheck = false;
			bool bFailed = false;
		};

		struct FInternal
		{
			FTimeValue SegmentBaseTime {(int64)0};
			TOptional<uint64> RawAdjustmentValue;
			struct FRollover
			{
				uint64 RawDTSOffset = 0;
				uint64 RawPTSOffset = 0;
			};
			FRollover Rollover[4];
			TOptional<uint64> PrevRawID3StartPTS;
		};
		FInternal Internal;

		FSegmentTimes Local;
		FNextExpectedStartTimes Next;
	};

	DECLARE_DELEGATE_OneParam(FFirstTimestampReceivedDelegate, TSharedPtrTS<FStreamSegmentRequestCommon>);



	// Streaming protocol for which this segment is requested.
	EStreamingProtocol StreamingProtocol = EStreamingProtocol::Undefined;

	// The container format that is expected to be used. Not necessarily what it will be!
	EContainerFormat ExpectedContainerFormat = EContainerFormat::Undefined;
	EStreamType StreamType = EStreamType::Unsupported;						// Type of stream (video, audio, etc.)
	int32 QualityIndex = 0;
	int32 MaxQualityIndex = 0;
	TSharedPtrTS<ITimelineMediaAsset> Period;								// The period the adaptation set belongs to.
	TSharedPtrTS<IPlaybackAssetAdaptationSet> AdaptationSet;				// The adaptation set the representation belongs to.
	TSharedPtrTS<IPlaybackAssetRepresentation> Representation;				// The representation this request belongs to.
	FStreamCodecInformation CodecInfo[4];									// Partial codec info as can be collected from the MPD.
	FSegmentInformationCommon Segment;										// Segment information (URLs and timing values)
	TArray<TSharedPtrTS<FStreamSegmentRequestCommon>> DependentStreams;		// Streams this segment depends on. Currently only used to hold the set of requests for the initial playback start.
	bool bIsFalloffSegment = false;											// true if this segment no longer exists on the timeline and is used only to skip ahead.
	bool bIsGapSegment = false;												// true if segment information is given, but the segment is known to be missing and not to be loaded.
	bool bIsEOSSegment = false;												// true if this is not an actual request but a stream-has-already-ended request.
	bool bIsInitialStartRequest = false;									// true if this is the initial playback start request.
	FTimeValue PeriodStart;													// Value to add to all DTS & PTS to map them into the Period timeline
	FTimeValue AST = FTimeValue::GetZero();									// Value of AST to add to all time to generate wallclock time
	FTimeValue AdditionalAdjustmentTime = FTimeValue::GetZero();			// Sum of any other time corrections
	bool bInsertFillerData = false;											// true to insert empty access units into the buffer instead of reading actual data.
	int64 TimestampSequenceIndex = 0;										// Sequence index to set in all timestamp values of the decoded access unit.
	FTimeValue FrameAccurateStartTime;										// If set, the start time as was requested in a Seek() (not in media local time)

	// Encryption
	FEncryption DrmInit;
	FEncryption DrmMedia;

	// Muxed stream types to ignore (only used with HLS for now)
	bool bIsMultiplex = false;
	bool bIgnoreVideo = false;
	bool bIgnoreAudio = false;
	bool bIgnoreSubtitles = false;

	// UTC wallclock times during which this segment can be fetched;
	FTimeValue ASAST;
	FTimeValue SAET;
	FTimeValue DownloadDelayTime;


	// Internal work variables
	TSharedPtrTS<FBufferSourceInfo> SourceBufferInfo[4];
	int32 NumOverallRetries = 0;											// Number of retries for this _segment_ across all possible quality levels and CDNs.
	uint32 CurrentPlaybackSequenceID = ~0U;									// Set by the player before adding the request to the stream reader.
	Metrics::FSegmentDownloadStats DownloadStats;
	HTTP::FConnectionInfo ConnectionInfo;
	bool bWarnedAboutTimescale = false;
	FTimestampVars TimestampVars;
	FFirstTimestampReceivedDelegate FirstTimestampReceivedDelegate;			// Optional notification callback to invoke when the first AU timestamp is parsed
};



/**
 *
**/
class FStreamSegmentReaderCommon : public IStreamReader
{
public:
	FStreamSegmentReaderCommon();
	virtual ~FStreamSegmentReaderCommon();

	UEMediaError Create(IPlayerSessionServices* PlayerSessionService, const CreateParam& InCreateParam) override;
	bool LoadsMultiplexedTracksSequentially() override;
	void Close() override;

	//! Adds a request to read from a stream
	EAddResult AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> Request) override;

	//! Cancels any ongoing requests of the given stream type. Silent cancellation will not notify OnFragmentClose() or OnFragmentReachedEOS().
	void CancelRequest(EStreamType StreamType, bool bSilent) override;

	//! Cancels all pending requests.
	void CancelRequests() override;

private:
	ELECTRA_IMPL_DEFAULT_ERROR_METHODS(CommonSegmentReader);

	struct FStreamHandler : public FMediaThread, public IGenericDataReader
	{
		enum class EMediaSegmentTriggerResult
		{
			Started,
			IsFiller,
			IsSideloaded,
			DontHandle,
		};

		enum class EHandleResult
		{
			DontHandle,
			Finished,
			Skipped,
			Aborted,
			Failed
		};

		enum class EEmitType
		{
			UntilBlocked
		};

		enum class EEmitResult
		{
			HaveRemaining,
			SentEverything
		};

		struct FReadBuffer
		{
			FReadBuffer()
			{ Reset(); }
			~FReadBuffer()
			{ Reset(); }

			void Reset()
			{
				ReceiveBuffer.Reset();
				ParsePos = 0;
				bIsEncrypted = false;
				bDecrypterReady = false;
				bIsDecrypterGood = false;
				if (BlockDecrypter.IsValid() && BlockDecrypterHandle)
				{
					BlockDecrypter->BlockStreamDecryptEnd(BlockDecrypterHandle);
				}
				BlockDecrypter.Reset();
				BlockDecrypterIV.Empty();
				BlockDecrypterKID.Empty();
				BlockDecrypterHandle = nullptr;
				DecryptedDataBuffer.Reset();
			}

			bool DidDecryptionFail() const
			{ return bIsEncrypted && bDecrypterReady && !bIsDecrypterGood; }
			bool WaitUntilSizeAvailable(int64 SizeNeeded, int32 TimeoutMicroseconds);
			FCriticalSection* GetLock();
			const uint8* GetLinearReadData() const;
			uint8* GetLinearReadData();
			int64 GetLinearReadSize() const;
			bool GetEOD() const;
			bool WasAborted() const;


			FWaitableBuffer DecryptedDataBuffer;
			TArray<uint8> BlockDecrypterIV;
			TArray<uint8> BlockDecrypterKID;
			TSharedPtr<ElectraCDM::IMediaCDMDecrypter, ESPMode::ThreadSafe> BlockDecrypter;
			TSharedPtrTS<FWaitableBuffer> ReceiveBuffer;
			ElectraCDM::IMediaCDMDecrypter::IStreamDecryptHandle* BlockDecrypterHandle = nullptr;
			int64 ParsePos = 0;
			int32 BlockDecrypterBlockSize = 0;
			bool bIsEncrypted = false;
			bool bDecrypterReady = false;
			bool bIsDecrypterGood = false;
		};

		struct FProducerTime
		{
			FTimeValue Base;
			int64 Media = 0;
		};

		struct FActiveTrackData
		{
			void Reset()
			{
				BufferSourceInfo.Reset();
				DurationSuccessfullyRead.SetToZero();
				DurationSuccessfullyDelivered.SetToZero();
				AverageDuration.SetToZero();
				LargestDTS.SetToInvalid();
				SmallestPTS.SetToInvalid();
				LargestPTS.SetToInvalid();
				NumAddedTotal = 0;
				bIsFirstInSequence = true;
				bReadPastLastPTS = false;
				bTaggedLastSample = false;
				bGotAllSamples = false;

				AccessUnitFIFO.Empty();
				SortedAccessUnitFIFO.Empty();
				CSD.Reset();
				StreamType = EStreamType::Unsupported;
				Bitrate = 0;
				bNeedToRecalculateDurations = false;

				DefaultDurationFromCSD.SetToInvalid();

				PrevPTS90k = 0;
				PrevDTS90k = 0;
			}

			struct FSample
			{
				FTimeValue PTS;
				FAccessUnit* AU = nullptr;
				uint32 SequentialIndex = 0;
				FSample(FAccessUnit* InAU, uint32 InSequentialIndex) : PTS(InAU->PTS), AU(InAU), SequentialIndex(InSequentialIndex) { InAU->AddRef(); }
				FSample(const FSample& rhs)
				{
					PTS = rhs.PTS;
					SequentialIndex = rhs.SequentialIndex;
					if ((AU = rhs.AU) != nullptr)
					{
						AU->AddRef();
					}
				}
				void Release()
				{
					FAccessUnit::Release(AU);
					AU = nullptr;
				}
				~FSample()
				{
					Release();
				}
			};

			void AddAccessUnit(FAccessUnit* InAU)
			{
				if (InAU)
				{
					AccessUnitFIFO.Emplace(FActiveTrackData::FSample(InAU, NumAddedTotal));
					if (bNeedToRecalculateDurations)
					{
						SortedAccessUnitFIFO.Emplace(FActiveTrackData::FSample(InAU, NumAddedTotal));
						SortedAccessUnitFIFO.Sort([](const FActiveTrackData::FSample& a, const FActiveTrackData::FSample& b){return a.PTS < b.PTS;});
					}
					// If a valid non-zero duration exists on the AU we take it as the average duration.
					if ((!AverageDuration.IsValid() || AverageDuration.IsZero()) && InAU->Duration.IsValid() && InAU->Duration > FTimeValue::GetZero())
					{
						AverageDuration = InAU->Duration;
					}
					if (!LargestDTS.IsValid() || InAU->DTS > LargestDTS)
					{
						LargestDTS = InAU->DTS;
					}
					if (!SmallestPTS.IsValid() || InAU->PTS < SmallestPTS)
					{
						SmallestPTS = InAU->PTS;
					}
					if (!LargestPTS.IsValid() || AccessUnitFIFO.Last().AU->PTS > LargestPTS)
					{
						LargestPTS = AccessUnitFIFO.Last().AU->PTS;
					}
					++NumAddedTotal;
				}
			}

			TArray<FSample> AccessUnitFIFO;
			TArray<FSample> SortedAccessUnitFIFO;
			FTimeValue DurationSuccessfullyRead {(int64)0};
			FTimeValue DurationSuccessfullyDelivered {(int64)0};
			FTimeValue AverageDuration {(int64)0};
			FTimeValue SmallestPTS;
			FTimeValue LargestPTS;
			FTimeValue LargestDTS;
			FTimeValue TimeMappingOffset;
			TSharedPtrTS<FAccessUnit::CodecData> CSD;
			TSharedPtrTS<FBufferSourceInfo> BufferSourceInfo;
			EStreamType StreamType = EStreamType::Unsupported;
			uint32 NumAddedTotal = 0;
			int32 Bitrate = 0;
			bool bIsFirstInSequence = true;
			bool bReadPastLastPTS = false;
			bool bTaggedLastSample = false;
			bool bGotAllSamples = false;
			bool bNeedToRecalculateDurations = false;
			// Track local time values
			int64 MediaLocalFirstAUTime = 0;
			int64 MediaLocalLastAUTime = TNumericLimits<int64>::Max();
			int64 PTO = 0;
			FProducerTime ProducerTime;
			// Calculated values
			FTimeValue DefaultDurationFromCSD;
			// TS specific for DTS/PTR rollover detection
			int64 PrevPTS90k = 0;
			int64 PrevDTS90k = 0;
		};

		class FBoxDataLoader : public MP4Utilities::IMP4DataReaderBase, public TSharedFromThis<FBoxDataLoader, ESPMode::ThreadSafe>
		{
		public:
			FBoxDataLoader(IGenericDataReader* InGenericReader)
				: GenericReader(InGenericReader)
			{ }
			virtual ~FBoxDataLoader() = default;
			int64 ReadData(void* InOutBuffer, int64 InNumBytes, int64 InFromOffset, FCancellationCheckDelegate InCheckCancellationDelegate) override
			{
				check(InFromOffset == GetCurrentFileOffset());
				int64 nr = GenericReader->ReadData(InOutBuffer, InNumBytes, InFromOffset);
				return nr == InNumBytes ? nr : GenericReader->HasReadBeenAborted() ? EResult::Canceled : EResult::ReadError;
			}
			int64 GetTotalFileSize() override
			{ return GenericReader->GetTotalSize(); }
			int64 GetCurrentFileOffset() override
			{ return GenericReader->GetCurrentOffset(); }
			bool HasReachedEOF() override
			{ return GenericReader->HasReachedEOF(); }
			FString GetLastError() override
			{ return FString(); }
		private:
			IGenericDataReader* GenericReader = nullptr;
		};

		FStreamHandler();
		virtual ~FStreamHandler();
		void Cancel(bool bSilent);
		void SignalWork();
		void WorkerThread();
		EHandleResult SetError(const FString& InMessage, uint16 InCode);
		bool HasErrored() const;
		void LogMessage(IInfoLog::ELevel Level, const FString& Message);
		int32 HTTPProgressCallback(const IElectraHttpManager::FRequest* Request);
		void HTTPCompletionCallback(const IElectraHttpManager::FRequest* Request);
		void HTTPUpdateStats(const FTimeValue& CurrentTime, const IElectraHttpManager::FRequest* Request);

		void HandleRequest();
		void SetupSegmentDownloadStatsFromConnectionInfo(const HTTP::FConnectionInfo& ci);
		bool FetchInitSegment(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		EMediaSegmentTriggerResult TriggerMediaSegmentDownload(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		void HandleCommonMediaBegin(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		void HandleCommonMediaEnd(EHandleResult InSegmentResult, TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		EHandleResult HandleMP4Media(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		EHandleResult HandleMP4MediaLegacy(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		EHandleResult HandleMKVMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		EHandleResult HandleTSMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		EHandleResult HandleID3RawMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest, int32 InID3HeaderSize);
		EHandleResult HandleSideloadedMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		EHandleResult HandleRawSubtitleMedia(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		EHandleResult HandleFillerDataSetup(TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		void InsertFillerData(TSharedPtrTS<FActiveTrackData> InActiveTrackData, TSharedPtrTS<FStreamSegmentRequestCommon> InRequest);
		void CheckForInbandDASHEvents(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest);
		void HandleMP4Metadata(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest, const TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe>& InMoovMetaBox, const TSharedPtr<MP4Boxes::FMP4BoxMETA, ESPMode::ThreadSafe>& InMoofMetaBox);
		void HandleMP4EventMessages(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest, const TSharedPtr<MP4Boxes::FMP4BoxSIDX, ESPMode::ThreadSafe>& InSidxBox, const TArray<TSharedPtr<MP4Boxes::FMP4BoxEMSG, ESPMode::ThreadSafe>>& InEventMessageBoxes);
		void HandleMP4MetadataLegacy(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest, const TSharedPtrTS<IParserISO14496_12>& InMP4Parser, const TSharedPtrTS<const IParserISO14496_12>& InMP4InitSegment, const FTimeValue& InBaseTime);
		void HandleMP4EventMessagesLegacy(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest, const TSharedPtrTS<IParserISO14496_12>& InMP4Parser);

		void SelectPrimaryTrackData(const TSharedPtrTS<FStreamSegmentRequestCommon>& InRequest);
		EEmitResult EmitSamples(EEmitType InEmitType);

		// Methods from IGenericDataReader
		int64 ReadData(void* IntoBuffer, int64 NumBytesToRead, int64 InFromOffset) override;
		bool HasReachedEOF() const override;
		bool HasReadBeenAborted() const override;
		int64 GetCurrentOffset() const override;
		int64 GetTotalSize() const override;

		static bool IsWebVTTHeader(const uint8* InData, int32 InSize)
		{
			return (InSize >= 6 && (InData[0] == 0x57 && InData[1] == 0x45 && InData[2] == 0x42 && InData[3] == 0x56 && InData[4] == 0x54 && InData[5] == 0x54)) ||
					(InSize >= 9 && (InData[0] == 0xef && InData[1] == 0xbb && InData[2] == 0xbf && InData[3] == 0x57 && InData[4] == 0x45 && InData[5] == 0x42 && InData[6] == 0x56 && InData[7] == 0x54 && InData[8] == 0x54));
		}

		IStreamReader::CreateParam Parameters;
		TSharedPtrTS<FStreamSegmentRequestCommon> CurrentRequest;
		FMediaSemaphore WorkSignal;
		FMediaEvent IsIdleSignal;
		volatile bool bTerminate = false;
		volatile bool bWasStarted = false;
		volatile bool bRequestCanceled = false;
		volatile bool bSilentCancellation = false;
		volatile bool bHasErrored = false;
		bool bAbortedByABR = false;
		bool bAllowEarlyEmitting = false;
		bool bFillRemainingDuration = false;
		bool bAddedCMCDQueryToMediaURL = false;

		IPlayerSessionServices* PlayerSessionService = nullptr;
		FReadBuffer ReadBuffer;
		TArray<TSharedPtrTS<DASH::FPlayerEvent>> SegmentEventsFound;

		FCriticalSection MetricUpdateLock;
		int32 ProgressReportCount = 0;
		TSharedPtrTS<IAdaptiveStreamSelector> StreamSelector;
		FString ABRAbortReason;

		struct FNoInitSegmentData {int32 Zero=0;};
		TVariant<FNoInitSegmentData, TSharedPtrTS<FParsedMOOVCommon>, TSharedPtrTS<const IParserISO14496_12>, TSharedPtrTS<const IParserMKV>, TSharedPtrTS<const TArray<uint8>>> InitSegmentData;
		Metrics::FSegmentDownloadStats DownloadStats;
		HTTP::FConnectionInfo CurrentConnectionInfo;
		TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener;
		TSharedPtrTS<IElectraHttpManager::FRequest> HTTPRequest;
		FErrorDetail SegmentError;
		TSortedMap<uint64, TSharedPtrTS<FActiveTrackData>> TrackDataMap;
		TSharedPtrTS<FActiveTrackData> CurrentlyActiveTrackData;
		TSharedPtrTS<FActiveTrackData> PrimaryTrackData;
		TSharedPtr<ElectraCDM::IMediaCDMDecrypter, ESPMode::ThreadSafe> Decrypter;

		static uint32 UniqueDownloadID;
	};

	FStreamHandler StreamHandlers[3];		// 0 = video, 1 = audio, 2 = subtitle
	FErrorDetail ErrorDetail;
	IPlayerSessionServices* PlayerSessionService = nullptr;
	bool bIsStarted = false;
};


} // namespace Electra

