// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

namespace Electra
{
namespace ISOBMFF
{

const FName OptionKeyISOBMFFLoadConnectTimeout(TEXT("isobmff_connection_timeout"));				//!< (FTimeValue) value specifying connection timeout
const FName OptionKeyISOBMFFLoadNoDataTimeout(TEXT("isobmff_nodata_timeout"));					//!< (FTimeValue) value specifying no-data timeout

const FName OptionKeyISOBMFFTruncateToShortestTrack(TEXT("isobmff_trunc_to_shortest"));			//!< (bool) whether or not to truncate the movie duration to the shortest track

}

} // namespace Electra


