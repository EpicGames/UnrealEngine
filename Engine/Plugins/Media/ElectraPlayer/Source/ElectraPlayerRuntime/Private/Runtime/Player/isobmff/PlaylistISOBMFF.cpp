// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlaylistISOBMFF.h"
#include "OptionKeynamesISOBMFF.h"
#include "StreamReaderISOBMFF.h"
#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "Player/PlayerStreamFilter.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/AdaptivePlayerMetadataKeynames.h"
#include "Player/DRM/DRMManager.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/BCP47-Helpers.h"
#include "Utilities/MP4Helpers.h"
#include "Utils/MPEG/ElectraUtilsMP4.h"
#include "ElectraPlayerPrivate.h"
#include "HTTP/HTTPManager.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameNumber.h"
#include "Containers/Set.h"
#include "HAL/LowLevelMemTracker.h"
#include "Stats/Stats.h"

/*********************************************************************************************************************/

#include "HAL/IConsoleManager.h"
static TAutoConsoleVariable<int32> CVarElectraISOBMFFPlayerForceDuration(
	TEXT("Electra.ISOBMFFPlayer.ForceDuration"),
	0,
	TEXT("Set movie duration to the shortest or longest duration tracks in the file.\n")
	TEXT(" 0: keep whatever is in effect\n 1: use shortest duration\n 2: use longest duration"),
	ECVF_Default);

/*********************************************************************************************************************/

#define ERRCODE_MANIFEST_ISOBMFF_NO_PLAYABLE_STREAMS		1
#define ERRCODE_MANIFEST_ISOBMFF_STARTSEGMENT_NOT_FOUND		2
#define ERRCODE_MANIFEST_ISOBMFF_DRM_ERROR					3


#define DEFAULT_TRUNCATE_MOVIE_TO_SHORTEST_TRACK false


DECLARE_CYCLE_STAT(TEXT("FPlayPeriodISOBMFF::FindSegment"), STAT_ElectraPlayer_ISOBMFF_FindSegment, STATGROUP_ElectraPlayer);

namespace Electra
{

//-----------------------------------------------------------------------------
/**
 * CTOR
 */
FManifestISOBMFFInternal::FManifestISOBMFFInternal(IPlayerSessionServices* InPlayerSessionServices)
	: PlayerSessionServices(InPlayerSessionServices)
{
}


//-----------------------------------------------------------------------------
/**
 * DTOR
 */
FManifestISOBMFFInternal::~FManifestISOBMFFInternal()
{
}


//-----------------------------------------------------------------------------
/**
 * Builds the internal manifest from the mp4's moov box.
 */
FErrorDetail FManifestISOBMFFInternal::Build(TArray<TSharedPtr<MP4Boxes::FMP4BoxBase>>&& InParsedRootBoxes, const FString& InURL,
											 TSharedPtr<FMP4DataLoader, ESPMode::ThreadSafe> InDataLoader, FMP4DataLoader::FCancellationCheckDelegate InCancelCheck)
{
	MediaAsset = MakeSharedTS<FTimelineAssetISOBMFF>();
	FErrorDetail Result = MediaAsset->Build(PlayerSessionServices, MoveTemp(InParsedRootBoxes), InURL, InDataLoader, InCancelCheck);
	FTimeRange PlaybackRange = GetPlaybackRange(IManifest::EPlaybackRangeType::TemporaryPlaystartRange);
	DefaultStartTime = PlaybackRange.Start;
	DefaultEndTime = PlaybackRange.End;
	return Result;
}


//-----------------------------------------------------------------------------
/**
 * Logs a message.
 */
void FManifestISOBMFFInternal::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::ISOBMFFPlaylist, Level, Message);
	}
}



//-----------------------------------------------------------------------------
/**
 * Returns track metadata.
 */
void FManifestISOBMFFInternal::GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const
{
	if (MediaAsset.IsValid())
	{
		MediaAsset->GetMetaData(OutMetadata, StreamType);
	}
}



//-----------------------------------------------------------------------------
/**
 * Returns the playback range on the timeline, which is a subset of the total
 * time range. This may be set through manifest internal means or by URL fragment
 * parameters where permissable (eg. example.mp4#t=22,50).
 * If start or end are not specified they will be set to invalid.
 */
FTimeRange FManifestISOBMFFInternal::GetPlaybackRange(EPlaybackRangeType InRangeType) const
{
	FTimeRange FromTo;

	// We are interested in the 't' or 'r' fragment value here.
	FString Time;
	for(auto& Fragment : URLFragmentComponents)
	{
		if ((InRangeType == IManifest::EPlaybackRangeType::TemporaryPlaystartRange && Fragment.Name.Equals(TEXT("t"))) ||
			(InRangeType == IManifest::EPlaybackRangeType::LockedPlaybackRange && Fragment.Name.Equals(TEXT("r"))))
		{
			Time = Fragment.Value;
		}
	}
	if (!Time.IsEmpty())
	{
		FTimeRange TotalRange = GetTotalTimeRange();
		TArray<FString> TimeRange;
		const TCHAR* const TimeDelimiter = TEXT(",");
		Time.ParseIntoArray(TimeRange, TimeDelimiter, false);
		if (TimeRange.Num() && !TimeRange[0].IsEmpty())
		{
			RFC2326::ParseNPTTime(FromTo.Start, TimeRange[0]);
		}
		if (TimeRange.Num() > 1 && !TimeRange[1].IsEmpty())
		{
			RFC2326::ParseNPTTime(FromTo.End, TimeRange[1]);
		}
		// Need to clamp this into the total time range to prevent any issues.
		if (FromTo.Start.IsValid() && TotalRange.Start.IsValid() && FromTo.Start < TotalRange.Start)
		{
			FromTo.Start = TotalRange.Start;
		}
		if (FromTo.End.IsValid() && TotalRange.End.IsValid() && FromTo.End > TotalRange.End)
		{
			FromTo.End = TotalRange.End;
		}
	}
	return FromTo;
}


//-----------------------------------------------------------------------------
/**
 * Returns the minimum duration of content that must be buffered up before playback
 * will begin. This is an arbitrary choice that could be controlled by a 'pdin' box.
 */
FTimeValue FManifestISOBMFFInternal::GetMinBufferTime() const
{
	// NOTE: This could come from a 'pdin' (progressive download information) box, but those are rarely, if ever, set by any tool.
	return FTimeValue().SetFromSeconds(1.0);
}

TSharedPtrTS<IProducerReferenceTimeInfo> FManifestISOBMFFInternal::GetProducerReferenceTimeInfo(int64 ID) const
{
	return nullptr;
}

TRangeSet<double> FManifestISOBMFFInternal::GetPossiblePlaybackRates(EPlayRateType InForType) const
{
	TRangeSet<double> Ranges;
//	Ranges.Add(TRange<double>{1.0}); // normal (real-time) playback rate
	Ranges.Add(TRange<double>::Inclusive(0.5, 4.0));
	Ranges.Add(TRange<double>{0.0}); // and pause
	return Ranges;
}


//-----------------------------------------------------------------------------
/**
 * Creates an instance of a stream reader to stream from the mp4 file.
 */
IStreamReader* FManifestISOBMFFInternal::CreateStreamReaderHandler()
{
	FStreamReaderISOBMFF* Reader = new FStreamReaderISOBMFF;
	if (MediaAsset)
	{
		switch(MediaAsset->GetLoaderType())
		{
			default:
			case FTimelineAssetISOBMFF::ELoaderType::Chunks:
			{
				Reader->SetLoaderType(FStreamSegmentRequestISOBMFF::ELoaderType::Chunks);
				break;
			}
			case FTimelineAssetISOBMFF::ELoaderType::Muxed:
			{
				Reader->SetLoaderType(FStreamSegmentRequestISOBMFF::ELoaderType::Muxed);
				break;
			}
			case FTimelineAssetISOBMFF::ELoaderType::Fragmented:
			{
				Reader->SetLoaderType(FStreamSegmentRequestISOBMFF::ELoaderType::Fragmented);
				break;
			}
		}
	}
	return Reader;
}


//-----------------------------------------------------------------------------
/**
 * Returns the playback period for the given time.
 */
IManifest::FResult FManifestISOBMFFInternal::FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	if (MediaAsset.IsValid() && StartPosition.Time.IsValid() && StartPosition.Time < MediaAsset->GetDuration())
	{
		OutPlayPeriod = MakeSharedTS<FPlayPeriodISOBMFF>(MediaAsset);
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}
	else
	{
		return IManifest::FResult(IManifest::FResult::EType::PastEOS);
	}
}

IManifest::FResult FManifestISOBMFFInternal::FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment)
{
	// There is no following period.
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}





//-----------------------------------------------------------------------------
/**
 * Constructs a playback period.
 */
FManifestISOBMFFInternal::FPlayPeriodISOBMFF::FPlayPeriodISOBMFF(TSharedPtrTS<FManifestISOBMFFInternal::FTimelineAssetISOBMFF> InMediaAsset)
	: MediaAsset(InMediaAsset)
	, CurrentReadyState(IManifest::IPlayPeriod::EReadyState::NotLoaded)
	, LoaderType(InMediaAsset->GetLoaderType())
{
	AggregateMinChunkDuration = FTimespan::FromSeconds(1.0);
	MuxedSegmentDuration = FTimespan::FromSeconds(2.0);
}


//-----------------------------------------------------------------------------
/**
 * Destroys a playback period.
 */
FManifestISOBMFFInternal::FPlayPeriodISOBMFF::~FPlayPeriodISOBMFF()
{
}


//-----------------------------------------------------------------------------
/**
 * Sets stream playback preferences for this playback period.
 */
void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	if (ForStreamType == EStreamType::Video)
	{
		VideoPreferences = StreamAttributes;
	}
	else if (ForStreamType == EStreamType::Audio)
	{
		AudioPreferences = StreamAttributes;
	}
	else if (ForStreamType == EStreamType::Subtitle)
	{
		SubtitlePreferences = StreamAttributes;
	}
}


//-----------------------------------------------------------------------------
/**
 * Returns the starting bitrate.
 *
 * This is merely informational and not strictly required.
 * If fetching of the moov box provided us with the total size of the mp4 file
 * we will use that divided by the duration.
 */
int64 FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetDefaultStartingBitrate() const
{
	return 2000000;
}

//-----------------------------------------------------------------------------
/**
 * Returns the ready state of this playback period.
 */
IManifest::IPlayPeriod::EReadyState FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetReadyState()
{
	return CurrentReadyState;
}


void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::Load()
{
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::Loaded;
}

//-----------------------------------------------------------------------------
/**
 * Prepares the playback period for playback.
 * With an mp4 file we are actually always ready for playback, but we say we're not
 * one time to get here with any possible options.
 */
void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::PrepareForPlay()
{
	SelectedVideoMetadata.Reset();
	SelectedAudioMetadata.Reset();
	SelectedSubtitleMetadata.Reset();
	VideoBufferSourceInfo.Reset();
	AudioBufferSourceInfo.Reset();
	SubtitleBufferSourceInfo.Reset();
	SelectInitialStream(EStreamType::Video);
	SelectInitialStream(EStreamType::Audio);
	SelectInitialStream(EStreamType::Subtitle);
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::IsReady;
}


TSharedPtrTS<FBufferSourceInfo> FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetSelectedStreamBufferSourceInfo(EStreamType StreamType)
{
	return StreamType == EStreamType::Video ? VideoBufferSourceInfo :
		   StreamType == EStreamType::Audio ? AudioBufferSourceInfo :
		   StreamType == EStreamType::Subtitle ? SubtitleBufferSourceInfo : nullptr;
}

FString FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetSelectedAdaptationSetID(EStreamType StreamType)
{
	switch(StreamType)
	{
		case EStreamType::Video:
		{
			return SelectedVideoMetadata.IsValid() ? SelectedVideoMetadata->ID : FString();
		}
		case EStreamType::Audio:
		{
			return SelectedAudioMetadata.IsValid() ? SelectedAudioMetadata->ID : FString();
		}
		case EStreamType::Subtitle:
		{
			return SelectedSubtitleMetadata.IsValid() ? SelectedSubtitleMetadata->ID : FString();
		}
		default:
		{
			return FString();
		}
	}
}


IManifest::IPlayPeriod::ETrackChangeResult FManifestISOBMFFInternal::FPlayPeriodISOBMFF::ChangeTrackStreamPreference(EStreamType StreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Chunks || LoaderType == FTimelineAssetISOBMFF::ELoaderType::Fragmented)
	{
		return IManifest::IPlayPeriod::ETrackChangeResult::NewPeriodNeeded;
	}
	else if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Muxed)
	{
		TSharedPtrTS<FTrackMetadata> Metadata = SelectMetadataForAttributes(StreamType, StreamAttributes);
		if (Metadata.IsValid())
		{
			if (StreamType == EStreamType::Video)
			{
				if (!(SelectedVideoMetadata.IsValid() && Metadata->Equals(*SelectedVideoMetadata)))
				{
					SelectedVideoMetadata = Metadata;
					MakeBufferSourceInfoFromMetadata(StreamType, VideoBufferSourceInfo, SelectedVideoMetadata);
					return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
				}
			}
			else if (StreamType == EStreamType::Audio)
			{
				if (!(SelectedAudioMetadata.IsValid() && Metadata->Equals(*SelectedAudioMetadata)))
				{
					SelectedAudioMetadata = Metadata;
					MakeBufferSourceInfoFromMetadata(StreamType, AudioBufferSourceInfo, SelectedAudioMetadata);
					return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
				}
			}
			else if (StreamType == EStreamType::Subtitle)
			{
				if (!(SelectedSubtitleMetadata.IsValid() && Metadata->Equals(*SelectedSubtitleMetadata)))
				{
					SelectedSubtitleMetadata = Metadata;
					MakeBufferSourceInfoFromMetadata(StreamType, SubtitleBufferSourceInfo, SelectedSubtitleMetadata);
					return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
				}
			}
		}
		return IManifest::IPlayPeriod::ETrackChangeResult::NotChanged;
	}
	else
	{
		unimplemented();
		return IManifest::IPlayPeriod::ETrackChangeResult::StartOver;
	}
}

void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::SelectInitialStream(EStreamType StreamType)
{
	if (StreamType == EStreamType::Video)
	{
		SelectedVideoMetadata = SelectMetadataForAttributes(StreamType, VideoPreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, VideoBufferSourceInfo, SelectedVideoMetadata);
	}
	else if (StreamType == EStreamType::Audio)
	{
		SelectedAudioMetadata = SelectMetadataForAttributes(StreamType, AudioPreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, AudioBufferSourceInfo, SelectedAudioMetadata);
	}
	else if (StreamType == EStreamType::Subtitle)
	{
		SelectedSubtitleMetadata = SelectMetadataForAttributes(StreamType, SubtitlePreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, SubtitleBufferSourceInfo, SelectedSubtitleMetadata);
	}
}

TSharedPtrTS<FTrackMetadata> FManifestISOBMFFInternal::FPlayPeriodISOBMFF::SelectMetadataForAttributes(EStreamType StreamType, const FStreamSelectionAttributes& InAttributes)
{
	TSharedPtrTS<FTimelineAssetISOBMFF> Asset = MediaAsset.Pin();
	if (Asset.IsValid())
	{
		TArray<FTrackMetadata> Metadata;
		Asset->GetMetaData(Metadata, StreamType);
		// Is there a fixed index to be used?
		if (InAttributes.OverrideIndex.IsSet() && InAttributes.OverrideIndex.GetValue() >= 0 && InAttributes.OverrideIndex.GetValue() < Metadata.Num())
		{
			// Use this.
			return MakeSharedTS<FTrackMetadata>(Metadata[InAttributes.OverrideIndex.GetValue()]);
		}
		if (Metadata.Num())
		{
			// We do not look at the 'kind' or 'codec' here, only the language.
			// Set the first track as default in case we do not find the one we're looking for.
			if (InAttributes.Language_RFC4647.IsSet())
			{
				TArray<int32> CandidateIndices;
				TArray<BCP47::FLanguageTag> CandList;
				for(auto &Meta : Metadata)
				{
					CandList.Emplace(Meta.LanguageTagRFC5646);
				}
				CandidateIndices = BCP47::FindExtendedFilteringMatch(CandList, InAttributes.Language_RFC4647.GetValue());
				return MakeSharedTS<FTrackMetadata>(CandidateIndices.Num() ? Metadata[CandidateIndices[0]] : Metadata[0]);
			}
			// Subtitle tracks are not selected by default. If there is no explicit selection asked for we ignore them.
			if (StreamType != EStreamType::Subtitle)
			{
				return MakeSharedTS<FTrackMetadata>(Metadata[0]);
			}
		}
	}
	return nullptr;
}

void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::MakeBufferSourceInfoFromMetadata(EStreamType StreamType, TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FTrackMetadata> InMetadata)
{
	if (InMetadata.IsValid())
	{
		OutBufferSourceInfo = MakeSharedTS<FBufferSourceInfo>();
		OutBufferSourceInfo->Kind = InMetadata->Kind;
		OutBufferSourceInfo->LanguageTag = InMetadata->LanguageTagRFC5646;
		OutBufferSourceInfo->Codec = InMetadata->HighestBandwidthCodec.GetCodecName();
		TSharedPtrTS<FTimelineAssetISOBMFF> Asset = MediaAsset.Pin();
		OutBufferSourceInfo->PeriodID = Asset->GetUniqueIdentifier();
		OutBufferSourceInfo->PeriodAdaptationSetID = Asset->GetUniqueIdentifier() + TEXT(".") + InMetadata->ID;
		TArray<FTrackMetadata> Metadata;
		Asset->GetMetaData(Metadata, StreamType);
		for(int32 i=0; i<Metadata.Num(); ++i)
		{
			if (Metadata[i].Equals(*InMetadata))
			{
				OutBufferSourceInfo->HardIndex = i;
				break;
			}
		}
	}
}



//-----------------------------------------------------------------------------
/**
 * Returns the timeline media asset. We have a weak pointer to it only to
 * prevent any cyclic locks, so we need to lock it first.
 */
TSharedPtrTS<ITimelineMediaAsset> FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetMediaAsset() const
{
	return MediaAsset.Pin();
}


//-----------------------------------------------------------------------------
/**
 * Selects a particular stream (== internal track ID) for playback.
 */
void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex)
{
	// No-op.
}

void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload)
{
	// No-op.
}

void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::AggregateChunkSamples(TSharedPtr<IStreamSegment, ESPMode::ThreadSafe> InOutRequest, const FTimespan& InMinDuration)
{
/*
	const FStreamSegmentRequestISOBMFF* Request = static_cast<const FStreamSegmentRequestISOBMFF*>(InOutRequest.Get());
	auto It = Request->Track.TrackIterator->Clone();
	int64 ChunkEndOffset = It->GetSampleFileOffset() + Request->Track.TrackChunkSampleInfo.SizeInBytes;
*/
}

void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::AdjustMuxedDownload(TSharedPtr<IStreamSegment, ESPMode::ThreadSafe> InOutRequest, const FTimespan& InMinDuration, EAdjustMuxType InType)
{
	FStreamSegmentRequestISOBMFF* Request = static_cast<FStreamSegmentRequestISOBMFF*>(InOutRequest.Get());

	auto GetFirstSelectedIterator = [&]() -> TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe>
	{
		for(int32 i=0,iMax=Request->DependentStreams.Num(); i<iMax; ++i)
		{
			if (Request->DependentStreams[i]->bWasSelectedAtStart && Request->DependentStreams[i]->Track.TrackIterator)
			{
				Request->ReportingIndexInMux = i;
				return Request->DependentStreams[i]->Track.TrackIterator->Clone();
			}
		}
		return nullptr;
	};

	auto GetNextActiveIterator = [&]() -> TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe>
	{
		for(int32 i=0,iMax=Request->DependentStreams.Num(); i<iMax; ++i)
		{
			if (Request->DependentStreams[i]->Track.TrackIterator)
			{
				Request->ReportingIndexInMux = i;
				return Request->DependentStreams[i]->Track.TrackIterator->Clone();
			}
		}
		return nullptr;
	};

	if (InType == EAdjustMuxType::FirstStart)
	{
		auto PrimaryIterator = GetFirstSelectedIterator();
		// If there is no track with an an iterator we're done.
		if (!PrimaryIterator)
		{
			Request->bIsLastSegment = true;
			return;
		}

		int64 FirstSampleFileOffset = PrimaryIterator->GetSampleFileOffset();
		FTimespan AccumulatedPrimaryDuration(0);
		bool bReachedEnd = false;
		while(AccumulatedPrimaryDuration < InMinDuration)
		{
			AccumulatedPrimaryDuration += PrimaryIterator->GetDurationAsTimespan();
			if (!PrimaryIterator->Next())
			{
				bReachedEnd = true;
				break;
			}
		}
		Request->Bitrate = 0;
		Request->SegmentDuration.SetFromTimespan(AccumulatedPrimaryDuration);
		Request->PrimarySampleFileOffsetStart = FirstSampleFileOffset;
		Request->PrimarySampleFileOffsetEnd = bReachedEnd ? -1 : PrimaryIterator->GetSampleFileOffset() + PrimaryIterator->GetSampleSize();
		// Adjust tracks accordingly.
		for(int32 nTrk=0,nTrkMax=Request->DependentStreams.Num(); nTrk<nTrkMax; ++nTrk)
		{
			// Add the bitrate of the track even if it's already EOS.
			Request->Bitrate += Request->DependentStreams[nTrk]->Bitrate;

			const TSharedPtrTS<FStreamSegmentRequestISOBMFF>& DepStr(Request->DependentStreams[nTrk]);
			// Track already at EOS?
			if (!DepStr->Track.TrackIterator)
			{
				continue;
			}
			// If the track has the selected sample position before the first sample of the primary track
			// and this track is not also a selected track or is a subtitle track (for which it may be possible
			// that the first selected sample is way too old) we step forward.
			TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe>& TkIt(DepStr->Track.TrackIterator);
			if (!DepStr->bWasSelectedAtStart || DepStr->StreamType == EStreamType::Subtitle)
			{
				while(TkIt->GetSampleFileOffset() < FirstSampleFileOffset)
				{
					// If advancing results in an EOS we reset the iterator to indicate this.
					if (!TkIt->Next())
					{
						DepStr->Track.TrackIterator.Reset();
						DepStr->bIsLastSegment = true;
						break;
					}
				}
			}
			else
			{
				// Otherwise, if this sample comes before the one from the primary track we
				// need to load that one and thus have to adjust the file position backward.
				int64 SmpPos = TkIt->GetSampleFileOffset();
				if (SmpPos < Request->PrimarySampleFileOffsetStart)
				{
					Request->PrimarySampleFileOffsetStart = SmpPos;
				}
			}
			if (TkIt)
			{
				TkIt->GetCurrentChunkRemainingSampleInfo(DepStr->Track.TrackChunkSampleInfo);
				DepStr->SegmentDuration.SetFromND(DepStr->Track.TrackChunkSampleInfo.Duration, TkIt->GetTimescale());
				auto DTS = TkIt->GetEffectiveDTS();
				DepStr->FirstDTS.SetFromND(DTS.GetNumerator(), DTS.GetDenominator());
				DepStr->bIsLastSegment = DepStr->Track.TrackChunkSampleInfo.bIsLastChunk;
			}
		}
	}
	else if (InType == EAdjustMuxType::Continuation)
	{
		Request->bIsLastSegment = true;
		// Get the file offset where we have to continue reading from.
		// This will be the smallest sample file offset of all remaining tracks.
		Request->PrimarySampleFileOffsetStart = TNumericLimits<int64>::Max();
		for(auto &tkit : Request->DependentStreams)
		{
			if (tkit->Track.TrackIterator)
			{
				Request->bIsLastSegment = false;
				const int64 TkSmpOff = tkit->Track.TrackIterator->GetSampleFileOffset();
				if (TkSmpOff < Request->PrimarySampleFileOffsetStart)
				{
					Request->PrimarySampleFileOffsetStart = TkSmpOff;
				}
				tkit->Track.TrackIterator->GetCurrentChunkRemainingSampleInfo(tkit->Track.TrackChunkSampleInfo);
				tkit->SegmentDuration.SetFromND(tkit->Track.TrackChunkSampleInfo.Duration, tkit->Track.TrackIterator->GetTimescale());
				auto DTS = tkit->Track.TrackIterator->GetEffectiveDTS();
				tkit->FirstDTS.SetFromND(DTS.GetNumerator(), DTS.GetDenominator());
				tkit->bIsLastSegment = tkit->Track.TrackChunkSampleInfo.bIsLastChunk;
			}
		}
		auto PrimaryIterator = GetNextActiveIterator();
		if (!PrimaryIterator)
		{
			Request->PrimarySampleFileOffsetStart = -1;
			Request->PrimarySampleFileOffsetEnd = -1;
			Request->bIsLastSegment = true;
			return;
		}
		FTimespan AccumulatedPrimaryDuration(0);
		bool bReachedEnd = false;
		while(AccumulatedPrimaryDuration < InMinDuration)
		{
			AccumulatedPrimaryDuration += PrimaryIterator->GetDurationAsTimespan();
			if (!PrimaryIterator->Next())
			{
				bReachedEnd = true;
				break;
			}
		}
		Request->SegmentDuration.SetFromTimespan(AccumulatedPrimaryDuration);
		Request->PrimarySampleFileOffsetEnd = bReachedEnd ? -1 : PrimaryIterator->GetSampleFileOffset() + PrimaryIterator->GetSampleSize();
	}
	else //if (InType == EAdjustMuxType::Retry)
	{
		Request->bIsLastSegment = true;
		// Get the file offset where we have to continue reading from.
		// This will be the smallest sample file offset of all remaining tracks.
		Request->PrimarySampleFileOffsetStart = TNumericLimits<int64>::Max();
		for(auto &tkit : Request->DependentStreams)
		{
			if (tkit->Track.TrackIterator)
			{
				Request->bIsLastSegment = false;
				const int64 TkSmpOff = tkit->Track.TrackIterator->GetSampleFileOffset();
				if (TkSmpOff < Request->PrimarySampleFileOffsetStart)
				{
					Request->PrimarySampleFileOffsetStart = TkSmpOff;
				}
				tkit->Track.TrackIterator->GetCurrentChunkRemainingSampleInfo(tkit->Track.TrackChunkSampleInfo);
				tkit->SegmentDuration.SetFromND(tkit->Track.TrackChunkSampleInfo.Duration, tkit->Track.TrackIterator->GetTimescale());
				auto DTS = tkit->Track.TrackIterator->GetEffectiveDTS();
				tkit->FirstDTS.SetFromND(DTS.GetNumerator(), DTS.GetDenominator());
				tkit->bIsLastSegment = tkit->Track.TrackChunkSampleInfo.bIsLastChunk;
			}
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Creates the starting segment request to start playback with.
 */
IManifest::FResult FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	TSharedPtrTS<FTimelineAssetISOBMFF> ma = MediaAsset.Pin();
	if (!ma)
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound);
	}
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_ISOBMFF_FindSegment);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, ISOBMFF_FindSegment);

	// Frame accurate seek required?
	const bool bFrameAccurateSearch = StartPosition.Options.bFrameAccuracy;
	FTimeValue PlayRangeEnd = StartPosition.Options.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());
	if (PlayRangeEnd > ma->GetDuration())
	{
		PlayRangeEnd = ma->GetDuration();
	}

	FTimespan SearchTime = StartPosition.Time.GetAsTimespan();


	auto SetError = [&]() -> IManifest::FResult
	{
		IManifest::FResult res(IManifest::FResult::EType::NotFound);
		res.SetErrorDetail(FErrorDetail().SetError(UEMEDIA_ERROR_DETAIL)
							.SetFacility(Facility::EFacility::ISOBMFFPlaylist)
							.SetCode(ERRCODE_MANIFEST_ISOBMFF_STARTSEGMENT_NOT_FOUND)
							.SetMessage(FString::Printf(TEXT("Could not find start segment for time %lld"), (long long int)StartPosition.Time.GetAsHNS())));
		return res;
	};

	auto SetupTrackRequest = [&](TSharedPtrTS<FAdaptationSetISOBMFF> InAdapt, EStreamType InStreamType) -> TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe>
	{
		if (!InAdapt)
		{
			return nullptr;
		}
		auto Representation = InAdapt->GetRepresentation();
		auto TrkInf = Representation->GetTrackInfo();
		auto TrkIt = TrkInf->MP4Track->CreateIteratorAtKeyframe(SearchTime, FTimespan::Zero());
		auto Req = MakeShared<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe>();
		Req->StreamType = InStreamType;
		Req->bWasSelectedAtStart = true;
		Req->Bitrate = Representation->GetBitrate();
		Req->AdaptationSetID = InAdapt->GetUniqueIdentifier();
		Req->AdaptationSetTrackIndex = InAdapt->GetInternalTrackIndex();
		Req->RepresentationID = Representation->GetUniqueIdentifier();
		Req->LanguageTag = InAdapt->GetLanguageTag();
		Req->KindOfTrack = InAdapt->GetTrackKind();
		Req->Track.TrackInfo = TrkInf;
		if (TrkIt)
		{
			TrkIt->GetCurrentChunkRemainingSampleInfo(Req->Track.TrackChunkSampleInfo);
			Req->Track.TrackIterator = TrkIt;
			Req->SegmentDuration.SetFromND(Req->Track.TrackChunkSampleInfo.Duration, TrkIt->GetTimescale());
			auto DTS = TrkIt->GetEffectiveDTS();
			Req->FirstDTS.SetFromND(DTS.GetNumerator(), DTS.GetDenominator());
			Req->bIsLastSegment = Req->Track.TrackChunkSampleInfo.bIsLastChunk;
		}
		else
		{
			Req->bIsLastSegment = true;
		}
		return Req;
	};

	// Depending on how we need to load the data we have to locate the first sample to start with differently.
	if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Chunks || LoaderType == FTimelineAssetISOBMFF::ELoaderType::Fragmented)
	{
		TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe> Requests[3];

		// Is there a selected video track?
		if (SelectedVideoMetadata && !SelectedVideoMetadata->ID.IsEmpty())
		{
			Requests[0] = SetupTrackRequest(ma->GetAdaptationSetByTypeAndID(EStreamType::Video, SelectedVideoMetadata->ID), EStreamType::Video);
			if (!Requests[0] || !Requests[0]->Track.TrackIterator)
			{
				return SetError();
			}
			// When not using frame accurate seeking we set the searchtime for audio and subtitles to the time of the video keyframe
			// since we will start there and need the audio to be there, too.
			if (!bFrameAccurateSearch)
			{
				SearchTime = Requests[0]->Track.TrackIterator->GetEffectivePTS().GetAsTimespan();
			}
		}
		// Is there a selected audio track?
		if (SelectedAudioMetadata && !SelectedAudioMetadata->ID.IsEmpty())
		{
			Requests[1] = SetupTrackRequest(ma->GetAdaptationSetByTypeAndID(EStreamType::Audio, SelectedAudioMetadata->ID), EStreamType::Audio);
			if (!Requests[1] || !Requests[1]->Track.TrackIterator)
			{
				return SetError();
			}
		}
		// Is there a selected subtitle track?
		if (SelectedSubtitleMetadata && !SelectedSubtitleMetadata->ID.IsEmpty())
		{
			// For subtitles if we can't create a request we do not fail.
			Requests[2] = SetupTrackRequest(ma->GetAdaptationSetByTypeAndID(EStreamType::Subtitle, SelectedSubtitleMetadata->ID), EStreamType::Subtitle);
			if (!Requests[2] || !Requests[2]->Track.TrackIterator)
			{
				Requests[2].Reset();
			}
		}

		TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe> StartRequest = MakeShared<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe>();
		StartRequest->MediaAsset = ma;
		StartRequest->LoaderType = LoaderType == FTimelineAssetISOBMFF::ELoaderType::Chunks ? FStreamSegmentRequestISOBMFF::ELoaderType::Chunks : FStreamSegmentRequestISOBMFF::ELoaderType::Fragmented;
		StartRequest->MediaURL = ma->GetMediaURL();
		StartRequest->AssetID = ma->GetUniqueIdentifier();
		StartRequest->EarliestPTS = bFrameAccurateSearch ? StartPosition.Time : FTimeValue().SetFromTimespan(SearchTime);
		StartRequest->LastPTS = PlayRangeEnd;
		StartRequest->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
		for(int32 i=0; i<3; ++i)
		{
			if (!Requests[i])
			{
				continue;
			}
			Requests[i]->MediaAsset = StartRequest->MediaAsset;
			Requests[i]->LoaderType = StartRequest->LoaderType;
			Requests[i]->AssetID = StartRequest->AssetID;
			Requests[i]->MediaURL = StartRequest->MediaURL;
			Requests[i]->EarliestPTS = StartRequest->EarliestPTS;
			Requests[i]->LastPTS = StartRequest->LastPTS;
			Requests[i]->TimestampSequenceIndex = StartRequest->TimestampSequenceIndex;
			Requests[i]->DrmClient = Requests[i]->Track.TrackInfo->DrmFormat.IsEncrypted() ? ma->GetDrmClient() : nullptr;

			if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Chunks)
			{
				// TBD: aggregate consecutive chunks into a single request?
				AggregateChunkSamples(Requests[i], AggregateMinChunkDuration);
			}

			// If this isn't already the last segment we need to check if this segment contains
			// or is past the playback range end.
			if (!Requests[i]->bIsLastSegment)
			{
				FTimeValue LastDTS = Requests[i]->FirstDTS + Requests[i]->SegmentDuration;
				if (LastDTS > PlayRangeEnd)
				{
					Requests[i]->bIsLastSegment = true;
				}
			}

			// Set the first request type as the type for the start request.
			if (StartRequest->StreamType == EStreamType::Unsupported)
			{
				StartRequest->StreamType = Requests[i]->StreamType;
			}
			StartRequest->Bitrate += Requests[i]->Bitrate;
			if (!StartRequest->SegmentDuration.IsValid() || StartRequest->SegmentDuration < Requests[i]->SegmentDuration)
			{
				StartRequest->SegmentDuration = Requests[i]->SegmentDuration;
			}
			StartRequest->DependentStreams.Emplace(MoveTemp(Requests[i]));
		}
		OutSegment = StartRequest;
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}
	else if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Muxed)
	{
		TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe> PrimaryReq;
		// Pick the selected video or audio track as the primary track we need to start with.
		if (SelectedVideoMetadata && !SelectedVideoMetadata->ID.IsEmpty())
		{
			PrimaryReq = SetupTrackRequest(ma->GetAdaptationSetByTypeAndID(EStreamType::Video, SelectedVideoMetadata->ID), EStreamType::Video);
		}
		else if (SelectedAudioMetadata && !SelectedAudioMetadata->ID.IsEmpty())
		{
			PrimaryReq = SetupTrackRequest(ma->GetAdaptationSetByTypeAndID(EStreamType::Audio, SelectedAudioMetadata->ID), EStreamType::Audio);
		}
		if (!PrimaryReq || !PrimaryReq->Track.TrackIterator)
		{
			return SetError();
		}
		if (!bFrameAccurateSearch)
		{
			SearchTime = PrimaryReq->Track.TrackIterator->GetEffectivePTS().GetAsTimespan();
		}

		TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe> StartRequest = MakeShared<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe>();
		StartRequest->MediaAsset = ma;
		StartRequest->LoaderType = FStreamSegmentRequestISOBMFF::ELoaderType::Muxed;
		StartRequest->MediaURL = ma->GetMediaURL();
		StartRequest->AssetID = ma->GetUniqueIdentifier();
		StartRequest->EarliestPTS = bFrameAccurateSearch ? StartPosition.Time : FTimeValue().SetFromTimespan(SearchTime);
		StartRequest->LastPTS = PlayRangeEnd;
		StartRequest->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
		StartRequest->StreamType = PrimaryReq->StreamType;
		StartRequest->AdaptationSetID = PrimaryReq->AdaptationSetID;
		StartRequest->RepresentationID = PrimaryReq->RepresentationID;
		StartRequest->AdaptationSetTrackIndex = PrimaryReq->AdaptationSetTrackIndex;
		// Using the sample information from the "primary" track we now need to go over all tracks and set them up.
		EStreamType AllStreamTypes[] { EStreamType::Video, EStreamType::Audio, EStreamType::Subtitle };
		FString SelectedStreamTypeIDs[3];
		SelectedStreamTypeIDs[0] = SelectedVideoMetadata && !SelectedVideoMetadata->ID.IsEmpty() ? SelectedVideoMetadata->ID : FString();
		SelectedStreamTypeIDs[1] = SelectedAudioMetadata && !SelectedAudioMetadata->ID.IsEmpty() ? SelectedAudioMetadata->ID : FString();
		SelectedStreamTypeIDs[2] = SelectedSubtitleMetadata && !SelectedSubtitleMetadata->ID.IsEmpty() ? SelectedSubtitleMetadata->ID : FString();
		for(int32 stIdx=0; stIdx<UE_ARRAY_COUNT(AllStreamTypes); ++stIdx)
		{
			for(int32 i=0,iMax=ma->GetNumberOfAdaptationSets(AllStreamTypes[stIdx]); i<iMax; ++i)
			{
				TSharedPtrTS<FAdaptationSetISOBMFF> AS = StaticCastSharedPtr<FAdaptationSetISOBMFF>(ma->GetAdaptationSetByTypeAndIndex(AllStreamTypes[stIdx], i));
				auto TrkReq = SetupTrackRequest(AS, AllStreamTypes[stIdx]);
				if (TrkReq)
				{
					TrkReq->MediaAsset = StartRequest->MediaAsset;
					TrkReq->LoaderType = StartRequest->LoaderType;
					TrkReq->AssetID = StartRequest->AssetID;
					TrkReq->MediaURL = StartRequest->MediaURL;
					TrkReq->EarliestPTS = StartRequest->EarliestPTS;
					TrkReq->LastPTS = StartRequest->LastPTS;
					TrkReq->TimestampSequenceIndex = StartRequest->TimestampSequenceIndex;
					TrkReq->DrmClient = TrkReq->Track.TrackInfo->DrmFormat.IsEncrypted() ? ma->GetDrmClient() : nullptr;
					TrkReq->bWasSelectedAtStart = AS->GetUniqueIdentifier().Equals(SelectedStreamTypeIDs[stIdx]);
					// Add this track request to the list of dependent streams.
					// Note: This track could be at EOS at that time already, which is indicated by it not having an iterator!
					StartRequest->DependentStreams.Emplace(MoveTemp(TrkReq));
				}
			}
		}
		AdjustMuxedDownload(StartRequest, MuxedSegmentDuration, EAdjustMuxType::FirstStart);
		OutSegment = StartRequest;
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}
	else
	{
		return IManifest::FResult(IManifest::FResult::EType::NotFound);
	}
}

//-----------------------------------------------------------------------------
/**
 * Creates the next segment request.
 */
IManifest::FResult FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options)
{
	const FStreamSegmentRequestISOBMFF* Request = static_cast<const FStreamSegmentRequestISOBMFF*>(CurrentSegment.Get());
	if (Request)
	{
		TSharedPtrTS<FTimelineAssetISOBMFF> ma = MediaAsset.Pin();
		if (!ma)
		{
			return IManifest::FResult(IManifest::FResult::EType::NotFound);
		}
		// Check if the current request did not already go up to the end of the stream. If so there is no next segment.
		if (!Request->bIsLastSegment)
		{
			if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Chunks || LoaderType == FTimelineAssetISOBMFF::ELoaderType::Fragmented)
			{
				TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe> NextRequest = MakeShared<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe>();
				*NextRequest = *Request;
				NextRequest->DownloadStats = {};
				NextRequest->ConnectionInfo = {};

				if (NextRequest->Track.TrackIterator)
				{
					NextRequest->Track.TrackIterator->GetCurrentChunkRemainingSampleInfo(NextRequest->Track.TrackChunkSampleInfo);
					NextRequest->SegmentDuration.SetFromND(NextRequest->Track.TrackChunkSampleInfo.Duration, NextRequest->Track.TrackIterator->GetTimescale());
					auto DTS = NextRequest->Track.TrackIterator->GetEffectiveDTS();
					auto PTS = NextRequest->Track.TrackIterator->GetEffectivePTS();
					NextRequest->FirstDTS.SetFromND(DTS.GetNumerator(), DTS.GetDenominator());
					NextRequest->bIsLastSegment = NextRequest->Track.TrackChunkSampleInfo.bIsLastChunk;
				}
				else
				{
					NextRequest->bIsLastSegment = true;
				}

				// Check if this is still covered by the playback range
				FTimeValue PlayRangeEnd = Options.PlaybackRange.End;
				if (PlayRangeEnd > ma->GetDuration())
				{
					PlayRangeEnd = ma->GetDuration();
				}
				NextRequest->LastPTS = PlayRangeEnd;
				if (NextRequest->FirstDTS > PlayRangeEnd)
				{
					return IManifest::FResult(IManifest::FResult::EType::PastEOS);
				}

				OutSegment = MoveTemp(NextRequest);
				return IManifest::FResult(IManifest::FResult::EType::Found);
			}
			else if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Muxed)
			{
				// The next request for a multiplex merely continues where this one ended.
				TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe> NextRequest = MakeShared<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe>();
				*NextRequest = *Request;
				NextRequest->DownloadStats = {};
				NextRequest->ConnectionInfo = {};
				AdjustMuxedDownload(NextRequest, MuxedSegmentDuration, EAdjustMuxType::Continuation);
				if (NextRequest->bIsLastSegment)
				{
					return IManifest::FResult(IManifest::FResult::EType::PastEOS);
				}
				OutSegment = MoveTemp(NextRequest);
				return IManifest::FResult(IManifest::FResult::EType::Found);
			}
			else
			{
				unimplemented();
				return IManifest::FResult(IManifest::FResult::EType::NotFound);
			}
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}


//-----------------------------------------------------------------------------
/**
 * Sets up a starting segment request to loop playback to.
 * The streams selected through SelectStream() will be used.
 */
IManifest::FResult FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	return GetStartingSegment(OutSegment, SequenceState, StartPosition, SearchType);
}


//-----------------------------------------------------------------------------
/**
 * Same as GetStartingSegment() except this is for a specific stream (video, audio, ...) only.
 * To be used when a track (language) change is made and a new segment is needed at the current playback position.
 */
IManifest::FResult FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType InStreamType, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& InStartPosition, ESearchType InSearchType)
{
	if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Chunks || LoaderType == FTimelineAssetISOBMFF::ELoaderType::Fragmented)
	{
		TSharedPtrTS<IStreamSegment> NewReq;
		FResult Result = GetStartingSegment(NewReq, InSequenceState, InStartPosition, InSearchType);
		if (Result.IsSuccess())
		{
			// We do not need the starting request itself, only the dependent stream of the desired type.
			const FStreamSegmentRequestISOBMFF* Request = static_cast<const FStreamSegmentRequestISOBMFF*>(NewReq.Get());
			for(int32 i=0; i<Request->DependentStreams.Num(); ++i)
			{
				if (Request->DependentStreams[i]->StreamType == InStreamType)
				{
					OutSegment = Request->DependentStreams[i];
					return IManifest::FResult(IManifest::FResult::EType::Found);
				}
			}
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Creates a segment retry request.
 */
IManifest::FResult FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData)
{
	const FStreamSegmentRequestISOBMFF* Request = static_cast<const FStreamSegmentRequestISOBMFF*>(CurrentSegment.Get());
	if (Request)
	{
		if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Chunks || LoaderType == FTimelineAssetISOBMFF::ELoaderType::Fragmented)
		{
			// The retry is actually a copy of the current request with just updated
			// remaining sample per chunk values of what has not been read so far.
			TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe> NextRequest = MakeShared<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe>();
			*NextRequest = *Request;
			NextRequest->DownloadStats = {};
			NextRequest->ConnectionInfo = {};
			++NextRequest->NumOverallRetries;
			if (NextRequest->Track.TrackIterator)
			{
				NextRequest->Track.TrackIterator->GetCurrentChunkRemainingSampleInfo(NextRequest->Track.TrackChunkSampleInfo);
				NextRequest->SegmentDuration.SetFromND(NextRequest->Track.TrackChunkSampleInfo.Duration, NextRequest->Track.TrackIterator->GetTimescale());
				auto DTS = NextRequest->Track.TrackIterator->GetEffectiveDTS();
				auto PTS = NextRequest->Track.TrackIterator->GetEffectivePTS();
				NextRequest->FirstDTS.SetFromND(DTS.GetNumerator(), DTS.GetDenominator());
				NextRequest->bIsLastSegment = NextRequest->Track.TrackChunkSampleInfo.bIsLastChunk;
			}
			else
			{
				// This cannot happen because there will be an iterator. Just in case it went away though...
				return IManifest::FResult(IManifest::FResult::EType::NotFound);
			}

			OutSegment = MoveTemp(NextRequest);
			return IManifest::FResult(IManifest::FResult::EType::Found);
		}
		else if (LoaderType == FTimelineAssetISOBMFF::ELoaderType::Muxed)
		{
			// The retry request for a multiplex merely continues where this one failed.
			TSharedPtr<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe> NextRequest = MakeShared<FStreamSegmentRequestISOBMFF, ESPMode::ThreadSafe>();
			*NextRequest = *Request;
			NextRequest->DownloadStats = {};
			NextRequest->ConnectionInfo = {};
			++NextRequest->NumOverallRetries;
			AdjustMuxedDownload(NextRequest, MuxedSegmentDuration, EAdjustMuxType::Retry);
			OutSegment = MoveTemp(NextRequest);
			return IManifest::FResult(IManifest::FResult::EType::Found);
		}
		else
		{
			unimplemented();
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Returns the average segment duration.
 */
void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	TSharedPtrTS<FTimelineAssetISOBMFF> ma = MediaAsset.Pin();
	if (ma.IsValid())
	{
		ma->GetAverageSegmentDuration(OutAverageSegmentDuration, AdaptationSetID, RepresentationID);
	}
}


//-----------------------------------------------------------------------------
/**
 * Called by the ABR to increase the delay in fetching the next segment in case the segment returned a 404 when fetched at
 * the announced availability time. This may reduce 404's on the next segment fetches.
 */
void FManifestISOBMFFInternal::FPlayPeriodISOBMFF::IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount)
{
	// No-op.
}


//-----------------------------------------------------------------------------
/**
 * Builds the timeline asset.
 */
FErrorDetail FManifestISOBMFFInternal::FTimelineAssetISOBMFF::Build(IPlayerSessionServices* InPlayerSessionServices, TArray<TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>>&& InParsedRootBoxes, const FString& InURL,
																	TSharedPtr<FMP4DataLoader, ESPMode::ThreadSafe> InDataLoader, FMP4DataLoader::FCancellationCheckDelegate InCancelCheck)
{
	PlayerSessionServices = InPlayerSessionServices;
	ParsedRootBoxes = MoveTemp(InParsedRootBoxes);
	MediaURL = InURL;
	bIsLocalFile = MediaURL.StartsWith(TEXT("file:"), ESearchCase::IgnoreCase);

	TArray<ElectraCDM::IMediaCDM::FCDMCandidate> DRMCandidates;
	auto DRMManager = PlayerSessionServices->GetDRMManager();

	FErrorDetail Error;
	auto SetError = [&Error](const FString& InMsg) -> void
	{
		Error.SetFacility(Facility::EFacility::ISOBMFFPlaylist);
		Error.SetMessage(InMsg);
		Error.SetCode(ERRCODE_MANIFEST_ISOBMFF_NO_PLAYABLE_STREAMS);
	};

	auto FindRootBox = [&](uint32 InType) -> TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>
	{
		auto rb = ParsedRootBoxes.FindByPredicate([Type=InType](const TSharedPtr<MP4Boxes::FMP4BoxBase>& In) { return In->GetType() == Type;});
		return rb ? *rb : nullptr;
	};
	TSharedPtr<MP4Boxes::FMP4BoxMOOV, ESPMode::ThreadSafe> MoovBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxMOOV>(FindRootBox(MP4Utilities::MakeBoxAtom('m','o','o','v')));
	if (!MoovBox)
	{
		SetError(TEXT("No `moov` box found, cannot play this file."));
		return Error;
	}
	// Get the `mvhd` box for the duration of the movie and the timescale other values are measured in.
	auto MvhdBox = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxMVHD>(MP4Utilities::MakeBoxAtom('m','v','h','d'), 0);
	if (!MvhdBox)
	{
		SetError(TEXT("No `mvhd` box found. Invalid file."));
		return Error;
	}

	MP4Utilities::FFractionalTime FullMovieDuration = MvhdBox->GetDuration();

	// Check if this is a fragmented file. We do not look for `sidx` or `mfra` boxes since these are purely optional.
	TSharedPtr<MP4Boxes::FMP4BoxMOOF, ESPMode::ThreadSafe> FirstMoofBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxMOOF>(FindRootBox(MP4Utilities::MakeBoxAtom('m','o','o','f')));
	auto MvexBox = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxMVEX>(MP4Utilities::MakeBoxAtom('m','v','e','x'), 0);
	if (FirstMoofBox && !MvexBox)
	{
		SetError(TEXT("No `mvex` box present in a file containing `moof` fragment boxes."));
		return Error;
	}
	const bool bIsFragmented = MvexBox || FirstMoofBox;
	if (bIsFragmented)
	{
		auto MehdBox = MvexBox->FindBoxRecursive<MP4Boxes::FMP4BoxMEHD>(MP4Utilities::MakeBoxAtom('m','e','h','d'), 0);
		if (MehdBox)
		{
			FullMovieDuration.SetNumerator(MehdBox->GetFragmentDuration());
		}
		else if (FullMovieDuration.GetNumerator() == 0)
		{
			SetError(TEXT("Duration in `mvhd` box is set to zero in a fragmented file that is missing a `mehd` box. This is not a usable mp4/mov file."));
			return Error;
		}
	}

	TArray<TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>> UsableTracks;
	TArray<TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>> AllTracks;
	MoovBox->GetAllBoxInstances(AllTracks, MP4Utilities::MakeBoxAtom('t','r','a','k'));
	int64 LongestTkD = 0;
	int64 ShortestTkD = TNumericLimits<int64>::Max();
	TMap<TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>, TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>> TrakMap;
	TSet<uint32> ReferencedTimecodeTrackIDs;
	TArray<TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>> ExistingTimecodeTracks;
	for(int32 i=0; i<AllTracks.Num(); ++i)
	{
		const TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>& ThisTrakBox(AllTracks[i]);

		TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> TrkInf = MakeShared<FTrackInfo, ESPMode::ThreadSafe>();
		auto Tkhd = ThisTrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxTKHD>(MP4Utilities::MakeBoxAtom('t','k','h','d'), 1);
		auto Mdhd = ThisTrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxMDHD>(MP4Utilities::MakeBoxAtom('m','d','h','d'), 3);
		if (!Tkhd || !Mdhd)
		{
			continue;
		}
		TrkInf->TrackID = Tkhd->GetTrackID();
		TrkInf->TrackTimescale = Mdhd->GetTimescale();
		int64 TrackDuration = Tkhd->GetDuration();
		if (TrackDuration == TNumericLimits<int64>::Max())
		{
			continue;
		}

		// Take note if this track references a timecode track. We need to do this for every track, even those that
		// we cannot use to prevent using any unreferenced timecode track for another usable track if the track
		// itself does not reference any timecode track.
		auto Tref = ThisTrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxTREF>(MP4Utilities::MakeBoxAtom('t','r','e','f'), 1);
		if (Tref)
		{
			auto References = Tref->GetEntriesOfType(MP4Utilities::MakeBoxAtom('t','m','c','d'));
			if (References.Num())
			{
				if (References.Num() > 1)
				{
					LogMessage(IInfoLog::Warning, FString::Printf(TEXT("Track #%u contains more than one `tmcd` reference box. Using first reference only."), TrkInf->TrackID));
				}
				if (References[0].TrackIDs.Num())
				{
					if (References[0].TrackIDs.Num() > 1)
					{
						LogMessage(IInfoLog::Warning, FString::Printf(TEXT("Track #%u references more than one timecode track. Using first reference only."), TrkInf->TrackID));
					}
					TrkInf->ReferencedTimecodeTrackID = References[0].TrackIDs[0];
					// Tag every track nonetheless.
					for(auto& RefTkId : References[0].TrackIDs)
					{
						ReferencedTimecodeTrackIDs.Add(RefTkId);
					}
				}
			}
		}
		/*
			if (!Tkhd->IsEnabled())
			{
				continue;
			}
		*/

		FString Msg;
		if (!ElectraDecodersUtil::MP4::GetTrackFormatInfo(Msg, TrkInf->CodecFormat, TrkInf->DrmFormat, TrkInf->DecoderInformation, ThisTrakBox, TrkInf->TrackID, true))
		{
			continue;
		}
		// Skip tracks that cannot be decoded.
		if (!TrkInf->DecoderInformation.bIsDecodable)
		{
			continue;
		}
		// Can we decode this encrypted track?
		if (TrkInf->DrmFormat.IsEncrypted())
		{
			if (!TrkInf->DrmFormat.EncryptionInfo.IsType<FDRMTypeFormat::FISOEncryptionInfo>() || !DRMManager)
			{
				continue;
			}
			const FDRMTypeFormat::FISOEncryptionInfo& EncInfo(TrkInf->DrmFormat.EncryptionInfo.Get<FDRMTypeFormat::FISOEncryptionInfo>());
			if (!EncInfo.Scheme)
			{
				continue;
			}
			FString Scheme = FDRMManager::MakePrintableStringFromUint32(EncInfo.Scheme);
			TSharedPtr<ElectraCDM::IMediaCDMCapabilities, ESPMode::ThreadSafe> DRMCapabilities;
			bool bCanBeDecrypted = false;
			for(int32 nI=0; nI<EncInfo.CDMInfos.Num(); ++nI)
			{
				FString uuid = FDRMManager::MakeHexStringFromArray(EncInfo.CDMInfos[nI].SystemID);
				DRMCapabilities = DRMManager->GetCDMCapabilitiesForScheme(uuid, FString(), FString());
				if (DRMCapabilities.IsValid())
				{
					ElectraCDM::IMediaCDMCapabilities::ESupportResult Result = DRMCapabilities->SupportsCipher(Scheme);
					if (Result == ElectraCDM::IMediaCDMCapabilities::ESupportResult::Supported)
					{
						bCanBeDecrypted = true;
						ElectraCDM::IMediaCDM::FCDMCandidate cand;
						cand.SchemeId = uuid;
						cand.CommonScheme = Scheme;
						for(int32 j=0; j<EncInfo.CDMInfos[nI].KIDs.Num(); ++j)
						{
							cand.DefaultKIDs.Emplace(FDRMManager::MakeHexStringFromArray(EncInfo.CDMInfos[nI].KIDs[j]));
						}
						DRMCandidates.Emplace(MoveTemp(cand));
					}
				}
			}
			// If this track cannot be decrypted we skip over it.
			if (!bCanBeDecrypted)
			{
				continue;
			}
		}

		// Can this track be decoded and meets the current playback restrictions?
		FStreamCodecInformation ci(TrkInf->CodecFormat);
		IPlayerStreamFilter* StreamFilter = PlayerSessionServices->GetStreamFilter();
		if (TrkInf->CodecFormat.Type == FCodecTypeFormat::EType::Timecode || (StreamFilter && StreamFilter->CanDecodeStream(ci)))
		{
			// If the file is fragmented we collect all the track fragment boxes for this track.
			if (bIsFragmented)
			{
				TrkInf->FragmentInfo = MakeShared<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe>();
				if (!TrkInf->FragmentInfo->Prepare(ThisTrakBox, MvexBox, ParsedRootBoxes))
				{
					LogMessage(IInfoLog::Warning, FString::Printf(TEXT("Fragmented track #%u could not be prepared, ignoring this track."), TrkInf->TrackID));
					continue;
				}

				// The summed duration is in media timescale, so we need to transform it into movie timescale.
				FTimeFraction MediaDur(TrkInf->FragmentInfo->GetTotalDuration(), Mdhd->GetTimescale());
				TrackDuration = MediaDur.GetAsTimebase(FullMovieDuration.GetDenominator());
				// It is possible that the movie duration is only describing the content contained in the moov box.
				// If that is smaller than the track duration, update the movie duration with the track duration.
				if (TrkInf->CodecFormat.Type != FCodecTypeFormat::EType::Timecode && FullMovieDuration.GetNumerator() < TrackDuration)
				{
					FullMovieDuration.SetNumerator(TrackDuration);
				}
			}

			// Update track durations only for video and audio tracks.
			if (TrkInf->CodecFormat.Type == FCodecTypeFormat::EType::Video || TrkInf->CodecFormat.Type == FCodecTypeFormat::EType::Audio)
			{
				if (TrackDuration > LongestTkD)
				{
					LongestTkD = TrackDuration;
				}
				if (TrackDuration < ShortestTkD)
				{
					ShortestTkD = TrackDuration;
				}
			}
			TrakMap.Add(TrkInf, ThisTrakBox);
			// If this is a timecode add it to a separate list for convenience later.
			if (TrkInf->CodecFormat.Type == FCodecTypeFormat::EType::Timecode)
			{
				ExistingTimecodeTracks.Emplace(TrkInf);
			}
			UsableTracks.Emplace(MoveTemp(TrkInf));
		}
	}

	MP4Utilities::FFractionalTime AdjustedMovieDuration;
	bool bTruncateToShortestTrack = DEFAULT_TRUNCATE_MOVIE_TO_SHORTEST_TRACK;
	auto UserOpt = PlayerSessionServices->GetOptionValue(ISOBMFF::OptionKeyISOBMFFTruncateToShortestTrack);
	if (UserOpt.IsValid())
	{
		bTruncateToShortestTrack = UserOpt.SafeGetBool(false);
	}
	int32 CVarOpt = CVarElectraISOBMFFPlayerForceDuration.GetValueOnAnyThread();
	if (CVarOpt == 1)
	{
		bTruncateToShortestTrack = true;
	}
	else if (CVarOpt == 2)
	{
		bTruncateToShortestTrack = false;
	}
	if (bTruncateToShortestTrack)
	{
		OverallTrackDuration.SetFromND(ShortestTkD, FullMovieDuration.GetDenominator());
		AdjustedMovieDuration.SetFromND(ShortestTkD, FullMovieDuration.GetDenominator());
	}
	else
	{
		OverallTrackDuration.SetFromND(FullMovieDuration.GetNumerator(), FullMovieDuration.GetDenominator());
		AdjustedMovieDuration = FullMovieDuration;
	}

	int32 NumAddedTracks = 0;
	for(int32 i=0; i<UsableTracks.Num(); ++i)
	{
		TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> TrkInf = UsableTracks[i];

		auto TRAKBox = TrakMap[TrkInf];
		TrkInf->MP4Track = MP4Boxes::FMP4Track::Create(TRAKBox, TrkInf->FragmentInfo);
		if (!TrkInf->MP4Track->Prepare(FullMovieDuration, AdjustedMovieDuration))
		{
			continue;
		}
		// Timecode tracks were set up for convenience only. They are not to be handled any further.
		if (TrkInf->CodecFormat.Type == FCodecTypeFormat::EType::Timecode)
		{
			continue;
		}

		// In an mp4 file we treat every track as a single adaptation set with one representation only.
		// That's because by definition an adaptation set contains the same content at different bitrates and resolutions, but
		// the type, language and codec has to be the same.
		switch(TrkInf->CodecFormat.Type)
		{
			case FCodecTypeFormat::EType::Video:
			{
				TSharedPtrTS<FAdaptationSetISOBMFF> AdaptationSet = MakeSharedTS<FAdaptationSetISOBMFF>();
				AdaptationSet->CreateFrom(TrkInf, FStreamCodecInformation(TrkInf->CodecFormat), NumAddedTracks, VideoAdaptationSets.Num());
				VideoAdaptationSets.Add(MoveTemp(AdaptationSet));
				break;
			}
			case FCodecTypeFormat::EType::Audio:
			{
				TSharedPtrTS<FAdaptationSetISOBMFF> AdaptationSet = MakeSharedTS<FAdaptationSetISOBMFF>();
				AdaptationSet->CreateFrom(TrkInf, FStreamCodecInformation(TrkInf->CodecFormat), NumAddedTracks, AudioAdaptationSets.Num());
				AudioAdaptationSets.Add(MoveTemp(AdaptationSet));
				break;
			}
			case FCodecTypeFormat::EType::Subtitle:
			{
				TSharedPtrTS<FAdaptationSetISOBMFF> AdaptationSet = MakeSharedTS<FAdaptationSetISOBMFF>();
				AdaptationSet->CreateFrom(TrkInf, FStreamCodecInformation(TrkInf->CodecFormat), NumAddedTracks, SubtitleAdaptationSets.Num());
				SubtitleAdaptationSets.Add(MoveTemp(AdaptationSet));
				break;
			}
			default:
			{
				break;
			}
		}
		++NumAddedTracks;
	}

	// No playable content?
	if (VideoAdaptationSets.Num() == 0 && AudioAdaptationSets.Num() == 0)
	{
		SetError(TEXT("No playable streams in this mp4"));
		return Error;
	}

	// Determine the presumably best way to load the movie
	if (bIsFragmented)
	{
		// If the movie is fragmented we have to load fragments.
		LoaderType = ELoaderType::Fragmented;
	}
	else if (NumAddedTracks == 1)
	{
		// If there is only a single track we use the multiplex loader to reduce the
		// number of individual chunk requests.
		LoaderType = ELoaderType::Muxed;
	}
	else //if (NumAddedTracks == 1)
	{
		// When there are multiple tracks we try to determine the interleave duration.
		// If it is tight enough we can use the multiplex loader but if it is too
		// large we load by chunks to not risk any one buffer not filling up enough
		// while reading multiplexed.
		LoaderType = ELoaderType::Muxed;
		const int32 kNumProbeSamples = 100;
		const FTimespan kMaxPermittedInterleaveDuration(FTimespan::FromSeconds(0.75));
		bool bProbeDone = false;
		for(int32 i=0; !bProbeDone && i<UsableTracks.Num(); ++i)
		{
			TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> TrkInf = UsableTracks[i];
			// Only look at video or audio tracks.
			if (TrkInf->CodecFormat.Type == FCodecTypeFormat::EType::Video || TrkInf->CodecFormat.Type == FCodecTypeFormat::EType::Audio)
			{
				auto TkIt = TrkInf->MP4Track->CreateIterator();
				FTimespan Dur = TkIt->GetDurationAsTimespan();
				int64 Off = TkIt->GetSampleFileOffset();
				int64 NxtOff = Off + TkIt->GetSampleSize();
				for(int32 si=1; si<kNumProbeSamples && TkIt->Next(); ++si)
				{
					if (TkIt->GetSampleFileOffset() != NxtOff)
					{
						break;
					}
					Dur += TkIt->GetDurationAsTimespan();
					if (Dur > kMaxPermittedInterleaveDuration)
					{
						LogMessage(IInfoLog::Verbose, FString::Printf(TEXT("Track interleave duration seems large, using chunk read mode.")));
						LoaderType = ELoaderType::Chunks;
						bProbeDone = true;
						break;
					}
					NxtOff += TkIt->GetSampleSize();
				}
			}
		}
	}

	// Should we parse any timecodes?
	bool bPrepareFirstTimecodes = PlayerSessionServices ? PlayerSessionServices->HaveOptionValue(Electra::OptionKeyParseTimecodeInfo) : false;

	// Is there a metadata box?
	auto Udta = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxUDTA>(MP4Utilities::MakeBoxAtom('u','d','t','a'), 1);
	if (Udta)
	{
		auto Meta = Udta->FindBoxRecursive<MP4Boxes::FMP4BoxMETA>(MP4Utilities::MakeBoxAtom('m','e','t','a'), 1);
		if (Meta)
		{
			auto Hdlr = Meta->GetHandler();
			if (Hdlr)
			{
				uint32 hdlr = Hdlr->GetHandlerType();
				uint32 res0 = Hdlr->GetReservedValue(0);
				TArray<MP4Utilities::FMP4BoxInfo> Boxes;
				for(auto &ch : Meta->GetChildren())
				{
					Boxes.Emplace(ch->GetBoxInfo());
				}
				MediaMetadata = MakeSharedTS<FMetadataParser>();
				if (MediaMetadata->Parse(hdlr, res0, Boxes) == FMetadataParser::EResult::Success)
				{
					PlayerSessionServices->SendMessageToPlayer(FPlaylistMetadataUpdateMessage::Create(FTimeValue(), MediaMetadata, false));
				}
				else
				{
					MediaMetadata.Reset();
				}
			}
		}
		// When we are to handle first timecodes let's see if there is a global timecode.
		if (bPrepareFirstTimecodes)
		{
			auto CTIMBox = Udta->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xa9U,'T','I','M'), 0);
			auto CTSCBox = Udta->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xa9U,'T','S','C'), 0);
			auto CTSZBox = Udta->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xa9U,'T','S','Z'), 0);
			if (CTIMBox.IsValid() && CTSCBox.IsValid() && CTSZBox.IsValid())
			{
				auto GetValue = [](const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& InBox) -> FString
				{
					MP4Utilities::FMP4AtomReader timReader(InBox->GetBoxData());
					FString String;
					uint16 StringLength, UnknownValue;
					if (timReader.Read(StringLength)&&  timReader.Read(UnknownValue))
					{
						if (timReader.ReadString(String, StringLength))
						{
							return String;
						}
					}
					return FString();
				};
				FTrackInfo::FFirstSampleTimecode tc;
				FFrameRate fr;
				tc.Timecode = GetValue(CTIMBox);
				tc.Framerate = FString::Printf(TEXT("%s/%s"), *GetValue(CTSCBox), *GetValue(CTSZBox));
				TOptional<FTimecode> ptc = FTimecode::ParseTimecode(tc.Timecode);
				if (ptc.IsSet() && TryParseString(fr, *tc.Framerate))
				{
					FFrameNumber fn = ptc.GetValue().ToFrameNumber(fr);
					tc.TimecodeValue = (uint32) fn.Value;
				}
				// Note: this also sets it on timecode tracks, which is ok since the value isn't used there.
				for(int32 i=0; i<UsableTracks.Num(); ++i)
				{
					UsableTracks[i]->FirstSampleTimecode = tc;
				}
			}
		}
	}

	// Resolve explicitly referenced timecode tracks?
	if (bPrepareFirstTimecodes && ExistingTimecodeTracks.Num())
	{
		TMap<uint32, FManifestISOBMFFInternal::FTrackInfo::FFirstSampleTimecode> TrackFirstTimecodeMap;
		auto LoadFirstTimecodeSample = [&](uint32 InTrackID) -> bool
		{
			if (auto TrInfp = ExistingTimecodeTracks.FindByPredicate([InTrackID](const TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>& tk) { return tk->TrackID == InTrackID; } ))
			{
				if (auto TkIt = (*TrInfp)->MP4Track ? (*TrInfp)->MP4Track->CreateIterator() : nullptr)
				{
					TArray<uint8> Data;
					Data.SetNumUninitialized(TkIt->GetSampleSize());
					int64 Result = InDataLoader->ReadData(Data.GetData(), TkIt->GetSampleSize(), TkIt->GetSampleFileOffset(), InCancelCheck);
					if (Result == TkIt->GetSampleSize())
					{
						// Only handle 4 byte size timecode for now.
						if (TkIt->GetSampleSize() == 4)
						{
							FCodecTypeFormat::FTMCDTimecode TimecodeInfo((*TrInfp)->CodecFormat.Properties.Get<FCodecTypeFormat::FTMCDTimecode>());
							FManifestISOBMFFInternal::FTrackInfo::FFirstSampleTimecode firstTC;
							firstTC.TimecodeValue = MP4Utilities::GetFromBigEndian(*reinterpret_cast<const uint32*>(Data.GetData()));
							firstTC.Framerate = TimecodeInfo.GetFrameRate().ToPrettyText().ToString();
							firstTC.Timecode = TimecodeInfo.ConvertToTimecode(firstTC.TimecodeValue).ToString();
							TrackFirstTimecodeMap.Add(InTrackID, firstTC);
							return true;
						}
					}
				}
			}
			return false;
		};

		// If any track explicitly references a timecode track we only handle those.
		if (ReferencedTimecodeTrackIDs.Num())
		{
			// Load the first timecode from each referenced track and populate a map with it.
			// If this fails we just ignore it.
			for(auto& rttid : ReferencedTimecodeTrackIDs)
			{
				LoadFirstTimecodeSample(rttid);
			}
			// Go over the tracks and assign the timecodes we loaded successfully.
			for(auto &Tki : UsableTracks)
			{
				if (Tki->ReferencedTimecodeTrackID.IsSet() && TrackFirstTimecodeMap.Contains(Tki->ReferencedTimecodeTrackID.GetValue()))
				{
					Tki->FirstSampleTimecode = TrackFirstTimecodeMap[Tki->ReferencedTimecodeTrackID.GetValue()];
				}
			}
		}
		// Otherwise, if no track references a timecode explicitly, then if there is only a single timecode track it applies to all tracks.
		else if (ExistingTimecodeTracks.Num() == 1)
		{
			if (LoadFirstTimecodeSample(ExistingTimecodeTracks[0]->TrackID))
			{
				for(auto &Tki : UsableTracks)
				{
					Tki->FirstSampleTimecode = TrackFirstTimecodeMap[ExistingTimecodeTracks[0]->TrackID];
				}
			}
		}
	}
	// If timecodes are to be extracted, set the one of the first video track to begin with.
	if (bPrepareFirstTimecodes)
	{
		if (VideoAdaptationSets.Num())
		{
			auto &tc = VideoAdaptationSets[0]->GetRepresentation()->GetTrackInfo()->FirstSampleTimecode;
			if (tc.IsSet())
			{
				const FVariantValue StartTimecodeValue(tc.GetValue().Timecode);
				const FVariantValue StartTimecodeFrameRate(tc.GetValue().Framerate);
				PlayerSessionServices->GetMediaInfoDictionary().Set(MediaInfoStartTimecodeValue, StartTimecodeValue);
				PlayerSessionServices->GetMediaInfoDictionary().Set(MediaInfoStartTimecodeFrameRate, StartTimecodeFrameRate);
			}
		}
	}

	// Request licenses
	if (DRMCandidates.Num())
	{
		ElectraCDM::ECDMError Result = DRMManager->CreateDRMClient(DrmClient, DRMCandidates);
		if (Result == ElectraCDM::ECDMError::Success && DrmClient.IsValid())
		{
			DrmClient->RegisterEventListener(DRMManager);
			DrmClient->SetLicenseServerURL(MediaURL);
			DrmClient->PrepareLicenses();
		}
		else
		{
			Error.SetFacility(Facility::EFacility::ISOBMFFPlaylist);
			Error.SetMessage(FString::Printf(TEXT("Failed to create DRM client with error %d"), (int32)Result));
			Error.SetCode(ERRCODE_MANIFEST_ISOBMFF_DRM_ERROR);
		}
	}
	return Error;
}


void FManifestISOBMFFInternal::FTimelineAssetISOBMFF::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::ISOBMFFPlaylist, Level, Message);
	}
}


void FManifestISOBMFFInternal::FTimelineAssetISOBMFF::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	// This is not expected to be called. And if it does we return a dummy entry.
	OutAverageSegmentDuration.SetFromSeconds(5.0);
}


void FManifestISOBMFFInternal::FAdaptationSetISOBMFF::CreateFrom(TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> InTrack, const FStreamCodecInformation& InCodecInfo, int32 /*InSequentialIndex*/, int32 InIndexOfType)
{
	Representation = MakeSharedTS<FRepresentationISOBMFF>();
	Representation->CreateFrom(InTrack, InCodecInfo);
	CodecRFC6381 = InTrack->CodecFormat.RFC6381;
	KindOfTrack = InIndexOfType == 0 ? TEXT("main") : TEXT("translation");
	LanguageTag = InTrack->CodecFormat.LanguageTag;
	UniqueIdentifier = Representation->GetUniqueIdentifier();
	Duration = InTrack->MP4Track->GetMappedTrackDuration().GetAsTimespan();
	InternalTrackIndex = InIndexOfType;
}

void FManifestISOBMFFInternal::FRepresentationISOBMFF::CreateFrom(TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> InTrack, const FStreamCodecInformation& InCodecInfo)
{
	Track = MoveTemp(InTrack);
	CodecInformation = InCodecInfo;

	// The unique identifier will be the track ID inside the mp4.
	// NOTE: This *MUST* be just a number since it gets parsed back out from a string into a number later! Do *NOT* prepend/append any string literals!!
	UniqueIdentifier = LexToString(Track->TrackID);

	Name = Track->MP4Track->GetCommonMetadata().Name;
	if (Name.IsEmpty())
	{
		Name = FString::Printf(TEXT("%s (ID=%u)"), *Track->MP4Track->GetCommonMetadata().HandlerName, Track->TrackID);
	}

	// Get bitrate from the average or max bitrate as stored in the track. If not stored it will be 0.
	Bitrate = Track->CodecFormat.Bitrate ? Track->CodecFormat.Bitrate : Track->CodecFormat.AverageBitrate;

	// With no bitrate available we set some defaults. This is mainly to avoid a bitrate of 0 from being surfaced that would prevent
	// events like the initial bitrate change that needs to transition away from 0 to something real.
	if (Bitrate == 0)
	{
		switch(Track->CodecFormat.Type)
		{
			case FCodecTypeFormat::EType::Video:
			{
				Bitrate = 1 * 1024 * 1024;
				break;
			}
			case FCodecTypeFormat::EType::Audio:
			{
				Bitrate = 64 * 1024;
				break;
			}
			case FCodecTypeFormat::EType::Subtitle:
			{
				Bitrate = 8 * 1024;
				break;
			}
			default:
			{
				// Whatever it is, assume it's a low bitrate.
				Bitrate = 32 * 1024;
				break;
			}
		}
	}
	if (!CodecInformation.GetBitrate())
	{
		CodecInformation.SetBitrate(Bitrate);
	}
}

} // namespace Electra

