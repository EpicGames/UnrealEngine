// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "Player/PlaybackTimeline.h"
#include "Player/PlayerStreamReader.h"
#include "Utilities/URLParser.h"
#include "Utilities/MP4Helpers.h"
#include "MediaStreamMetadata.h"
#include "MP4Utilities.h"
#include "MP4Boxes.h"
#include "MP4Track.h"
#include "ElectraCDMClient.h"

namespace Electra
{

/**
 * This class represents the internal "playlist" of an mp4 file.
 * All metadata of the tracks it is composed of are maintained by this class.
 */
class FManifestISOBMFFInternal : public IManifest, public TSharedFromThis<FManifestISOBMFFInternal, ESPMode::ThreadSafe>
{
public:
	struct FTrackInfo
	{
		TSharedPtr<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe> FragmentInfo;
		TSharedPtr<MP4Boxes::FMP4Track, ESPMode::ThreadSafe> MP4Track;

		FCodecTypeFormat CodecFormat;
		FDRMTypeFormat DrmFormat;
		FDecoderInformation DecoderInformation;
		uint32 TrackID = 0;
		uint32 TrackTimescale = 0;

		struct FFirstSampleTimecode
		{
			FString Timecode;
			FString Framerate;
			uint32 TimecodeValue = 0;
		};
		TOptional<FFirstSampleTimecode> FirstSampleTimecode;
		TOptional<uint32> ReferencedTimecodeTrackID;
	};

public:
	FManifestISOBMFFInternal(IPlayerSessionServices* InPlayerSessionServices);
	virtual ~FManifestISOBMFFInternal();

	FErrorDetail Build(TArray<TSharedPtr<MP4Boxes::FMP4BoxBase>>&& InParsedRootBoxes, const FString& URL,
					   TSharedPtr<FMP4DataLoader, ESPMode::ThreadSafe> InDataLoader, FMP4DataLoader::FCancellationCheckDelegate InCancelCheck);

	EMediaFormatType GetMediaFormatType() const override
	{ return EMediaFormatType::ISOBMFF; }
	bool ReloadsPlaylistPeriodically() const override
	{ return false; }
	bool SupportsMultipleTracksPerKind() const override
	{ return true; }
	bool OrderOfTracksCanChangeDynamically() const override
	{ return false; }
	EType GetPresentationType() const override
	{ return EType::OnDemand; }
	EReplayEventType GetReplayType() const override
	{ return IManifest::EReplayEventType::NoReplay; }
	TSharedPtrTS<const FLowLatencyDescriptor> GetLowLatencyDescriptor() const override
	{ return nullptr; }
	FTimeValue CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const override
	{ return FTimeValue(); }
	FTimeValue GetAnchorTime() const override
	{ return FTimeValue::GetZero(); }
	FTimeRange GetTotalTimeRange() const override
	{ return MediaAsset.IsValid() ? MediaAsset->GetTimeRange() : FTimeRange(); }
	FTimeRange GetSeekableTimeRange() const override
	{ return GetTotalTimeRange(); }
	FTimeRange GetPlaybackRange(EPlaybackRangeType InRangeType) const override;
	FTimeValue GetDuration() const override
	{ return MediaAsset.IsValid() ? MediaAsset->GetDuration() : FTimeValue(); }
	FTimeValue GetDefaultStartTime() const override
	{ return DefaultStartTime; }
	void ClearDefaultStartTime() override
	{ DefaultStartTime.SetToInvalid(); }
	FTimeValue GetDefaultEndTime() const override
	{ return DefaultEndTime; }
	void ClearDefaultEndTime() override
	{ DefaultEndTime.SetToInvalid(); }
	void GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const override;
	void UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData) override
	{ }
	FTimeValue GetMinBufferTime() const override;
	FTimeValue GetDesiredLiveLatency() const override
	{ return FTimeValue(); }
	ELiveEdgePlayMode GetLiveEdgePlayMode() const override
	{ return IManifest::ELiveEdgePlayMode::Never; }
	TRangeSet<double> GetPossiblePlaybackRates(EPlayRateType InForType) const override;
	TSharedPtrTS<IProducerReferenceTimeInfo> GetProducerReferenceTimeInfo(int64 ID) const override;
	void UpdateDynamicRefetchCounter() override
	{ }
	void PrepareForLooping(int32 InNumLoopsToAdd) override
	{ }
	void TriggerClockSync(EClockSyncType InClockSyncType) override
	{ }
	void TriggerPlaylistRefresh() override
	{ }
	void ReachedStableBuffer() override
	{ }

	IStreamReader* CreateStreamReaderHandler() override;
	FResult FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
	FResult FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment) override;


	class FRepresentationISOBMFF : public IPlaybackAssetRepresentation
	{
	public:
		virtual ~FRepresentationISOBMFF() = default;

		void CreateFrom(TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> InTrack, const FStreamCodecInformation& InCodecInfo);

		FString GetUniqueIdentifier() const override
		{ return UniqueIdentifier; }
		const FStreamCodecInformation& GetCodecInformation() const override
		{ return CodecInformation; }
		int32 GetBitrate() const override
		{ return Bitrate; }
		int32 GetQualityIndex() const override
		{ return 0; }
		bool CanBePlayed() const override
		{ return true; }

		const FString& GetName() const
		{ return Name; }
		TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> GetTrackInfo() const
		{ return Track; }
	private:
		TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> Track;
		FStreamCodecInformation CodecInformation;
		FString UniqueIdentifier;
		FString Name;
		int32 Bitrate;
	};

	class FAdaptationSetISOBMFF : public IPlaybackAssetAdaptationSet
	{
	public:
		virtual ~FAdaptationSetISOBMFF() = default;

		void CreateFrom(TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> InTrack, const FStreamCodecInformation& InCodecInfo, int32 InSequentialIndex, int32 InIndexOfType);

		FString GetUniqueIdentifier() const override
		{ return UniqueIdentifier; }
		FString GetListOfCodecs() const override
		{ return CodecRFC6381; }
		const BCP47::FLanguageTag& GetLanguageTag() const override
		{ return LanguageTag; }
		int32 GetNumberOfRepresentations() const override
		{ return 1; }
		bool IsLowLatencyEnabled() const override
		{ return false; }
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByIndex(int32 RepresentationIndex) const override
		{ return RepresentationIndex == 0 ? Representation : TSharedPtrTS<IPlaybackAssetRepresentation>(); }
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByUniqueIdentifier(const FString& InUniqueIdentifier) const override
		{ return Representation.IsValid() && Representation->GetUniqueIdentifier() == InUniqueIdentifier ? Representation : TSharedPtrTS<IPlaybackAssetRepresentation>(); }
		TSharedPtrTS<FRepresentationISOBMFF> GetRepresentation() const
		{ return Representation; }
		FString GetTrackKind() const
		{ return KindOfTrack; }
		int32 GetInternalTrackIndex() const
		{ return InternalTrackIndex; }
		FTimespan GetInternalTrackDuration() const
		{ return Duration; }
	private:
		TSharedPtrTS<FRepresentationISOBMFF> Representation;
		BCP47::FLanguageTag LanguageTag;
		FString CodecRFC6381;
		FString KindOfTrack;
		FString UniqueIdentifier;
		FTimespan Duration;
		int32 InternalTrackIndex = -1;
	};

	class FTimelineAssetISOBMFF : public ITimelineMediaAsset, public TSharedFromThis<FTimelineAssetISOBMFF, ESPMode::ThreadSafe>
	{
	public:
		FTimelineAssetISOBMFF()
			: PlayerSessionServices(nullptr)
		{ }

		virtual ~FTimelineAssetISOBMFF() = default;

		FErrorDetail Build(IPlayerSessionServices* InPlayerSessionServices, TArray<TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>>&& InParsedRootBoxes, const FString& InURL,
						   TSharedPtr<FMP4DataLoader, ESPMode::ThreadSafe> InDataLoader, FMP4DataLoader::FCancellationCheckDelegate InCancelCheck);

		FTimeRange GetTimeRange() const override
		{
			FTimeRange tr;
			tr.Start.SetToZero();
			tr.End = GetDuration();
			return tr;
		}

		FTimeValue GetDuration() const override
		{ return OverallTrackDuration; }

		FString GetAssetIdentifier() const override
		{ return FString("mp4-asset.0"); }
		FString GetUniqueIdentifier() const override
		{ return FString("mp4-media.0"); }
		int32 GetNumberOfAdaptationSets(EStreamType OfStreamType) const override
		{
			switch(OfStreamType)
			{
				case EStreamType::Video:
				{
					return VideoAdaptationSets.Num();
				}
				case EStreamType::Audio:
				{
					return AudioAdaptationSets.Num();
				}
				case EStreamType::Subtitle:
				{
					return SubtitleAdaptationSets.Num();
				}
				default:
				{
					return 0;
				}
			}
		}
		TSharedPtrTS<IPlaybackAssetAdaptationSet> GetAdaptationSetByTypeAndIndex(EStreamType OfStreamType, int32 AdaptationSetIndex) const override
		{
			switch(OfStreamType)
			{
				case EStreamType::Video:
				{
					return AdaptationSetIndex < VideoAdaptationSets.Num() ? VideoAdaptationSets[AdaptationSetIndex] : TSharedPtrTS<IPlaybackAssetAdaptationSet>();
				}
				case EStreamType::Audio:
				{
					return AdaptationSetIndex < AudioAdaptationSets.Num() ? AudioAdaptationSets[AdaptationSetIndex] : TSharedPtrTS<IPlaybackAssetAdaptationSet>();
				}
				case EStreamType::Subtitle:
				{
					return AdaptationSetIndex < SubtitleAdaptationSets.Num() ? SubtitleAdaptationSets[AdaptationSetIndex] : TSharedPtrTS<IPlaybackAssetAdaptationSet>();
				}
			}
			return TSharedPtrTS<IPlaybackAssetAdaptationSet>();
		}

		void GetMetaData(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const override
		{
			for(int32 i=0, iMax=GetNumberOfAdaptationSets(StreamType); i<iMax; ++i)
			{
				TSharedPtrTS<FAdaptationSetISOBMFF> AdaptSet = StaticCastSharedPtr<FAdaptationSetISOBMFF>(GetAdaptationSetByTypeAndIndex(StreamType, i));
				if (AdaptSet.IsValid())
				{
					FTrackMetadata tm;
					tm.ID = AdaptSet->GetUniqueIdentifier();
					tm.LanguageTagRFC5646 = AdaptSet->GetLanguageTag();
					tm.Kind = i==0 ? TEXT("main") : TEXT("translation");
					tm.Duration = AdaptSet->GetInternalTrackDuration();

					for(int32 j=0, jMax=AdaptSet->GetNumberOfRepresentations(); j<jMax; ++j)
					{
						TSharedPtrTS<FRepresentationISOBMFF> Repr = StaticCastSharedPtr<FRepresentationISOBMFF>(AdaptSet->GetRepresentationByIndex(j));
						if (Repr.IsValid())
						{
							FStreamMetadata sd;
							sd.Bandwidth = Repr->GetBitrate();
							sd.CodecInformation = Repr->GetCodecInformation();
							sd.ID = Repr->GetUniqueIdentifier();
							// There is only 1 "stream" per "track" so we can set the highest bitrate and codec info the same as the track.
							tm.HighestBandwidth = sd.Bandwidth;
							tm.HighestBandwidthCodec = sd.CodecInformation;

							tm.Label = Repr->GetName();

							tm.StreamDetails.Emplace(MoveTemp(sd));
						}
					}
					OutMetadata.Emplace(MoveTemp(tm));
				}
			}
		}

		void UpdateRunningMetaData(const FString& InKindOfValue, const FVariant& InNewValue) override
		{ }

		void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID);
		const FString& GetMediaURL() const
		{
			return MediaURL;
		}

		TSharedPtrTS<FAdaptationSetISOBMFF> GetAdaptationSetByTypeAndID(EStreamType InStreamType, const FString& InID) const
		{
			const TArray<TSharedPtrTS<FAdaptationSetISOBMFF>>* AdSet = nullptr;
			switch(InStreamType)
			{
				case EStreamType::Video:
				{
					AdSet = &VideoAdaptationSets;
					break;
				}
				case EStreamType::Audio:
				{
					AdSet = &AudioAdaptationSets;
					break;
				}
				case EStreamType::Subtitle:
				{
					AdSet = &SubtitleAdaptationSets;
					break;
				}
				default:
				{
					return nullptr;
				}
			}
			auto Adpt = (*AdSet).FindByPredicate([InID](const TSharedPtrTS<FAdaptationSetISOBMFF>& e) {return e->GetUniqueIdentifier().Equals(InID);});
			return Adpt ? *Adpt : nullptr;
		}

		enum class ELoaderType
		{
			Chunks,
			Muxed,
			Fragmented
		};
		ELoaderType GetLoaderType() const
		{ return LoaderType; }

		TSharedPtrTS<ElectraCDM::IMediaCDMClient> GetDrmClient() const
		{ return DrmClient; }

	private:
		struct FPlayRangeEndInfo
		{
			void Clear()
			{
				Time.SetToInvalid();
				FileOffsetsPerTrackID.Empty();
				TotalFileEndOffset = -1;
			}
			FTimeValue Time;
			TArray<int64> FileOffsetsPerTrackID;
			int64 TotalFileEndOffset = -1;
		};

		void LogMessage(IInfoLog::ELevel Level, const FString& Message);
		IPlayerSessionServices* PlayerSessionServices = nullptr;
		FString MediaURL;
		bool bIsLocalFile { false };
		ELoaderType LoaderType { ELoaderType::Chunks };
		FTimeValue OverallTrackDuration;
		TArray<TSharedPtr<MP4Boxes::FMP4BoxBase>> ParsedRootBoxes;
		TArray<TSharedPtrTS<FAdaptationSetISOBMFF>> VideoAdaptationSets;
		TArray<TSharedPtrTS<FAdaptationSetISOBMFF>> AudioAdaptationSets;
		TArray<TSharedPtrTS<FAdaptationSetISOBMFF>> SubtitleAdaptationSets;
		FPlayRangeEndInfo PlayRangeEndInfo;
		// Metadata from the 'meta' box, if any.
		TSharedPtrTS<FMetadataParser> MediaMetadata;
		TSharedPtrTS<ElectraCDM::IMediaCDMClient> DrmClient;
	};


	class FPlayPeriodISOBMFF : public IManifest::IPlayPeriod
	{
	public:
		FPlayPeriodISOBMFF(TSharedPtrTS<FTimelineAssetISOBMFF> InMediaAsset);
		virtual ~FPlayPeriodISOBMFF();
		void SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) override;
		EReadyState GetReadyState() override;
		void Load() override;
		void PrepareForPlay() override;
		int64 GetDefaultStartingBitrate() const override;
		TSharedPtrTS<FBufferSourceInfo> GetSelectedStreamBufferSourceInfo(EStreamType StreamType) override;
		FString GetSelectedAdaptationSetID(EStreamType StreamType) override;
		ETrackChangeResult ChangeTrackStreamPreference(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) override;
		TSharedPtrTS<ITimelineMediaAsset> GetMediaAsset() const override;
		void SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex) override;
		void TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload) override;
		FResult GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
		FResult GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType InStreamType, const FPlayerSequenceState& InLoopState, const FPlayStartPosition& InStartPosition, ESearchType InSearchType) override;
		FResult GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options) override;
		FResult GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData) override;
		FResult GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
		void IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount) override;
		void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID) override;

	private:
		void SelectInitialStream(EStreamType StreamType);
		TSharedPtrTS<FTrackMetadata> SelectMetadataForAttributes(EStreamType StreamType, const FStreamSelectionAttributes& InAttributes);
		void MakeBufferSourceInfoFromMetadata(EStreamType StreamType, TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FTrackMetadata> InMetadata);
		void AggregateChunkSamples(TSharedPtr<IStreamSegment, ESPMode::ThreadSafe> InOutRequest, const FTimespan& InMinDuration);
		enum class EAdjustMuxType
		{
			FirstStart,
			Continuation,
			Retry
		};
		void AdjustMuxedDownload(TSharedPtr<IStreamSegment, ESPMode::ThreadSafe> InOutRequest, const FTimespan& InMinDuration, EAdjustMuxType InType);

		TWeakPtrTS<FTimelineAssetISOBMFF> MediaAsset;
		FStreamSelectionAttributes VideoPreferences;
		FStreamSelectionAttributes AudioPreferences;
		FStreamSelectionAttributes SubtitlePreferences;
		TSharedPtrTS<FTrackMetadata> SelectedVideoMetadata;
		TSharedPtrTS<FTrackMetadata> SelectedAudioMetadata;
		TSharedPtrTS<FTrackMetadata> SelectedSubtitleMetadata;
		TSharedPtrTS<FBufferSourceInfo> VideoBufferSourceInfo;
		TSharedPtrTS<FBufferSourceInfo> AudioBufferSourceInfo;
		TSharedPtrTS<FBufferSourceInfo> SubtitleBufferSourceInfo;
		EReadyState CurrentReadyState;
		FTimelineAssetISOBMFF::ELoaderType LoaderType;
		FTimespan AggregateMinChunkDuration;
		FTimespan MuxedSegmentDuration;
	};

	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

	const TArray<FURL_RFC3986::FQueryParam>& GetURLFragmentComponents() const
	{ return URLFragmentComponents;	}

	void SetURLFragmentComponents(TArray<FURL_RFC3986::FQueryParam> InURLFragmentComponents)
	{ URLFragmentComponents = MoveTemp(InURLFragmentComponents); }

	const TArray<FURL_RFC3986::FQueryParam>& GetURLFragmentComponents()
	{ return URLFragmentComponents;	}


	IPlayerSessionServices* PlayerSessionServices = nullptr;
	TSharedPtrTS<FTimelineAssetISOBMFF> MediaAsset;
	// The URL fragment components
	TArray<FURL_RFC3986::FQueryParam> URLFragmentComponents;
	FTimeValue DefaultStartTime;
	FTimeValue DefaultEndTime;
};


} // namespace Electra
