// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlaylistReaderISOBMFF.h"
#include "Player/Manifest.h"
#include "Player/PlayerSessionServices.h"
#include "HTTP/HTTPManager.h"
#include "HTTP/HTTPResponseCache.h"
#include "MP4Utilities.h"
#include "MP4Boxes.h"
#include "Utilities/MP4Helpers.h"
#include "ElectraPlayerPrivate.h"
#include "Player/isobmff/PlaylistISOBMFF.h"
#include "Stats/Stats.h"
#include "HAL/LowLevelMemTracker.h"

#define ERRCODE_ISOBMFF_INVALID_FILE	1
#define ERRCODE_ISOBMFF_DOWNLOAD_ERROR	2


DECLARE_CYCLE_STAT(TEXT("FPlaylistReaderISOBMFF_WorkerThread"), STAT_ElectraPlayer_ISOBMFF_PlaylistWorker, STATGROUP_ElectraPlayer);


namespace Electra
{

/**
 * This class is responsible for downloading the mp4 non-mdat boxes and parsing them.
 */
class FPlaylistReaderISOBMFF : public IPlaylistReaderISOBMFF, public FMediaThread
{
public:
	FPlaylistReaderISOBMFF();
	void Initialize(IPlayerSessionServices* PlayerSessionServices);

	virtual ~FPlaylistReaderISOBMFF();

	EMediaFormatType GetMediaFormatType() const override
	{ return EMediaFormatType::ISOBMFF; }
	void Close() override;
	void HandleOnce() override
	{ }

	const FString& GetPlaylistType() const override
	{
		static FString Type("mp4");
		return Type;
	}

	void LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams) override;

	FString GetURL() const override
	{ return MainPlaylistURL; }


	TSharedPtrTS<IManifest> GetManifest() override
	{ return Manifest; }


private:

	void StartWorkerThread();
	void StopWorkerThread();
	void WorkerThread();

	void PostError(const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	FString MainPlaylistURL;
	FString URLFragment;
	FMediaEvent WorkerThreadQuitSignal;
	bool bIsWorkerThreadStarted = false;

	TSharedPtr<FMP4DataLoader, ESPMode::ThreadSafe> DataLoader;
	HTTP::FConnectionInfo ConnectionInfo;

	volatile bool bAbort = false;

	TSharedPtrTS<FManifestISOBMFFInternal> Manifest;
	FErrorDetail LastErrorDetail;
};


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtrTS<IPlaylistReader> IPlaylistReaderISOBMFF::Create(IPlayerSessionServices* PlayerSessionServices)
{
	TSharedPtrTS<FPlaylistReaderISOBMFF> PlaylistReader = MakeSharedTS<FPlaylistReaderISOBMFF>();
	if (PlaylistReader)
	{
		PlaylistReader->Initialize(PlayerSessionServices);
	}
	return PlaylistReader;
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


FPlaylistReaderISOBMFF::FPlaylistReaderISOBMFF()
	: FMediaThread("ElectraPlayer::ISOBMFF Playlist")
{
}

FPlaylistReaderISOBMFF::~FPlaylistReaderISOBMFF()
{
	Close();
}

void FPlaylistReaderISOBMFF::Initialize(IPlayerSessionServices* InPlayerSessionServices)
{
	PlayerSessionServices = InPlayerSessionServices;
}

void FPlaylistReaderISOBMFF::Close()
{
	bAbort = true;
	StopWorkerThread();
}

void FPlaylistReaderISOBMFF::StartWorkerThread()
{
	check(!bIsWorkerThreadStarted);
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FPlaylistReaderISOBMFF::WorkerThread));
	bIsWorkerThreadStarted = true;
}

void FPlaylistReaderISOBMFF::StopWorkerThread()
{
	if (bIsWorkerThreadStarted)
	{
		WorkerThreadQuitSignal.Signal();
		ThreadWaitDone();
		ThreadReset();
		bIsWorkerThreadStarted = false;
	}
}

void FPlaylistReaderISOBMFF::PostError(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	LastErrorDetail.Clear();
	LastErrorDetail.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	LastErrorDetail.SetFacility(Facility::EFacility::ISOBMFFPlaylistReader);
	LastErrorDetail.SetCode(InCode);
	LastErrorDetail.SetMessage(InMessage);
	check(PlayerSessionServices);
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostError(LastErrorDetail);
	}
}

void FPlaylistReaderISOBMFF::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::ISOBMFFPlaylistReader, Level, Message);
	}
}

void FPlaylistReaderISOBMFF::LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams)
{
	FURL_RFC3986 UrlParser;
	UrlParser.Parse(InURL);
	MainPlaylistURL = UrlParser.Get(true, false);
	URLFragment = UrlParser.GetFragment();

	StartWorkerThread();
}


void FPlaylistReaderISOBMFF::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_ISOBMFF_PlaylistWorker);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, ISOBMFF_PlaylistWorker);

	TSharedPtrTS<IHTTPResponseCache> LocalCache = IHTTPResponseCache::Create(1024*1024, 32, nullptr);

	DataLoader = FMP4DataLoader::Create();
	auto DataLoaderCancel = FMP4DataLoader::FCancellationCheckDelegate::CreateLambda([&](){return bAbort;});

	FMP4DataLoader::FHttpParams HttpParams;
	HttpParams.HTTPManager = PlayerSessionServices->GetHTTPManager();
	HttpParams.HTTPResponseCache = LocalCache;
	DataLoader->Open(MainPlaylistURL, HttpParams);
	bool bIsLocalFile = MainPlaylistURL.StartsWith(TEXT("file:"), ESearchCase::IgnoreCase);

	MP4Utilities::FMP4BoxLocator BoxLocator;
	TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> RootBoxes;
	const TArray<uint32> FirstBoxes { MP4Utilities::MakeBoxAtom('f','t','y','p'), MP4Utilities::MakeBoxAtom('s','t','y','p'), MP4Utilities::MakeBoxAtom('s','i','d','x'), MP4Utilities::MakeBoxAtom('f','r','e','e'), MP4Utilities::MakeBoxAtom('s','k','i','p') };
	const TArray<uint32> StopAfterBoxes { MP4Utilities::MakeBoxAtom('m','o','o','v') };	// Empty means to read all boxes and not stop after a specific one.
	const TArray<uint32> ReadBoxes;			// Empty means to read in all boxes except `mdat`
	bool bGotBoxes = BoxLocator.LocateAndReadRootBoxes(RootBoxes, DataLoader, FirstBoxes, StopAfterBoxes, false, ReadBoxes, DataLoaderCancel);
	ConnectionInfo = DataLoader->GetConnectionInfo();
	bool bHasErrored = ConnectionInfo.StatusInfo.ErrorDetail.IsSet();

	if (BoxLocator.GetLastError().Len())
	{
		PostError(BoxLocator.GetLastError(), ERRCODE_ISOBMFF_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
	}

	// There are currently no retries, so this is the first (and only) attempt we make.
	const int32 Attempt = 1;
	// Notify the download of the "main playlist". This indicates the download only, not the parsing thereof.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistDownloadMessage::Create(&ConnectionInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempt));
	// Notify that the "main playlist" has been parsed, successfully or not.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ConnectionInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempt));
	// Failed to get the boxes but was not aborted?
	if (!bAbort && (!bGotBoxes || bHasErrored))
	{
		// See if there was a download error
		if (ConnectionInfo.StatusInfo.ErrorDetail.IsError())
		{
			PostError(FString::Printf(TEXT("%s while downloading \"%s\""), *ConnectionInfo.StatusInfo.ErrorDetail.GetMessage(), *ConnectionInfo.EffectiveURL), ERRCODE_ISOBMFF_DOWNLOAD_ERROR, UEMEDIA_ERROR_READ_ERROR);
		}
	}
	else if (bGotBoxes && !bAbort && !bHasErrored)
	{
		// Do we have the `moov` box?
		if (RootBoxes.ContainsByPredicate([](const TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>& In){ return In->Type == MP4Utilities::MakeBoxAtom('m','o','o','v'); }))
		{
			if (RootBoxes.ContainsByPredicate([](const TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>& In){ return In->Type == MP4Utilities::MakeBoxAtom('m','d','a','t'); }))
			{
				LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("The movie at \"%s\" is not fast-startable. Consider moving the 'moov' box in front of the 'mdat' for faster startup times."), *ConnectionInfo.EffectiveURL));
			}

			// Parse all the root boxes. There are typically no more than 4 to 6.
			TArray<TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>> ParsedRootBoxes;
			ParsedRootBoxes.SetNum(RootBoxes.Num());
			bool bIsFragmented = false;
			for(int32 i=0; i<RootBoxes.Num(); ++i)
			{
				MP4Boxes::FMP4BoxTreeParser tp;
				if (tp.ParseBoxTree(RootBoxes[i]))
				{
					ParsedRootBoxes[i] = tp.GetBoxTree();
					// Check if this is a fragmented file.
					if (ParsedRootBoxes[i]->GetType() == MP4Utilities::MakeBoxAtom('m','o','o','v'))
					{
						auto MoovBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxMOOV>(ParsedRootBoxes[i]);
						auto MvexBox = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxMVEX>(MP4Utilities::MakeBoxAtom('m','v','e','x'), 0);
						if (MvexBox)
						{
							bIsFragmented = true;
						}
					}
				}
				else
				{
					PostError(FString::Printf(TEXT("Failed to parse the file's box structure.")), ERRCODE_ISOBMFF_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
					bHasErrored = true;
				}
			}

			if (bIsFragmented)
			{
				if (!bIsLocalFile)
				{
					LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("The movie at \"%s\" is fragmented and not a local file. Parsing the movie structure will be slow."), *ConnectionInfo.EffectiveURL));
				}

				// Continue loading root boxes
				TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> FrgRootBoxes;
				const TArray<uint32> FrgFirstBoxes;
				const TArray<uint32> FrgStopAfterBoxes;	// Empty means to read all boxes and not stop after a specific one.
				const TArray<uint32> FrgReadBoxes;		// Empty means to read in all boxes except `mdat`
				bGotBoxes = BoxLocator.LocateAndReadRootBoxes(FrgRootBoxes, DataLoader, FrgFirstBoxes, FrgStopAfterBoxes, false, FrgReadBoxes, DataLoaderCancel);
				ConnectionInfo = DataLoader->GetConnectionInfo();
				bHasErrored = ConnectionInfo.StatusInfo.ErrorDetail.IsSet();
				if (BoxLocator.GetLastError().Len())
				{
					PostError(BoxLocator.GetLastError(), ERRCODE_ISOBMFF_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
				}
				if (!bHasErrored)
				{
					int32 CurNumRootBoxes = RootBoxes.Num();
					RootBoxes.Append(MoveTemp(FrgRootBoxes));
					ParsedRootBoxes.SetNum(RootBoxes.Num());
					for(int32 i=CurNumRootBoxes; i<RootBoxes.Num(); ++i)
					{
						MP4Boxes::FMP4BoxTreeParser tp;
						if (tp.ParseBoxTree(RootBoxes[i]))
						{
							ParsedRootBoxes[i] = tp.GetBoxTree();
						}
						else
						{
							PostError(FString::Printf(TEXT("Failed to parse the file's box structure.")), ERRCODE_ISOBMFF_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
							bHasErrored = true;
						}
					}
				}
			}

			if (!bHasErrored)
			{
				TArray<FURL_RFC3986::FQueryParam> URLFragmentComponents;
				FURL_RFC3986::GetQueryParams(URLFragmentComponents, URLFragment, false);
				Manifest = MakeSharedTS<FManifestISOBMFFInternal>(PlayerSessionServices);
				Manifest->SetURLFragmentComponents(MoveTemp(URLFragmentComponents));
				LastErrorDetail = Manifest->Build(MoveTemp(ParsedRootBoxes), MainPlaylistURL, DataLoader, DataLoaderCancel);
				if (LastErrorDetail.IsSet())
				{
					Manifest.Reset();
				}

				// Let the external registry know that we have no properties with an end-of-properties call.
				PlayerSessionServices->ValidateMainPlaylistCustomProperty(GetPlaylistType(), MainPlaylistURL, TArray<FElectraHTTPStreamHeader>(), IPlayerSessionServices::FPlaylistProperty());
				// Notify that the "variant playlists" are ready. There are no variants in an mp4, but this is the trigger that the playlists are all set up and are good to go now.
				PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ConnectionInfo, Playlist::EListType::Variant, Playlist::ELoadType::Initial, Attempt));
			}
		}
		else
		{
			PostError(FString::Printf(TEXT("No `moov` box found in \"%s\". This is not a valid file."), *ConnectionInfo.EffectiveURL), ERRCODE_ISOBMFF_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
		}
	}

	DataLoader->Close();
	DataLoader.Reset();
	// This thread's work is done. We only wait for termination now.
	WorkerThreadQuitSignal.Wait();
}




} // namespace Electra
