// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlaylistReader.h"

namespace Electra
{
class IPlayerSessionServices;

class IPlaylistReaderISOBMFF : public IPlaylistReader
{
public:
	static TSharedPtrTS<IPlaylistReader> Create(IPlayerSessionServices* PlayerSessionServices);

	virtual ~IPlaylistReaderISOBMFF() = default;
};

} // namespace Electra
