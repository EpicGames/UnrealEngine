// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamReaderISOBMFF.h"
#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserISO14496-12.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/isobmff/OptionKeynamesISOBMFF.h"
#include "Player/PlayerSessionServices.h"
#include "Player/PlayerStreamFilter.h"


DECLARE_CYCLE_STAT(TEXT("FStreamReaderISOBMFF_HandleRequest"), STAT_ElectraPlayer_ISOBMFF_StreamReader, STATGROUP_ElectraPlayer);


namespace Electra
{

void FStreamSegmentRequestISOBMFF::GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const
{
	OutDependentStreams.Empty();
	for(auto& Stream : DependentStreams)
	{
		OutDependentStreams.Emplace(Stream);
	}
}

void FStreamSegmentRequestISOBMFF::GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams)
{
	OutRequestedStreams.Empty();
	if (LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Chunks || LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Fragmented)
	{
		for(auto& Stream : DependentStreams)
		{
			OutRequestedStreams.Emplace(Stream);
		}
	}
	else
	{
		OutRequestedStreams.Emplace(AsShared());
	}
}

void FStreamSegmentRequestISOBMFF::GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams)
{
	OutAlreadyEndedStreams.Empty();
	if (LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Muxed && PrimarySampleFileOffsetStart < 0)
	{
		for(auto& Stream : DependentStreams)
		{
			OutAlreadyEndedStreams.Emplace(Stream);
		}
	}
}

FTimeRange FStreamSegmentRequestISOBMFF::GetTimeRange() const
{
	FTimeRange tr;
	tr.Start = FirstDTS;
	tr.End = tr.Start + SegmentDuration;
	tr.Start.SetSequenceIndex(TimestampSequenceIndex);
	tr.End.SetSequenceIndex(TimestampSequenceIndex);
	return tr;
}


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/


FStreamReaderISOBMFF::~FStreamReaderISOBMFF()
{
	Close();
}

void FStreamReaderISOBMFF::SetLoaderType(FStreamSegmentRequestISOBMFF::ELoaderType InLoaderType)
{
	LoaderType = InLoaderType;
}

bool FStreamReaderISOBMFF::LoadsMultiplexedTracksSequentially()
{
	return LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Muxed;
}

UEMediaError FStreamReaderISOBMFF::Create(IPlayerSessionServices* InPlayerSessionService, const CreateParam &InCreateParam)
{
	check(InPlayerSessionService);
	PlayerSessionService = InPlayerSessionService;

	if (!InCreateParam.MemoryProvider || !InCreateParam.EventListener)
	{
		return UEMEDIA_ERROR_BAD_ARGUMENTS;
	}

	bIsStarted = true;
	if (LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Chunks || LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Fragmented)
	{
		for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(StreamHandlers); ++i)
		{
			StreamHandlers[i].PlayerSessionServices = PlayerSessionService;
			StreamHandlers[i].Parameters = InCreateParam;
			StreamHandlers[i].bTerminate = false;
			StreamHandlers[i].bWasStarted = false;
			StreamHandlers[i].bRequestCanceled = false;
			StreamHandlers[i].bSilentCancellation = false;
			StreamHandlers[i].bHasErrored  = false;
			StreamHandlers[i].IsIdleSignal.Signal();
			StreamHandlers[i].ThreadSetName(i==0 ? "Electra ISOBMFF video loader" :
											i==1 ? "Electra ISOBMFF audio loader" :
												   "Electra ISOBMFF subtitle loader");
		}
	}
	else
	{
		StreamHandlers[0].PlayerSessionServices = PlayerSessionService;
		StreamHandlers[0].Parameters = InCreateParam;
		StreamHandlers[0].bTerminate = false;
		StreamHandlers[0].bWasStarted = false;
		StreamHandlers[0].bRequestCanceled = false;
		StreamHandlers[0].bSilentCancellation = false;
		StreamHandlers[0].bHasErrored  = false;
		StreamHandlers[0].IsIdleSignal.Signal();
		StreamHandlers[0].ThreadSetName("Electra ISOBMFF loader");
	}
	return UEMEDIA_ERROR_OK;
}


void FStreamReaderISOBMFF::Close()
{
	if (bIsStarted)
	{
		bIsStarted = false;
		// Signal the worker threads to end.
		for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(StreamHandlers); ++i)
		{
			StreamHandlers[i].bTerminate = true;
			StreamHandlers[i].Cancel(true);
			StreamHandlers[i].SignalWork();
		}
		// Wait until they finished.
		for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(StreamHandlers); ++i)
		{
			if (StreamHandlers[i].bWasStarted)
			{
				StreamHandlers[i].ThreadWaitDone();
				StreamHandlers[i].ThreadReset();
			}
		}
	}
}


void FStreamReaderISOBMFF::FStreamHandler::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	PlayerSessionServices->PostLog(Facility::EFacility::ISOBMFFStreamReader, Level, Message);
}

IStreamReader::EAddResult FStreamReaderISOBMFF::AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> InRequest)
{
	TSharedPtrTS<FStreamSegmentRequestISOBMFF> Request = StaticCastSharedPtr<FStreamSegmentRequestISOBMFF>(InRequest);
	if ((LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Chunks || LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Fragmented) && !Request->DependentStreams.IsEmpty())
	{
		PostError(PlayerSessionService, TEXT("Initial start request segments cannot be enqueued!"), 0);
		return IStreamReader::EAddResult::Error;
	}
	// Do not add empty mux requests.
	else if (LoaderType != FStreamSegmentRequestISOBMFF::ELoaderType::Muxed || Request->PrimarySampleFileOffsetStart >= 0)
	{
		// Get the handler for the main request.
		FStreamHandler* Handler = nullptr;
		if (LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Chunks || LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Fragmented)
		{
			switch(Request->GetType())
			{
				case EStreamType::Video:
				{
					Handler = &StreamHandlers[0];
					break;
				}
				case EStreamType::Audio:
				{
					Handler = &StreamHandlers[1];
					break;
				}
				case EStreamType::Subtitle:
				{
					Handler = &StreamHandlers[2];
					break;
				}
				default:
				{
					break;
				}
			}
		}
		else
		{
			Handler = &StreamHandlers[0];
		}
		if (!Handler)
		{
			ErrorDetail.SetMessage(FString::Printf(TEXT("No handler for stream type")));
			return IStreamReader::EAddResult::Error;
		}
		// Is the handler busy?
		bool bIsIdle = Handler->IsIdleSignal.WaitTimeout(1000 * 1000);
		if (!bIsIdle)
		{
			ErrorDetail.SetMessage(FString::Printf(TEXT("The handler for this stream type is busy!?")));
			return IStreamReader::EAddResult::Error;
		}

		Request->SetPlaybackSequenceID(CurrentPlaybackSequenceID);
		if (!Handler->bWasStarted)
		{
			Handler->ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(Handler, &FStreamHandler::WorkerThread));
			Handler->bWasStarted = true;
		}

		Handler->bRequestCanceled = false;
		Handler->bSilentCancellation = false;
		Handler->bHasErrored = false;
		Handler->CurrentRequest = Request;
		Handler->SignalWork();
	}
	return EAddResult::Added;
}

void FStreamReaderISOBMFF::CancelRequest(EStreamType StreamType, bool bSilent)
{
	if (LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Chunks || LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Fragmented)
	{
		if (StreamType == EStreamType::Video)
		{
			StreamHandlers[0].Cancel(bSilent);
		}
		else if (StreamType == EStreamType::Audio)
		{
			StreamHandlers[1].Cancel(bSilent);
		}
		else if (StreamType == EStreamType::Subtitle)
		{
			StreamHandlers[2].Cancel(bSilent);
		}
	}
	else
	{
		StreamHandlers[0].Cancel(bSilent);
	}
}

void FStreamReaderISOBMFF::CancelRequests()
{
	for(int32 i=0; i<FMEDIA_STATIC_ARRAY_COUNT(StreamHandlers); ++i)
	{
		StreamHandlers[i].Cancel(false);
	}
}


std::atomic<uint32> FStreamReaderISOBMFF::FStreamHandler::UniqueDownloadID;

void FStreamReaderISOBMFF::FStreamHandler::Cancel(bool bSilent)
{
	bSilentCancellation = bSilent;
	bRequestCanceled = true;
}

void FStreamReaderISOBMFF::FStreamHandler::SignalWork()
{
	WorkSignal.Release();
}


bool FStreamReaderISOBMFF::FStreamHandler::HasBeenAborted() const
{
	return bRequestCanceled;
}

bool FStreamReaderISOBMFF::FStreamHandler::HasErrored() const
{
	return bHasErrored;
}

int32 FStreamReaderISOBMFF::FStreamHandler::HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest)
{
	// Aborted?
	return HasBeenAborted() ? 1 : 0;
}

void FStreamReaderISOBMFF::FStreamHandler::HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest)
{
	bHasErrored = InRequest->ConnectionInfo.StatusInfo.ErrorDetail.IsError();
}


void FStreamReaderISOBMFF::FStreamHandler::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	while(!bTerminate)
	{
		WorkSignal.Obtain();
		if (!bTerminate)
		{
			if (CurrentRequest.IsValid())
			{
				IsIdleSignal.Reset();
				if (!bRequestCanceled)
				{
					if (CurrentRequest->LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Chunks || CurrentRequest->LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Fragmented)
					{
						HandleChunkOrFragmentRequest();
					}
					else if (CurrentRequest->LoaderType == FStreamSegmentRequestISOBMFF::ELoaderType::Muxed)
					{
						HandleMuxedRequest();
					}
					else
					{
						unimplemented();
						CurrentRequest.Reset();
					}
				}
				else
				{
					CurrentRequest.Reset();
				}
				IsIdleSignal.Signal();
			}
			bRequestCanceled = false;
			bSilentCancellation = false;
			bHasErrored = false;
		}
	}
}


void FStreamReaderISOBMFF::FStreamHandler::HandleMuxedRequest()
{
	TSharedPtrTS<FStreamSegmentRequestISOBMFF> Request = CurrentRequest;

	// Create CSD and buffer info for all the streams.
	struct FTrackVars
	{
		TSharedPtr<FAccessUnit::CodecData, ESPMode::ThreadSafe> CSD;
		TSharedPtr<FBufferSourceInfo, ESPMode::ThreadSafe> BufferInfo;
		TSharedPtr<ElectraCDM::IMediaCDMDecrypter, ESPMode::ThreadSafe> Decrypter;
		FDRMTypeFormat DrmFormat;
		ElectraCDM::FMediaCDMSampleInfo SampleEncryptionInfo;
		TArray<FAccessUnit*> AccessUnits;
		FTimeValue DurationSuccessfullyRead { FTimeValue::GetZero() };
		FTimeValue DurationSuccessfullyDelivered { FTimeValue::GetZero() };
		EStreamType StreamType { EStreamType::Unsupported };
		bool bIsFirstInSequence { true };
		bool bEndOfTrack { false };
		bool bDeliveredLastAccessUnit { false };
		bool bNotifiedOfTrackEnd { false };
		bool bIsEncrypted { false };
		int32 DependendStreamIdx { -1 };
	};
	TMap<uint32, FTrackVars> TrackMap;
	for(int32 i=0; i<Request->DependentStreams.Num(); ++i)
	{
		TSharedPtr<FAccessUnit::CodecData, ESPMode::ThreadSafe> CSD(new FAccessUnit::CodecData);
		CSD->CodecSpecificData = Request->DependentStreams[i]->Track.TrackInfo->CodecFormat.CSD;
		CSD->RawCSD = Request->DependentStreams[i]->Track.TrackInfo->CodecFormat.DCR;
		CSD->ParsedInfo = FStreamCodecInformation(Request->DependentStreams[i]->Track.TrackInfo->CodecFormat);
		TSharedPtr<FBufferSourceInfo, ESPMode::ThreadSafe> BufferInfo = MakeShared<FBufferSourceInfo, ESPMode::ThreadSafe>();
		BufferInfo->PlaybackSequenceID = Request->GetPlaybackSequenceID();
		BufferInfo->PeriodID = Request->AssetID;
		BufferInfo->PeriodAdaptationSetID = Request->AssetID + TEXT(".") + Request->DependentStreams[i]->AdaptationSetID;
		BufferInfo->LanguageTag = Request->DependentStreams[i]->LanguageTag;
		BufferInfo->HardIndex = Request->DependentStreams[i]->AdaptationSetTrackIndex;
		BufferInfo->Codec = CSD->ParsedInfo.GetCodecName();
		BufferInfo->Kind = Request->DependentStreams[i]->KindOfTrack;
		FTrackVars tv;
		tv.DependendStreamIdx = i;
		tv.CSD = MoveTemp(CSD);
		tv.BufferInfo = MoveTemp(BufferInfo);
		tv.StreamType = Request->DependentStreams[i]->StreamType;
		// If there is no iterator for this track then it is already at EOT.
		tv.bEndOfTrack = !Request->DependentStreams[i]->Track.TrackIterator;
		// If this track is at its end we let the player know that this buffer will not be
		// receiving any data.
		if (tv.bEndOfTrack)
		{
			tv.bNotifiedOfTrackEnd = true;
			Parameters.EventListener->OnFragmentReachedEOS(tv.StreamType, tv.BufferInfo);
		}
		// Encrypted?
		if (Request->DependentStreams[i]->Track.TrackInfo->DrmFormat.IsEncrypted())
		{
			tv.bIsEncrypted = true;
			tv.DrmFormat = Request->DependentStreams[i]->Track.TrackInfo->DrmFormat;
			if (tv.DrmFormat.EncryptionInfo.IsType<FDRMTypeFormat::FISOEncryptionInfo>())
			{
				const FDRMTypeFormat::FISOEncryptionInfo& encInf(tv.DrmFormat.EncryptionInfo.Get<FDRMTypeFormat::FISOEncryptionInfo>());
				tv.SampleEncryptionInfo.DefaultKID = encInf.DefaultKID;
				tv.SampleEncryptionInfo.IV = encInf.DefaultIV;
				tv.SampleEncryptionInfo.Scheme4CC = encInf.Scheme;
				if (encInf.BlockPattern.IsSet())
				{
					tv.SampleEncryptionInfo.Pattern.PatternType = 1;
					tv.SampleEncryptionInfo.Pattern.CryptByteBlock = encInf.BlockPattern.GetValue().CryptByteBlock;
					tv.SampleEncryptionInfo.Pattern.SkipByteBlock = encInf.BlockPattern.GetValue().SkipByteBlock;
				}
				if (Request->DependentStreams[i]->DrmClient)
				{
					Request->DependentStreams[i]->DrmClient->CreateDecrypter(tv.Decrypter, FString());
				}
			}
		}
		TrackMap.Emplace(Request->DependentStreams[i]->Track.TrackInfo->TrackID, MoveTemp(tv));
	}

	Metrics::FSegmentDownloadStats& ds = Request->DownloadStats;
	ds.StatsID = ++UniqueDownloadID;
	ds.MediaAssetID = Request->AssetID;
	ds.AdaptationSetID = Request->AdaptationSetID;
	ds.RepresentationID = Request->RepresentationID;
	ds.Bitrate = Request->Bitrate;
	ds.FailureReason.Empty();
	ds.bWasSuccessful = true;
	ds.bWasAborted = false;
	ds.bDidTimeout = false;
	ds.HTTPStatusCode = 0;
	ds.StreamType = Request->GetType();
	ds.SegmentType = Metrics::ESegmentType::Media;
	ds.PresentationTime = Request->EarliestPTS.GetAsSeconds();
	ds.Duration = Request->SegmentDuration.GetAsSeconds();
	ds.DurationDownloaded = 0.0;
	ds.DurationDelivered = 0.0;
	ds.TimeToFirstByte = 0.0;
	ds.TimeToDownload = 0.0;
	ds.ByteSize = -1;
	ds.NumBytesDownloaded = 0;
	ds.bInsertedFillerData = false;
	ds.URL.URL = Request->MediaURL;
	ds.bIsMissingSegment = false;
	ds.bParseFailure = false;
	ds.RetryNumber = Request->NumOverallRetries;

	Parameters.EventListener->OnFragmentOpen(Request);

	TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateRaw(this, &FStreamReaderISOBMFF::FStreamHandler::HTTPCompletionCallback);
	ProgressListener->ProgressDelegate = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateRaw(this, &FStreamReaderISOBMFF::FStreamHandler::HTTPProgressCallback);

	const int64 StartOffset = Request->PrimarySampleFileOffsetStart;
	const int64 EndOffset = Request->PrimarySampleFileOffsetEnd;

	ReceiveBuffer.Reset();
	ReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
	TSharedPtrTS<IElectraHttpManager::FRequest> HTTP(new IElectraHttpManager::FRequest);
	HTTP->Parameters.URL = Request->MediaURL;
	HTTP->Parameters.Range.Start = StartOffset;
	HTTP->Parameters.Range.EndIncluding = EndOffset - 1;
	HTTP->Parameters.AcceptEncoding.Set(TEXT("identity"));
	HTTP->Parameters.ConnectTimeout = PlayerSessionServices->GetOptionValue(ISOBMFF::OptionKeyISOBMFFLoadConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 8));
	HTTP->Parameters.NoDataTimeout = PlayerSessionServices->GetOptionValue(ISOBMFF::OptionKeyISOBMFFLoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 6));
	HTTP->ReceiveBuffer = ReceiveBuffer;
	HTTP->ProgressListener = ProgressListener;
	HTTP->ResponseCache = PlayerSessionServices->GetHTTPResponseCache();
	PlayerSessionServices->GetHTTPManager()->AddRequest(HTTP, false);

	bool bDone = false;

	const uint32 PlaybackSequenceID = Request->GetPlaybackSequenceID();

	// Locate the track iterator that will handle the next incoming data.
	auto GetNextTrackIndexToIterate = [&]() -> int32
	{
		int64 NextIteratorFileDataOffset = TNumericLimits<int64>::Max();
		int32 ItIdx = -1;
		for(int32 i=0; i<Request->DependentStreams.Num(); ++i)
		{
			if (Request->DependentStreams[i]->Track.TrackIterator && Request->DependentStreams[i]->Track.TrackIterator->GetSampleFileOffset() < NextIteratorFileDataOffset)
			{
				NextIteratorFileDataOffset = Request->DependentStreams[i]->Track.TrackIterator->GetSampleFileOffset();
				ItIdx = i;
			}
		}
		return ItIdx;
	};

	auto ShouldStop = [&]() -> bool
	{
		return HasErrored() || HasBeenAborted() || bTerminate;
	};


	auto DiscardAllPendingAccessUnits = [&]() -> void
	{
		for(auto &tvit : TrackMap)
		{
			FTrackVars& tv(tvit.Value);
			while(!tv.AccessUnits.IsEmpty())
			{
				FAccessUnit* AU = tv.AccessUnits.Pop();
				FAccessUnit::Release(AU);
				AU = nullptr;
			}
		}
	};

	auto DeliverAccessUnits = [&](bool bUntilDone) -> void
	{
		while(!HasBeenAborted() && !bTerminate)
		{
			int32 NumBlocked = 0;
			int32 NumWithAUs = 0;
			for(auto &tvit : TrackMap)
			{
				FTrackVars& tv(tvit.Value);
				if (tv.AccessUnits.Num())
				{
					if (!tv.bDeliveredLastAccessUnit)
					{
						++NumWithAUs;
						FAccessUnit* AU = tv.AccessUnits[0];
						if (Parameters.EventListener->OnFragmentAccessUnitReceived(AU))
						{
							tv.DurationSuccessfullyDelivered += AU->Duration;
							tv.bDeliveredLastAccessUnit |= AU->bIsLastInPeriod;
							tv.AccessUnits.RemoveAt(0);
							AU = nullptr;
						}
						else
						{
							++NumBlocked;
						}
					}
					else
					{
						while(!tv.AccessUnits.IsEmpty())
						{
							FAccessUnit* AU = tv.AccessUnits.Pop();
							FAccessUnit::Release(AU);
							AU = nullptr;
						}
					}
				}
				else if (!tv.bNotifiedOfTrackEnd && !Request->DependentStreams[tv.DependendStreamIdx]->Track.TrackIterator)
				{
					tv.bNotifiedOfTrackEnd = true;
					Parameters.EventListener->OnFragmentReachedEOS(tv.StreamType, tv.BufferInfo);
				}
			}
			if (!bUntilDone || NumWithAUs == 0)
			{
				break;
			}
			if (NumBlocked == NumWithAUs)
			{
				FMediaRunnable::SleepMicroseconds(1000 * 10);
			}
		}
	};

	int64 SizeNeededInBuffer = 0;
	int32 NextTrackIndex = -1;
	while(!bDone)
	{
		if (NextTrackIndex < 0)
		{
			NextTrackIndex = GetNextTrackIndexToIterate();
			// All at EOS?
			if (NextTrackIndex < 0)
			{
				Request->bIsLastSegment = true;
				bDone = true;
				break;
			}
		}
		TSharedPtrTS<FStreamSegmentRequestISOBMFF>& DepStr(Request->DependentStreams[NextTrackIndex]);

		FTrackVars& TrackVars(TrackMap[DepStr->Track.TrackInfo->TrackID]);
		check(DepStr->Track.TrackChunkSampleInfo.NumSamples);

		const int64 SampleRelativeOffset = DepStr->Track.TrackIterator->GetSampleFileOffset() - StartOffset;
		const int64 SampleSize = DepStr->Track.TrackIterator->GetSampleSize();
		SizeNeededInBuffer = SampleRelativeOffset + SampleSize;

		// Wait for the data to arrive. While it does emit already accumulated access units.
		while(!ShouldStop())
		{
			if (ReceiveBuffer->WaitUntilSizeAvailable(SizeNeededInBuffer, 1000 * 100))
			{
				// Did we get what we need or did we hit EOF?
				if (ReceiveBuffer->GetLinearReadSize() < SizeNeededInBuffer)
				{
					bDone = true;
				}
				break;
			}
			else
			{
				// If the read buffer doesn't have the required size yet we try to deliver accumulated access units
				DeliverAccessUnits(false);
			}
		}
		if (bDone || ShouldStop())
		{
			break;
		}

		// Get the sample properties
		auto TrackIt = DepStr->Track.TrackIterator;
		uint32 SampleNumber = TrackIt->GetSampleNumber();
		MP4Utilities::FFractionalTime DTS = TrackIt->GetEffectiveDTS();
		MP4Utilities::FFractionalTime PTS = TrackIt->GetEffectivePTS();
		MP4Utilities::FFractionalTime Duration = TrackIt->GetDuration();
		bool bIsSyncSample = TrackIt->IsSyncOrRAPSample();

		// Create an access unit.
		FAccessUnit* AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
		if (AccessUnit)
		{
			const int64 TimestampSequenceIndex = Request->TimestampSequenceIndex;
			AccessUnit->ESType = DepStr->StreamType;
			AccessUnit->PTS.SetFromND(PTS.GetNumerator(), PTS.GetDenominator(), TimestampSequenceIndex);
			AccessUnit->DTS.SetFromND(DTS.GetNumerator(), DTS.GetDenominator(), TimestampSequenceIndex);
			AccessUnit->Duration.SetFromND(Duration.GetNumerator(), Duration.GetDenominator());

			AccessUnit->SequenceIndex = TimestampSequenceIndex;
			AccessUnit->EarliestPTS = DepStr->EarliestPTS;
			AccessUnit->EarliestPTS.SetSequenceIndex(TimestampSequenceIndex);
			AccessUnit->LatestPTS = DepStr->LastPTS;
			AccessUnit->LatestPTS.SetSequenceIndex(TimestampSequenceIndex);

			AccessUnit->AUSize = (uint32) SampleSize;
			AccessUnit->AUCodecData = TrackVars.CSD;
			AccessUnit->BufferSourceInfo = TrackVars.BufferInfo;

			AccessUnit->bIsFirstInSequence = TrackVars.bIsFirstInSequence;
			TrackVars.bIsFirstInSequence = false;
			AccessUnit->bIsSyncSample = bIsSyncSample;
			AccessUnit->bIsDummyData = false;
			AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
			{
				FScopeLock lock(ReceiveBuffer->GetLock());
				FMemory::Memcpy(AccessUnit->AUData, reinterpret_cast<const uint8*>(ReceiveBuffer->GetLinearReadData()) + SampleRelativeOffset, SampleSize);
			}
			// Decrypt?
			if (TrackVars.bIsEncrypted)
			{
				if (TrackVars.Decrypter)
				{
					// Wait for the decrypter to get ready.
					while(!ShouldStop() && (TrackVars.Decrypter->GetState() == ElectraCDM::ECDMState::WaitingForKey || TrackVars.Decrypter->GetState() == ElectraCDM::ECDMState::Idle))
					{
						FMediaRunnable::SleepMilliseconds(100);
					}
					// Set up the sample encryption info.
					const MP4Boxes::FMP4BoxSENC::FEntry& enc = TrackIt->GetEncryptionInfo();
					TrackVars.SampleEncryptionInfo.SubSamples.Empty();
					TrackVars.SampleEncryptionInfo.IV = enc.IV;
					for(auto &sub : enc.SubSamples)
					{
						TrackVars.SampleEncryptionInfo.SubSamples.Add({.NumClearBytes = sub.NumClearBytes, .NumEncryptedBytes = sub.NumEncryptedBytes});
					}
					// Decrypt
					ElectraCDM::ECDMError DecryptResult = ElectraCDM::ECDMError::Failure;
					if (TrackVars.Decrypter->GetState() == ElectraCDM::ECDMState::Ready)
					{
						DecryptResult = TrackVars.Decrypter->DecryptInPlace((uint8*) AccessUnit->AUData, (int32) AccessUnit->AUSize, TrackVars.SampleEncryptionInfo);
					}
					if (DecryptResult != ElectraCDM::ECDMError::Success)
					{
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;
						ds.FailureReason = FString::Printf(TEXT("Failed to decrypt segment with error %d (%s)"), (int32)DecryptResult, *TrackVars.Decrypter->GetLastErrorMessage());
						bHasErrored = true;
						bDone = true;
						break;
					}
				}
				else
				{
					FAccessUnit::Release(AccessUnit);
					AccessUnit = nullptr;
					ds.FailureReason = FString::Printf(TEXT("No valid decrypter to decrypt track"));
					bHasErrored = true;
					bDone = true;
					break;
				}
			}
			if (AccessUnit->DTS >= AccessUnit->LatestPTS && AccessUnit->PTS >= AccessUnit->LatestPTS)
			{
				AccessUnit->bIsLastInPeriod = true;
				TrackVars.bEndOfTrack = true;
			}
			TrackVars.DurationSuccessfullyRead += AccessUnit->Duration;
			TrackVars.AccessUnits.Emplace(AccessUnit);
			AccessUnit = nullptr;
		}
		DeliverAccessUnits(false);

		// Exhausted all samples in this chunk?
		const bool bEOT = !TrackIt->Next();
		if (bEOT)
		{
			NextTrackIndex = -1;
			TrackVars.bEndOfTrack = true;
			// Dump the iterator so we know that this track is done.
			DepStr->Track.TrackIterator.Reset();
		}
		else if (--DepStr->Track.TrackChunkSampleInfo.NumSamples == 0)
		{
			TrackIt->GetCurrentChunkRemainingSampleInfo(DepStr->Track.TrackChunkSampleInfo);
			NextTrackIndex = -1;
		}
	}
	// Remove the download request.
	ProgressListener.Reset();
	PlayerSessionServices->GetHTTPManager()->RemoveRequest(HTTP, false);
	Request->ConnectionInfo = HTTP->ConnectionInfo;
	HTTP.Reset();
	ReceiveBuffer.Reset();

	// Deliver all pending access units unless we get canceled.
	DeliverAccessUnits(true);
	// Anything that's still there we now need to discard.
	DiscardAllPendingAccessUnits();
	// If all tracks reached their end we mark the request as being the last one.
	bool bAllAtLastSample = true;
	for(auto &tvit : TrackMap)
	{
		FTrackVars& tv(tvit.Value);
		if (!tv.bEndOfTrack)
		{
			bAllAtLastSample = false;
		}
	}
	Request->bIsLastSegment |= bAllAtLastSample;

	// Set up download stat fields.
	ds.HTTPStatusCode = Request->ConnectionInfo.StatusInfo.HTTPStatus;
	ds.TimeToFirstByte = Request->ConnectionInfo.TimeUntilFirstByte;
	ds.TimeToDownload = (Request->ConnectionInfo.RequestEndTime - Request->ConnectionInfo.RequestStartTime).GetAsSeconds();
	ds.ByteSize = Request->ConnectionInfo.ContentLength;
	ds.NumBytesDownloaded = Request->ConnectionInfo.BytesReadSoFar;
	if (ds.FailureReason.IsEmpty())
	{
		ds.FailureReason = Request->ConnectionInfo.StatusInfo.ErrorDetail.GetMessage();
	}
	ds.bWasSuccessful = !bHasErrored;
	ds.URL.URL = Request->ConnectionInfo.EffectiveURL;
	if (Request->ReportingIndexInMux >= 0)
	{
		const FTrackVars& tv(TrackMap[Request->DependentStreams[Request->ReportingIndexInMux]->Track.TrackInfo->TrackID]);
		ds.DurationDownloaded = tv.DurationSuccessfullyRead.GetAsSeconds();
		ds.DurationDelivered  = tv.DurationSuccessfullyDelivered.GetAsSeconds();
	}
	ds.bIsCachedResponse = Request->ConnectionInfo.bIsCachedResponse;

	// Reset the current request so another one can be added immediately when we call OnFragmentClose()
	CurrentRequest.Reset();
	if (!bSilentCancellation)
	{
		PlayerSessionServices->GetStreamSelector()->ReportDownloadEnd(ds);
		Parameters.EventListener->OnFragmentClose(Request);
	}
}

void FStreamReaderISOBMFF::FStreamHandler::HandleChunkOrFragmentRequest()
{
	TSharedPtrTS<FStreamSegmentRequestISOBMFF> Request = CurrentRequest;

	Metrics::FSegmentDownloadStats& ds = Request->DownloadStats;
	ds.StatsID = ++UniqueDownloadID;
	ds.MediaAssetID = Request->AssetID;
	ds.AdaptationSetID = Request->AdaptationSetID;
	ds.RepresentationID = Request->RepresentationID;
	ds.Bitrate = Request->Bitrate;
	ds.FailureReason.Empty();
	ds.bWasSuccessful = true;
	ds.bWasAborted = false;
	ds.bDidTimeout = false;
	ds.HTTPStatusCode = 0;
	ds.StreamType = Request->GetType();
	ds.SegmentType = Metrics::ESegmentType::Media;
	ds.PresentationTime = Request->EarliestPTS.GetAsSeconds();
	ds.Duration = Request->SegmentDuration.GetAsSeconds();
	ds.DurationDownloaded = 0.0;
	ds.DurationDelivered = 0.0;
	ds.TimeToFirstByte = 0.0;
	ds.TimeToDownload = 0.0;
	ds.ByteSize = -1;
	ds.NumBytesDownloaded = 0;
	ds.bInsertedFillerData = false;
	ds.URL.URL = Request->MediaURL;
	ds.bIsMissingSegment = false;
	ds.bParseFailure = false;
	ds.RetryNumber = Request->NumOverallRetries;

	Parameters.EventListener->OnFragmentOpen(Request);

	TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateRaw(this, &FStreamReaderISOBMFF::FStreamHandler::HTTPCompletionCallback);
	ProgressListener->ProgressDelegate = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateRaw(this, &FStreamReaderISOBMFF::FStreamHandler::HTTPProgressCallback);

	const int64 StartOffset = Request->Track.TrackIterator->GetSampleFileOffset();
	const int64 EndOffset = StartOffset + Request->Track.TrackChunkSampleInfo.SizeInBytes;

	ReceiveBuffer.Reset();
	ReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
	TSharedPtrTS<IElectraHttpManager::FRequest> HTTP(new IElectraHttpManager::FRequest);
	HTTP->Parameters.URL = Request->MediaURL;
	HTTP->Parameters.Range.Start = StartOffset;
	HTTP->Parameters.Range.EndIncluding = EndOffset - 1;
	HTTP->Parameters.AcceptEncoding.Set(TEXT("identity"));
	HTTP->Parameters.ConnectTimeout = PlayerSessionServices->GetOptionValue(ISOBMFF::OptionKeyISOBMFFLoadConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 8));
	HTTP->Parameters.NoDataTimeout = PlayerSessionServices->GetOptionValue(ISOBMFF::OptionKeyISOBMFFLoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 6));
	HTTP->ReceiveBuffer = ReceiveBuffer;
	HTTP->ProgressListener = ProgressListener;
	HTTP->ResponseCache = PlayerSessionServices->GetHTTPResponseCache();
	PlayerSessionServices->GetHTTPManager()->AddRequest(HTTP, false);

	bool bDone = false;

	// Create CSD and buffer info
	TSharedPtrTS<FAccessUnit::CodecData> CSD(new FAccessUnit::CodecData);
	CSD->CodecSpecificData = Request->Track.TrackInfo->CodecFormat.CSD;
	CSD->RawCSD = Request->Track.TrackInfo->CodecFormat.DCR;
	CSD->ParsedInfo = FStreamCodecInformation(Request->Track.TrackInfo->CodecFormat);
	TSharedPtr<FBufferSourceInfo, ESPMode::ThreadSafe> BufferSource = MakeShared<FBufferSourceInfo, ESPMode::ThreadSafe>();
	BufferSource->PlaybackSequenceID = Request->GetPlaybackSequenceID();
	BufferSource->PeriodID = Request->AssetID;
	BufferSource->PeriodAdaptationSetID = Request->AssetID + TEXT(".") + Request->AdaptationSetID;
	BufferSource->LanguageTag = Request->LanguageTag;
	BufferSource->HardIndex = Request->AdaptationSetTrackIndex;
	BufferSource->Codec = CSD->ParsedInfo.GetCodecName();
	BufferSource->Kind = Request->KindOfTrack;

	struct FDecryptVars
	{
		ElectraCDM::FMediaCDMSampleInfo SampleEncryptionInfo;
		FDRMTypeFormat DrmFormat;
		TSharedPtr<ElectraCDM::IMediaCDMDecrypter, ESPMode::ThreadSafe> Decrypter;
		bool bIsEncrypted { false };
	};
	FDecryptVars DecryptVars;
	if (Request->Track.TrackInfo->DrmFormat.IsEncrypted())
	{
		DecryptVars.bIsEncrypted = true;
		DecryptVars.DrmFormat = Request->Track.TrackInfo->DrmFormat;
		if (DecryptVars.DrmFormat.EncryptionInfo.IsType<FDRMTypeFormat::FISOEncryptionInfo>())
		{
			const FDRMTypeFormat::FISOEncryptionInfo& encInf(DecryptVars.DrmFormat.EncryptionInfo.Get<FDRMTypeFormat::FISOEncryptionInfo>());
			DecryptVars.SampleEncryptionInfo.DefaultKID = encInf.DefaultKID;
			DecryptVars.SampleEncryptionInfo.IV = encInf.DefaultIV;
			DecryptVars.SampleEncryptionInfo.Scheme4CC = encInf.Scheme;
			if (encInf.BlockPattern.IsSet())
			{
				DecryptVars.SampleEncryptionInfo.Pattern.PatternType = 1;
				DecryptVars.SampleEncryptionInfo.Pattern.CryptByteBlock = encInf.BlockPattern.GetValue().CryptByteBlock;
				DecryptVars.SampleEncryptionInfo.Pattern.SkipByteBlock = encInf.BlockPattern.GetValue().SkipByteBlock;
			}
			if (Request->DrmClient)
			{
				Request->DrmClient->CreateDecrypter(DecryptVars.Decrypter, FString());
			}
		}
	}

	auto ShouldStop = [&]() -> bool
	{
		return HasErrored() || HasBeenAborted() || bTerminate;
	};

	uint32 PlaybackSequenceID = Request->GetPlaybackSequenceID();
	auto TrackIt = Request->Track.TrackIterator;
	int64 SizeNeededInBuffer = 0;
	FTimeValue DurationSuccessfullyRead(FTimeValue::GetZero());
	FTimeValue DurationSuccessfullyDelivered(FTimeValue::GetZero());
	bool bIsFirstInSequence = true;
	bool bSentLastAccessUnit = false;
	while(!bDone && !ShouldStop())
	{
		// Get the number of bytes we need for the next sample.
		int64 SampleSize = TrackIt->GetSampleSize();
		SizeNeededInBuffer = TrackIt->GetSampleFileOffset() + SampleSize - StartOffset;
		// Wait until the required amount of data has been read.
		while(!ShouldStop())
		{
			// Check periodically if we are to abort in case we do not have enough data yet.
			if (ReceiveBuffer->WaitUntilSizeAvailable(SizeNeededInBuffer, 1000 * 100))
			{
				break;
			}
		}
		// Leave the loop if we have to abort.
		if (ShouldStop())
		{
			break;
		}
		// Did we get enough data?
		if (ReceiveBuffer->GetLinearReadSize() < SizeNeededInBuffer)
		{
			bDone = true;
			break;
		}


		// Get the sample properties
		uint32 SampleNumber = TrackIt->GetSampleNumber();
		MP4Utilities::FFractionalTime DTS = TrackIt->GetEffectiveDTS();
		MP4Utilities::FFractionalTime PTS = TrackIt->GetEffectivePTS();
		MP4Utilities::FFractionalTime Duration = TrackIt->GetDuration();
		bool bIsSyncSample = TrackIt->IsSyncOrRAPSample();

		// Create an access unit.
		FAccessUnit* AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
		if (AccessUnit)
		{
			AccessUnit->ESType = Request->StreamType;
			AccessUnit->PTS.SetFromND(PTS.GetNumerator(), PTS.GetDenominator(), Request->TimestampSequenceIndex);
			AccessUnit->DTS.SetFromND(DTS.GetNumerator(), DTS.GetDenominator(), Request->TimestampSequenceIndex);
			AccessUnit->Duration.SetFromND(Duration.GetNumerator(), Duration.GetDenominator());

			AccessUnit->SequenceIndex = Request->TimestampSequenceIndex;
			AccessUnit->EarliestPTS = Request->EarliestPTS;
			AccessUnit->EarliestPTS.SetSequenceIndex(Request->TimestampSequenceIndex);
			AccessUnit->LatestPTS = Request->LastPTS;
			AccessUnit->LatestPTS.SetSequenceIndex(Request->TimestampSequenceIndex);

			AccessUnit->AUSize = (uint32) SampleSize;
			AccessUnit->AUCodecData = CSD;
			AccessUnit->BufferSourceInfo = BufferSource;

			AccessUnit->bIsFirstInSequence = bIsFirstInSequence;
			bIsFirstInSequence = false;
			AccessUnit->bIsSyncSample = bIsSyncSample;
			AccessUnit->bIsDummyData = false;
			AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
			{
				const int64 SampleBufferOffset = TrackIt->GetSampleFileOffset() - StartOffset;
				FScopeLock lock(ReceiveBuffer->GetLock());
				FMemory::Memcpy(AccessUnit->AUData, reinterpret_cast<const uint8*>(ReceiveBuffer->GetLinearReadData()) + SampleBufferOffset, SampleSize);
			}

			// Decrypt?
			if (DecryptVars.bIsEncrypted)
			{
				if (DecryptVars.Decrypter)
				{
					// Wait for the decrypter to get ready.
					while(!ShouldStop() && (DecryptVars.Decrypter->GetState() == ElectraCDM::ECDMState::WaitingForKey || DecryptVars.Decrypter->GetState() == ElectraCDM::ECDMState::Idle))
					{
						FMediaRunnable::SleepMilliseconds(100);
					}
					// Set up the sample encryption info.
					const MP4Boxes::FMP4BoxSENC::FEntry& enc = TrackIt->GetEncryptionInfo();
					DecryptVars.SampleEncryptionInfo.SubSamples.Empty();
					DecryptVars.SampleEncryptionInfo.IV = enc.IV;
					for(auto &sub : enc.SubSamples)
					{
						DecryptVars.SampleEncryptionInfo.SubSamples.Add({.NumClearBytes = sub.NumClearBytes, .NumEncryptedBytes = sub.NumEncryptedBytes});
					}
					// Decrypt
					ElectraCDM::ECDMError DecryptResult = ElectraCDM::ECDMError::Failure;
					if (DecryptVars.Decrypter->GetState() == ElectraCDM::ECDMState::Ready)
					{
						DecryptResult = DecryptVars.Decrypter->DecryptInPlace((uint8*) AccessUnit->AUData, (int32) AccessUnit->AUSize, DecryptVars.SampleEncryptionInfo);
					}
					if (DecryptResult != ElectraCDM::ECDMError::Success)
					{
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;
						ds.FailureReason = FString::Printf(TEXT("Failed to decrypt segment with error %d (%s)"), (int32)DecryptResult, *DecryptVars.Decrypter->GetLastErrorMessage());
						bHasErrored = true;
						bDone = true;
						break;
					}
				}
				else
				{
					FAccessUnit::Release(AccessUnit);
					AccessUnit = nullptr;
					ds.FailureReason = FString::Printf(TEXT("No valid decrypter to decrypt track"));
					bHasErrored = true;
					bDone = true;
					break;
				}
			}


			if (AccessUnit->DTS >= AccessUnit->LatestPTS && AccessUnit->PTS >= AccessUnit->LatestPTS)
			{
				// Tag the last one and send it off, but stop doing so for the remainder of the segment.
				// Note: we continue reading this chunk all the way to the end on purpose to avoid prematurely closing the connection.
				AccessUnit->bIsLastInPeriod = true;
				Request->bIsLastSegment = true;
			}

			DurationSuccessfullyRead += AccessUnit->Duration;
			bool bSentOff = bSentLastAccessUnit;
			while(!bSentOff && !HasBeenAborted() && !bTerminate)
			{
				if (Parameters.EventListener->OnFragmentAccessUnitReceived(AccessUnit))
				{
					DurationSuccessfullyDelivered += AccessUnit->Duration;
					bSentLastAccessUnit |= AccessUnit->bIsLastInPeriod;
					bSentOff = true;
					AccessUnit = nullptr;
				}
				else
				{
					FMediaRunnable::SleepMicroseconds(1000 * 10);
				}
			}

			// Release the AU if we still have it.
			FAccessUnit::Release(AccessUnit);
			AccessUnit = nullptr;
		}

		// Advance to the next sample unless we are to abort.
		if (!HasErrored() && !HasBeenAborted() && !bTerminate)
		{
			bool bReachedLastSample = !TrackIt->Next();
			if (bReachedLastSample)
			{
				bDone = true;
			}
		}
	}

	// Remove the download request.
	ProgressListener.Reset();
	PlayerSessionServices->GetHTTPManager()->RemoveRequest(HTTP, false);
	Request->ConnectionInfo = HTTP->ConnectionInfo;
	HTTP.Reset();
	ReceiveBuffer.Reset();

	// Set up download stat fields.
	ds.HTTPStatusCode = Request->ConnectionInfo.StatusInfo.HTTPStatus;
	ds.TimeToFirstByte = Request->ConnectionInfo.TimeUntilFirstByte;
	ds.TimeToDownload = (Request->ConnectionInfo.RequestEndTime - Request->ConnectionInfo.RequestStartTime).GetAsSeconds();
	ds.ByteSize = Request->ConnectionInfo.ContentLength;
	ds.NumBytesDownloaded = Request->ConnectionInfo.BytesReadSoFar;
	if (ds.FailureReason.IsEmpty())
	{
		ds.FailureReason = Request->ConnectionInfo.StatusInfo.ErrorDetail.GetMessage();
	}
	ds.bWasSuccessful = !bHasErrored;
	ds.URL.URL = Request->ConnectionInfo.EffectiveURL;
	ds.DurationDownloaded = DurationSuccessfullyRead.GetAsSeconds();
	ds.DurationDelivered  = DurationSuccessfullyDelivered.GetAsSeconds();
	ds.bIsCachedResponse = Request->ConnectionInfo.bIsCachedResponse;

	// Reset the current request so another one can be added immediately when we call OnFragmentClose()
	CurrentRequest.Reset();
	if (!bSilentCancellation)
	{
		PlayerSessionServices->GetStreamSelector()->ReportDownloadEnd(ds);
		Parameters.EventListener->OnFragmentClose(Request);
	}
}
} // namespace Electra
