// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "Player/PlayerStreamReader.h"
#include "Player/DRM/DRMManager.h"
#include "Player/isobmff/PlaylistISOBMFF.h"
#include "HTTP/HTTPManager.h"
#include "Utilities/HashFunctions.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/BCP47-Helpers.h"
#include "SynchronizedClock.h"
#include "StreamAccessUnitBuffer.h"
#include "StreamDataBuffer.h"
#include "MP4Track.h"

#include <atomic>


namespace Electra
{


class FStreamSegmentRequestISOBMFF : public IStreamSegment
{
public:
	FStreamSegmentRequestISOBMFF() = default;
	virtual ~FStreamSegmentRequestISOBMFF() = default;

	void SetPlaybackSequenceID(uint32 InPlaybackSequenceID) override
	{ PlaybackSequenceID = InPlaybackSequenceID; }
	uint32 GetPlaybackSequenceID() const override
	{ return PlaybackSequenceID; }
	void SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay) override
	{ }
	FTimeValue GetExecuteAtUTCTime() const override
	{ return FTimeValue::GetInvalid(); }
	EStreamType GetType() const override
	{ return StreamType; }

	void GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const override;
	void GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams) override;
	void GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams) override;

	FTimeValue GetFirstPTS() const override
	{
		FTimeValue First(EarliestPTS);
		First.SetSequenceIndex(TimestampSequenceIndex);
		return First; 
	}
	FTimeRange GetTimeRange() const override;

	int32 GetQualityIndex() const override
	{ return 0; }
	int32 GetBitrate() const override
	{ return Bitrate; }

	void GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const override
	{ OutStats = DownloadStats; }
	bool GetStartupDelay(FTimeValue& OutStartTime, FTimeValue& OutTimeIntoSegment, FTimeValue& OutSegmentDuration) const override
	{ return false; }

	struct FTrack
	{
		TSharedPtr<FManifestISOBMFFInternal::FTrackInfo, ESPMode::ThreadSafe> TrackInfo;
		TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe> TrackIterator;
		MP4Boxes::FMP4Track::FIterator::FChunkInfo TrackChunkSampleInfo;
	};
	enum class ELoaderType
	{
		Chunks,
		Muxed,
		Fragmented
	};

	TSharedPtrTS<ITimelineMediaAsset> MediaAsset;
	ELoaderType LoaderType { ELoaderType::Chunks };
	FString AssetID;
	FString AdaptationSetID;
	FString RepresentationID;
	int32 AdaptationSetTrackIndex { 0 };
	FString MediaURL;
	BCP47::FLanguageTag LanguageTag;
	FString KindOfTrack;
	TArray<TSharedPtrTS<FStreamSegmentRequestISOBMFF>> DependentStreams;
	FTrack Track;
	FTimeValue FirstDTS;
	FTimeValue EarliestPTS;
	FTimeValue LastPTS;
	FTimeValue SegmentDuration;
	EStreamType StreamType {EStreamType::Unsupported};
	int64 TimestampSequenceIndex { 0 };
	uint32 PlaybackSequenceID { ~0U };
	int32 Bitrate { 0 };
	int32 NumOverallRetries { 0 };
	bool bIsLastSegment { false };

	// File offsets when reading sequentially all track data in the multiplex
	int64 PrimarySampleFileOffsetStart { -1 };
	int64 PrimarySampleFileOffsetEnd { -1 };
	int32 ReportingIndexInMux { -1 };
	bool bWasSelectedAtStart { false };

	// Encryption
	TSharedPtrTS<ElectraCDM::IMediaCDMClient> DrmClient;

	// Download stats that are set during and after download.
	Metrics::FSegmentDownloadStats DownloadStats;
	HTTP::FConnectionInfo ConnectionInfo;
};


/**
 * This class implements an interface to read from an mp4 file.
 */
class FStreamReaderISOBMFF : public IStreamReader
{
public:
	FStreamReaderISOBMFF() = default;
	virtual ~FStreamReaderISOBMFF();
	void SetLoaderType(FStreamSegmentRequestISOBMFF::ELoaderType InLoaderType);
	UEMediaError Create(IPlayerSessionServices* PlayerSessionService, const CreateParam &createParam) override;
	bool LoadsMultiplexedTracksSequentially() override;
	void Close() override;
	EAddResult AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> Request) override;
	void CancelRequest(EStreamType StreamType, bool bSilent) override;
	void CancelRequests() override;

private:
	ELECTRA_IMPL_DEFAULT_ERROR_METHODS(ISOBMFFStreamReader);

	struct FStreamHandler : public FMediaThread
	{
		FStreamHandler() = default;
		virtual ~FStreamHandler() = default;
		void Cancel(bool bSilent);
		void SignalWork();
		void WorkerThread();
		void SetError(const FString& InMessage, uint16 InCode);
		bool HasErrored() const;
		void LogMessage(IInfoLog::ELevel Level, const FString& Message);
		int32 HTTPProgressCallback(const IElectraHttpManager::FRequest* Request);
		void HTTPCompletionCallback(const IElectraHttpManager::FRequest* Request);
		void HTTPUpdateStats(const FTimeValue& CurrentTime, const IElectraHttpManager::FRequest* Request);
		void HandleMuxedRequest();
		void HandleChunkOrFragmentRequest();
		bool HasBeenAborted() const;

		IStreamReader::CreateParam Parameters;
		FMediaSemaphore WorkSignal;
		FMediaEvent IsIdleSignal;
		TSharedPtrTS<FStreamSegmentRequestISOBMFF> CurrentRequest;
		IPlayerSessionServices* PlayerSessionServices = nullptr;
		volatile bool bTerminate = false;
		volatile bool bWasStarted = false;
		volatile bool bRequestCanceled = false;
		volatile bool bSilentCancellation = false;
		volatile bool bHasErrored = false;

		TSharedPtrTS<FWaitableBuffer> ReceiveBuffer;

		static std::atomic<uint32> UniqueDownloadID;
	};


	FStreamSegmentRequestISOBMFF::ELoaderType LoaderType { FStreamSegmentRequestISOBMFF::ELoaderType::Chunks };
	FStreamHandler StreamHandlers[3];		// 0 = video (or multiplexed), 1 = audio, 2 = subtitle
	FErrorDetail ErrorDetail;
	IPlayerSessionServices* PlayerSessionService = nullptr;
	bool bIsStarted = false;
};



} // namespace Electra
