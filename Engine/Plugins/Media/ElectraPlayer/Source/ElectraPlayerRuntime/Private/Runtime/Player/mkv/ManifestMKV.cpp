// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserMKV.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMKV.h"
#include "Utilities/HashFunctions.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/mkv/StreamReaderMKV.h"
#include "Player/mkv/ManifestMKV.h"


#define ERRCODE_MANIFEST_MKV_NO_PLAYABLE_STREAMS		1
#define ERRCODE_MANIFEST_MKV_STARTSEGMENT_NOT_FOUND		2


DECLARE_CYCLE_STAT(TEXT("FPlayPeriodMKV::FindSegment"), STAT_ElectraPlayer_MKV_FindSegment, STATGROUP_ElectraPlayer);

namespace Electra
{

//-----------------------------------------------------------------------------
/**
 * CTOR
 */
FManifestMKVInternal::FManifestMKVInternal(IPlayerSessionServices* InPlayerSessionServices)
	: PlayerSessionServices(InPlayerSessionServices)
{
}


//-----------------------------------------------------------------------------
/**
 * DTOR
 */
FManifestMKVInternal::~FManifestMKVInternal()
{
}


//-----------------------------------------------------------------------------
/**
 * Builds the internal manifest.
 *
 * @param MKVParser
 * @param URL
 * @param InConnectionInfo
 *
 * @return
 */
FErrorDetail FManifestMKVInternal::Build(TSharedPtrTS<IParserMKV> MKVParser, const FString& URL, const HTTP::FConnectionInfo& InConnectionInfo)
{
	ConnectionInfo = InConnectionInfo;
	MediaAsset = MakeSharedTS<FTimelineAssetMKV>();
	FErrorDetail Error = MediaAsset->Build(PlayerSessionServices, MKVParser, URL);
	FTimeRange PlaybackRange = GetPlaybackRange(IManifest::EPlaybackRangeType::TemporaryPlaystartRange);
	DefaultStartTime = PlaybackRange.Start;
	DefaultEndTime = PlaybackRange.End;
	return Error;
}


EMediaFormatType FManifestMKVInternal::GetMediaFormatType() const
{
	return EMediaFormatType::MKV;
}

bool FManifestMKVInternal::ReloadsPlaylistPeriodically() const
{
	return false;
}

bool FManifestMKVInternal::SupportsMultipleTracksPerKind() const
{
	return true;
}

bool FManifestMKVInternal::OrderOfTracksCanChangeDynamically() const
{
	return false;
}


//-----------------------------------------------------------------------------
/**
 * Returns the type of presentation.
 * For now this is always VoD.
 *
 * @return
 */
IManifest::EType FManifestMKVInternal::GetPresentationType() const
{
	return IManifest::EType::OnDemand;
}


//-----------------------------------------------------------------------------
/**
 * Returns track metadata.
 *
 * @param OutMetadata
 * @param StreamType
 */
void FManifestMKVInternal::GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const
{
	if (MediaAsset.IsValid())
	{
		MediaAsset->GetMetaData(OutMetadata, StreamType);
	}
}

void FManifestMKVInternal::UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData)
{
	// No-op.
}


//-----------------------------------------------------------------------------
/**
 * Returns the playback range on the timeline, which is a subset of the total
 * time range. This may be set through manifest internal means or by URL fragment
 * parameters where permissable (eg. example.mkv#t=22,50).
 * If start or end are not specified they will be set to invalid.
 *
 * @return Optionally set time range to which playback is restricted.
 */
FTimeRange FManifestMKVInternal::GetPlaybackRange(EPlaybackRangeType InRangeType) const
{
	FTimeRange FromTo;

	// We are interested in the 't' or 'r' fragment value here.
	FString Time;
	for(auto& Fragment : URLFragmentComponents)
	{
		if ((InRangeType == IManifest::EPlaybackRangeType::TemporaryPlaystartRange && Fragment.Name.Equals(TEXT("t"))) ||
			(InRangeType == IManifest::EPlaybackRangeType::LockedPlaybackRange && Fragment.Name.Equals(TEXT("r"))))
		{
			Time = Fragment.Value;
		}
	}
	if (!Time.IsEmpty())
	{
		FTimeRange TotalRange = GetTotalTimeRange();
		TArray<FString> TimeRange;
		const TCHAR* const TimeDelimiter = TEXT(",");
		Time.ParseIntoArray(TimeRange, TimeDelimiter, false);
		if (TimeRange.Num() && !TimeRange[0].IsEmpty())
		{
			RFC2326::ParseNPTTime(FromTo.Start, TimeRange[0]);
		}
		if (TimeRange.Num() > 1 && !TimeRange[1].IsEmpty())
		{
			RFC2326::ParseNPTTime(FromTo.End, TimeRange[1]);
		}
		// Need to clamp this into the total time range to prevent any issues.
		if (FromTo.Start.IsValid() && TotalRange.Start.IsValid() && FromTo.Start < TotalRange.Start)
		{
			FromTo.Start = TotalRange.Start;
		}
		if (FromTo.End.IsValid() && TotalRange.End.IsValid() && FromTo.End > TotalRange.End)
		{
			FromTo.End = TotalRange.End;
		}
	}
	return FromTo;
}


//-----------------------------------------------------------------------------
/**
 * Returns the minimum duration of content that must be buffered up before playback
 * will begin.
 *
 * @return
 */
FTimeValue FManifestMKVInternal::GetMinBufferTime() const
{
	return FTimeValue().SetFromSeconds(0.5);
}


TSharedPtrTS<IProducerReferenceTimeInfo> FManifestMKVInternal::GetProducerReferenceTimeInfo(int64 ID) const
{
	return nullptr;
}

TRangeSet<double> FManifestMKVInternal::GetPossiblePlaybackRates(EPlayRateType InForType) const
{
	TRangeSet<double> Ranges;
	if (InForType == IManifest::EPlayRateType::ThinnedRate)
	{
		//Ranges.Add(TRange<double>{1.0}); // normal (real-time) playback rate
		Ranges.Add(TRange<double>::Inclusive(0.5, 4.0));
	}
	Ranges.Add(TRange<double>{0.0}); // pause
	return Ranges;
}

void FManifestMKVInternal::UpdateDynamicRefetchCounter()
{
	// No-op.
}

void FManifestMKVInternal::TriggerClockSync(IManifest::EClockSyncType InClockSyncType)
{
	// No-op.
}

void FManifestMKVInternal::TriggerPlaylistRefresh()
{
	// No-op.
}

//-----------------------------------------------------------------------------
/**
 * Creates an instance of a stream reader to stream from the mkv file.
 *
 * @return
 */
IStreamReader* FManifestMKVInternal::CreateStreamReaderHandler()
{
	return new FStreamReaderMKV;
}

//-----------------------------------------------------------------------------
/**
 * Returns the playback period for the given time.
 *
 * @param OutPlayPeriod
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMKVInternal::FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	OutPlayPeriod = MakeSharedTS<FPlayPeriodMKV>(MediaAsset);
	return IManifest::FResult(IManifest::FResult::EType::Found);
}

//-----------------------------------------------------------------------------
/**
 * Returns the next playback period for the given one.
 *
 * @param OutPlayPeriod
 * @param CurrentSegment
 *
 * @return Always PastEOS since there are no periods in a Matroska file.
 */
IManifest::FResult FManifestMKVInternal::FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment)
{
	// There is no following period.
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}

//-----------------------------------------------------------------------------
/**
 * Constructs a playback period.
 *
 * @param InMediaAsset
 */
FManifestMKVInternal::FPlayPeriodMKV::FPlayPeriodMKV(TSharedPtrTS<FManifestMKVInternal::FTimelineAssetMKV> InMediaAsset)
	: MediaAsset(InMediaAsset)
	, CurrentReadyState(IManifest::IPlayPeriod::EReadyState::NotLoaded)
{
}

//-----------------------------------------------------------------------------
/**
 * Destroys a playback period.
 */
FManifestMKVInternal::FPlayPeriodMKV::~FPlayPeriodMKV()
{
}


//-----------------------------------------------------------------------------
/**
 * Sets stream playback preferences for this playback period.
 *
 * @param ForStreamType
 * @param StreamAttributes
 */
void FManifestMKVInternal::FPlayPeriodMKV::SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	if (ForStreamType == EStreamType::Video)
	{
		VideoPreferences = StreamAttributes;
	}
	else if (ForStreamType == EStreamType::Audio)
	{
		AudioPreferences = StreamAttributes;
	}
	else if (ForStreamType == EStreamType::Subtitle)
	{
		SubtitlePreferences = StreamAttributes;
	}
}

//-----------------------------------------------------------------------------
/**
 * Returns the starting bitrate.
 *
 * This is merely informational and not strictly required.
 *
 * @return
 */
int64 FManifestMKVInternal::FPlayPeriodMKV::GetDefaultStartingBitrate() const
{
	return 1000000;
}

//-----------------------------------------------------------------------------
/**
 * Returns the ready state of this playback period.
 *
 * @return
 */
IManifest::IPlayPeriod::EReadyState FManifestMKVInternal::FPlayPeriodMKV::GetReadyState()
{
	return CurrentReadyState;
}

void FManifestMKVInternal::FPlayPeriodMKV::Load()
{
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::Loaded;
}

//-----------------------------------------------------------------------------
/**
 * Prepares the playback period for playback.
 * With an mkv file we are actually always ready for playback, but we say we're not
 * one time to get here with any possible options.
 */
void FManifestMKVInternal::FPlayPeriodMKV::PrepareForPlay()
{
	SelectedVideoMetadata.Reset();
	SelectedAudioMetadata.Reset();
	SelectedSubtitleMetadata.Reset();
	VideoBufferSourceInfo.Reset();
	AudioBufferSourceInfo.Reset();
	SubtitleBufferSourceInfo.Reset();
	SelectInitialStream(EStreamType::Video);
	SelectInitialStream(EStreamType::Audio);
	SelectInitialStream(EStreamType::Subtitle);
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::IsReady;
}

TSharedPtrTS<FBufferSourceInfo> FManifestMKVInternal::FPlayPeriodMKV::GetSelectedStreamBufferSourceInfo(EStreamType StreamType)
{
	return StreamType == EStreamType::Video ? VideoBufferSourceInfo :
		   StreamType == EStreamType::Audio ? AudioBufferSourceInfo :
		   StreamType == EStreamType::Subtitle ? SubtitleBufferSourceInfo : nullptr;
}

FString FManifestMKVInternal::FPlayPeriodMKV::GetSelectedAdaptationSetID(EStreamType StreamType)
{
	switch(StreamType)
	{
		case EStreamType::Video:
		{
			return SelectedVideoMetadata.IsValid() ? SelectedVideoMetadata->ID : FString();
		}
		case EStreamType::Audio:
		{
			return SelectedAudioMetadata.IsValid() ? SelectedAudioMetadata->ID : FString();
		}
		case EStreamType::Subtitle:
		{
			return SelectedSubtitleMetadata.IsValid() ? SelectedSubtitleMetadata->ID : FString();
		}
		default:
		{
			return FString();
		}
	}
}

IManifest::IPlayPeriod::ETrackChangeResult FManifestMKVInternal::FPlayPeriodMKV::ChangeTrackStreamPreference(EStreamType StreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	TSharedPtrTS<FTrackMetadata> Metadata = SelectMetadataForAttributes(StreamType, StreamAttributes);
	if (Metadata.IsValid())
	{
		if (StreamType == EStreamType::Video)
		{
			if (!(SelectedVideoMetadata.IsValid() && Metadata->Equals(*SelectedVideoMetadata)))
			{
				SelectedVideoMetadata = Metadata;
				MakeBufferSourceInfoFromMetadata(StreamType, VideoBufferSourceInfo, SelectedVideoMetadata);
				return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
			}
		}
		else if (StreamType == EStreamType::Audio)
		{
			if (!(SelectedAudioMetadata.IsValid() && Metadata->Equals(*SelectedAudioMetadata)))
			{
				SelectedAudioMetadata = Metadata;
				MakeBufferSourceInfoFromMetadata(StreamType, AudioBufferSourceInfo, SelectedAudioMetadata);
				return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
			}
		}
		else if (StreamType == EStreamType::Subtitle)
		{
			if (!(SelectedSubtitleMetadata.IsValid() && Metadata->Equals(*SelectedSubtitleMetadata)))
			{
				SelectedSubtitleMetadata = Metadata;
				MakeBufferSourceInfoFromMetadata(StreamType, SubtitleBufferSourceInfo, SelectedSubtitleMetadata);
				return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
			}
		}
	}
	return IManifest::IPlayPeriod::ETrackChangeResult::NotChanged;
}

void FManifestMKVInternal::FPlayPeriodMKV::SelectInitialStream(EStreamType StreamType)
{
	if (StreamType == EStreamType::Video)
	{
		SelectedVideoMetadata = SelectMetadataForAttributes(StreamType, VideoPreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, VideoBufferSourceInfo, SelectedVideoMetadata);
	}
	else if (StreamType == EStreamType::Audio)
	{
		SelectedAudioMetadata = SelectMetadataForAttributes(StreamType, AudioPreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, AudioBufferSourceInfo, SelectedAudioMetadata);
	}
	else if (StreamType == EStreamType::Subtitle)
	{
		SelectedSubtitleMetadata = SelectMetadataForAttributes(StreamType, SubtitlePreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, SubtitleBufferSourceInfo, SelectedSubtitleMetadata);
	}
}

TSharedPtrTS<FTrackMetadata> FManifestMKVInternal::FPlayPeriodMKV::SelectMetadataForAttributes(EStreamType StreamType, const FStreamSelectionAttributes& InAttributes)
{
	TSharedPtrTS<FTimelineAssetMKV> Asset = MediaAsset.Pin();
	if (Asset.IsValid())
	{
		TArray<FTrackMetadata> Metadata;
		Asset->GetMetaData(Metadata, StreamType);
		// Is there a fixed index to be used?
		if (InAttributes.OverrideIndex.IsSet() && InAttributes.OverrideIndex.GetValue() >= 0 && InAttributes.OverrideIndex.GetValue() < Metadata.Num())
		{
			// Use this.
			return MakeSharedTS<FTrackMetadata>(Metadata[InAttributes.OverrideIndex.GetValue()]);
		}
		if (Metadata.Num())
		{
			// We do not look at the 'kind' or 'codec' here, only the language.
			// Set the first track as default in case we do not find the one we're looking for.
			if (InAttributes.Language_RFC4647.IsSet())
			{
				TArray<int32> CandidateIndices;
				TArray<BCP47::FLanguageTag> CandList;
				for(auto &Meta : Metadata)
				{
					CandList.Emplace(Meta.LanguageTagRFC5646);
				}
				CandidateIndices = BCP47::FindExtendedFilteringMatch(CandList, InAttributes.Language_RFC4647.GetValue());
				return MakeSharedTS<FTrackMetadata>(CandidateIndices.Num() ? Metadata[CandidateIndices[0]] : Metadata[0]);
			}
			// Subtitle tracks are not selected by default. If there is no explicit selection asked for we ignore them.
			if (StreamType != EStreamType::Subtitle)
			{
				return MakeSharedTS<FTrackMetadata>(Metadata[0]);
			}
		}
	}
	return nullptr;
}

void FManifestMKVInternal::FPlayPeriodMKV::MakeBufferSourceInfoFromMetadata(EStreamType StreamType, TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FTrackMetadata> InMetadata)
{
	if (InMetadata.IsValid())
	{
		OutBufferSourceInfo = MakeSharedTS<FBufferSourceInfo>();
		OutBufferSourceInfo->Kind = InMetadata->Kind;
		OutBufferSourceInfo->LanguageTag = InMetadata->LanguageTagRFC5646;
		OutBufferSourceInfo->Codec = InMetadata->HighestBandwidthCodec.GetCodecName();
		TSharedPtrTS<FTimelineAssetMKV> Asset = MediaAsset.Pin();
		OutBufferSourceInfo->PeriodID = Asset->GetUniqueIdentifier();
		OutBufferSourceInfo->PeriodAdaptationSetID = Asset->GetUniqueIdentifier() + TEXT(".") + InMetadata->ID;
		TArray<FTrackMetadata> Metadata;
		Asset->GetMetaData(Metadata, StreamType);
		for(int32 i=0; i<Metadata.Num(); ++i)
		{
			if (Metadata[i].Equals(*InMetadata))
			{
				OutBufferSourceInfo->HardIndex = i;
				break;
			}
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Returns the timeline media asset. We have a weak pointer to it only to
 * prevent any cyclic locks, so we need to lock it first.
 *
 * @return
 */
TSharedPtrTS<ITimelineMediaAsset> FManifestMKVInternal::FPlayPeriodMKV::GetMediaAsset() const
{
	TSharedPtrTS<ITimelineMediaAsset> ma = MediaAsset.Pin();
	return ma;
}

//-----------------------------------------------------------------------------
/**
 * Selects a particular stream (== internal track ID) for playback.
 *
 * @param AdaptationSetID
 * @param RepresentationID
 */
void FManifestMKVInternal::FPlayPeriodMKV::SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex)
{
	// Presently this method is only called by the ABR to switch between quality levels.
	// Since a single mkv doesn't have different quality levels we ignore this for now.
	// This may need an implementation when switching between different languages though.
	// .....
}

void FManifestMKVInternal::FPlayPeriodMKV::TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload)
{
	// No-op.
}

//-----------------------------------------------------------------------------
/**
 * Creates the starting segment request to start playback with.
 *
 * @param OutSegment
 * @param InSequenceState
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMKVInternal::FPlayPeriodMKV::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	TSharedPtrTS<FTimelineAssetMKV> ma = MediaAsset.Pin();
	FString SelectedStreamID = SelectedVideoMetadata.IsValid() ? GetSelectedAdaptationSetID(EStreamType::Video) : GetSelectedAdaptationSetID(EStreamType::Audio);
	return ma.IsValid() ? ma->GetStartingSegment(OutSegment, InSequenceState, SelectedStreamID, StartPosition, SearchType, -1) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}

//-----------------------------------------------------------------------------
/**
 * Same as GetStartingSegment() except this is for a specific stream (video, audio, ...) only.
 * To be used when a track (language) change is made and a new segment is needed at the current playback position.
 */
IManifest::FResult FManifestMKVInternal::FPlayPeriodMKV::GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType StreamType, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	// Not supported
	return IManifest::FResult(IManifest::FResult::EType::NotFound);
}

//-----------------------------------------------------------------------------
/**
 * Sets up a starting segment request to loop playback to.
 * The streams selected through SelectStream() will be used.
 *
 * @param OutSegment
 * @param SequenceState
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMKVInternal::FPlayPeriodMKV::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	TSharedPtrTS<FTimelineAssetMKV> ma = MediaAsset.Pin();
	FString SelectedStreamID = SelectedVideoMetadata.IsValid() ? GetSelectedAdaptationSetID(EStreamType::Video) : GetSelectedAdaptationSetID(EStreamType::Audio);
	return ma.IsValid() ? ma->GetLoopingSegment(OutSegment, SequenceState, SelectedStreamID, StartPosition, SearchType) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}

//-----------------------------------------------------------------------------
/**
 * Called by the ABR to increase the delay in fetching the next segment in case the segment returned a 404 when fetched at
 * the announced availability time. This may reduce 404's on the next segment fetches.
 *
 * @param IncreaseAmount
 */
void FManifestMKVInternal::FPlayPeriodMKV::IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount)
{
	// No-op.
}

//-----------------------------------------------------------------------------
/**
 * Creates the next segment request.
 *
 * @param OutSegment
 * @param CurrentSegment
 * @param Options
 *
 * @return
 */
IManifest::FResult FManifestMKVInternal::FPlayPeriodMKV::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options)
{
	TSharedPtrTS<FTimelineAssetMKV> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetNextSegment(OutSegment, CurrentSegment, Options) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}

//-----------------------------------------------------------------------------
/**
 * Creates a segment retry request.
 *
 * @param OutSegment
 * @param CurrentSegment
 * @param Options
 * @param bReplaceWithFillerData
 *
 * @return
 */
IManifest::FResult FManifestMKVInternal::FPlayPeriodMKV::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData)
{
	TSharedPtrTS<FTimelineAssetMKV> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetRetrySegment(OutSegment, CurrentSegment, Options, bReplaceWithFillerData) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}

//-----------------------------------------------------------------------------
/**
 * Returns the average segment duration.
 */
void FManifestMKVInternal::FPlayPeriodMKV::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	TSharedPtrTS<FTimelineAssetMKV> ma = MediaAsset.Pin();
	if (ma.IsValid())
	{
		ma->GetAverageSegmentDuration(OutAverageSegmentDuration, AdaptationSetID, RepresentationID);
	}
}

//-----------------------------------------------------------------------------
/**
 * Builds the timeline asset.
 *
 * @param InPlayerSessionServices
 * @param InMKVParser
 * @param URL
 *
 * @return
 */
FErrorDetail FManifestMKVInternal::FTimelineAssetMKV::Build(IPlayerSessionServices* InPlayerSessionServices, TSharedPtrTS<IParserMKV> InMKVParser, const FString& URL)
{
	PlayerSessionServices = InPlayerSessionServices;
	MediaURL = URL;

	MovieDuration = InMKVParser->GetDuration();
	// Go over the supported tracks and create an internal manifest-like structure for the player to work with.
	for(int32 nTrack=0,nMaxTrack=InMKVParser->GetNumberOfTracks(); nTrack < nMaxTrack; ++nTrack)
	{
		const IParserMKV::ITrack* Track = InMKVParser->GetTrackByIndex(nTrack);
		if (Track)
		{
			// In an mkv file we treat every track as a single adaptation set with one representation only.
			// That's because by definition an adaptation set contains the same content at different bitrates and resolutions, but
			// the type, language and codec has to be the same.
			FErrorDetail Error;
			TSharedPtrTS<FAdaptationSetMKV> AdaptationSet = MakeSharedTS<FAdaptationSetMKV>();
			Error = AdaptationSet->CreateFrom(Track, URL);
			if (Error.IsOK())
			{
				// Add this track to the proper category.
				switch(Track->GetCodecInformation().GetStreamType())
				{
					case EStreamType::Video:
					{
						VideoAdaptationSets.Add(AdaptationSet);
						UsableTrackIDs.Emplace(Track->GetID());
						break;
					}
					case EStreamType::Audio:
					{
						AudioAdaptationSets.Add(AdaptationSet);
						UsableTrackIDs.Emplace(Track->GetID());
						break;
					}
					case EStreamType::Subtitle:
					{
						SubtitleAdaptationSets.Add(AdaptationSet);
						UsableTrackIDs.Emplace(Track->GetID());
						break;
					}
					default:
					{
						break;
					}
				}
			}
			else
			{
				return Error;
			}
		}
	}

	// No playable content?
	if (VideoAdaptationSets.Num() == 0 && AudioAdaptationSets.Num() == 0)
	{
		FErrorDetail Error;
		Error.SetFacility(Facility::EFacility::MKVPlaylist);
		Error.SetMessage("No playable streams in this file");
		Error.SetCode(ERRCODE_MANIFEST_MKV_NO_PLAYABLE_STREAMS);
		return Error;
	}

	// Hold on to the parser for future reference.
	MKVParser = InMKVParser;
	return FErrorDetail();
}

IManifest::FResult FManifestMKVInternal::FTimelineAssetMKV::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, FString InSelectedStreamID, const FPlayStartPosition& StartPosition, ESearchType SearchType, int64 AtAbsoluteFilePos)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MKV_FindSegment);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, MKV_FindSegment);

	// Frame accurate seek required?
	bool bFrameAccurateSearch = AtAbsoluteFilePos < 0 ? StartPosition.Options.bFrameAccuracy : false;
	FTimeValue PlayRangeEnd = StartPosition.Options.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());
	if (PlayRangeEnd > MovieDuration)
	{
		PlayRangeEnd = MovieDuration;
	}

	// Do we have video?
	TSharedPtrTS<FRepresentationMKV> Repr;
	EStreamType ReprType = EStreamType::Unsupported;
	if (VideoAdaptationSets.Num())
	{
		Repr = StaticCastSharedPtr<FRepresentationMKV>(VideoAdaptationSets[0]->GetRepresentationByIndex(0));
		ReprType = EStreamType::Video;
		if (InSelectedStreamID.Len())
		{
			for(int32 i=0; i<VideoAdaptationSets.Num(); ++i)
			{
				if (VideoAdaptationSets[i]->GetRepresentationByIndex(0)->GetUniqueIdentifier().Equals(InSelectedStreamID))
				{
					Repr = StaticCastSharedPtr<FRepresentationMKV>(VideoAdaptationSets[i]->GetRepresentationByIndex(0));
					break;
				}
			}
		}
	}
	else if (AudioAdaptationSets.Num())
	{
		Repr = StaticCastSharedPtr<FRepresentationMKV>(AudioAdaptationSets[0]->GetRepresentationByIndex(0));
		ReprType = EStreamType::Audio;
		if (InSelectedStreamID.Len())
		{
			for(int32 i=0; i<AudioAdaptationSets.Num(); ++i)
			{
				if (AudioAdaptationSets[i]->GetRepresentationByIndex(0)->GetUniqueIdentifier().Equals(InSelectedStreamID))
				{
					Repr = StaticCastSharedPtr<FRepresentationMKV>(AudioAdaptationSets[i]->GetRepresentationByIndex(0));
					break;
				}
			}
		}
	}

	if (Repr.IsValid())
	{
		int32 TrackID = 0;
		LexFromString(TrackID, *Repr->GetUniqueIdentifier());
		const IParserMKV::ITrack* Track = MKVParser->GetTrackByTrackID(TrackID);
		if (Track)
		{
			TSharedPtrTS<IParserMKV::ICueIterator> TrackIt(Track->CreateCueIterator());
			IParserMKV::ICueIterator::ESearchMode SearchMode =
				SearchType == ESearchType::After  || SearchType == ESearchType::StrictlyAfter  ? IParserMKV::ICueIterator::ESearchMode::After  :
				SearchType == ESearchType::Before || SearchType == ESearchType::StrictlyBefore ? IParserMKV::ICueIterator::ESearchMode::Before : IParserMKV::ICueIterator::ESearchMode::Closest;
			UEMediaError Error;
			FTimeValue firstTimestamp;
			int64 firstByteOffset = 0;

			FTimeValue StreamDuration = MKVParser->GetDuration();
			if (StreamDuration.IsValid() && StreamDuration <= StartPosition.Time && (SearchType == ESearchType::After || SearchType == ESearchType::StrictlyAfter || SearchType == ESearchType::Closest))
			{
				firstTimestamp = StreamDuration;
				Error = UEMEDIA_ERROR_END_OF_STREAM;
			}
			else
			{
				Error = TrackIt->StartAtTime(StartPosition.Time, SearchMode);
				firstTimestamp = Error == UEMEDIA_ERROR_OK ? TrackIt->GetTimestamp() : StreamDuration;
			}
			if (Error == UEMEDIA_ERROR_OK || Error == UEMEDIA_ERROR_END_OF_STREAM)
			{
				// Time found. Set up the fragment request.
				TSharedPtrTS<FStreamSegmentRequestMKV> req(new FStreamSegmentRequestMKV);
				OutSegment = req;
				req->MediaAsset = SharedThis(this);
				req->MKVParser = MKVParser;
				req->PrimaryTrackID = Track->GetID();
				req->EnabledTrackIDs = UsableTrackIDs;
				req->FirstPTS = firstTimestamp;
				req->PrimaryStreamType = ReprType;
				req->Bitrate = Repr->GetBitrate();
				// Set the PTS range for which to present samples.
				req->EarliestPTS = bFrameAccurateSearch ? StartPosition.Time : firstTimestamp;
				check(req->EarliestPTS.IsValid());
				req->LastPTS = PlayRangeEnd;
				req->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
				// Set info from the track iterator if found
				if (Error == UEMEDIA_ERROR_OK)
				{
					req->bIsFirstSegment = true;
					req->CueUniqueID = TrackIt->GetUniqueID();
					req->NextCueUniqueID = TrackIt->GetNextUniqueID();
					req->FileStartOffset = TrackIt->GetClusterFileOffset();
					req->FileEndOffset = TrackIt->GetClusterFileOffset() + TrackIt->GetClusterFileSize() - 1;
					req->bIsLastSegment = TrackIt->IsLastCluster() || TrackIt->GetTimestamp() + TrackIt->GetClusterDuration() >= PlayRangeEnd;
					req->SegmentDuration = TrackIt->GetClusterDuration();
				}
				else
				{
					// Set end-of-stream info.
					req->bIsLastSegment = true;
					req->bIsEOSSegment = true;
				}
				return IManifest::FResult(IManifest::FResult::EType::Found);
			}
			else if (Error == UEMEDIA_ERROR_INSUFFICIENT_DATA)
			{
				return IManifest::FResult(IManifest::FResult::EType::BeforeStart);
			}
			else
			{
				IManifest::FResult res(IManifest::FResult::EType::NotFound);
				res.SetErrorDetail(FErrorDetail().SetError(Error)
									.SetFacility(Facility::EFacility::MKVPlaylist)
									.SetCode(ERRCODE_MANIFEST_MKV_STARTSEGMENT_NOT_FOUND)
									.SetMessage(FString::Printf(TEXT("Could not find start segment for time %lld"), (long long int)StartPosition.Time.GetAsHNS())));
				return res;
			}
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetError(UEMEDIA_ERROR_INSUFFICIENT_DATA)
					   .SetFacility(Facility::EFacility::MKVPlaylist)
					   .SetCode(ERRCODE_MANIFEST_MKV_STARTSEGMENT_NOT_FOUND)
					   .SetMessage(FString::Printf(TEXT("Could not find start segment for time %lld, no valid tracks"), (long long int)StartPosition.Time.GetAsHNS())));
}

IManifest::FResult FManifestMKVInternal::FTimelineAssetMKV::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options)
{
	const FStreamSegmentRequestMKV* Request = static_cast<const FStreamSegmentRequestMKV*>(CurrentSegment.Get());
	if (Request)
	{
		// Check if the current request did not already go up to the end of the stream. If so there is no next segment.
		if (Request->NextCueUniqueID != ~0U && !Request->bIsLastSegment)
		{
			const IParserMKV::ITrack* Track = MKVParser->GetTrackByTrackID(Request->PrimaryTrackID);
			if (Track)
			{
				TSharedPtrTS<IParserMKV::ICueIterator> TrackIt(Track->CreateCueIterator());
				UEMediaError Error = TrackIt->StartAtUniqueID(Request->NextCueUniqueID);
				if (Error == UEMEDIA_ERROR_OK)
				{
					// Time found. Set up the fragment request.
					TSharedPtrTS<FStreamSegmentRequestMKV> req(new FStreamSegmentRequestMKV);
					OutSegment = req;

					req->MediaAsset = SharedThis(this);
					req->MKVParser = MKVParser;
					req->PrimaryTrackID = Track->GetID();
					req->CueUniqueID = TrackIt->GetUniqueID();
					req->NextCueUniqueID = TrackIt->GetNextUniqueID();
					req->EnabledTrackIDs = UsableTrackIDs;
					req->FirstPTS = TrackIt->GetTimestamp();
					req->FileStartOffset = TrackIt->GetClusterFileOffset();
					req->FileEndOffset = TrackIt->GetClusterFileOffset() + TrackIt->GetClusterFileSize() - 1;
					req->SegmentDuration = TrackIt->GetClusterDuration();
					req->Bitrate = Request->Bitrate;
					req->PrimaryStreamType = Request->PrimaryStreamType;
					req->bIsFirstSegment = false;
					req->bIsLastSegment = TrackIt->IsLastCluster() || TrackIt->GetTimestamp() + TrackIt->GetClusterDuration() >= Request->LastPTS;
					req->EarliestPTS = Request->EarliestPTS;
					req->LastPTS = Request->LastPTS;
					req->TimestampSequenceIndex = Request->TimestampSequenceIndex;
					return IManifest::FResult(IManifest::FResult::EType::Found);
				}
			}
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}

IManifest::FResult FManifestMKVInternal::FTimelineAssetMKV::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData)
{
	const FStreamSegmentRequestMKV* Request = static_cast<const FStreamSegmentRequestMKV*>(CurrentSegment.Get());
	if (Request)
	{
		TSharedPtrTS<FStreamSegmentRequestMKV> req(new FStreamSegmentRequestMKV);
		OutSegment = req;

		// Copy all relevant values from the failed request over to the new retry request.
		req->EnabledTrackIDs = Request->EnabledTrackIDs;
		req->MediaAsset = Request->MediaAsset;
		req->MKVParser = Request->MKVParser;
		req->FirstPTS = Request->FirstPTS;
		req->SegmentDuration = Request->SegmentDuration;
		req->EarliestPTS = Request->EarliestPTS;
		req->LastPTS = Request->LastPTS;
		req->TimestampSequenceIndex = Request->TimestampSequenceIndex;
		req->PrimaryStreamType = Request->PrimaryStreamType;
		req->FileEndOffset = Request->FileEndOffset;
		req->PrimaryTrackID = Request->PrimaryTrackID;
		req->CueUniqueID = Request->CueUniqueID;
		req->NextCueUniqueID = Request->NextCueUniqueID;
		req->PlaybackSequenceID = Request->PlaybackSequenceID;
		req->Bitrate = Request->Bitrate;
		req->bIsFirstSegment = Request->bIsFirstSegment;
		req->bIsLastSegment = Request->bIsLastSegment;
		req->bIsEOSSegment = Request->bIsEOSSegment;
		req->NumOverallRetries = Request->NumOverallRetries + 1;

		// The file retry position is that of the failed cluster. This is for the case
		// where several clusters are between two Cue Points and we did not fail on the
		// first one. There's no need to load and re-parse earlier clusters.
		req->FileStartOffset = Request->FailedClusterFileOffset;
		req->RetryBlockOffset = Request->FailedClusterDataOffset;
		// We need to remember where we failed at. This is for the case where the retry
		// also fails, but an an earlier position than before, in which case we have to
		// remember the farthest position we ever got in that cluster.
		req->FailedClusterDataOffset = Request->FailedClusterDataOffset;
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}
	return IManifest::FResult(IManifest::FResult::EType::NotFound);
}

IManifest::FResult FManifestMKVInternal::FTimelineAssetMKV::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& SequenceState, FString SelectedStreamID, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	return GetStartingSegment(OutSegment, SequenceState, SelectedStreamID, StartPosition, SearchType, -1);
}

void FManifestMKVInternal::FTimelineAssetMKV::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	// This is not expected to be called. And if it does we return a dummy entry.
	OutAverageSegmentDuration.SetFromSeconds(5.0);
}

TSharedPtrTS<IParserMKV> FManifestMKVInternal::FTimelineAssetMKV::GetMKVParser()
{
	return MKVParser;
}

FErrorDetail FManifestMKVInternal::FAdaptationSetMKV::CreateFrom(const IParserMKV::ITrack* InTrack, const FString& InURL)
{
	FErrorDetail Error;
	Representation = MakeSharedTS<FRepresentationMKV>();
	Error = Representation->CreateFrom(InTrack, InURL);
	if (Error.IsOK())
	{
		CodecRFC6381 = Representation->GetCodecInformation().GetCodecSpecifierRFC6381();
		UniqueIdentifier = Representation->GetUniqueIdentifier();
		LanguageTag = InTrack->GetLanguageTag();
	}
	return Error;
}

FErrorDetail FManifestMKVInternal::FRepresentationMKV::CreateFrom(const IParserMKV::ITrack* InTrack, const FString& URL)
{
	CodecInformation = InTrack->GetCodecInformation();
	CodecSpecificData = InTrack->GetCodecSpecificData();
	// The unique identifier will be the track ID inside the mkv.
	// NOTE: This *MUST* be just a number since it gets parsed back out from a string into a number later! Do *NOT* prepend/append any string literals!!
	UniqueIdentifier = LexToString(InTrack->GetID());
	Name = InTrack->GetName();
	switch(CodecInformation.GetStreamType())
	{
		case EStreamType::Video:
		{
			Bitrate = 1 * 1024 * 1024;
			break;
		}
		case EStreamType::Audio:
		{
			Bitrate = 64 * 1024;
			break;
		}
		case EStreamType::Subtitle:
		{
			Bitrate = 8 * 1024;
			break;
		}
		default:
		{
			// Whatever it is, assume it's a low bitrate.
			Bitrate = 32 * 1024;
			break;
		}
	}
	if (!CodecInformation.GetBitrate())
	{
		CodecInformation.SetBitrate(Bitrate);
	}
	// Not a whole lot that could have gone wrong here.
	return FErrorDetail();
}

} // namespace Electra
