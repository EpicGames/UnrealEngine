// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "Player/PlaybackTimeline.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserMKV.h"
#include "SynchronizedClock.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "Utilities/URLParser.h"
#include "MediaStreamMetadata.h"


namespace Electra
{

/**
 * This class represents the internal "manifest" or "playlist" of an mkv file.
 * All metadata of the tracks it is composed of are maintained by this class.
 */
class FManifestMKVInternal : public IManifest, public TSharedFromThis<FManifestMKVInternal, ESPMode::ThreadSafe>
{
public:
	FManifestMKVInternal(IPlayerSessionServices* InPlayerSessionServices);
	virtual ~FManifestMKVInternal();

	FErrorDetail Build(TSharedPtrTS<IParserMKV> MP4Parser, const FString& URL, const HTTP::FConnectionInfo& InConnectionInfo);

	EMediaFormatType GetMediaFormatType() const override;
	bool ReloadsPlaylistPeriodically() const override;
	bool SupportsMultipleTracksPerKind() const override;
	bool OrderOfTracksCanChangeDynamically() const override;
	EType GetPresentationType() const override;
	EReplayEventType GetReplayType() const override
	{ return IManifest::EReplayEventType::NoReplay; }
	TSharedPtrTS<const FLowLatencyDescriptor> GetLowLatencyDescriptor() const override
	{ return nullptr; }
	FTimeValue CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const override
	{ return FTimeValue(); }
	FTimeValue GetAnchorTime() const override
	{ return FTimeValue::GetZero(); }
	FTimeRange GetTotalTimeRange() const override
	{ return MediaAsset.IsValid() ? MediaAsset->GetTimeRange() : FTimeRange(); }
	FTimeRange GetSeekableTimeRange() const override
	{
		// For the time being the seekable range equals the total range.
		return GetTotalTimeRange();
	}
	FTimeRange GetPlaybackRange(EPlaybackRangeType InRangeType) const override;
	FTimeValue GetDuration() const override
	{ return MediaAsset.IsValid() ? MediaAsset->GetDuration() : FTimeValue(); }
	FTimeValue GetDefaultStartTime() const override
	{ return DefaultStartTime; }
	void ClearDefaultStartTime() override
	{ DefaultStartTime.SetToInvalid(); }
	FTimeValue GetDefaultEndTime() const override
	{ return DefaultEndTime; }
	void ClearDefaultEndTime() override
	{ DefaultEndTime.SetToInvalid(); }
	void GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const override;
	void UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData) override;
	FTimeValue GetMinBufferTime() const override;
	FTimeValue GetDesiredLiveLatency() const override
	{ return FTimeValue(); }
	ELiveEdgePlayMode GetLiveEdgePlayMode() const override
	{ return IManifest::ELiveEdgePlayMode::Never; }
	TRangeSet<double> GetPossiblePlaybackRates(EPlayRateType InForType) const override;
	TSharedPtrTS<IProducerReferenceTimeInfo> GetProducerReferenceTimeInfo(int64 ID) const override;
	void UpdateDynamicRefetchCounter() override;
	void PrepareForLooping(int32 InNumLoopsToAdd) override
	{ }
	void TriggerClockSync(EClockSyncType InClockSyncType) override;
	void TriggerPlaylistRefresh() override;
	void ReachedStableBuffer() override
	{ }

	IStreamReader* CreateStreamReaderHandler() override;
	FResult FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
	FResult FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment) override;


	class FRepresentationMKV : public IPlaybackAssetRepresentation
	{
	public:
		virtual ~FRepresentationMKV() = default;

		FErrorDetail CreateFrom(const IParserMKV::ITrack* InTrack, const FString& URL);

		FString GetUniqueIdentifier() const override
		{ return UniqueIdentifier; }
		const FStreamCodecInformation& GetCodecInformation() const override
		{ return CodecInformation; }
		int32 GetBitrate() const override
		{ return Bitrate; }
		int32 GetQualityIndex() const override
		{ return 0; }
		bool CanBePlayed() const override
		{ return true; }

		const FString& GetName() const
		{ return Name; }
	private:
		FStreamCodecInformation		CodecInformation;
		FString						UniqueIdentifier;
		FString						Name;
		int32						Bitrate;
		TArray<uint8>				CodecSpecificData;
	};

	class FAdaptationSetMKV : public IPlaybackAssetAdaptationSet
	{
	public:
		virtual ~FAdaptationSetMKV() = default;

		FErrorDetail CreateFrom(const IParserMKV::ITrack* InTrack, const FString& URL);

		FString GetUniqueIdentifier() const override
		{ return UniqueIdentifier; }
		FString GetListOfCodecs() const override
		{ return CodecRFC6381; }
		const BCP47::FLanguageTag& GetLanguageTag() const override
		{ return LanguageTag; }

		int32 GetNumberOfRepresentations() const override
		{ return Representation.IsValid() ? 1 : 0; }
		bool IsLowLatencyEnabled() const override
		{ return false; }
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByIndex(int32 RepresentationIndex) const override
		{ return RepresentationIndex == 0 ? Representation : TSharedPtrTS<IPlaybackAssetRepresentation>(); }
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByUniqueIdentifier(const FString& InUniqueIdentifier) const override
		{ return Representation.IsValid() && Representation->GetUniqueIdentifier() == InUniqueIdentifier ? Representation : TSharedPtrTS<IPlaybackAssetRepresentation>(); }
	private:
		TSharedPtrTS<FRepresentationMKV> Representation;
		BCP47::FLanguageTag LanguageTag;
		FString CodecRFC6381;
		FString UniqueIdentifier;
	};

	class FTimelineAssetMKV : public ITimelineMediaAsset, public TSharedFromThis<FTimelineAssetMKV, ESPMode::ThreadSafe>
	{
	public:
		FTimelineAssetMKV()
			: PlayerSessionServices(nullptr)
		{ }

		virtual ~FTimelineAssetMKV() = default;

		FErrorDetail Build(IPlayerSessionServices* InPlayerSessionServices, TSharedPtrTS<IParserMKV> InMKVParser, const FString& InURL);

		virtual FTimeRange GetTimeRange() const override
		{
			FTimeRange tr;
			tr.Start.SetToZero();
			tr.End = GetDuration();
			return tr;
		}

		virtual FTimeValue GetDuration() const override
		{
			return MovieDuration;
		}

		virtual FString GetAssetIdentifier() const override
		{ return FString("mkv-asset.0"); }
		virtual FString GetUniqueIdentifier() const override
		{ return FString("mkv-media.0"); }
		virtual int32 GetNumberOfAdaptationSets(EStreamType OfStreamType) const override
		{
			switch(OfStreamType)
			{
				case EStreamType::Video:
				{
					return VideoAdaptationSets.Num();
				}
				case EStreamType::Audio:
				{
					return AudioAdaptationSets.Num();
				}
				case EStreamType::Subtitle:
				{
					return SubtitleAdaptationSets.Num();
				}
				default:
				{
					return 0;
				}
			}
		}
		virtual TSharedPtrTS<IPlaybackAssetAdaptationSet> GetAdaptationSetByTypeAndIndex(EStreamType OfStreamType, int32 AdaptationSetIndex) const override
		{
			switch(OfStreamType)
			{
				case EStreamType::Video:
				{
					return AdaptationSetIndex < VideoAdaptationSets.Num() ? VideoAdaptationSets[AdaptationSetIndex] : TSharedPtrTS<IPlaybackAssetAdaptationSet>();
				}
				case EStreamType::Audio:
				{
					return AdaptationSetIndex < AudioAdaptationSets.Num() ? AudioAdaptationSets[AdaptationSetIndex] : TSharedPtrTS<IPlaybackAssetAdaptationSet>();
				}
				case EStreamType::Subtitle:
				{
					return AdaptationSetIndex < SubtitleAdaptationSets.Num() ? SubtitleAdaptationSets[AdaptationSetIndex] : TSharedPtrTS<IPlaybackAssetAdaptationSet>();
				}
				default:
				{
					break;
				}
			}
			return TSharedPtrTS<IPlaybackAssetAdaptationSet>();
		}

		virtual void GetMetaData(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const override
		{
			for(int32 i=0, iMax=GetNumberOfAdaptationSets(StreamType); i<iMax; ++i)
			{
				TSharedPtrTS<FAdaptationSetMKV> AdaptSet = StaticCastSharedPtr<FAdaptationSetMKV>(GetAdaptationSetByTypeAndIndex(StreamType, i));
				if (AdaptSet.IsValid())
				{
					FTrackMetadata tm;
					tm.ID = AdaptSet->GetUniqueIdentifier();
					tm.LanguageTagRFC5646 = AdaptSet->GetLanguageTag();
					tm.Kind = i==0 ? TEXT("main") : TEXT("translation");

					for(int32 j=0, jMax=AdaptSet->GetNumberOfRepresentations(); j<jMax; ++j)
					{
						TSharedPtrTS<FRepresentationMKV> Repr = StaticCastSharedPtr<FRepresentationMKV>(AdaptSet->GetRepresentationByIndex(j));
						if (Repr.IsValid())
						{
							FStreamMetadata sd;
							sd.Bandwidth = Repr->GetBitrate();
							sd.CodecInformation = Repr->GetCodecInformation();
							sd.ID = Repr->GetUniqueIdentifier();
							// There is only 1 "stream" per "track" so we can set the highest bitrate and codec info the same as the track.
							tm.HighestBandwidth = sd.Bandwidth;
							tm.HighestBandwidthCodec = sd.CodecInformation;

							tm.Label = Repr->GetName();

							tm.StreamDetails.Emplace(MoveTemp(sd));
						}
					}
					OutMetadata.Emplace(MoveTemp(tm));
				}
			}
		}

		void UpdateRunningMetaData(const FString& InKindOfValue, const FVariant& InNewValue) override
		{ }

		FResult GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, FString InSelectedStreamID, const FPlayStartPosition& StartPosition, ESearchType SearchType, int64 AtAbsoluteFilePos);
		FResult GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options);
		FResult GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData);
		FResult GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, FString SelectedStreamID, const FPlayStartPosition& StartPosition, ESearchType SearchType);

		void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID);
		TSharedPtrTS<IParserMKV> GetMKVParser();
		const FString& GetMediaURL() const
		{
			return MediaURL;
		}

	private:
		IPlayerSessionServices* PlayerSessionServices;
		FString MediaURL;
		FTimeValue MovieDuration;
		TSharedPtrTS<IParserMKV> MKVParser;
		TArray<TSharedPtrTS<FAdaptationSetMKV>> VideoAdaptationSets;
		TArray<TSharedPtrTS<FAdaptationSetMKV>> AudioAdaptationSets;
		TArray<TSharedPtrTS<FAdaptationSetMKV>> SubtitleAdaptationSets;
		TArray<uint64> UsableTrackIDs;
	};


	class FPlayPeriodMKV : public IManifest::IPlayPeriod
	{
	public:
		FPlayPeriodMKV(TSharedPtrTS<FTimelineAssetMKV> InMediaAsset);
		virtual ~FPlayPeriodMKV();
		void SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) override;
		EReadyState GetReadyState() override;
		void Load() override;
		void PrepareForPlay() override;
		int64 GetDefaultStartingBitrate() const override;
		TSharedPtrTS<FBufferSourceInfo> GetSelectedStreamBufferSourceInfo(EStreamType StreamType) override;
		FString GetSelectedAdaptationSetID(EStreamType StreamType) override;
		ETrackChangeResult ChangeTrackStreamPreference(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) override;
		TSharedPtrTS<ITimelineMediaAsset> GetMediaAsset() const override;
		void SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex) override;
		void TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload) override;
		FResult GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
		FResult GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType StreamType, const FPlayerSequenceState& LoopState, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
		FResult GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options) override;
		FResult GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData) override;
		FResult GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
		void IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount) override;
		void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID) override;

	private:
		void SelectInitialStream(EStreamType StreamType);
		TSharedPtrTS<FTrackMetadata> SelectMetadataForAttributes(EStreamType StreamType, const FStreamSelectionAttributes& InAttributes);
		void MakeBufferSourceInfoFromMetadata(EStreamType StreamType, TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FTrackMetadata> InMetadata);

		TWeakPtrTS<FTimelineAssetMKV> MediaAsset;
		FStreamSelectionAttributes VideoPreferences;
		FStreamSelectionAttributes AudioPreferences;
		FStreamSelectionAttributes SubtitlePreferences;
		TSharedPtrTS<FTrackMetadata> SelectedVideoMetadata;
		TSharedPtrTS<FTrackMetadata> SelectedAudioMetadata;
		TSharedPtrTS<FTrackMetadata> SelectedSubtitleMetadata;
		TSharedPtrTS<FBufferSourceInfo> VideoBufferSourceInfo;
		TSharedPtrTS<FBufferSourceInfo> AudioBufferSourceInfo;
		TSharedPtrTS<FBufferSourceInfo> SubtitleBufferSourceInfo;
		EReadyState CurrentReadyState;
	};

	const TArray<FURL_RFC3986::FQueryParam>& GetURLFragmentComponents() const
	{
		return URLFragmentComponents;
	}

	void SetURLFragmentComponents(TArray<FURL_RFC3986::FQueryParam> InURLFragmentComponents)
	{
		URLFragmentComponents = MoveTemp(InURLFragmentComponents);
	}

	const TArray<FURL_RFC3986::FQueryParam>& GetURLFragmentComponents()
	{
		return URLFragmentComponents;
	}

	IPlayerSessionServices* PlayerSessionServices;
	TSharedPtrTS<FTimelineAssetMKV> MediaAsset;
	HTTP::FConnectionInfo ConnectionInfo;
	TArray<FURL_RFC3986::FQueryParam> URLFragmentComponents;
	FTimeValue DefaultStartTime;
	FTimeValue DefaultEndTime;
};

} // namespace Electra
