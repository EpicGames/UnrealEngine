// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{
namespace MKV
{

const FName OptionKeyMKVLoadConnectTimeout(TEXT("mkv_connection_timeout"));				//!< (FTimeValue) value specifying connection timeout fetching the MKV
const FName OptionKeyMKVLoadNoDataTimeout(TEXT("mkv_nodata_timeout"));					//!< (FTimeValue) value specifying no-data timeout fetching the MKV

}

} // namespace Electra


