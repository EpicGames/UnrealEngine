// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlaylistReaderMKV.h"
#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserMKV.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "Utilities/HashFunctions.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/PlayerStreamReader.h"
#include "Player/mkv/ManifestMKV.h"
#include "Player/mkv/OptionKeynamesMKV.h"

#define ERRCODE_MKV_INVALID_FILE	1
#define ERRCODE_MKV_DOWNLOAD_ERROR	2


DECLARE_CYCLE_STAT(TEXT("FPlaylistReaderMKV_WorkerThread"), STAT_ElectraPlayer_MKV_PlaylistWorker, STATGROUP_ElectraPlayer);


namespace Electra
{

/**
 * This class is responsible for downloading the mkv non-mdat boxes and parsing them.
 */
class FPlaylistReaderMKV : public IPlaylistReaderMKV, public IGenericDataReader, public FMediaThread
{
public:
	FPlaylistReaderMKV();
	void Initialize(IPlayerSessionServices* PlayerSessionServices);

	virtual ~FPlaylistReaderMKV();
	
	EMediaFormatType GetMediaFormatType() const override
	{ return EMediaFormatType::MKV; }
	void Close() override;
	void HandleOnce() override;

	/**
	 * Returns the type of playlist format.
	 * For this implementation it will be "mkv".
	 * A "webm" file is also just an "mkv" (because it is).
	 *
	 * @return "mkv" to indicate this is an mkv file.
	 */
	const FString& GetPlaylistType() const override
	{
		static FString Type("mkv");
		return Type;
	}

	/**
	 * Loads and parses the playlist.
	 *
	 * @param URL URL of the playlist to load
	 * @param InReplayEventParams URL fragments pertaining to replay event which have been removed from the source URL
	 */
	void LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams) override;

	/**
	 * Returns the URL from which the playlist was loaded (or supposed to be loaded).
	 *
	 * @return The playlist URL
	 */
	FString GetURL() const override;

	/**
	 * Returns an interface to the manifest created from the loaded mkv playlists.
	 *
	 * @return A shared manifest interface pointer.
	 */
	TSharedPtrTS<IManifest> GetManifest() override;

private:
	// Methods from IParserMKV::IReader
	bool HasReadBeenAborted() const override;
	int64 ReadData(void* IntoBuffer, int64 NumBytesToRead, int64 InFromOffset) override;
	int64 GetCurrentOffset() const override;
	int64 GetTotalSize() const override
	{
		return FileSize;
	}
	bool HasReachedEOF() const override
	{
		check(!"this should not be called");
		return false;
	}

	void StartWorkerThread();
	void StopWorkerThread();
	void WorkerThread();

	void PostError(const FErrorDetail& InError);
	void PostError(const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

	FErrorDetail CreateErrorAndLog(const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);

	int32 HTTPProgressCallback(const IElectraHttpManager::FRequest* Request);
	void HTTPCompletionCallback(const IElectraHttpManager::FRequest* Request);

	void ClearRequest();

	void ReadChunk(uint8* DestinationBuffer, int64 InFromOffset, int64 ChunkSize);

	bool HasErrored() const
	{
		return bHasErrored;
	}

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	FString PlaylistURL;
	FString URLFragment;
	FMediaEvent WorkerThreadQuitSignal;
	bool bIsWorkerThreadStarted = false;

	FCriticalSection Lock;
	TSharedPtrTS<IElectraHttpManager::FRequest> Request;
	TSharedPtrTS<FWaitableBuffer> ReceiveBuffer;
	TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener;
	HTTP::FConnectionInfo ConnectionInfo;
	FMediaEvent RequestFinished;
	int64 FileSize = -1;

	bool bAbort = false;
	bool bHasErrored = false;

	TSharedPtrTS<IParserMKV> MKVParser;
	TSharedPtrTS<FManifestMKVInternal> Manifest;
	FErrorDetail LastErrorDetail;
};


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtrTS<IPlaylistReader> IPlaylistReaderMKV::Create(IPlayerSessionServices* PlayerSessionServices)
{
	check(PlayerSessionServices);
	TSharedPtrTS<FPlaylistReaderMKV> PlaylistReader = MakeSharedTS<FPlaylistReaderMKV>();
	if (PlaylistReader)
	{
		PlaylistReader->Initialize(PlayerSessionServices);
	}
	return PlaylistReader;
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


FPlaylistReaderMKV::FPlaylistReaderMKV()
	: FMediaThread("ElectraPlayer::MKV Playlist")
{
}

FPlaylistReaderMKV::~FPlaylistReaderMKV()
{
	Close();
}

FString FPlaylistReaderMKV::GetURL() const
{
	return PlaylistURL;
}

TSharedPtrTS<IManifest> FPlaylistReaderMKV::GetManifest()
{
	return Manifest;
}

void FPlaylistReaderMKV::Initialize(IPlayerSessionServices* InPlayerSessionServices)
{
	PlayerSessionServices = InPlayerSessionServices;
}

void FPlaylistReaderMKV::Close()
{
	bAbort = true;
	ClearRequest();

	StopWorkerThread();
}

void FPlaylistReaderMKV::HandleOnce()
{
	// No-op. This class is using a dedicated thread to read data from the stream
	// which can stall at any moment and thus not lend itself to a tickable instance.
}

void FPlaylistReaderMKV::StartWorkerThread()
{
	check(!bIsWorkerThreadStarted);
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FPlaylistReaderMKV::WorkerThread));
	bIsWorkerThreadStarted = true;
}

void FPlaylistReaderMKV::StopWorkerThread()
{
	if (bIsWorkerThreadStarted)
	{
		WorkerThreadQuitSignal.Signal();
		ThreadWaitDone();
		ThreadReset();
		bIsWorkerThreadStarted = false;
	}
}

void FPlaylistReaderMKV::PostError(const FErrorDetail& InError)
{
	LastErrorDetail = InError;
	PlayerSessionServices->PostError(LastErrorDetail);
}

void FPlaylistReaderMKV::PostError(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	LastErrorDetail.Clear();
	LastErrorDetail.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	LastErrorDetail.SetFacility(Facility::EFacility::MKVPlaylistReader);
	LastErrorDetail.SetCode(InCode);
	LastErrorDetail.SetMessage(InMessage);
	PlayerSessionServices->PostError(LastErrorDetail);
}

FErrorDetail FPlaylistReaderMKV::CreateErrorAndLog(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	FErrorDetail err;
	err.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	err.SetFacility(Facility::EFacility::MKVPlaylistReader);
	err.SetCode(InCode);
	err.SetMessage(InMessage);
	PlayerSessionServices->PostLog(Facility::EFacility::MKVPlaylistReader, IInfoLog::ELevel::Error, err.GetPrintable());
	return err;
}

void FPlaylistReaderMKV::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	PlayerSessionServices->PostLog(Facility::EFacility::MKVPlaylistReader, Level, Message);
}

void FPlaylistReaderMKV::LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams)
{
	FURL_RFC3986 UrlParser;
	UrlParser.Parse(InURL);
	PlaylistURL = UrlParser.Get(true, false);
	URLFragment = UrlParser.GetFragment();
	StartWorkerThread();
}

int32 FPlaylistReaderMKV::HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest)
{
	// Aborted?
	return bAbort ? 1 : 0;
}

void FPlaylistReaderMKV::HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest)
{
	bool bFailed = InRequest->ConnectionInfo.StatusInfo.ErrorDetail.IsError();
	ConnectionInfo = InRequest->ConnectionInfo;
	if (!bFailed)
	{
		// Set the size of the resource if we don't have it yet.
		if (FileSize < 0)
		{
			ElectraHTTPStream::FHttpRange crh;
			if (crh.ParseFromContentRangeResponse(InRequest->ConnectionInfo.ContentRangeHeader))
			{
				FileSize = crh.GetDocumentSize();
			}
		}

		if (ConnectionInfo.EffectiveURL.Len())
		{
			PlaylistURL = ConnectionInfo.EffectiveURL;
		}
	}
	bHasErrored = bFailed;
	RequestFinished.Signal();
}

void FPlaylistReaderMKV::ClearRequest()
{
	RequestFinished.Reset();
	FScopeLock lock(&Lock);
	if (Request.IsValid())
	{
		PlayerSessionServices->GetHTTPManager()->RemoveRequest(Request, false);
		Request.Reset();
	}
	ProgressListener.Reset();
	ReceiveBuffer.Reset();
}

void FPlaylistReaderMKV::ReadChunk(uint8* DestinationBuffer, int64 InFromOffset, int64 ChunkSize)
{
	ClearRequest();

	FScopeLock lock(&Lock);
	ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateRaw(this, &FPlaylistReaderMKV::HTTPCompletionCallback);
	ProgressListener->ProgressDelegate   = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateRaw(this, &FPlaylistReaderMKV::HTTPProgressCallback);

	ReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
	ReceiveBuffer->SetExternalBuffer(DestinationBuffer, ChunkSize);

	Request = MakeSharedTS<IElectraHttpManager::FRequest>();
	Request->Parameters.URL = PlaylistURL;
	Request->Parameters.Range.SetStart(InFromOffset);
	int64 LastByte = InFromOffset + ChunkSize - 1;
	if (FileSize >= 0 && LastByte > FileSize-1)
	{
		LastByte = FileSize - 1;
	}
	Request->Parameters.Range.SetEndIncluding(LastByte);
	Request->Parameters.ConnectTimeout = PlayerSessionServices->GetOptionValue(MKV::OptionKeyMKVLoadConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 8));
	Request->Parameters.NoDataTimeout = PlayerSessionServices->GetOptionValue(MKV::OptionKeyMKVLoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 6));
	Request->ReceiveBuffer = ReceiveBuffer;
	Request->ProgressListener = ProgressListener;
	Request->ResponseCache = PlayerSessionServices->GetHTTPResponseCache();
	PlayerSessionServices->GetHTTPManager()->AddRequest(Request, false);
}

void FPlaylistReaderMKV::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MKV_PlaylistWorker);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, MKV_PlaylistWorker);

	MKVParser = IParserMKV::CreateParser(PlayerSessionServices);
	LastErrorDetail = MKVParser->ParseHeader(this, IParserMKV::EParserFlags::ParseFlag_Default);
	ClearRequest();

	if (!HasReadBeenAborted())
	{
		// There are currently no retries, so this is the first (and only) attempt we make.
		const int32 Attempt = 1;
		// Notify the download of the "main playlist". This indicates the download only, not the parsing thereof.
		PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistDownloadMessage::Create(&ConnectionInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempt));
		// Notify that the "main playlist" has been parsed, successfully or not.
		PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ConnectionInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempt));

		if (LastErrorDetail.IsOK())
		{
			// Prepare the tracks in the stream that are of a supported codec.
			LastErrorDetail = MKVParser->PrepareTracks();
			if (LastErrorDetail.IsOK())
			{
				Manifest = MakeSharedTS<FManifestMKVInternal>(PlayerSessionServices);

				TArray<FURL_RFC3986::FQueryParam> URLFragmentComponents;
				FURL_RFC3986::GetQueryParams(URLFragmentComponents, URLFragment, false);	// The fragment is already URL escaped, so no need to do it again.
				Manifest->SetURLFragmentComponents(MoveTemp(URLFragmentComponents));

				LastErrorDetail = Manifest->Build(MKVParser, PlaylistURL, ConnectionInfo);

				// Let the external registry know that we have no properties with an end-of-properties call.
				PlayerSessionServices->ValidateMainPlaylistCustomProperty(GetPlaylistType(), PlaylistURL, TArray<FElectraHTTPStreamHeader>(), IPlayerSessionServices::FPlaylistProperty());

				// Notify that the "variant playlists" are ready. There are no variants in an mkv, but this is the trigger that the playlists are all set up and are good to go now.
				PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ConnectionInfo, Playlist::EListType::Variant, Playlist::ELoadType::Initial, Attempt));
			}
			else
			{
				PostError(FString::Printf(TEXT("Failed to parse tracks in mkv \"%s\". %s"), *ConnectionInfo.EffectiveURL, *LastErrorDetail.GetMessage()), ERRCODE_MKV_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
			}
		}
		else
		{
			// See if there was a download error
			if (ConnectionInfo.StatusInfo.ErrorDetail.IsError())
			{
				PostError(FString::Printf(TEXT("%s while downloading \"%s\""), *ConnectionInfo.StatusInfo.ErrorDetail.GetMessage(), *ConnectionInfo.EffectiveURL), ERRCODE_MKV_DOWNLOAD_ERROR, UEMEDIA_ERROR_READ_ERROR);
			}
			else
			{
				PlayerSessionServices->PostError(LastErrorDetail);
			}
		}
	}

	// This thread's work is done. We only wait for termination now.
	WorkerThreadQuitSignal.Wait();
}

/**
 * Read n bytes of data into the provided buffer.
 *
 * Reading must return the number of bytes asked to get, if necessary by blocking.
 * If a read error prevents reading the number of bytes -1 must be returned.
 *
 * @param IntoBuffer Buffer into which to store the data bytes. If nullptr is passed the data must be skipped over.
 * @param NumBytesToRead The number of bytes to read. Must not read more bytes and no less than requested.
 * @return The number of bytes read or -1 on a read error.
 */
int64 FPlaylistReaderMKV::ReadData(void* IntoBuffer, int64 NumBytesToRead, int64 InFromOffset)
{
	ReadChunk(reinterpret_cast<uint8*>(IntoBuffer), InFromOffset, NumBytesToRead);
	while(1)
	{
		TSharedPtrTS<FWaitableBuffer> rb = ReceiveBuffer;
		if (bHasErrored || bAbort || !rb.IsValid())
		{
			return -1;
		}
		// Wait for data to arrive.
		if (RequestFinished.WaitTimeout(1000 * 100))
		{
			break;
		}
	}
	return NumBytesToRead;
}

int64 FPlaylistReaderMKV::GetCurrentOffset() const
{
	check(!"This is not expected to be called using a buffered reader");
	return -1;
}

/**
 * Checks if reading of the file and therefor parsing has been aborted.
 *
 * @return true if reading/parsing has been aborted, false otherwise.
 */
bool FPlaylistReaderMKV::HasReadBeenAborted() const
{
	return bAbort;
}

} // namespace Electra
