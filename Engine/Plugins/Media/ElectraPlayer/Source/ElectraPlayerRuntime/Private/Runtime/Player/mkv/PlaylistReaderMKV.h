// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlaylistReader.h"
#include "Player/PlayerSessionServices.h"


namespace Electra
{

class IPlaylistReaderMKV : public IPlaylistReader
{
public:
	static TSharedPtrTS<IPlaylistReader> Create(IPlayerSessionServices* PlayerSessionServices);

	virtual ~IPlaylistReaderMKV() = default;
};

} // namespace Electra
