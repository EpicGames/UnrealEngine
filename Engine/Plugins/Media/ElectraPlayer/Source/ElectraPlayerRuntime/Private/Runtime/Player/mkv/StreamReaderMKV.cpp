// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserMKV.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMKV.h"
#include "Utilities/Utilities.h"
#include "Utilities/TimeUtilities.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/mkv/ManifestMKV.h"
#include "Player/mkv/StreamReaderMKV.h"
#include "Player/mkv/OptionKeynamesMKV.h"
#include "Player/PlayerSessionServices.h"
#include "Player/PlayerStreamFilter.h"


DECLARE_CYCLE_STAT(TEXT("FStreamReaderMKV_HandleRequest"), STAT_ElectraPlayer_MKV_StreamReader, STATGROUP_ElectraPlayer);


namespace Electra
{

void FStreamSegmentRequestMKV::SetPlaybackSequenceID(uint32 InPlaybackSequenceID)
{
	PlaybackSequenceID = InPlaybackSequenceID;
}

uint32 FStreamSegmentRequestMKV::GetPlaybackSequenceID() const
{
	return PlaybackSequenceID;
}

EStreamType FStreamSegmentRequestMKV::GetType() const
{
	return PrimaryStreamType;
}

void FStreamSegmentRequestMKV::SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay)
{
	// No op for mkv/webm.
}

FTimeValue FStreamSegmentRequestMKV::GetExecuteAtUTCTime() const
{
	// Execute right now.
	return FTimeValue::GetInvalid();
}

void FStreamSegmentRequestMKV::GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const
{
	// Those are not "real" dependent streams in that they are multiplexed and do not need to be fetched from
	// a different source. This merely indicates the types of non-primary streams we will be demuxing.
	TArray<EStreamType> DependentStreamTypes;
	for(auto &TrkId : EnabledTrackIDs)
	{
		DependentStreamTypes.AddUnique(MKVParser->GetTrackByTrackID(TrkId)->GetCodecInformation().GetStreamType());
	}
	DependentStreamTypes.Remove(PrimaryStreamType);
	for(int32 i=0; i<DependentStreamTypes.Num(); ++i)
	{
		FStreamSegmentRequestMKV* DepReq = new FStreamSegmentRequestMKV(*this);
		DepReq->PrimaryStreamType = DependentStreamTypes[i];
		TSharedPtrTS<IStreamSegment> p(DepReq);
		OutDependentStreams.Push(p);
	}
}

void FStreamSegmentRequestMKV::GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams)
{
	OutRequestedStreams.Empty();
	OutRequestedStreams.Emplace(SharedThis(this));
}

void FStreamSegmentRequestMKV::GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams)
{
	OutAlreadyEndedStreams.Empty();
	if (bIsEOSSegment)
	{
		TArray<EStreamType> DependentStreamTypes;
		for(auto &TrkId : EnabledTrackIDs)
		{
			DependentStreamTypes.AddUnique(MKVParser->GetTrackByTrackID(TrkId)->GetCodecInformation().GetStreamType());
		}
		for(int32 i=0; i<DependentStreamTypes.Num(); ++i)
		{
			FStreamSegmentRequestMKV* DepReq = new FStreamSegmentRequestMKV(*this);
			DepReq->PrimaryStreamType = DependentStreamTypes[i];
			TSharedPtrTS<IStreamSegment> p(DepReq);
			OutAlreadyEndedStreams.Push(p);
		}
	}
}

FTimeValue FStreamSegmentRequestMKV::GetFirstPTS() const
{
	FTimeValue First(EarliestPTS > FirstPTS ? EarliestPTS : FirstPTS);
	First.SetSequenceIndex(TimestampSequenceIndex);
	return First;
}

FTimeRange FStreamSegmentRequestMKV::GetTimeRange() const
{
	FTimeRange tr;
	tr.Start = FirstPTS;
	tr.End = FirstPTS + SegmentDuration;
	tr.Start.SetSequenceIndex(TimestampSequenceIndex);
	tr.End.SetSequenceIndex(TimestampSequenceIndex);
	return tr;
}

int32 FStreamSegmentRequestMKV::GetQualityIndex() const
{
	// No quality choice here.
	return 0;
}

int32 FStreamSegmentRequestMKV::GetBitrate() const
{
	return Bitrate;
}

void FStreamSegmentRequestMKV::GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const
{
	OutStats = DownloadStats;
}


uint32 FStreamReaderMKV::UniqueDownloadID = 1;

FStreamReaderMKV::~FStreamReaderMKV()
{
	Close();
}

UEMediaError FStreamReaderMKV::Create(IPlayerSessionServices* InPlayerSessionService, const CreateParam& InCreateParam)
{
	if (!InCreateParam.MemoryProvider || !InCreateParam.EventListener)
	{
		return UEMEDIA_ERROR_BAD_ARGUMENTS;
	}

	PlayerSessionServices = InPlayerSessionService;
	Parameters = InCreateParam;
	bTerminate = false;
	bIsStarted = true;

	ThreadSetName("ElectraPlayer::MKV streamer");
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FStreamReaderMKV::WorkerThread));

	return UEMEDIA_ERROR_OK;
}

bool FStreamReaderMKV::LoadsMultiplexedTracksSequentially()
{
	return true;
}

void FStreamReaderMKV::Close()
{
	if (bIsStarted)
	{
		bIsStarted = false;

		TSharedPtrTS<FStreamSegmentRequestMKV>	Request = CurrentRequest;
		if (Request.IsValid())
		{
			// ...
		}
		CancelRequests();
		bTerminate = true;
		WorkSignal.Signal();
		ThreadWaitDone();
		ThreadReset();
		CurrentRequest.Reset();
	}
}

IStreamReader::EAddResult FStreamReaderMKV::AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> InRequest)
{
	TSharedPtrTS<FStreamSegmentRequestMKV>	Request = CurrentRequest;
	if (Request.IsValid())
	{
		check(!"why is the handler busy??");
		return IStreamReader::EAddResult::TryAgainLater;
	}
	Request = StaticCastSharedPtr<FStreamSegmentRequestMKV>(InRequest);
	Request->SetPlaybackSequenceID(CurrentPlaybackSequenceID);
	bRequestCanceled = false;
	bHasErrored = false;
	// Only add the request if it is not an all-EOS one!
	if (!Request->bIsEOSSegment)
	{
		CurrentRequest = Request;
		WorkSignal.Signal();
	}
	return EAddResult::Added;
}

void FStreamReaderMKV::CancelRequest(EStreamType StreamType, bool bSilent)
{
	// No-op.
}

void FStreamReaderMKV::CancelRequests()
{
	bRequestCanceled = true;
	ReadBuffer.Abort();
}

bool FStreamReaderMKV::HasBeenAborted() const
{
	return bRequestCanceled || ReadBuffer.bAbort;
}

bool FStreamReaderMKV::HasErrored() const
{
	return bHasErrored;
}

int32 FStreamReaderMKV::HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest)
{
	HTTPUpdateStats(MEDIAutcTime::Current(), InRequest);
	// Aborted?
	return HasBeenAborted() ? 1 : 0;
}

void FStreamReaderMKV::HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest)
{
	HTTPUpdateStats(FTimeValue::GetInvalid(), InRequest);
	bHasErrored = InRequest->ConnectionInfo.StatusInfo.ErrorDetail.IsError();
}

void FStreamReaderMKV::HTTPUpdateStats(const FTimeValue& CurrentTime, const IElectraHttpManager::FRequest* Request)
{
	TSharedPtrTS<FStreamSegmentRequestMKV> SegmentRequest = CurrentRequest;
	if (SegmentRequest.IsValid())
	{
		FScopeLock lock(&MetricUpdateLock);
		SegmentRequest->ConnectionInfo = Request->ConnectionInfo;
		// Update the current download stats which we report periodically to the ABR.
		Metrics::FSegmentDownloadStats& ds = SegmentRequest->DownloadStats;
		if (Request->ConnectionInfo.EffectiveURL.Len())
		{
			ds.URL.URL = Request->ConnectionInfo.EffectiveURL;
		}
		ds.HTTPStatusCode = Request->ConnectionInfo.StatusInfo.HTTPStatus;
		ds.TimeToFirstByte = Request->ConnectionInfo.TimeUntilFirstByte;
		ds.TimeToDownload = ((CurrentTime.IsValid() ? CurrentTime : Request->ConnectionInfo.RequestEndTime) - Request->ConnectionInfo.RequestStartTime).GetAsSeconds();
		ds.ByteSize = Request->ConnectionInfo.ContentLength;
		ds.NumBytesDownloaded = Request->ConnectionInfo.BytesReadSoFar;
	}
}

void FStreamReaderMKV::HandleRequest()
{
	TSharedPtrTS<FStreamSegmentRequestMKV> Request = CurrentRequest;
	FManifestMKVInternal::FTimelineAssetMKV* TimelineAsset = static_cast<FManifestMKVInternal::FTimelineAssetMKV*>(Request->MediaAsset.Get());

	// Clear the active track map with a new starting request that is not a retry.
	// Otherwise we keep previous values.
	if (Request->bIsFirstSegment && Request->RetryBlockOffset < 0)
	{
		ActiveTrackMap.Reset();
	}
	else
	{
		for(auto &ActiveTrk : ActiveTrackMap)
		{
			FSelectedTrackData& td = ActiveTrk.Value;
			td.Clear();
		}
	}
	struct FPlaylistTrackMetadata
	{
		EStreamType Type;
		FString Kind;
		BCP47::FLanguageTag LanguageTag;
		FString PeriodID;
		FString AdaptationSetID;
		FString RepresentationID;
		int32 Bitrate;
		int32 Index;
		FStreamCodecInformation CodecInfo;
	};
	TMap<uint64, FPlaylistTrackMetadata> SelectedTrackMap;
	const EStreamType TypesOfSupportedTracks[] = { EStreamType::Video, EStreamType::Audio, EStreamType::Subtitle };
	{
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MKV_StreamReader);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, MKV_StreamReader);
		for(int32 nStrType=0; nStrType<UE_ARRAY_COUNT(TypesOfSupportedTracks); ++nStrType)
		{
			int32 NumAdapt = Request->MediaAsset->GetNumberOfAdaptationSets(TypesOfSupportedTracks[nStrType]);
			for(int32 nAdapt=0; nAdapt<NumAdapt; ++nAdapt)
			{
				TSharedPtrTS<IPlaybackAssetAdaptationSet> AdaptationSet = Request->MediaAsset->GetAdaptationSetByTypeAndIndex(TypesOfSupportedTracks[nStrType], nAdapt);
				const BCP47::FLanguageTag& LanguageTag = AdaptationSet->GetLanguageTag();
				FString AdaptID = AdaptationSet->GetUniqueIdentifier();
				int32 NumRepr = AdaptationSet->GetNumberOfRepresentations();
				for(int32 nRepr=0; nRepr<NumRepr; ++nRepr)
				{
					TSharedPtrTS<IPlaybackAssetRepresentation> Representation = AdaptationSet->GetRepresentationByIndex(nRepr);
					// Note: By definition the representations unique identifier is a string of the numeric track ID and can thus be parsed back into a number.
					FString ReprID = Representation->GetUniqueIdentifier();
					uint32 TrackId;
					LexFromString(TrackId, *ReprID);
					FPlaylistTrackMetadata tmd;
					tmd.Type = TypesOfSupportedTracks[nStrType];
					tmd.LanguageTag = LanguageTag;
					tmd.Kind = nRepr == 0 ? TEXT("main") : TEXT("translation");	// perhaps this should come from querying the metadata instead of doing this over here...
					tmd.PeriodID = Request->MediaAsset->GetUniqueIdentifier();
					tmd.AdaptationSetID = AdaptID;
					tmd.RepresentationID = ReprID;
					tmd.Bitrate = Representation->GetBitrate();
					tmd.Index = nAdapt;
					tmd.CodecInfo = Representation->GetCodecInformation();
					SelectedTrackMap.Emplace(TrackId, tmd);
				}
			}
		}
	}

	Metrics::FSegmentDownloadStats& ds = Request->DownloadStats;
	ds.StatsID = FMediaInterlockedIncrement(UniqueDownloadID);

	// Set the segment request metadata from the primary track
	const FPlaylistTrackMetadata* PrimaryTrackMetadata = SelectedTrackMap.Find(Request->PrimaryTrackID);
	check(PrimaryTrackMetadata);
	if (PrimaryTrackMetadata)
	{
		ds.MediaAssetID = PrimaryTrackMetadata->PeriodID;
		ds.AdaptationSetID = PrimaryTrackMetadata->AdaptationSetID;
		ds.RepresentationID = PrimaryTrackMetadata->RepresentationID;
		ds.Bitrate = PrimaryTrackMetadata->Bitrate;
	}

	ds.FailureReason.Empty();
	ds.bWasSuccessful = true;
	ds.bWasAborted = false;
	ds.bDidTimeout = false;
	ds.HTTPStatusCode = 0;
	ds.StreamType = Request->GetType();
	ds.SegmentType = Metrics::ESegmentType::Media;
	ds.PresentationTime = Request->FirstPTS.GetAsSeconds();
	ds.Duration = Request->SegmentDuration.GetAsSeconds();
	ds.DurationDownloaded = 0.0;
	ds.DurationDelivered = 0.0;
	ds.TimeToFirstByte = 0.0;
	ds.TimeToDownload = 0.0;
	ds.ByteSize = -1;
	ds.NumBytesDownloaded = 0;
	ds.bInsertedFillerData = false;
	ds.URL.URL = TimelineAsset->GetMediaURL();
	ds.bIsMissingSegment = false;
	ds.bParseFailure = false;
	ds.RetryNumber = Request->NumOverallRetries;

	Parameters.EventListener->OnFragmentOpen(Request);

	TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener;
	ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateRaw(this, &FStreamReaderMKV::HTTPCompletionCallback);
	ProgressListener->ProgressDelegate = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateRaw(this, &FStreamReaderMKV::HTTPProgressCallback);

	ReadBuffer.Reset();
	ReadBuffer.ReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
	ReadBuffer.SetStartOffset(Request->FileStartOffset);
	ReadBuffer.SetCurrentOffset(0);
	ReadBuffer.SetEndOffset(Request->FileEndOffset);

	HTTPRequest = MakeSharedTS<IElectraHttpManager::FRequest>();
	HTTPRequest->Parameters.URL = TimelineAsset->GetMediaURL();
	HTTPRequest->Parameters.Range.Start = Request->FileStartOffset;
	HTTPRequest->Parameters.Range.EndIncluding = Request->FileEndOffset;
	// No compression as this would not yield much with already compressed video/audio data.
	HTTPRequest->Parameters.AcceptEncoding.Set(TEXT("identity"));
	// Timeouts
	HTTPRequest->Parameters.ConnectTimeout = PlayerSessionServices->GetOptionValue(MKV::OptionKeyMKVLoadConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 8));
	HTTPRequest->Parameters.NoDataTimeout = PlayerSessionServices->GetOptionValue(MKV::OptionKeyMKVLoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 6));

	// Explicit range?
	int64 NumRequestedBytes = HTTPRequest->Parameters.Range.GetNumberOfBytes();

	HTTPRequest->ReceiveBuffer = ReadBuffer.ReceiveBuffer;
	HTTPRequest->ProgressListener = ProgressListener;
	HTTPRequest->ResponseCache = PlayerSessionServices->GetHTTPResponseCache();
	PlayerSessionServices->GetHTTPManager()->AddRequest(HTTPRequest, false);

	check(Request->MKVParser.IsValid());
	TSharedPtrTS<IParserMKV::IClusterParser> ClusterParser = Request->MKVParser->CreateClusterParser(this, Request->EnabledTrackIDs, IParserMKV::EClusterParseFlags::ClusterParseFlag_Default);

	check(ClusterParser.IsValid());

	uint32 PlaybackSequenceID = Request->GetPlaybackSequenceID();
	auto UpdateSelectedTrack = [&SelectedTrackMap, &PlaybackSequenceID](const IParserMKV::IClusterParser::IAction* InAction, TMap<uint64, FSelectedTrackData>& ActiveTrks) -> FSelectedTrackData&
	{
		const FPlaylistTrackMetadata* SelectedTrackMetadata = SelectedTrackMap.Find(InAction->GetTrackID());
		// Check if this track ID is already in our map of active tracks.
		FSelectedTrackData& st = ActiveTrks.FindOrAdd(InAction->GetTrackID());
		if (SelectedTrackMetadata)
		{
			if (!st.CSD.IsValid())
			{
				TSharedPtrTS<FAccessUnit::CodecData> CSD(new FAccessUnit::CodecData);
				CSD->ParsedInfo = SelectedTrackMetadata->CodecInfo;
				CSD->CodecSpecificData = SelectedTrackMetadata->CodecInfo.GetCodecSpecificData();
				FVariantValue dcr = SelectedTrackMetadata->CodecInfo.GetExtras().GetValue(StreamCodecInformationOptions::DecoderConfigurationRecord);
				if (dcr.IsValid() && dcr.IsType(FVariantValue::EDataType::TypeU8Array))
				{
					CSD->RawCSD = dcr.GetArray();
				}
				st.CSD = MoveTemp(CSD);
			}
			if (!st.BufferSourceInfo.IsValid())
			{
				auto meta = MakeShared<FBufferSourceInfo, ESPMode::ThreadSafe>();
				st.StreamType = SelectedTrackMetadata->Type;
				st.Bitrate = SelectedTrackMetadata->Bitrate;
				if (st.CSD.IsValid())
				{
					st.CSD->ParsedInfo.SetBitrate(st.Bitrate);
				}
				meta->Kind = SelectedTrackMetadata->Kind;
				meta->LanguageTag = SelectedTrackMetadata->LanguageTag;
				meta->Codec = st.CSD.IsValid() ? st.CSD->ParsedInfo.GetCodecName() : FString();
				meta->PeriodID = SelectedTrackMetadata->PeriodID;
				meta->PeriodAdaptationSetID = SelectedTrackMetadata->PeriodID + TEXT(".") + SelectedTrackMetadata->AdaptationSetID;
				meta->HardIndex = SelectedTrackMetadata->Index;
				meta->PlaybackSequenceID = PlaybackSequenceID;
				st.BufferSourceInfo = MoveTemp(meta);
			}
		}
		st.bNeedToRecalculateDurations = st.StreamType == EStreamType::Video;
		return st;
	};

	FAccessUnit* AccessUnit = nullptr;
	bool bDone = false;
	bool bIsParserError = false;
	bool bIsReadError = false;
	while(!bDone && !HasErrored() && !HasBeenAborted() && !bTerminate)
	{
		auto PrepareAccessUnit = [this, &AccessUnit](int64 NumToRead) -> void*
		{
			if (!AccessUnit)
			{
				AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
				AccessUnit->AUSize = NumToRead;
				AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);
				return AccessUnit->AUData;
			}
			else
			{
				void* NewBuffer = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize + NumToRead);
				FMemory::Memcpy(NewBuffer, AccessUnit->AUData, AccessUnit->AUSize);
				void* ReadTo = AdvancePointer(NewBuffer, AccessUnit->AUSize);
				AccessUnit->AdoptNewPayloadBuffer(NewBuffer, AccessUnit->AUSize + NumToRead);
				return ReadTo;
			}
		};

		IParserMKV::IClusterParser::EParseAction ParseAction = ClusterParser->NextParseAction();
		switch(ParseAction)
		{
			case IParserMKV::IClusterParser::EParseAction::ReadFrameData:
			{
				const IParserMKV::IClusterParser::IActionReadFrameData* Action = static_cast<const IParserMKV::IClusterParser::IActionReadFrameData*>(ClusterParser->GetAction());
				check(Action);

				int64 NumToRead = Action->GetNumBytesToRead();
				void* ReadTo = PrepareAccessUnit(NumToRead);
				int64 nr = ReadBuffer.ReadTo(ReadTo, NumToRead);
				if (nr != NumToRead)
				{
					bHasErrored = true;
					bIsReadError = true;
				}
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::FrameDone:
			{
				const IParserMKV::IClusterParser::IActionFrameDone* Action = static_cast<const IParserMKV::IClusterParser::IActionFrameDone*>(ClusterParser->GetAction());
				check(Action);
				if (AccessUnit)
				{
					FSelectedTrackData& SelectedTrack = UpdateSelectedTrack(Action, ActiveTrackMap);
					AccessUnit->ESType = SelectedTrack.StreamType;
					AccessUnit->PTS = Action->GetPTS();
					AccessUnit->DTS = Action->GetDTS();
					AccessUnit->Duration = Action->GetDuration();
					AccessUnit->EarliestPTS = Request->EarliestPTS;
					AccessUnit->LatestPTS = Request->LastPTS;

					AccessUnit->AUCodecData = SelectedTrack.CSD;

					// Set the sequence index member and update all timestamps with it as well.
					AccessUnit->SequenceIndex = Request->TimestampSequenceIndex;
					AccessUnit->DTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->PTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->EarliestPTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->LatestPTS.SetSequenceIndex(Request->TimestampSequenceIndex);

					AccessUnit->bIsFirstInSequence = SelectedTrack.bIsFirstInSequence;
					AccessUnit->bIsSyncSample = Action->IsKeyFrame();
					AccessUnit->bIsDummyData = false;

					// Set the associated stream metadata
					AccessUnit->BufferSourceInfo = SelectedTrack.BufferSourceInfo;

					// VP9 codec?
					if (SelectedTrack.CSD->ParsedInfo.GetCodec4CC() == Utils::Make4CC('v','p','0','9'))
					{
						// We cannot trust the keyframe indicator from the demuxer.
						ElectraDecodersUtil::VPxVideo::FVP9UncompressedHeader Header;
						if (ElectraDecodersUtil::VPxVideo::ParseVP9UncompressedHeader(Header, AccessUnit->AUData, AccessUnit->AUSize))
						{
							AccessUnit->bIsSyncSample = Header.IsKeyframe();
						}

						// Any additional data?
						const TMap<uint64, TArray<uint8>>& BlockAdditionalData = Action->GetBlockAdditionalData();
						for(auto& badIt : BlockAdditionalData)
						{
							auto AddDynamicBlockData = [&](const FName& InName, const TArray<uint8>& InData) -> void
							{
								if (!AccessUnit->DynamicSidebandData.IsValid())
								{
									AccessUnit->DynamicSidebandData = MakeUnique<TMap<FName, TArray<uint8>>>();
								}
								AccessUnit->DynamicSidebandData->Emplace(InName, InData);
							};

							// What type of additional data is there?
							if (BlockAdditionalData.Contains(1))		// VP9 alpha channel?
							{
								AddDynamicBlockData(DynamicSidebandData::VPxAlpha, BlockAdditionalData[1]);
							}
							else if (BlockAdditionalData.Contains(4))	// VP9 ITU T.35 metadata?
							{
								AddDynamicBlockData(DynamicSidebandData::ITU_T_35, BlockAdditionalData[4]);
							}
						}
					}
					// VP8 codec?
					else if (SelectedTrack.CSD->ParsedInfo.GetCodec4CC() == Utils::Make4CC('v','p','0','8'))
					{
						ElectraDecodersUtil::VPxVideo::FVP8UncompressedHeader Header;
						if (ElectraDecodersUtil::VPxVideo::ParseVP8UncompressedHeader(Header, AccessUnit->AUData, AccessUnit->AUSize))
						{
							AccessUnit->bIsSyncSample = Header.IsKeyframe();
						}
					}


					SelectedTrack.bIsFirstInSequence = false;
					// Update the duration read with the duration from the sample. This may not be correct since we are
					// recalculating the duration of video frames later, but this is only for download statistics and
					// ABR choices, so it does not need to be that accurate.
					SelectedTrack.DurationSuccessfullyRead += AccessUnit->Duration;

					// For error handling, if we managed to get additional data we reset the retry count.
					if (ds.RetryNumber && SelectedTrack.DurationSuccessfullyRead.GetAsSeconds() > 2.0)
					{
						ds.RetryNumber = 0;
						Request->NumOverallRetries = 0;
					}

					// If already read in the previous attempt we skip over this and do not add it to the FIFO
					bool bAlreadyRead = Request->RetryBlockOffset >= 0 && ClusterParser->GetClusterBlockPosition() < Request->RetryBlockOffset;
					if (!bAlreadyRead)
					{
						// Add to the track AU FIFO unless we already reached the last sample of the time range.
						if (!SelectedTrack.bReadPastLastPTS)
						{
							SelectedTrack.AccessUnitFIFO.Emplace(FSelectedTrackData::FSample(AccessUnit));
							if (SelectedTrack.bNeedToRecalculateDurations)
							{
								SelectedTrack.SortedAccessUnitFIFO.Emplace(FSelectedTrackData::FSample(AccessUnit));
								SelectedTrack.SortedAccessUnitFIFO.Sort([](const FSelectedTrackData::FSample& a, const FSelectedTrackData::FSample& b){return a.PTS < b.PTS;});
							}
						}

						// Add video keyframes as a Cue since not all of them may have been added as Cues in the multiplexer.
						if (AccessUnit->ESType == EStreamType::Video && AccessUnit->bIsSyncSample)
						{
							Request->MKVParser->AddCue(Action->GetTimestamp(), Action->GetTrackID(), Action->GetSegmentRelativePosition(), 0, Action->GetClusterPosition());
						}
					}
				}
				FAccessUnit::Release(AccessUnit);
				AccessUnit = nullptr;
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::SkipOver:
			{
				const IParserMKV::IClusterParser::IActionSkipOver* Action = static_cast<const IParserMKV::IClusterParser::IActionSkipOver*>(ClusterParser->GetAction());
				check(Action);
				int64 NumBytesToSkip = Action->GetNumBytesToSkip();
				int64 nr = ReadBuffer.ReadTo(nullptr, NumBytesToSkip);
				if (nr != NumBytesToSkip)
				{
					bHasErrored = true;
					bIsReadError = true;
				}
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::PrependData:
			{
				const IParserMKV::IClusterParser::IActionPrependData* Action = static_cast<const IParserMKV::IClusterParser::IActionPrependData*>(ClusterParser->GetAction());
				check(Action);
				int64 NumToRead = Action->GetPrependData().Num();
				void* ReadTo = PrepareAccessUnit(NumToRead);
				FMemory::Memcpy(ReadTo, Action->GetPrependData().GetData(), Action->GetPrependData().Num());
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::DecryptData:
			{
				break;
			}
			case IParserMKV::IClusterParser::EParseAction::EndOfData:
			{
				bDone = true;
				break;
			}
			default:
			case IParserMKV::IClusterParser::EParseAction::Failure:
			{
				bIsParserError = true;
				bDone = true;
				break;
			}
		}
		// If all tracks had their last access unit sent into the player buffer we can end early.
		// This is for play range requests where the last cluster size may be large and we do not have to read the excess.
		if (EmitSamples(EEmitType::One, Request) == EEmitResult::AllReachedEOS)
		{
			bDone = true;
		}
	}
	// Any open access unit that we may have failed on we delete.
	FAccessUnit::Release(AccessUnit);
	AccessUnit = nullptr;

	// Remove the download request.
	ProgressListener.Reset();
	PlayerSessionServices->GetHTTPManager()->RemoveRequest(HTTPRequest, false);
	Request->ConnectionInfo = HTTPRequest->ConnectionInfo;
	HTTPRequest.Reset();

	// Set downloaded and delivered duration from the primary track.
	double DurationRead = 0.0, DurationDelivered = 0.0;
	{
		FSelectedTrackData& PrimaryTrack = ActiveTrackMap.FindOrAdd(Request->PrimaryTrackID);
		DurationRead = PrimaryTrack.DurationSuccessfullyRead.GetAsSeconds();
		DurationDelivered = PrimaryTrack.DurationSuccessfullyDelivered.GetAsSeconds();
	}

	// Was there an error?
	if (!bHasErrored && !bIsReadError && !bIsParserError)
	{
		// No, read loop ended, all buffers have received all the samples they will get.
		for(auto &ActiveTrk : ActiveTrackMap)
		{
			FSelectedTrackData& td = ActiveTrk.Value;
			td.bGotAllSamples = true;
		}

		// If this was the last cluster we must not hold back any samples.
		bool bWasLastCluster = Request->bIsLastSegment || Request->NextCueUniqueID == ~0U;
		// Emit all remaining pending AUs
		EmitSamples(bWasLastCluster ? EEmitType::AllRemaining : EEmitType::KnownDurationOnly, Request);
		if (bWasLastCluster)
		{
			ActiveTrackMap.Reset();
		}
		else
		{
			// Anything still not sent off we delete when we were aborted or asked to terminate.
			// On regular completion we keep the remaining AUs to emit with the next request.
			// We have to keep them in order to calculate the sample durations.
			if (bTerminate || HasReadBeenAborted())
			{
				for(auto &ActiveTrk : ActiveTrackMap)
				{
					FSelectedTrackData& td = ActiveTrk.Value;
					td.SortedAccessUnitFIFO.Empty();
					td.AccessUnitFIFO.Empty();
				}
			}
		}
	}
	else
	{
		// There has been an error. We leave the track map intact for the potential retry.
		Request->FailedClusterFileOffset = ClusterParser->GetClusterPosition();
		check(Request->FailedClusterFileOffset > 0);
		int64 FailedBlockOffset = ClusterParser->GetClusterBlockPosition();
		// Remember the farthest we got into a cluster if a retry fails earlier than the first failed attempt did.
		Request->FailedClusterDataOffset = FailedBlockOffset < Request->FailedClusterDataOffset ? Request->FailedClusterDataOffset : FailedBlockOffset;
	}

	// Set up remaining download stat fields.
	ds.FailureReason = Request->ConnectionInfo.StatusInfo.ErrorDetail.GetMessage();
	if (bIsParserError)
	{
		ds.FailureReason = ClusterParser->GetLastError().GetMessage();
	}
	ds.bWasSuccessful = !bHasErrored;
	ds.URL.URL = Request->ConnectionInfo.EffectiveURL;
	ds.HTTPStatusCode = Request->ConnectionInfo.StatusInfo.HTTPStatus;
	ds.DurationDownloaded = DurationRead;
	ds.DurationDelivered = DurationDelivered;
	ds.TimeToFirstByte = Request->ConnectionInfo.TimeUntilFirstByte;
	ds.TimeToDownload = (Request->ConnectionInfo.RequestEndTime - Request->ConnectionInfo.RequestStartTime).GetAsSeconds();
	ds.ByteSize = Request->ConnectionInfo.ContentLength;
	ds.NumBytesDownloaded = Request->ConnectionInfo.BytesReadSoFar;
	ds.bIsCachedResponse = Request->ConnectionInfo.bIsCachedResponse;

	// Reset the current request so another one can be added immediately when we call OnFragmentClose()
	CurrentRequest.Reset();
	PlayerSessionServices->GetStreamSelector()->ReportDownloadEnd(ds);
	Parameters.EventListener->OnFragmentClose(Request);
}

FStreamReaderMKV::EEmitResult FStreamReaderMKV::EmitSamples(EEmitType InEmitType, const TSharedPtrTS<FStreamSegmentRequestMKV>& InRequest)
{
	EEmitResult Result = EEmitResult::SentNothing;
	// Emit all remaining pending AUs
	bool bAllSentOff = false;
	while(!bAllSentOff && !bTerminate && !HasReadBeenAborted())
	{
		bAllSentOff = true;
		for(auto &ActiveTrk : ActiveTrackMap)
		{
			FSelectedTrackData& td = ActiveTrk.Value;

			if (td.bReadPastLastPTS)
			{
				td.AccessUnitFIFO.Empty();
				td.SortedAccessUnitFIFO.Empty();
			}

			while(td.AccessUnitFIFO.Num() && !HasReadBeenAborted())
			{
				if (td.bNeedToRecalculateDurations)
				{
					// Need to have a certain amount of upcoming samples to be able to
					// (more or less) safely calculate timestamp differences.
					int32 NumToCheck = !td.bGotAllSamples ? 10 : td.AccessUnitFIFO.Num();
					if (td.AccessUnitFIFO.Num() < NumToCheck)
					{
						break;
					}
					// Locate the sample in the time-sorted list
					for(int32 i=0; i<td.SortedAccessUnitFIFO.Num(); ++i)
					{
						if (td.SortedAccessUnitFIFO[i].PTS == td.AccessUnitFIFO[0].PTS)
						{
							if (i < td.SortedAccessUnitFIFO.Num()-1)
							{
								td.AccessUnitFIFO[0].AU->Duration = td.SortedAccessUnitFIFO[i+1].PTS - td.SortedAccessUnitFIFO[i].PTS;
							}
							else if (InEmitType == EEmitType::KnownDurationOnly)
							{
								td.bReachedEndOfKnownDuration = true;
								break;
							}
							td.SortedAccessUnitFIFO[i].Release();
							break;
						}
					}
					// Reduce the sorted list
					for(int32 i=0; i<td.SortedAccessUnitFIFO.Num(); ++i)
					{
						if (td.SortedAccessUnitFIFO[i].AU)
						{
							if (i)
							{
								td.SortedAccessUnitFIFO.RemoveAt(0, i);
							}
							break;
						}
					}
				}

				// Could not recalculate the duration of the remaining samples, so we do not emit them now
				// and hold on to them until the next segment brings in more samples.
				if (td.bReachedEndOfKnownDuration)
				{
					break;
				}

				FAccessUnit* pNext = td.AccessUnitFIFO[0].AU;
				// Check if this is the last access unit in the requested time range.
				if (pNext->PTS >= pNext->LatestPTS)
				{
					// Because of B frames the last frame that must be decoded could actually be
					// a later frame in decode order.
					// Suppose the sequence IPBB with timestamps 0,3,1,2 respectively. Even though the
					// P frame with timestamp 3 is "the last" one, it will enter the decoder before the B frames.
					// As such we need to tag the last B frame as "the last one" even though its timstamp
					// is before the last time requested.
					// Note: This is not necessary to do for audio frames, but the logic is the same
					//       so we do not differentiate here.
					FTimeValue HighestPTSBelow(-1.0);
					int32 LastIndex = 0;
					for(int32 i=1; i<td.AccessUnitFIFO.Num(); ++i)
					{
						if (td.AccessUnitFIFO[i].PTS < pNext->LatestPTS)
						{
							if (td.AccessUnitFIFO[i].PTS > HighestPTSBelow)
							{
								HighestPTSBelow = td.AccessUnitFIFO[i].PTS;
								LastIndex = i;
							}
						}
					}
					td.AccessUnitFIFO[LastIndex].AU->bIsLastInPeriod = true;
				}

				if (Parameters.EventListener->OnFragmentAccessUnitReceived(pNext))
				{
					td.DurationSuccessfullyDelivered += pNext->Duration;
					td.AccessUnitFIFO[0].AU = nullptr;
					td.AccessUnitFIFO.RemoveAt(0);
					Result = Result == EEmitResult::SentNothing ? EEmitResult::Sent : Result;
					if (pNext->bIsLastInPeriod)
					{
						td.bReadPastLastPTS = true;
						break;
					}
				}
				else
				{
					break;
				}

				// If emitting only one sample we leave this loop.
				if (InEmitType == EEmitType::One)
				{
					break;
				}
			}
			if (td.AccessUnitFIFO.Num() && !td.bReachedEndOfKnownDuration)
			{
				bAllSentOff = false;
			}
		}
		// Wait for a while when emitting all remaining samples.
		if (!bAllSentOff && InEmitType != EEmitType::One)
		{
			FMediaRunnable::SleepMilliseconds(100);
		}
		else
		{
			break;
		}
	}

	// All at EOS?
	bool bAllAtEOS = !!ActiveTrackMap.Num();
	for(auto &ActiveTrk : ActiveTrackMap)
	{
		FSelectedTrackData& td = ActiveTrk.Value;
		bAllAtEOS = !td.bReadPastLastPTS ? false : bAllAtEOS;
	}
	Result = bAllAtEOS ? EEmitResult::AllReachedEOS : Result;


	// Check that buffers are how they are supposed to be
	if (!bTerminate && !HasReadBeenAborted() && InEmitType == EEmitType::AllRemaining)
	{
		for(auto &ActiveTrk : ActiveTrackMap)
		{
			FSelectedTrackData& td = ActiveTrk.Value;
			check(td.AccessUnitFIFO.IsEmpty());
			check(td.SortedAccessUnitFIFO.FindByPredicate([](const FSelectedTrackData::FSample& e){return e.AU != nullptr;}) == nullptr);
			td.SortedAccessUnitFIFO.Empty();
		}
	}
	return Result;
}


void FStreamReaderMKV::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);
	while(!bTerminate)
	{
		WorkSignal.WaitAndReset();
		if (bTerminate)
		{
			break;
		}
		TSharedPtrTS<FStreamSegmentRequestMKV>	Request = CurrentRequest;
		if (Request.IsValid())
		{
			HandleRequest();
		}
	}
}


int32 FStreamReaderMKV::FReadBuffer::ReadTo(void* IntoBuffer, int64 NumBytesToRead)
{
	FWaitableBuffer& SourceBuffer = *ReceiveBuffer;
	// Make sure the buffer will have the amount of data we need.
	while(1)
	{
		if (!SourceBuffer.WaitUntilSizeAvailable(CurrentOffset + NumBytesToRead, 1000 * 100))
		{
			if (bHasErrored || SourceBuffer.WasAborted() || bAbort)
			{
				return -1;
			}
		}
		else
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MKV_StreamReader);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, MKV_StreamReader);
			SourceBuffer.Lock();
			if (SourceBuffer.Num() >= CurrentOffset + NumBytesToRead)
			{
				if (IntoBuffer)
				{
					FMemory::Memcpy(IntoBuffer, SourceBuffer.GetLinearReadData() + CurrentOffset, NumBytesToRead);
				}
				SourceBuffer.Unlock();
				CurrentOffset += NumBytesToRead;
				return NumBytesToRead;
			}
			else
			{
				// Return 0 at EOF and -1 on error.
				SourceBuffer.Unlock();
				return bHasErrored ? -1 : 0;
			}
		}
	}
}


int64 FStreamReaderMKV::ReadData(void* InDestinationBuffer, int64 InNumBytesToRead, int64 InFromOffset)
{
	check(InFromOffset == GetCurrentOffset());
	if (InFromOffset != GetCurrentOffset())
	{
		return -1;
	}
	int64 nb = ReadBuffer.ReadTo(InDestinationBuffer, InNumBytesToRead);
	// We may get zero bytes due to the source buffer being set to EOD before the HTTP request
	// has called our completion delegate with an error.
	if (nb == 0 && HTTPRequest.IsValid() && HTTPRequest->ConnectionInfo.StatusInfo.ErrorCode)
	{
		nb = -1;
	}
	return nb;
}

int64 FStreamReaderMKV::GetCurrentOffset() const
{
	return ReadBuffer.GetStartOffset() + ReadBuffer.GetCurrentOffset();
}

int64 FStreamReaderMKV::GetTotalSize() const
{
	return ReadBuffer.GetTotalSize();
}

bool FStreamReaderMKV::HasReadBeenAborted() const
{
	return HasBeenAborted();
}

bool FStreamReaderMKV::HasReachedEOF() const
{
	check(!"this should not be called");
	return false;
}

} // namespace Electra
