// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserMKV.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "StreamAccessUnitBuffer.h"


namespace Electra
{

class FStreamSegmentRequestMKV : public IStreamSegment
{
public:
	FStreamSegmentRequestMKV() = default;
	virtual ~FStreamSegmentRequestMKV() = default;

	void SetPlaybackSequenceID(uint32 InPlaybackSequenceID) override;
	uint32 GetPlaybackSequenceID() const override;

	void SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay) override;
	FTimeValue GetExecuteAtUTCTime() const override;

	EStreamType GetType() const override;

	void GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const override;
	void GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams) override;
	void GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams) override;

	FTimeValue GetFirstPTS() const override;
	FTimeRange GetTimeRange() const override;

	int32 GetQualityIndex() const override;
	int32 GetBitrate() const override;

	void GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const override;
	bool GetStartupDelay(FTimeValue& OutStartTime, FTimeValue& OutTimeIntoSegment, FTimeValue& OutSegmentDuration) const override
	{ return false; }


	// Which tracks are enabled and will be extracted.
	TArray<uint64> EnabledTrackIDs;

	// The generated MKV asset
	TSharedPtrTS<ITimelineMediaAsset> MediaAsset;
	// MKV parser instance
	TSharedPtrTS<IParserMKV> MKVParser;

	// The PTS of the first sample
	FTimeValue FirstPTS;
	FTimeValue SegmentDuration;

	// PTS of the first sample to be presented.
	FTimeValue EarliestPTS;
	// PTS at which no further samples are to be presented.
	FTimeValue LastPTS;
	// Sequence index to set in all timestamp values of the decoded access unit.
	int64 TimestampSequenceIndex = 0;

	// Type of the stream for which the request was generated
	EStreamType PrimaryStreamType = EStreamType::Unsupported;
	// Where to start in the file
	int64 FileStartOffset = -1;
	// Where to end in the file (for HTTP range GET requests)
	int64 FileEndOffset = -1;
	// Where to skip to on a retry.
	int64 RetryBlockOffset = -1;

	uint64 PrimaryTrackID = 0;
	uint32 CueUniqueID = ~0U;
	uint32 NextCueUniqueID = ~0U;
	uint32 PlaybackSequenceID = ~0U;

	//! Offset of the cluster in which an error occurred.
	int64 FailedClusterFileOffset = -1;
	int64 FailedClusterDataOffset = -1;
	//!< Number of retries
	int32 NumOverallRetries = 0;

	int32 Bitrate = 0;
	// true if this segment is the first to start with or the first after a seek.
	bool bIsFirstSegment = false;
	// true if this segment is the last.
	bool bIsLastSegment = false;
	// true if this segment it beyond the end of the stream.
	bool bIsEOSSegment = false;

	Metrics::FSegmentDownloadStats DownloadStats;
	HTTP::FConnectionInfo ConnectionInfo;
};


/**
 * This class implements an interface to read from an mkv/webm file.
 */
class FStreamReaderMKV : public IStreamReader, public FMediaThread, IGenericDataReader
{
public:
	FStreamReaderMKV() = default;
	virtual ~FStreamReaderMKV();
	UEMediaError Create(IPlayerSessionServices* PlayerSessionService, const CreateParam &createParam) override;
	bool LoadsMultiplexedTracksSequentially() override;
	void Close() override;
	EAddResult AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> Request) override;
	void CancelRequest(EStreamType StreamType, bool bSilent) override;
	void CancelRequests() override;

	// Methods from IGenericDataReader
	int64 ReadData(void* InDestinationBuffer, int64 InNumBytesToRead, int64 InFromOffset) override;
	int64 GetCurrentOffset() const override;
	int64 GetTotalSize() const override;
	bool HasReadBeenAborted() const override;
	bool HasReachedEOF() const override;

private:
	void WorkerThread();
	int32 HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest);
	void HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest);
	void HTTPUpdateStats(const FTimeValue& CurrentTime, const IElectraHttpManager::FRequest* Request);
	void HandleRequest();

	enum class EEmitType
	{
		One,
		AllRemaining,
		KnownDurationOnly
	};
	enum class EEmitResult
	{
		SentNothing,
		Sent,
		AllReachedEOS,
	};
	EEmitResult EmitSamples(EEmitType InEmitType, const TSharedPtrTS<FStreamSegmentRequestMKV>& InRequest);

	bool HasBeenAborted() const;
	bool HasErrored() const;

	struct FReadBuffer
	{
		FReadBuffer()
		{
			Reset();
		}
		void Reset()
		{
			ReceiveBuffer.Reset();
			StartOffset = 0;
			EndOffset = 0;
			CurrentOffset = 0;
			bAbort = false;
			bHasErrored = false;
		}
		void SetStartOffset(int64 InPos)
		{
			StartOffset = InPos;
		}
		int64 GetStartOffset() const
		{
			return StartOffset;
		}
		void SetEndOffset(int64 InPos)
		{
			EndOffset = InPos;
		}
		void SetCurrentOffset(int64 InPos)
		{
			CurrentOffset = InPos;
		}
		int64 GetCurrentOffset() const
		{
			return CurrentOffset;
		}
		int64 GetTotalSize() const
		{
			return EndOffset + 1 - StartOffset;
		}
		void Abort()
		{
			bAbort = true;
			TSharedPtrTS<FWaitableBuffer> Buffer = ReceiveBuffer;
			if (Buffer.IsValid())
			{
				Buffer->Abort();
			}
		}
		void SetHasErrored()
		{
			bHasErrored = true;
		}
		int32 ReadTo(void* ToBuffer, int64 NumBytes);
		TSharedPtrTS<FWaitableBuffer> ReceiveBuffer;
		int64 StartOffset = 0;
		int64 EndOffset = 0;
		int64 CurrentOffset = 0;
		bool bAbort = false;
		bool bHasErrored = false;
	};

	struct FSelectedTrackData
	{
		void Clear()
		{
			BufferSourceInfo.Reset();
			DurationSuccessfullyRead.SetToZero();
			DurationSuccessfullyDelivered.SetToZero();
			bIsFirstInSequence = true;
			bReadPastLastPTS = false;
			bGotAllSamples = false;
			bReachedEndOfKnownDuration = false;
		}

		struct FSample
		{
			FTimeValue PTS;
			FAccessUnit* AU = nullptr;
			FSample(FAccessUnit* InAU) : PTS(InAU->PTS), AU(InAU) { InAU->AddRef(); }
			FSample(const FSample& rhs)
			{
				PTS = rhs.PTS;
				if ((AU = rhs.AU) != nullptr)
				{
					AU->AddRef();
				}
			}
			void Release()
			{
				FAccessUnit::Release(AU);
				AU = nullptr;
			}
			~FSample()
			{
				Release();
			}
		};

		TSharedPtrTS<FAccessUnit::CodecData> CSD;
		TSharedPtrTS<FBufferSourceInfo> BufferSourceInfo;
		FTimeValue DurationSuccessfullyRead {(int64)0};
		FTimeValue DurationSuccessfullyDelivered {(int64)0};
		EStreamType StreamType = EStreamType::Unsupported;
		int32 Bitrate = 0;
		bool bIsFirstInSequence = true;
		bool bReadPastLastPTS = false;
		TArray<FSample> AccessUnitFIFO;
		TArray<FSample> SortedAccessUnitFIFO;
		bool bGotAllSamples = false;
		bool bNeedToRecalculateDurations = false;
		bool bReachedEndOfKnownDuration = false;
	};

	CreateParam Parameters;
	IPlayerSessionServices* PlayerSessionServices = nullptr;
	bool bIsStarted = false;
	bool bTerminate = false;
	bool bRequestCanceled = false;
	bool bHasErrored = false;
	FErrorDetail ErrorDetail;

	TSharedPtrTS<FStreamSegmentRequestMKV> CurrentRequest;
	TSharedPtrTS<IElectraHttpManager::FRequest> HTTPRequest;
	FMediaEvent WorkSignal;
	FReadBuffer ReadBuffer;
	FCriticalSection MetricUpdateLock;
	TMap<uint64, FSelectedTrackData> ActiveTrackMap;
	static uint32 UniqueDownloadID;
};


} // namespace Electra
