// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserISO14496-12.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMP4.h"
#include "Utilities/HashFunctions.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "Utilities/BCP47-Helpers.h"
#include "MediaStreamMetadata.h"
#include "ElectraPlayerPrivate.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/PlayerStreamFilter.h"
#include "Player/mp4/StreamReaderMP4.h"
#include "Player/mp4/ManifestMP4.h"
#include "Player/DRM/DRMManager.h"


#define ERRCODE_MANIFEST_MP4_NO_PLAYABLE_STREAMS		1
#define ERRCODE_MANIFEST_MP4_STARTSEGMENT_NOT_FOUND		2
#define ERRCODE_MANIFEST_MP4_DRM_ERROR					3



DECLARE_CYCLE_STAT(TEXT("FPlayPeriodMP4::FindSegment"), STAT_ElectraPlayer_MP4_FindSegment, STATGROUP_ElectraPlayer);

namespace Electra
{

//-----------------------------------------------------------------------------
/**
 * CTOR
 */
FManifestMP4Internal::FManifestMP4Internal(IPlayerSessionServices* InPlayerSessionServices)
	: PlayerSessionServices(InPlayerSessionServices)
{
}


//-----------------------------------------------------------------------------
/**
 * DTOR
 */
FManifestMP4Internal::~FManifestMP4Internal()
{
}


//-----------------------------------------------------------------------------
/**
 * Builds the internal manifest from the mp4's moov box.
 *
 * @param MP4Parser
 * @param URL
 * @param InConnectionInfo
 *
 * @return
 */
FErrorDetail FManifestMP4Internal::Build(TSharedPtrTS<IParserISO14496_12> MP4Parser, const FString& URL, const HTTP::FConnectionInfo& /*InConnectionInfo*/)
{
	MediaAsset = MakeSharedTS<FTimelineAssetMP4>();
	FErrorDetail Result = MediaAsset->Build(PlayerSessionServices, MP4Parser, URL);
	FTimeRange PlaybackRange = GetPlaybackRange(IManifest::EPlaybackRangeType::TemporaryPlaystartRange);
	DefaultStartTime = PlaybackRange.Start;
	DefaultEndTime = PlaybackRange.End;
	return Result;
}


//-----------------------------------------------------------------------------
/**
 * Logs a message.
 *
 * @param Level
 * @param Message
 */
void FManifestMP4Internal::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::MP4Playlist, Level, Message);
	}
}



EMediaFormatType FManifestMP4Internal::GetMediaFormatType() const
{
	return EMediaFormatType::ISOBMFF;
}

bool FManifestMP4Internal::ReloadsPlaylistPeriodically() const
{
	return false;
}

bool FManifestMP4Internal::SupportsMultipleTracksPerKind() const
{
	return true;
}

bool FManifestMP4Internal::OrderOfTracksCanChangeDynamically() const
{
	return false;
}

//-----------------------------------------------------------------------------
/**
 * Returns the type of presentation.
 * For a single mp4 file this is always VoD.
 *
 * @return
 */
IManifest::EType FManifestMP4Internal::GetPresentationType() const
{
	return IManifest::EType::OnDemand;
}


//-----------------------------------------------------------------------------
/**
 * Returns track metadata.
 *
 * @param OutMetadata
 * @param StreamType
 */
void FManifestMP4Internal::GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const
{
	if (MediaAsset.IsValid())
	{
		MediaAsset->GetMetaData(OutMetadata, StreamType);
	}
}

void FManifestMP4Internal::UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData)
{
	// No-op.
}


//-----------------------------------------------------------------------------
/**
 * Returns the playback range on the timeline, which is a subset of the total
 * time range. This may be set through manifest internal means or by URL fragment
 * parameters where permissable (eg. example.mp4#t=22,50).
 * If start or end are not specified they will be set to invalid.
 *
 * @return Optionally set time range to which playback is restricted.
 */
FTimeRange FManifestMP4Internal::GetPlaybackRange(EPlaybackRangeType InRangeType) const
{
	FTimeRange FromTo;

	// We are interested in the 't' or 'r' fragment value here.
	FString Time;
	for(auto& Fragment : URLFragmentComponents)
	{
		if ((InRangeType == IManifest::EPlaybackRangeType::TemporaryPlaystartRange && Fragment.Name.Equals(TEXT("t"))) ||
			(InRangeType == IManifest::EPlaybackRangeType::LockedPlaybackRange && Fragment.Name.Equals(TEXT("r"))))
		{
			Time = Fragment.Value;
		}
	}
	if (!Time.IsEmpty())
	{
		FTimeRange TotalRange = GetTotalTimeRange();
		TArray<FString> TimeRange;
		const TCHAR* const TimeDelimiter = TEXT(",");
		Time.ParseIntoArray(TimeRange, TimeDelimiter, false);
		if (TimeRange.Num() && !TimeRange[0].IsEmpty())
		{
			RFC2326::ParseNPTTime(FromTo.Start, TimeRange[0]);
		}
		if (TimeRange.Num() > 1 && !TimeRange[1].IsEmpty())
		{
			RFC2326::ParseNPTTime(FromTo.End, TimeRange[1]);
		}
		// Need to clamp this into the total time range to prevent any issues.
		if (FromTo.Start.IsValid() && TotalRange.Start.IsValid() && FromTo.Start < TotalRange.Start)
		{
			FromTo.Start = TotalRange.Start;
		}
		if (FromTo.End.IsValid() && TotalRange.End.IsValid() && FromTo.End > TotalRange.End)
		{
			FromTo.End = TotalRange.End;
		}
	}
	return FromTo;
}


//-----------------------------------------------------------------------------
/**
 * Returns the minimum duration of content that must be buffered up before playback
 * will begin. This is an arbitrary choice that could be controlled by a 'pdin' box.
 *
 * @return
 */
FTimeValue FManifestMP4Internal::GetMinBufferTime() const
{
	// NOTE: This could come from a 'pdin' (progressive download information) box, but those are rarely, if ever, set by any tool.
	return FTimeValue().SetFromSeconds(1.0);
}

TSharedPtrTS<IProducerReferenceTimeInfo> FManifestMP4Internal::GetProducerReferenceTimeInfo(int64 ID) const
{
	return nullptr;
}

TRangeSet<double> FManifestMP4Internal::GetPossiblePlaybackRates(EPlayRateType InForType) const
{
	TRangeSet<double> Ranges;
//	Ranges.Add(TRange<double>{1.0}); // normal (real-time) playback rate
	Ranges.Add(TRange<double>::Inclusive(0.5, 4.0));
	Ranges.Add(TRange<double>{0.0}); // and pause
	return Ranges;
}


void FManifestMP4Internal::UpdateDynamicRefetchCounter()
{
	// No-op.
}

void FManifestMP4Internal::TriggerClockSync(IManifest::EClockSyncType InClockSyncType)
{
	// No-op.
}

void FManifestMP4Internal::TriggerPlaylistRefresh()
{
	// No-op.
}


//-----------------------------------------------------------------------------
/**
 * Creates an instance of a stream reader to stream from the mp4 file.
 *
 * @return
 */
IStreamReader* FManifestMP4Internal::CreateStreamReaderHandler()
{
	return new FStreamReaderMP4;
}


//-----------------------------------------------------------------------------
/**
 * Returns the playback period for the given time.
 *
 * @param OutPlayPeriod
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMP4Internal::FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	if (MediaAsset.IsValid() && StartPosition.Time.IsValid() && StartPosition.Time < MediaAsset->GetDuration())
	{
		OutPlayPeriod = MakeSharedTS<FPlayPeriodMP4>(MediaAsset);
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}
	else
	{
		return IManifest::FResult(IManifest::FResult::EType::PastEOS);
	}
}

IManifest::FResult FManifestMP4Internal::FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment)
{
	// There is no following period.
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}





//-----------------------------------------------------------------------------
/**
 * Constructs a playback period.
 *
 * @param InMediaAsset
 */
FManifestMP4Internal::FPlayPeriodMP4::FPlayPeriodMP4(TSharedPtrTS<FManifestMP4Internal::FTimelineAssetMP4> InMediaAsset)
	: MediaAsset(InMediaAsset)
	, CurrentReadyState(IManifest::IPlayPeriod::EReadyState::NotLoaded)
{
}


//-----------------------------------------------------------------------------
/**
 * Destroys a playback period.
 */
FManifestMP4Internal::FPlayPeriodMP4::~FPlayPeriodMP4()
{
}


//-----------------------------------------------------------------------------
/**
 * Sets stream playback preferences for this playback period.
 *
 * @param ForStreamType
 * @param StreamAttributes
 */
void FManifestMP4Internal::FPlayPeriodMP4::SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	if (ForStreamType == EStreamType::Video)
	{
		VideoPreferences = StreamAttributes;
	}
	else if (ForStreamType == EStreamType::Audio)
	{
		AudioPreferences = StreamAttributes;
	}
	else if (ForStreamType == EStreamType::Subtitle)
	{
		SubtitlePreferences = StreamAttributes;
	}
}


//-----------------------------------------------------------------------------
/**
 * Returns the starting bitrate.
 *
 * This is merely informational and not strictly required.
 * If fetching of the moov box provided us with the total size of the mp4 file
 * we will use that divided by the duration.
 *
 * @return
 */
int64 FManifestMP4Internal::FPlayPeriodMP4::GetDefaultStartingBitrate() const
{
	return 2000000;
}

//-----------------------------------------------------------------------------
/**
 * Returns the ready state of this playback period.
 *
 * @return
 */
IManifest::IPlayPeriod::EReadyState FManifestMP4Internal::FPlayPeriodMP4::GetReadyState()
{
	return CurrentReadyState;
}


void FManifestMP4Internal::FPlayPeriodMP4::Load()
{
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::Loaded;
}

//-----------------------------------------------------------------------------
/**
 * Prepares the playback period for playback.
 * With an mp4 file we are actually always ready for playback, but we say we're not
 * one time to get here with any possible options.
 */
void FManifestMP4Internal::FPlayPeriodMP4::PrepareForPlay()
{
	SelectedVideoMetadata.Reset();
	SelectedAudioMetadata.Reset();
	SelectedSubtitleMetadata.Reset();
	VideoBufferSourceInfo.Reset();
	AudioBufferSourceInfo.Reset();
	SubtitleBufferSourceInfo.Reset();
	SelectInitialStream(EStreamType::Video);
	SelectInitialStream(EStreamType::Audio);
	SelectInitialStream(EStreamType::Subtitle);
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::IsReady;
}


TSharedPtrTS<FBufferSourceInfo> FManifestMP4Internal::FPlayPeriodMP4::GetSelectedStreamBufferSourceInfo(EStreamType StreamType)
{
	return StreamType == EStreamType::Video ? VideoBufferSourceInfo :
		   StreamType == EStreamType::Audio ? AudioBufferSourceInfo :
		   StreamType == EStreamType::Subtitle ? SubtitleBufferSourceInfo : nullptr;
}

FString FManifestMP4Internal::FPlayPeriodMP4::GetSelectedAdaptationSetID(EStreamType StreamType)
{
	switch(StreamType)
	{
		case EStreamType::Video:
			return SelectedVideoMetadata.IsValid() ? SelectedVideoMetadata->ID : FString();
		case EStreamType::Audio:
			return SelectedAudioMetadata.IsValid() ? SelectedAudioMetadata->ID : FString();
		case EStreamType::Subtitle:
			return SelectedSubtitleMetadata.IsValid() ? SelectedSubtitleMetadata->ID : FString();
		default:
			return FString();
	}
}


IManifest::IPlayPeriod::ETrackChangeResult FManifestMP4Internal::FPlayPeriodMP4::ChangeTrackStreamPreference(EStreamType StreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	TSharedPtrTS<FTrackMetadata> Metadata = SelectMetadataForAttributes(StreamType, StreamAttributes);
	if (Metadata.IsValid())
	{
		if (StreamType == EStreamType::Video)
		{
			if (!(SelectedVideoMetadata.IsValid() && Metadata->Equals(*SelectedVideoMetadata)))
			{
				SelectedVideoMetadata = Metadata;
				MakeBufferSourceInfoFromMetadata(StreamType, VideoBufferSourceInfo, SelectedVideoMetadata);
				return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
			}
		}
		else if (StreamType == EStreamType::Audio)
		{
			if (!(SelectedAudioMetadata.IsValid() && Metadata->Equals(*SelectedAudioMetadata)))
			{
				SelectedAudioMetadata = Metadata;
				MakeBufferSourceInfoFromMetadata(StreamType, AudioBufferSourceInfo, SelectedAudioMetadata);
				return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
			}
		}
		else if (StreamType == EStreamType::Subtitle)
		{
			if (!(SelectedSubtitleMetadata.IsValid() && Metadata->Equals(*SelectedSubtitleMetadata)))
			{
				SelectedSubtitleMetadata = Metadata;
				MakeBufferSourceInfoFromMetadata(StreamType, SubtitleBufferSourceInfo, SelectedSubtitleMetadata);
				return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
			}
		}
	}
	return IManifest::IPlayPeriod::ETrackChangeResult::NotChanged;
}

void FManifestMP4Internal::FPlayPeriodMP4::SelectInitialStream(EStreamType StreamType)
{
	if (StreamType == EStreamType::Video)
	{
		SelectedVideoMetadata = SelectMetadataForAttributes(StreamType, VideoPreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, VideoBufferSourceInfo, SelectedVideoMetadata);
	}
	else if (StreamType == EStreamType::Audio)
	{
		SelectedAudioMetadata = SelectMetadataForAttributes(StreamType, AudioPreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, AudioBufferSourceInfo, SelectedAudioMetadata);
	}
	else if (StreamType == EStreamType::Subtitle)
	{
		SelectedSubtitleMetadata = SelectMetadataForAttributes(StreamType, SubtitlePreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, SubtitleBufferSourceInfo, SelectedSubtitleMetadata);
	}
}

TSharedPtrTS<FTrackMetadata> FManifestMP4Internal::FPlayPeriodMP4::SelectMetadataForAttributes(EStreamType StreamType, const FStreamSelectionAttributes& InAttributes)
{
	TSharedPtrTS<FTimelineAssetMP4> Asset = MediaAsset.Pin();
	if (Asset.IsValid())
	{
		TArray<FTrackMetadata> Metadata;
		Asset->GetMetaData(Metadata, StreamType);
		// Is there a fixed index to be used?
		if (InAttributes.OverrideIndex.IsSet() && InAttributes.OverrideIndex.GetValue() >= 0 && InAttributes.OverrideIndex.GetValue() < Metadata.Num())
		{
			// Use this.
			return MakeSharedTS<FTrackMetadata>(Metadata[InAttributes.OverrideIndex.GetValue()]);
		}
		if (Metadata.Num())
		{
			// We do not look at the 'kind' or 'codec' here, only the language.
			// Set the first track as default in case we do not find the one we're looking for.
			if (InAttributes.Language_RFC4647.IsSet())
			{
				TArray<int32> CandidateIndices;
				TArray<BCP47::FLanguageTag> CandList;
				for(auto &Meta : Metadata)
				{
					CandList.Emplace(Meta.LanguageTagRFC5646);
				}
				CandidateIndices = BCP47::FindExtendedFilteringMatch(CandList, InAttributes.Language_RFC4647.GetValue());
				return MakeSharedTS<FTrackMetadata>(CandidateIndices.Num() ? Metadata[CandidateIndices[0]] : Metadata[0]);
			}
			// Subtitle tracks are not selected by default. If there is no explicit selection asked for we ignore them.
			if (StreamType != EStreamType::Subtitle)
			{
				return MakeSharedTS<FTrackMetadata>(Metadata[0]);
			}
		}
	}
	return nullptr;
}

void FManifestMP4Internal::FPlayPeriodMP4::MakeBufferSourceInfoFromMetadata(EStreamType StreamType, TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FTrackMetadata> InMetadata)
{
	if (InMetadata.IsValid())
	{
		OutBufferSourceInfo = MakeSharedTS<FBufferSourceInfo>();
		OutBufferSourceInfo->Kind = InMetadata->Kind;
		OutBufferSourceInfo->LanguageTag = InMetadata->LanguageTagRFC5646;
		OutBufferSourceInfo->Codec = InMetadata->HighestBandwidthCodec.GetCodecName();
		TSharedPtrTS<FTimelineAssetMP4> Asset = MediaAsset.Pin();
		OutBufferSourceInfo->PeriodID = Asset->GetUniqueIdentifier();
		OutBufferSourceInfo->PeriodAdaptationSetID = Asset->GetUniqueIdentifier() + TEXT(".") + InMetadata->ID;
		TArray<FTrackMetadata> Metadata;
		Asset->GetMetaData(Metadata, StreamType);
		for(int32 i=0; i<Metadata.Num(); ++i)
		{
			if (Metadata[i].Equals(*InMetadata))
			{
				OutBufferSourceInfo->HardIndex = i;
				break;
			}
		}
	}
}



//-----------------------------------------------------------------------------
/**
 * Returns the timeline media asset. We have a weak pointer to it only to
 * prevent any cyclic locks, so we need to lock it first.
 *
 * @return
 */
TSharedPtrTS<ITimelineMediaAsset> FManifestMP4Internal::FPlayPeriodMP4::GetMediaAsset() const
{
	TSharedPtrTS<ITimelineMediaAsset> ma = MediaAsset.Pin();
	return ma;
}


//-----------------------------------------------------------------------------
/**
 * Selects a particular stream (== internal track ID) for playback.
 *
 * @param AdaptationSetID
 * @param RepresentationID
 */
void FManifestMP4Internal::FPlayPeriodMP4::SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex)
{
	// Presently this method is only called by the ABR to switch between quality levels.
	// Since a single mp4 doesn't have different quality levels (technically it could, but we are concerning ourselves only with different bitrates and that doesn't apply since we are streaming
	// the single file sequentially and selecting a different stream would not save any bandwidth so we don't bother) we ignore this for now.

	// This may need an implementation when switching between different languages though.
	// .....
}

void FManifestMP4Internal::FPlayPeriodMP4::TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload)
{
	// No-op.
}

//-----------------------------------------------------------------------------
/**
 * Creates the starting segment request to start playback with.
 *
 * @param OutSegment
 * @param InSequenceState
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMP4Internal::FPlayPeriodMP4::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	TSharedPtrTS<FTimelineAssetMP4> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetStartingSegment(OutSegment, InSequenceState, StartPosition, SearchType, -1) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}

//-----------------------------------------------------------------------------
/**
 * Same as GetStartingSegment() except this is for a specific stream (video, audio, ...) only.
 * To be used when a track (language) change is made and a new segment is needed at the current playback position.
 */
IManifest::FResult FManifestMP4Internal::FPlayPeriodMP4::GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType StreamType, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	// Not supported
	return IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Sets up a starting segment request to loop playback to.
 * The streams selected through SelectStream() will be used.
 *
 * @param OutSegment
 * @param SequenceState
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMP4Internal::FPlayPeriodMP4::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	TSharedPtrTS<FTimelineAssetMP4> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetLoopingSegment(OutSegment, SequenceState, StartPosition, SearchType) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Called by the ABR to increase the delay in fetching the next segment in case the segment returned a 404 when fetched at
 * the announced availability time. This may reduce 404's on the next segment fetches.
 *
 * @param IncreaseAmount
 */
void FManifestMP4Internal::FPlayPeriodMP4::IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount)
{
	// No-op.
}


//-----------------------------------------------------------------------------
/**
 * Creates the next segment request.
 *
 * @param OutSegment
 * @param CurrentSegment
 * @param Options
 *
 * @return
 */
IManifest::FResult FManifestMP4Internal::FPlayPeriodMP4::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options)
{
	TSharedPtrTS<FTimelineAssetMP4> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetNextSegment(OutSegment, CurrentSegment, Options) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Creates a segment retry request.
 *
 * @param OutSegment
 * @param CurrentSegment
 * @param Options
 * @param bReplaceWithFillerData
 *
 * @return
 */
IManifest::FResult FManifestMP4Internal::FPlayPeriodMP4::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData)
{
	TSharedPtrTS<FTimelineAssetMP4> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetRetrySegment(OutSegment, CurrentSegment, Options, bReplaceWithFillerData) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Returns the average segment duration.
 */
void FManifestMP4Internal::FPlayPeriodMP4::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	TSharedPtrTS<FTimelineAssetMP4> ma = MediaAsset.Pin();
	if (ma.IsValid())
	{
		ma->GetAverageSegmentDuration(OutAverageSegmentDuration, AdaptationSetID, RepresentationID);
	}
}





//-----------------------------------------------------------------------------
/**
 * Builds the timeline asset.
 *
 * @param InPlayerSessionServices
 * @param MP4Parser
 * @param URL
 *
 * @return
 */
FErrorDetail FManifestMP4Internal::FTimelineAssetMP4::Build(IPlayerSessionServices* InPlayerSessionServices, TSharedPtrTS<IParserISO14496_12> MP4Parser, const FString& URL)
{
	PlayerSessionServices = InPlayerSessionServices;
	MediaURL = URL;

	TSharedPtrTS<FDRMManager> DRMManager = InPlayerSessionServices->GetDRMManager();
	TArray<ElectraCDM::IMediaCDM::FCDMCandidate> DRMCandidates;

	// Go over the supported tracks and create an internal manifest-like structure for the player to work with.
	LongestTrackDuration = FTimeValue::GetZero();
	for(int32 nTrack=0,nMaxTrack=MP4Parser->GetNumberOfTracks(); nTrack < nMaxTrack; ++nTrack)
	{
		const IParserISO14496_12::ITrack* Track = MP4Parser->GetTrackByIndex(nTrack);
		if (Track)
		{
			//bool bIsLocalPlayback = TimelineAsset->GetMediaURL().StartsWith(TEXT("file:"));
			bool bIsUsable = false;

			// Can we decode this track?
			IPlayerStreamFilter* StreamFilter = InPlayerSessionServices->GetStreamFilter();
			if (StreamFilter && StreamFilter->CanDecodeStream(Track->GetCodecInformation()))
			{
				bIsUsable = true;
			}

			IParserISO14496_12::ITrack::FEncryptionInfo EncInfo;
			// Using encryption?
			if (bIsUsable && Track->GetEncryptionInfo(EncInfo))
			{
				// Encrypted.
				if (DRMManager.IsValid())
				{
					FString Scheme = FDRMManager::MakePrintableStringFromUint32(EncInfo.Scheme);
					TSharedPtr<ElectraCDM::IMediaCDMCapabilities, ESPMode::ThreadSafe> DRMCapabilities;
					for(int32 i=0; i<EncInfo.CDMInfos.Num(); ++i)
					{
						FString uuid = FDRMManager::MakeHexStringFromArray(EncInfo.CDMInfos[i].SystemID);
						DRMCapabilities = DRMManager->GetCDMCapabilitiesForScheme(uuid, FString(), FString());
						if (DRMCapabilities.IsValid())
						{
							ElectraCDM::IMediaCDMCapabilities::ESupportResult Result;
							Result = DRMCapabilities->SupportsCipher(Scheme);
							if (Result == ElectraCDM::IMediaCDMCapabilities::ESupportResult::Supported)
							{
								bIsUsable = true;

								ElectraCDM::IMediaCDM::FCDMCandidate cand;

								cand.SchemeId = uuid;
								cand.CommonScheme = Scheme;
								for(int32 j=0; j<EncInfo.CDMInfos[i].KIDs.Num(); ++j)
								{
									cand.DefaultKIDs.Emplace(FDRMManager::MakeHexStringFromArray(EncInfo.CDMInfos[i].KIDs[j]));
								}
								DRMCandidates.Emplace(MoveTemp(cand));
							}
							else
							{
								bIsUsable = false;
							}
						}
						else
						{
							bIsUsable = false;
						}
					}
				}
				else
				{
					bIsUsable = false;
				}
			}

			// In an mp4 file we treat every track as a single adaptation set with one representation only.
			// That's because by definition an adaptation set contains the same content at different bitrates and resolutions, but
			// the type, language and codec has to be the same.
			if (bIsUsable)
			{
				FErrorDetail err;
				TSharedPtrTS<FAdaptationSetMP4> AdaptationSet = MakeSharedTS<FAdaptationSetMP4>();
				err = AdaptationSet->CreateFrom(Track, URL);
				if (err.IsOK())
				{
					// Add this track to the proper category.
					switch(Track->GetCodecInformation().GetStreamType())
					{
						case EStreamType::Video:
						{
							VideoAdaptationSets.Add(AdaptationSet);
							break;
						}
						case EStreamType::Audio:
						{
							AudioAdaptationSets.Add(AdaptationSet);
							break;
						}
						case EStreamType::Subtitle:
						{
							SubtitleAdaptationSets.Add(AdaptationSet);
							break;
						}
						default:
						{
							break;
						}
					}
					FTimeValue TrkDur;
					TrkDur.SetFromTimeFraction(Track->GetDuration());
					if (TrkDur > LongestTrackDuration)
					{
						LongestTrackDuration = TrkDur;
					}
				}
				else
				{
					return err;
				}
			}
		}
	}

	// No playable content?
	if (VideoAdaptationSets.Num() == 0 && AudioAdaptationSets.Num() == 0)
	{
		FErrorDetail err;
		err.SetFacility(Facility::EFacility::MP4Playlist);
		err.SetMessage("No playable streams in this mp4");
		err.SetCode(ERRCODE_MANIFEST_MP4_NO_PLAYABLE_STREAMS);
		return err;
	}

// FIXME: fragmented mp4's with a sidx and moof boxes!

	// Hold on to the parsed MOOV box for future reference.
	MoovBoxParser = MP4Parser;

	// Get the metadata, if any.
	const IParserISO14496_12::IMetadata* md = MP4Parser->GetMetadata(IParserISO14496_12::EBaseBoxType::Moov);
	if (md)
	{
		uint32 hdlr = md->GetHandler();
		uint32 res0 = md->GetReserved(0);
		TArray<MP4Utilities::FMP4BoxInfo> Boxes;
		for(int32 i=0, iMax=md->GetNumChildBoxes(); i<iMax; ++i)
		{
			Boxes.Emplace(MP4Utilities::FMP4BoxInfo({.Data = MakeConstArrayView<uint8>(reinterpret_cast<const uint8*>(md->GetChildBoxData(i)), md->GetChildBoxDataSize(i)), .Type=md->GetChildBoxType(i)}));
		}
		MediaMetadata = MakeSharedTS<FMetadataParser>();
		if (MediaMetadata->Parse(hdlr, res0, Boxes) == FMetadataParser::EResult::Success)
		{
			PlayerSessionServices->SendMessageToPlayer(FPlaylistMetadataUpdateMessage::Create(FTimeValue(), MediaMetadata, false));
		}
		else
		{
			MediaMetadata.Reset();
		}
	}

	if (DRMCandidates.Num())
	{
		ElectraCDM::ECDMError Result = DRMManager->CreateDRMClient(DrmClient, DRMCandidates);
		if (Result == ElectraCDM::ECDMError::Success && DrmClient.IsValid())
		{
			DrmClient->RegisterEventListener(DRMManager);
			DrmClient->SetLicenseServerURL(URL);
			DrmClient->PrepareLicenses();
		}
		else
		{
			FErrorDetail err;
			err.SetFacility(Facility::EFacility::MP4Playlist);
			err.SetMessage(FString::Printf(TEXT("Failed to create DRM client with error %d"), (int32)Result));
			err.SetCode(ERRCODE_MANIFEST_MP4_DRM_ERROR);
			return err;
		}
	}

	return FErrorDetail();
}


void FManifestMP4Internal::FTimelineAssetMP4::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::MP4Playlist, Level, Message);
	}
}


void FManifestMP4Internal::FTimelineAssetMP4::LimitSegmentDownloadSize(TSharedPtrTS<IStreamSegment>& InOutSegment)
{
	// Limit the segment size by content duration and/or size.
	// This helps with downloads that might otherwise take too long or keep the connection open for too long (when downloading a large mp4 from start to finish).
	const int64 MaxSegmentDurationMSec = 3000;
	const int64 MaxSegmentByteCount = 1024 * 1024 * 64;
	if (InOutSegment.IsValid())
	{
		FStreamSegmentRequestMP4* Request = static_cast<FStreamSegmentRequestMP4*>(InOutSegment.Get());
		const int64 StartOffset = Request->FileStartOffset;
		const int64 EndOffset = PlayRangeEndInfo.TotalFileEndOffset >=0 ? PlayRangeEndInfo.TotalFileEndOffset : TNumericLimits<int64>::Max();

		TSharedPtr<IParserISO14496_12::IAllTrackIterator, ESPMode::ThreadSafe> AllTrackIterator = MoovBoxParser->CreateAllTrackIteratorByFilePos(StartOffset);
		bool bFirst = true;
		uint32 TrackId = ~0U;
		uint32 TrackTimeScale = 0;
		int64 TrackDur = 0;
		int64 TrackDurSoFar = 0;
		int64 LastTrackOffset = -1;
		int64 LastSampleSize = 0;
		int64 TrackDurationLimit = -1;
		while(AllTrackIterator.IsValid())
		{
			const IParserISO14496_12::ITrackIterator* CurrentTrackIt = AllTrackIterator->Current();
			if (CurrentTrackIt)
			{
				LastTrackOffset = CurrentTrackIt->GetSampleFileOffset();
				LastSampleSize = CurrentTrackIt->GetSampleSize();
				if (bFirst)
				{
					bFirst = false;
					TrackId = CurrentTrackIt->GetTrack()->GetID();
					TrackTimeScale = CurrentTrackIt->GetTimescale();
					if (MaxSegmentDurationMSec > 0)
					{
						TrackDurationLimit = MaxSegmentDurationMSec * TrackTimeScale / 1000;
					}
				}
				if (TrackId == CurrentTrackIt->GetTrack()->GetID())
				{
					TrackDurSoFar = TrackDur;
					TrackDur += CurrentTrackIt->GetDuration();
				}
				int64 CurrentTrackOffset = LastTrackOffset;
				bool bDurationLimitReached = TrackDurationLimit > 0 && TrackDur > TrackDurationLimit;
				bool bSizeLimitReached = /*(MaxSegmentByteCount > 0) &&*/ (CurrentTrackOffset - StartOffset >= MaxSegmentByteCount);
				if (CurrentTrackOffset >= EndOffset || bDurationLimitReached || bSizeLimitReached)
				{
					// Limit reached.
					Request->FileEndOffset = CurrentTrackOffset - 1;
					Request->SegmentInternalSize = CurrentTrackOffset - StartOffset;
					Request->SegmentDuration.SetFromND(bDurationLimitReached ? TrackDurSoFar : TrackDur, TrackTimeScale);
					Request->bIsLastSegment = CurrentTrackOffset >= EndOffset;
					break;
				}
				AllTrackIterator->Next();
			}
			else
			{
				// Done iterating
				Request->SegmentInternalSize = LastTrackOffset + LastSampleSize - StartOffset;
				break;
			}
		}
	}
}

void FManifestMP4Internal::FTimelineAssetMP4::UpdatePlayRangeEndInfo(const FTimeValue& PlayRangeEndTime)
{
	if (PlayRangeEndInfo.Time != PlayRangeEndTime)
	{
		PlayRangeEndInfo.Clear();
		PlayRangeEndInfo.Time = PlayRangeEndTime;
		if (PlayRangeEndInfo.Time.IsValid() && !PlayRangeEndInfo.Time.IsPositiveInfinity())
		{
			int32 nTracks = MoovBoxParser->GetNumberOfTracks();
			PlayRangeEndInfo.FileOffsetsPerTrackID.Reserve(nTracks);
			for(int32 nTrk=0; nTrk<nTracks; ++nTrk)
			{
				int64 TrackEndBytePos = -1;
				const IParserISO14496_12::ITrack* Track = MoovBoxParser->GetTrackByIndex(nTrk);
				IParserISO14496_12::ITrackIterator* TrkIt = Track ? Track->CreateIterator() : nullptr;
				if (TrkIt)
				{
					UEMediaError ItErr;
					bool bFirst = true;
					int64 TrackLocalTime = 0;
					for(ItErr = TrkIt->StartAtFirst(false); ItErr == UEMEDIA_ERROR_OK; ItErr = TrkIt->Next())
					{
						if (bFirst)
						{
							bFirst = false;
							TrackLocalTime = PlayRangeEndInfo.Time.GetAsTimebase(TrkIt->GetTimescale());
						}
						if (TrkIt->GetDTS() >= TrackLocalTime && TrkIt->GetPTS() >= TrackLocalTime)
						{
							break;
						}
					}
					TrackEndBytePos = TrkIt->GetSampleFileOffset() + TrkIt->GetSampleSize();
					if (TrackEndBytePos > PlayRangeEndInfo.TotalFileEndOffset)
					{
						PlayRangeEndInfo.TotalFileEndOffset = TrackEndBytePos;
					}
				}
				PlayRangeEndInfo.FileOffsetsPerTrackID.Emplace(TrackEndBytePos);
			}
		}
	}
}


IManifest::FResult FManifestMP4Internal::FTimelineAssetMP4::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType, int64 AtAbsoluteFilePos)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MP4_FindSegment);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, MP4_FindSegment);

// TODO: If there is a SIDX box we should look in there.

	// Frame accurate seek required?
	bool bFrameAccurateSearch = AtAbsoluteFilePos < 0 ? StartPosition.Options.bFrameAccuracy : false;
	FTimeValue PlayRangeEnd = StartPosition.Options.PlaybackRange.End;
	check(PlayRangeEnd.IsValid());
	if (PlayRangeEnd > LongestTrackDuration)
	{
		PlayRangeEnd = LongestTrackDuration;
	}
	UpdatePlayRangeEndInfo(PlayRangeEnd);

	// Look at the actual tracks. If there is video search there first for a keyframe/IDR frame.
	if (VideoAdaptationSets.Num())
	{
		// Start at the first track. The *assumption* is that the file is interleaved such that the video track with the lowest
		// track ID comes before the other tracks for every sync sample. This may not always be the case though.
		TSharedPtrTS<FRepresentationMP4> Repr = StaticCastSharedPtr<FRepresentationMP4>(VideoAdaptationSets[0]->GetRepresentationByIndex(0));
		if (Repr.IsValid())
		{
			int32 TrackID;
			LexFromString(TrackID, *Repr->GetUniqueIdentifier());
			const IParserISO14496_12::ITrack* Track = MoovBoxParser->GetTrackByTrackID(TrackID);
			if (Track)
			{
				TSharedPtrTS<IParserISO14496_12::ITrackIterator> TrackIt(Track->CreateIterator());
				IParserISO14496_12::ITrackIterator::ESearchMode SearchMode =
					SearchType == ESearchType::After  || SearchType == ESearchType::StrictlyAfter  ? IParserISO14496_12::ITrackIterator::ESearchMode::After  :
					SearchType == ESearchType::Before || SearchType == ESearchType::StrictlyBefore ? IParserISO14496_12::ITrackIterator::ESearchMode::Before : IParserISO14496_12::ITrackIterator::ESearchMode::Closest;
				// Frame accurate seeking requires to start from the preceeding sync sample!
				if (bFrameAccurateSearch)
				{
					SearchMode = IParserISO14496_12::ITrackIterator::ESearchMode::Before;
				}

				UEMediaError err;
				FTimeValue firstTimestamp;
				int64 firstByteOffset = 0;
				if (AtAbsoluteFilePos < 0)
				{
					// Get the track duration. If the start time is outside the track this could be a deliberate seek past the end of the video track
					// into a longer audio track. We do not want to snap to the last video IDR frame then.
					FTimeFraction TrackDuration = Track->GetDuration();
					firstTimestamp.SetFromTimeFraction(TrackDuration);
					if (TrackDuration.IsValid() && FTimeValue().SetFromTimeFraction(TrackDuration) <= StartPosition.Time && (SearchType == ESearchType::After || SearchType == ESearchType::StrictlyAfter || SearchType == ESearchType::Closest))
					{
						err = UEMEDIA_ERROR_END_OF_STREAM;
					}
					else
					{
						err = TrackIt->StartAtTime(StartPosition.Time, SearchMode, true);
						if (err == UEMEDIA_ERROR_OK)
						{
							firstTimestamp.SetFromND(TrackIt->GetPTS(), TrackIt->GetTimescale());
							firstByteOffset = TrackIt->GetSampleFileOffset();
						}
					}
				}
				else
				{
					TSharedPtr<IParserISO14496_12::IAllTrackIterator, ESPMode::ThreadSafe> AllTrackIterator = MoovBoxParser->CreateAllTrackIteratorByFilePos(AtAbsoluteFilePos);
					const IParserISO14496_12::ITrackIterator* CurrentTrackIt = AllTrackIterator->Current();
					if (CurrentTrackIt)
					{
						firstTimestamp.SetFromND(CurrentTrackIt->GetPTS(), CurrentTrackIt->GetTimescale());
						firstByteOffset = CurrentTrackIt->GetSampleFileOffset();
						err = UEMEDIA_ERROR_OK;
					}
					else
					{
						firstTimestamp.SetFromTimeFraction(Track->GetDuration());
						err = UEMEDIA_ERROR_END_OF_STREAM;
					}
				}
				if (err == UEMEDIA_ERROR_OK)
				{
					// Time found. Set up the fragment request.
					TSharedPtrTS<FStreamSegmentRequestMP4> req(new FStreamSegmentRequestMP4);
					OutSegment = req;
					req->MediaAsset = SharedThis(this);
					req->PrimaryTrackIterator = TrackIt;
					check(firstTimestamp.IsValid());
					req->FirstPTS = firstTimestamp;
					req->PrimaryStreamType = EStreamType::Video;
					req->FileStartOffset = firstByteOffset;
					req->FileEndOffset = -1;
					req->Bitrate = Repr->GetBitrate();
					req->bStartingOnMOOF = false;
					req->bIsFirstSegment = true;
					req->bIsLastSegment = true;
					req->SegmentDuration = GetDuration() - req->FirstPTS;
					// Set the PTS range for which to present samples.
					req->EarliestPTS = bFrameAccurateSearch ? StartPosition.Time : firstTimestamp;
					check(PlayRangeEnd.IsValid());
					req->LastPTS = PlayRangeEnd;
					req->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();

					// Add dependent stream types
					if (AudioAdaptationSets.Num())
					{
						req->DependentStreamTypes.Add(EStreamType::Audio);
					}
					if (SubtitleAdaptationSets.Num())
					{
						req->DependentStreamTypes.Add(EStreamType::Subtitle);
					}
					req->DrmClient = DrmClient;

					LimitSegmentDownloadSize(OutSegment);
					if (req->SegmentInternalSize > 0)
					{
						return IManifest::FResult(IManifest::FResult::EType::Found);
					}
					else
					{
						err = UEMEDIA_ERROR_END_OF_STREAM;
					}
				}
				if (err == UEMEDIA_ERROR_END_OF_STREAM)
				{
					// If there are no audio tracks we return an EOS request.
					// Otherwise we search the audio tracks for a start position.
					if (AudioAdaptationSets.Num() == 0)
					{
						TSharedPtrTS<FStreamSegmentRequestMP4> req(new FStreamSegmentRequestMP4);
						OutSegment = req;
						req->MediaAsset = SharedThis(this);
						req->FirstPTS = firstTimestamp;
						req->PrimaryStreamType = EStreamType::Video;
						req->Bitrate = Repr->GetBitrate();
						req->bStartingOnMOOF = false;
						req->bIsFirstSegment = false;
						req->bIsLastSegment = true;
						req->bAllTracksAtEOS = true;
						// Set the PTS range for which to present samples.
						req->EarliestPTS = bFrameAccurateSearch ? StartPosition.Time : firstTimestamp;
						req->LastPTS = PlayRangeEnd;
						req->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
						return IManifest::FResult(IManifest::FResult::EType::Found);
					}
				}
				else if (err == UEMEDIA_ERROR_INSUFFICIENT_DATA)
				{
					return IManifest::FResult(IManifest::FResult::EType::BeforeStart);
				}
				else
				{
					IManifest::FResult res(IManifest::FResult::EType::NotFound);
					res.SetErrorDetail(FErrorDetail().SetError(err)
									   .SetFacility(Facility::EFacility::MP4Playlist)
									   .SetCode(ERRCODE_MANIFEST_MP4_STARTSEGMENT_NOT_FOUND)
									   .SetMessage(FString::Printf(TEXT("Could not find start segment for time %lld"), (long long int)StartPosition.Time.GetAsHNS())));
					return res;
				}
			}
		}
	}
	// No video track(s). Are there audio tracks?
	if (AudioAdaptationSets.Num())
	{
		TSharedPtrTS<FRepresentationMP4> Repr = StaticCastSharedPtr<FRepresentationMP4>(AudioAdaptationSets[0]->GetRepresentationByIndex(0));
		if (Repr.IsValid())
		{
			int32 TrackID;
			LexFromString(TrackID, *Repr->GetUniqueIdentifier());
			const IParserISO14496_12::ITrack* Track = MoovBoxParser->GetTrackByTrackID(TrackID);
			if (Track)
			{
				TSharedPtrTS<IParserISO14496_12::ITrackIterator> TrackIt(Track->CreateIterator());
				IParserISO14496_12::ITrackIterator::ESearchMode SearchMode =
					SearchType == ESearchType::After  || SearchType == ESearchType::StrictlyAfter  ? IParserISO14496_12::ITrackIterator::ESearchMode::After  :
					SearchType == ESearchType::Before || SearchType == ESearchType::StrictlyBefore ? IParserISO14496_12::ITrackIterator::ESearchMode::Before : IParserISO14496_12::ITrackIterator::ESearchMode::Closest;
				// Frame accurate seeking requires to start from the preceeding sync sample!
				if (bFrameAccurateSearch)
				{
					SearchMode = IParserISO14496_12::ITrackIterator::ESearchMode::Before;
				}

				UEMediaError err;
				FTimeValue firstTimestamp;
				int64 firstByteOffset = 0;
				if (AtAbsoluteFilePos < 0)
				{
					// Get the track duration. If the start time is outside the track this could be a deliberate seek past the end of the audio track
					// for which we return EOS.
					FTimeFraction TrackDuration = Track->GetDuration();
					firstTimestamp.SetFromTimeFraction(TrackDuration);
					if (TrackDuration.IsValid() && FTimeValue().SetFromTimeFraction(TrackDuration) <= StartPosition.Time && (SearchType == ESearchType::After || SearchType == ESearchType::StrictlyAfter || SearchType == ESearchType::Closest))
					{
						err = UEMEDIA_ERROR_END_OF_STREAM;
					}
					else
					{
						err = TrackIt->StartAtTime(StartPosition.Time, SearchMode, true);
						if (err == UEMEDIA_ERROR_OK)
						{
							firstTimestamp.SetFromND(TrackIt->GetPTS(), TrackIt->GetTimescale());
							firstByteOffset = TrackIt->GetSampleFileOffset();
						}
					}
				}
				else
				{
					TSharedPtr<IParserISO14496_12::IAllTrackIterator, ESPMode::ThreadSafe> AllTrackIterator = MoovBoxParser->CreateAllTrackIteratorByFilePos(AtAbsoluteFilePos);
					const IParserISO14496_12::ITrackIterator* CurrentTrackIt = AllTrackIterator->Current();
					if (CurrentTrackIt)
					{
						firstTimestamp.SetFromND(CurrentTrackIt->GetPTS(), CurrentTrackIt->GetTimescale());
						firstByteOffset = CurrentTrackIt->GetSampleFileOffset();
						err = UEMEDIA_ERROR_OK;
					}
					else
					{
						firstTimestamp.SetFromTimeFraction(Track->GetDuration());
						err = UEMEDIA_ERROR_END_OF_STREAM;
					}
				}
				if (err == UEMEDIA_ERROR_OK)
				{
					// Time found. Set up the fragment request.
					TSharedPtrTS<FStreamSegmentRequestMP4> req(new FStreamSegmentRequestMP4);
					OutSegment = req;
					req->MediaAsset = SharedThis(this);
					req->PrimaryTrackIterator = TrackIt;
					check(firstTimestamp.IsValid());
					req->FirstPTS = firstTimestamp;
					req->PrimaryStreamType = EStreamType::Audio;
					req->FileStartOffset = firstByteOffset;
					req->FileEndOffset = -1;
					req->Bitrate = Repr->GetBitrate();
					req->bStartingOnMOOF = false;
					req->bIsFirstSegment = true;
					req->bIsLastSegment = true;
					req->SegmentDuration = GetDuration() - req->FirstPTS;
					// Set the PTS range for which to present samples.
					req->EarliestPTS = bFrameAccurateSearch ? StartPosition.Time : firstTimestamp;
					check(PlayRangeEnd.IsValid());
					req->LastPTS = PlayRangeEnd;
					req->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();

					// Add dependent stream types.
					// In case the video stream is shorter than audio we still need to add it as a dependent stream
					// (if it exists) in case the video will loop back to a point where there is video.
					if (VideoAdaptationSets.Num())
					{
						req->DependentStreamTypes.Add(EStreamType::Video);
					}
					if (SubtitleAdaptationSets.Num())
					{
						req->DependentStreamTypes.Add(EStreamType::Subtitle);
					}
					req->DrmClient = DrmClient;

					LimitSegmentDownloadSize(OutSegment);
					if (req->SegmentInternalSize > 0)
					{
						return IManifest::FResult(IManifest::FResult::EType::Found);
					}
					else
					{
						err = UEMEDIA_ERROR_END_OF_STREAM;
					}
				}
				if (err == UEMEDIA_ERROR_END_OF_STREAM)
				{
					// Regardless of there being a video stream or not we return an EOS request.
					TSharedPtrTS<FStreamSegmentRequestMP4> req(new FStreamSegmentRequestMP4);
					OutSegment = req;
					req->MediaAsset = SharedThis(this);
					req->FirstPTS = firstTimestamp;
					req->PrimaryStreamType = EStreamType::Audio;
					req->Bitrate = Repr->GetBitrate();
					req->bStartingOnMOOF = false;
					req->bIsFirstSegment = false;
					req->bIsLastSegment = true;
					req->bAllTracksAtEOS = true;
					// Set the PTS range for which to present samples.
					req->EarliestPTS = bFrameAccurateSearch ? StartPosition.Time : firstTimestamp;
					req->LastPTS = PlayRangeEnd;
					req->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
					// But if there is a video track we add it as a dependent stream that is also at EOS.
					if (VideoAdaptationSets.Num())
					{
						req->DependentStreamTypes.Add(EStreamType::Video);
					}
					return IManifest::FResult(IManifest::FResult::EType::Found);
				}
				else if (err == UEMEDIA_ERROR_INSUFFICIENT_DATA)
				{
					return IManifest::FResult(IManifest::FResult::EType::BeforeStart);
				}
				else
				{
					IManifest::FResult res(IManifest::FResult::EType::NotFound);
					res.SetErrorDetail(FErrorDetail().SetError(err)
									   .SetFacility(Facility::EFacility::MP4Playlist)
									   .SetCode(ERRCODE_MANIFEST_MP4_STARTSEGMENT_NOT_FOUND)
									   .SetMessage(FString::Printf(TEXT("Could not find start segment for time %lld"), (long long int)StartPosition.Time.GetAsHNS())));
					return res;
				}
			}
		}
	}

	return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetError(UEMEDIA_ERROR_INSUFFICIENT_DATA)
					   .SetFacility(Facility::EFacility::MP4Playlist)
					   .SetCode(ERRCODE_MANIFEST_MP4_STARTSEGMENT_NOT_FOUND)
					   .SetMessage(FString::Printf(TEXT("Could not find start segment for time %lld, no valid tracks"), (long long int)StartPosition.Time.GetAsHNS())));
}

IManifest::FResult FManifestMP4Internal::FTimelineAssetMP4::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options)
{
	const FStreamSegmentRequestMP4* Request = static_cast<const FStreamSegmentRequestMP4*>(CurrentSegment.Get());
	if (Request)
	{
		// Check if the current request did not already go up to the end of the stream. If so there is no next segment.
		if (Request->FileEndOffset >= 0)
		{
			FPlayStartPosition dummyPos;
			FPlayerSequenceState seqState;
			dummyPos.Options = Options;
			seqState.SetSequenceIndex(Request->TimestampSequenceIndex);
			IManifest::FResult res = GetStartingSegment(OutSegment, seqState, dummyPos, ESearchType::Same, Request->FileEndOffset + 1);
			if (res.GetType() == IManifest::FResult::EType::Found)
			{
				FStreamSegmentRequestMP4* NextRequest = static_cast<FStreamSegmentRequestMP4*>(OutSegment.Get());
				NextRequest->bIsFirstSegment = false;
				NextRequest->EarliestPTS = Request->EarliestPTS;
				NextRequest->LastPTS = Request->LastPTS;
				if (NextRequest->SegmentInternalSize > 0)
				{
					return res;
				}
			}
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}

IManifest::FResult FManifestMP4Internal::FTimelineAssetMP4::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData)
{
	const FStreamSegmentRequestMP4* Request = static_cast<const FStreamSegmentRequestMP4*>(CurrentSegment.Get());
	if (Request)
	{
		FPlayStartPosition dummyPos;
		FPlayerSequenceState seqState;
		dummyPos.Options = Options;
		seqState.SetSequenceIndex(Request->TimestampSequenceIndex);
		IManifest::FResult res = GetStartingSegment(OutSegment, seqState, dummyPos, ESearchType::Same, Request->CurrentIteratorBytePos);
		if (res.GetType() == IManifest::FResult::EType::Found)
		{
			FStreamSegmentRequestMP4* RetryRequest = static_cast<FStreamSegmentRequestMP4*>(OutSegment.Get());
			RetryRequest->NumOverallRetries = Request->NumOverallRetries + 1;
			RetryRequest->EarliestPTS = Request->EarliestPTS;
			RetryRequest->LastPTS = Request->LastPTS;
			return res;
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::NotFound);
}


IManifest::FResult FManifestMP4Internal::FTimelineAssetMP4::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	return GetStartingSegment(OutSegment, SequenceState, StartPosition, SearchType, -1);
}


void FManifestMP4Internal::FTimelineAssetMP4::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	// This is not expected to be called. And if it does we return a dummy entry.
	OutAverageSegmentDuration.SetFromSeconds(5.0);
}

TSharedPtrTS<IParserISO14496_12> FManifestMP4Internal::FTimelineAssetMP4::GetMoovBoxParser()
{
	return MoovBoxParser;
}



FErrorDetail FManifestMP4Internal::FAdaptationSetMP4::CreateFrom(const IParserISO14496_12::ITrack* InTrack, const FString& URL)
{
	Representation = MakeSharedTS<FRepresentationMP4>();
	FErrorDetail err;
	err = Representation->CreateFrom(InTrack, URL);
	if (err.IsOK())
	{
		CodecRFC6381	 = Representation->GetCodecInformation().GetCodecSpecifierRFC6381();
		UniqueIdentifier = Representation->GetUniqueIdentifier();
		LanguageTag		 = InTrack->GetLanguageTag();
	}
	return err;
}


FErrorDetail FManifestMP4Internal::FRepresentationMP4::CreateFrom(const IParserISO14496_12::ITrack* InTrack, const FString& URL)
{
	CodecInformation	 = InTrack->GetCodecInformation();
	// Get the CSD
	CodecSpecificData    = InTrack->GetCodecSpecificData();
	CodecSpecificDataRAW = InTrack->GetCodecSpecificDataRAW();

	// The unique identifier will be the track ID inside the mp4.
	// NOTE: This *MUST* be just a number since it gets parsed back out from a string into a number later! Do *NOT* prepend/append any string literals!!
	UniqueIdentifier = LexToString(InTrack->GetID());

	Name = InTrack->GetName();
	if (Name.Len() == 0)
	{
		Name = FString::Printf(TEXT("%s (ID=%u)"), *InTrack->GetNameFromHandler(), InTrack->GetID());
	}

	// Get bitrate from the average or max bitrate as stored in the track. If not stored it will be 0.
	Bitrate = InTrack->GetBitrateInfo().MaxBitrate ? InTrack->GetBitrateInfo().MaxBitrate : InTrack->GetBitrateInfo().AvgBitrate;

	// With no bitrate available we set some defaults. This is mainly to avoid a bitrate of 0 from being surfaced that would prevent
	// events like the initial bitrate change that needs to transition away from 0 to something real.
	if (Bitrate == 0)
	{
		switch(CodecInformation.GetStreamType())
		{
			case EStreamType::Video:
				Bitrate = 1 * 1024 * 1024;
				break;
			case EStreamType::Audio:
				Bitrate = 64 * 1024;
				break;
			case EStreamType::Subtitle:
				Bitrate = 8 * 1024;
				break;
			default:
				// Whatever it is, assume it's a low bitrate.
				Bitrate = 32 * 1024;
				break;
		}
	}
	if (!CodecInformation.GetBitrate())
	{
		CodecInformation.SetBitrate(Bitrate);
	}

	// Not a whole lot that could have gone wrong here.
	return FErrorDetail();
}


} // namespace Electra


