// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{
namespace MP4
{

const FName OptionKeyMP4LoadConnectTimeout(TEXT("mp4_connection_timeout"));				//!< (FTimeValue) value specifying connection timeout fetching the MP4
const FName OptionKeyMP4LoadNoDataTimeout(TEXT("mp4_nodata_timeout"));					//!< (FTimeValue) value specifying no-data timeout fetching the MP4

}

} // namespace Electra


