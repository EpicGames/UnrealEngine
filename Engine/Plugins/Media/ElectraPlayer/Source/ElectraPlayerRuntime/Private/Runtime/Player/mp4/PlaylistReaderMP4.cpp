// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "HTTP/HTTPResponseCache.h"
#include "MP4Utilities.h"
#include "Demuxer/ParserISO14496-12.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMP4.h"
#include "Utilities/HashFunctions.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/UtilsMP4.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/PlayerStreamReader.h"
#include "Player/mp4/ManifestMP4.h"
#include "Player/mp4/OptionKeynamesMP4.h"

#define ERRCODE_MP4_INVALID_FILE	1
#define ERRCODE_MP4_DOWNLOAD_ERROR	2


DECLARE_CYCLE_STAT(TEXT("FPlaylistReaderMP4_WorkerThread"), STAT_ElectraPlayer_MP4_PlaylistWorker, STATGROUP_ElectraPlayer);


namespace Electra
{

/**
 * This class is responsible for downloading the mp4 non-mdat boxes and parsing them.
 */
class FPlaylistReaderMP4 : public IPlaylistReaderMP4, IGenericDataReader, public IParserISO14496_12::IBoxCallback, public FMediaThread
{
public:
	FPlaylistReaderMP4();
	void Initialize(IPlayerSessionServices* PlayerSessionServices);

	virtual ~FPlaylistReaderMP4();

	EMediaFormatType GetMediaFormatType() const override
	{ return EMediaFormatType::ISOBMFF; }
	void Close() override;
	void HandleOnce() override;

	/**
	 * Returns the type of playlist format.
	 * For this implementation it will be "mp4".
	 *
	 * @return "mp4" to indicate this is an mp4 file.
	 */
	const FString& GetPlaylistType() const override
	{
		static FString Type("mp4");
		return Type;
	}

	/**
	 * Loads and parses the playlist.
	 *
	 * @param URL URL of the playlist to load
	 * @param InReplayEventParams URL fragments pertaining to replay event which have been removed from the source URL
	 */
	void LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams) override;

	/**
	 * Returns the URL from which the playlist was loaded (or supposed to be loaded).
	 *
	 * @return The playlist URL
	 */
	FString GetURL() const override;

	/**
	 * Returns an interface to the manifest created from the loaded mp4 playlists.
	 *
	 * @return A shared manifest interface pointer.
	 */
	TSharedPtrTS<IManifest> GetManifest() override;

private:
	// Methods from IGenericDataReader
	int64 ReadData(void* IntoBuffer, int64 NumBytesToRead, int64 InFromOffset) override;
	bool HasReachedEOF() const override;
	bool HasReadBeenAborted() const override;
	int64 GetCurrentOffset() const override;
	int64 GetTotalSize() const override;
	// Methods from IParserISO14496_12::IBoxCallback
	IParserISO14496_12::IBoxCallback::EParseContinuation OnFoundBox(IParserISO14496_12::FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset) override;
	IParserISO14496_12::IBoxCallback::EParseContinuation OnEndOfBox(IParserISO14496_12::FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset) override;

	void StartWorkerThread();
	void StopWorkerThread();
	void WorkerThread();

	void PostError(const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	FString MainPlaylistURL;
	FString URLFragment;
	FMediaEvent WorkerThreadQuitSignal;
	bool bIsWorkerThreadStarted = false;

	HTTP::FConnectionInfo ConnectionInfo;

	TSharedPtrTS<IParserISO14496_12> MP4Parser;
	TSharedPtrTS<FWaitableBuffer> ParseBuffer;
	int64 ParsePos = 0;
	int64 ParseBufferSize = 0;

	volatile bool bAbort = false;

	TSharedPtrTS<FManifestMP4Internal> Manifest;
	FErrorDetail LastErrorDetail;
};


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtrTS<IPlaylistReader> IPlaylistReaderMP4::Create(IPlayerSessionServices* PlayerSessionServices)
{
	TSharedPtrTS<FPlaylistReaderMP4> PlaylistReader = MakeSharedTS<FPlaylistReaderMP4>();
	if (PlaylistReader)
	{
		PlaylistReader->Initialize(PlayerSessionServices);
	}
	return PlaylistReader;
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


FPlaylistReaderMP4::FPlaylistReaderMP4()
	: FMediaThread("ElectraPlayer::MP4 Playlist")
{
}

FPlaylistReaderMP4::~FPlaylistReaderMP4()
{
	Close();
}

FString FPlaylistReaderMP4::GetURL() const
{
	return MainPlaylistURL;
}

TSharedPtrTS<IManifest> FPlaylistReaderMP4::GetManifest()
{
	return Manifest;
}

void FPlaylistReaderMP4::Initialize(IPlayerSessionServices* InPlayerSessionServices)
{
	PlayerSessionServices = InPlayerSessionServices;
}

void FPlaylistReaderMP4::Close()
{
	bAbort = true;
	StopWorkerThread();
}

void FPlaylistReaderMP4::HandleOnce()
{
	// No-op. This class is using a dedicated thread to read data from the stream
	// which can stall at any moment and thus not lend itself to a tickable instance.
}

void FPlaylistReaderMP4::StartWorkerThread()
{
	check(!bIsWorkerThreadStarted);
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FPlaylistReaderMP4::WorkerThread));
	bIsWorkerThreadStarted = true;
}

void FPlaylistReaderMP4::StopWorkerThread()
{
	if (bIsWorkerThreadStarted)
	{
		WorkerThreadQuitSignal.Signal();
		ThreadWaitDone();
		ThreadReset();
		bIsWorkerThreadStarted = false;
	}
}

void FPlaylistReaderMP4::PostError(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	LastErrorDetail.Clear();
	LastErrorDetail.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	LastErrorDetail.SetFacility(Facility::EFacility::MP4PlaylistReader);
	LastErrorDetail.SetCode(InCode);
	LastErrorDetail.SetMessage(InMessage);
	check(PlayerSessionServices);
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostError(LastErrorDetail);
	}
}

void FPlaylistReaderMP4::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::MP4PlaylistReader, Level, Message);
	}
}

void FPlaylistReaderMP4::LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams)
{
	FURL_RFC3986 UrlParser;
	UrlParser.Parse(InURL);
	MainPlaylistURL = UrlParser.Get(true, false);
	URLFragment = UrlParser.GetFragment();

	StartWorkerThread();
}


void FPlaylistReaderMP4::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MP4_PlaylistWorker);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, MP4_PlaylistWorker);

	const TArray<uint32> StopAfterBoxes { MP4Utilities::MakeBoxAtom('m','o','o','v') };
	const TArray<uint32> ReadBoxes { MP4Utilities::MakeBoxAtom('f','t','y','p'), MP4Utilities::MakeBoxAtom('m','o','o','v') };
	const TArray<uint32> FirstBoxes { MP4Utilities::MakeBoxAtom('f','t','y','p'), MP4Utilities::MakeBoxAtom('s','t','y','p'), MP4Utilities::MakeBoxAtom('s','i','d','x'), MP4Utilities::MakeBoxAtom('f','r','e','e'), MP4Utilities::MakeBoxAtom('s','k','i','p') };
	UtilsMP4::FMP4RootBoxLocator BoxLocator;
	TArray<UtilsMP4::FMP4RootBoxLocator::FBoxInfo> BoxInfos;
	TArray<HTTP::FHTTPHeader> ReqHeaders;
	bool bGotBoxes = BoxLocator.LocateRootBoxes(BoxInfos, PlayerSessionServices->GetHTTPManager(), MainPlaylistURL, ReqHeaders, FirstBoxes, StopAfterBoxes, ReadBoxes, UtilsMP4::FMP4RootBoxLocator::FCancellationCheckDelegate::CreateLambda([&]()
	{
		return bAbort;
	}));
	bool bHasErrored = BoxLocator.DidDownloadFail();
	ConnectionInfo = BoxLocator.GetConnectionInfo();
	if (!BoxLocator.GetErrorMessage().IsEmpty())
	{
		PostError(BoxLocator.GetErrorMessage(), ERRCODE_MP4_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
	}

	// There are currently no retries, so this is the first (and only) attempt we make.
	const int32 Attempt = 1;
	// Notify the download of the "main playlist". This indicates the download only, not the parsing thereof.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistDownloadMessage::Create(&ConnectionInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempt));
	// Notify that the "main playlist" has been parsed, successfully or not.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ConnectionInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempt));
	// Failed to get the boxes but was not aborted?
	if (!bAbort && (!bGotBoxes || bHasErrored))
	{
		// See if there was a download error
		if (ConnectionInfo.StatusInfo.ErrorDetail.IsError())
		{
			PostError(FString::Printf(TEXT("%s while downloading \"%s\""), *ConnectionInfo.StatusInfo.ErrorDetail.GetMessage(), *ConnectionInfo.EffectiveURL), ERRCODE_MP4_DOWNLOAD_ERROR, UEMEDIA_ERROR_READ_ERROR);
		}
	}
	else if (bGotBoxes && !bAbort && !bHasErrored)
	{
		// Do we have the `ftyp` and `moov` boxes?
		if (BoxInfos.ContainsByPredicate([InType=MP4Utilities::MakeBoxAtom('f','t','y','p')](const UtilsMP4::FMP4RootBoxLocator::FBoxInfo& InBox){return InBox.Type == InType;}) &&
			BoxInfos.ContainsByPredicate([InType=MP4Utilities::MakeBoxAtom('m','o','o','v')](const UtilsMP4::FMP4RootBoxLocator::FBoxInfo& InBox){return InBox.Type == InType;}))
		{
			if (BoxInfos.ContainsByPredicate([InType=MP4Utilities::MakeBoxAtom('m','d','a','t')](const UtilsMP4::FMP4RootBoxLocator::FBoxInfo& InBox){return InBox.Type == InType;}))
			{
				LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("The mp4 at \"%s\" is not fast-startable. Consider moving the 'moov' box in front of the 'mdat' for faster startup times."), *ConnectionInfo.EffectiveURL));
			}

			MP4Parser = IParserISO14496_12::CreateParser();

			// First parse the ftyp on which some elements in the moov depend.
			ParseBuffer = BoxInfos.FindByPredicate([InType=MP4Utilities::MakeBoxAtom('f','t','y','p')](const UtilsMP4::FMP4RootBoxLocator::FBoxInfo& InBox){return InBox.Type == InType;})->DataBuffer;
			ParsePos = 0;
			ParseBufferSize = ParseBuffer->Num();
			UEMediaError parseError = MP4Parser->ParseHeader(this, this, PlayerSessionServices, nullptr);
			// If successful parse the moov
			if (parseError == UEMEDIA_ERROR_OK || parseError == UEMEDIA_ERROR_END_OF_STREAM)
			{
				ParseBuffer = BoxInfos.FindByPredicate([InType=MP4Utilities::MakeBoxAtom('m','o','o','v')](const UtilsMP4::FMP4RootBoxLocator::FBoxInfo& InBox){return InBox.Type == InType;})->DataBuffer;
				ParsePos = 0;
				ParseBufferSize = ParseBuffer->Num();
				parseError = MP4Parser->ParseHeader(this, this, PlayerSessionServices, nullptr);
			}
			if (parseError == UEMEDIA_ERROR_OK || parseError == UEMEDIA_ERROR_END_OF_STREAM)
			{
				// Prepare the tracks in the stream that are of a supported codec.
				parseError = MP4Parser->PrepareTracks(PlayerSessionServices, TSharedPtrTS<const IParserISO14496_12>());
				if (parseError == UEMEDIA_ERROR_OK)
				{
					MP4Parser->ResolveTimecodeTracks(PlayerSessionServices, IParserISO14496_12::FCancellationCheckDelegate::CreateLambda([&]()
					{
						return bAbort;
					}));

					Manifest = MakeSharedTS<FManifestMP4Internal>(PlayerSessionServices);

					TArray<FURL_RFC3986::FQueryParam> URLFragmentComponents;
					FURL_RFC3986::GetQueryParams(URLFragmentComponents, URLFragment, false);	// The fragment is already URL escaped, so no need to do it again.
					Manifest->SetURLFragmentComponents(MoveTemp(URLFragmentComponents));

					LastErrorDetail = Manifest->Build(MP4Parser, MainPlaylistURL, ConnectionInfo);

					// Let the external registry know that we have no properties with an end-of-properties call.
					PlayerSessionServices->ValidateMainPlaylistCustomProperty(GetPlaylistType(), MainPlaylistURL, TArray<FElectraHTTPStreamHeader>(), IPlayerSessionServices::FPlaylistProperty());

					// Notify that the "variant playlists" are ready. There are no variants in an mp4, but this is the trigger that the playlists are all set up and are good to go now.
					PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ConnectionInfo, Playlist::EListType::Variant, Playlist::ELoadType::Initial, Attempt));
				}
				else
				{
					PostError(FString::Printf(TEXT("Failed to parse tracks in mp4 \"%s\" with error %u"), *ConnectionInfo.EffectiveURL, parseError), ERRCODE_MP4_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
				}
			}
			else
			{
				PostError(FString::Printf(TEXT("Failed to parse mp4 \"%s\" with error %u"), *ConnectionInfo.EffectiveURL, parseError), ERRCODE_MP4_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
			}
		}
		else
		{
			// No moov box usually means this is not a fast-start file.
			PostError(FString::Printf(TEXT("No ftyp or moov box found in \"%s\". This is not a valid file."), *ConnectionInfo.EffectiveURL), ERRCODE_MP4_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
		}
	}

	// This thread's work is done. We only wait for termination now.
	WorkerThreadQuitSignal.Wait();
}






/**
 * Read n bytes of data into the provided buffer.
 *
 * Reading must return the number of bytes asked to get, if necessary by blocking.
 * If a read error prevents reading the number of bytes -1 must be returned.
 *
 * @param IntoBuffer Buffer into which to store the data bytes. If nullptr is passed the data must be skipped over.
 * @param NumBytesToRead The number of bytes to read. Must not read more bytes and no less than requested.
 * @return The number of bytes read or -1 on a read error.
 */
int64 FPlaylistReaderMP4::ReadData(void* IntoBuffer, int64 NumBytesToRead, int64 InFromOffset)
{
	// We have all the data available.
	check(ParseBuffer.IsValid());

	if (ParsePos >= ParseBufferSize)
	{
		return 0;
	}
	const uint8* Src = ParseBuffer->GetLinearReadData() + ParsePos;
	if (IntoBuffer)
	{
		FMemory::Memcpy(IntoBuffer, Src, NumBytesToRead);
	}
	ParsePos += NumBytesToRead;
	return NumBytesToRead;
}

/**
 * Checks if the data source has reached the End Of File (EOF) and cannot provide any additional data.
 *
 * @return If EOF has been reached returns true, otherwise false.
 */
bool FPlaylistReaderMP4::HasReachedEOF() const
{
	return ParsePos >= ParseBufferSize;
}

/**
 * Checks if reading of the file and therefor parsing has been aborted.
 *
 * @return true if reading/parsing has been aborted, false otherwise.
 */
bool FPlaylistReaderMP4::HasReadBeenAborted() const
{
	return bAbort;
}

/**
 * Returns the current read offset.
 *
 * The first read offset is not necessarily zero. It could be anywhere inside the source.
 *
 * @return The current byte offset in the source.
 */
int64 FPlaylistReaderMP4::GetCurrentOffset() const
{
	return ParsePos;
}

int64 FPlaylistReaderMP4::GetTotalSize() const
{
	check(!"this should not be called");
	return -1;
}


IParserISO14496_12::IBoxCallback::EParseContinuation FPlaylistReaderMP4::OnFoundBox(IParserISO14496_12::FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset)
{
	return IParserISO14496_12::IBoxCallback::EParseContinuation::Continue;
}

IParserISO14496_12::IBoxCallback::EParseContinuation FPlaylistReaderMP4::OnEndOfBox(IParserISO14496_12::FBoxType Box, int64 BoxSizeInBytes, int64 FileDataOffset, int64 BoxDataOffset)
{
	return IParserISO14496_12::IBoxCallback::EParseContinuation::Stop;
}

} // namespace Electra


