// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/PlaylistReader.h"
#include "Player/PlayerSessionServices.h"


namespace Electra
{

class IPlaylistReaderMP4 : public IPlaylistReader
{
public:
	static TSharedPtrTS<IPlaylistReader> Create(IPlayerSessionServices* PlayerSessionServices);

	virtual ~IPlaylistReaderMP4() = default;
};


} // namespace Electra


