// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserISO14496-12.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMP4.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/mp4/ManifestMP4.h"
#include "Player/mp4/StreamReaderMP4.h"
#include "Player/mp4/OptionKeynamesMP4.h"
#include "Player/PlayerSessionServices.h"
#include "Player/PlayerStreamFilter.h"


DECLARE_CYCLE_STAT(TEXT("FStreamReaderMP4_HandleRequest"), STAT_ElectraPlayer_MP4_StreamReader, STATGROUP_ElectraPlayer);


namespace Electra
{

FStreamSegmentRequestMP4::FStreamSegmentRequestMP4()
{
	PrimaryStreamType      = EStreamType::Video;
	FileStartOffset 	   = -1;
	FileEndOffset   	   = -1;
	SegmentInternalSize	   = -1;
	Bitrate 			   = 0;
	PlaybackSequenceID     = ~0U;
	bStartingOnMOOF 	   = false;
	bIsFirstSegment		   = false;
	bIsLastSegment		   = false;
	bAllTracksAtEOS 	   = false;
	TimestampSequenceIndex = 0;
	CurrentIteratorBytePos = 0;
	NumOverallRetries      = 0;
}

FStreamSegmentRequestMP4::~FStreamSegmentRequestMP4()
{
}

void FStreamSegmentRequestMP4::SetPlaybackSequenceID(uint32 InPlaybackSequenceID)
{
	PlaybackSequenceID = InPlaybackSequenceID;
}

uint32 FStreamSegmentRequestMP4::GetPlaybackSequenceID() const
{
	return PlaybackSequenceID;
}

EStreamType FStreamSegmentRequestMP4::GetType() const
{
	return  PrimaryStreamType;
}

void FStreamSegmentRequestMP4::SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay)
{
	// No op for mp4.
}

FTimeValue FStreamSegmentRequestMP4::GetExecuteAtUTCTime() const
{
	// Right now.
	return FTimeValue::GetInvalid();
}

void FStreamSegmentRequestMP4::GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const
{
	// Those are not "real" dependent streams in that they are multiplexed and do not need to be fetched from
	// a different source. This merely indicates the types of non-primary streams we will be demuxing.
	for(int32 i=0; i<DependentStreamTypes.Num(); ++i)
	{
		FStreamSegmentRequestMP4* DepReq = new FStreamSegmentRequestMP4(*this);
		DepReq->PrimaryStreamType = DependentStreamTypes[i];
		DepReq->DependentStreamTypes.Empty();
		TSharedPtrTS<IStreamSegment> p(DepReq);
		OutDependentStreams.Push(p);
	}
}

void FStreamSegmentRequestMP4::GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams)
{
	OutRequestedStreams.Empty();
	OutRequestedStreams.Emplace(SharedThis(this));
}


void FStreamSegmentRequestMP4::GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams)
{
	OutAlreadyEndedStreams.Empty();
	if (bAllTracksAtEOS)
	{
		OutAlreadyEndedStreams.Push(SharedThis(this));
		for(int32 i=0; i<DependentStreamTypes.Num(); ++i)
		{
			FStreamSegmentRequestMP4* DepReq = new FStreamSegmentRequestMP4;
			// Only need to set the stream type here.
			DepReq->PrimaryStreamType = DependentStreamTypes[i];
			TSharedPtrTS<IStreamSegment> p(DepReq);
			OutAlreadyEndedStreams.Push(p);
		}
	}
}

FTimeValue FStreamSegmentRequestMP4::GetFirstPTS() const
{
	FTimeValue First(EarliestPTS > FirstPTS ? EarliestPTS : FirstPTS);
	First.SetSequenceIndex(TimestampSequenceIndex);
	return First;
}

FTimeRange FStreamSegmentRequestMP4::GetTimeRange() const
{
	FTimeRange tr;
	tr.Start = FirstPTS;
	tr.End = FirstPTS + SegmentDuration;
	tr.Start.SetSequenceIndex(TimestampSequenceIndex);
	tr.End.SetSequenceIndex(TimestampSequenceIndex);
	return tr;
}

int32 FStreamSegmentRequestMP4::GetQualityIndex() const
{
	// No quality choice here.
	return 0;
}

int32 FStreamSegmentRequestMP4::GetBitrate() const
{
	return Bitrate;
}

void FStreamSegmentRequestMP4::GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const
{
	OutStats = DownloadStats;
}






uint32	FStreamReaderMP4::UniqueDownloadID = 1;

FStreamReaderMP4::FStreamReaderMP4()
{
	bIsStarted  		  = false;
	bTerminate  		  = false;
	bRequestCanceled	  = false;
	bHasErrored 		  = false;
}

FStreamReaderMP4::~FStreamReaderMP4()
{
	Close();
}

UEMediaError FStreamReaderMP4::Create(IPlayerSessionServices* InPlayerSessionService, const CreateParam &InCreateParam)
{
	if (!InCreateParam.MemoryProvider || !InCreateParam.EventListener)
	{
		return UEMEDIA_ERROR_BAD_ARGUMENTS;
	}

	PlayerSessionServices = InPlayerSessionService;
	Parameters = InCreateParam;
	bTerminate = false;
	bIsStarted = true;

	ThreadSetName("ElectraPlayer::MP4 streamer");
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FStreamReaderMP4::WorkerThread));

	return UEMEDIA_ERROR_OK;
}

bool FStreamReaderMP4::LoadsMultiplexedTracksSequentially()
{
	return true;
}

void FStreamReaderMP4::Close()
{
	if (bIsStarted)
	{
		bIsStarted = false;

		TSharedPtrTS<FStreamSegmentRequestMP4>	Request = CurrentRequest;
		if (Request.IsValid())
		{
			// ...
		}
		CancelRequests();
		bTerminate = true;
		WorkSignal.Signal();
		ThreadWaitDone();
		ThreadReset();
		CurrentRequest.Reset();
	}
}


void FStreamReaderMP4::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	PlayerSessionServices->PostLog(Facility::EFacility::MP4StreamReader, Level, Message);
}

IStreamReader::EAddResult FStreamReaderMP4::AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> InRequest)
{
	TSharedPtrTS<FStreamSegmentRequestMP4>	Request = CurrentRequest;
	if (Request.IsValid())
	{
		check(!"why is the handler busy??");
		return IStreamReader::EAddResult::TryAgainLater;
	}
	Request = StaticCastSharedPtr<FStreamSegmentRequestMP4>(InRequest);
	Request->SetPlaybackSequenceID(CurrentPlaybackSequenceID);
	bRequestCanceled = false;
	bHasErrored = false;
	// Only add the request if it is not an all-EOS one!
	if (!Request->bAllTracksAtEOS)
	{
		CurrentRequest = Request;
		WorkSignal.Signal();
	}
	return EAddResult::Added;
}

void FStreamReaderMP4::CancelRequest(EStreamType StreamType, bool bSilent)
{
	// No-op.
}

void FStreamReaderMP4::CancelRequests()
{
	bRequestCanceled = true;
	ReadBuffer.Abort();
}

bool FStreamReaderMP4::HasBeenAborted() const
{
	return bRequestCanceled || ReadBuffer.bAbort;
}

bool FStreamReaderMP4::HasErrored() const
{
	return bHasErrored;
}

int32 FStreamReaderMP4::HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest)
{
	// Aborted?
	return HasBeenAborted() ? 1 : 0;
}

void FStreamReaderMP4::HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest)
{
	bHasErrored = InRequest->ConnectionInfo.StatusInfo.ErrorDetail.IsError();
	if (bHasErrored)
	{
		ReadBuffer.SetHasErrored();
	}
}

void FStreamReaderMP4::HandleRequest()
{
	TSharedPtrTS<FStreamSegmentRequestMP4>	Request = CurrentRequest;
	FString ParsingErrorMessage;

	FManifestMP4Internal::FTimelineAssetMP4* TimelineAsset = static_cast<FManifestMP4Internal::FTimelineAssetMP4*>(Request->MediaAsset.Get());

	// Clear the active track map.
	ActiveTrackMap.Reset();

	// Get the list of all the tracks that have been selected in the asset.
	// This does not mean their data will be _used_ for playback, only that the track is usable by the player
	// with regards to type and codec.
	struct FPlaylistTrackMetadata
	{
		EStreamType		Type;
		FString			Kind;
		BCP47::FLanguageTag LanguageTag;
		FString			PeriodID;
		FString			AdaptationSetID;
		FString			RepresentationID;
		int32			Bitrate;
		int32			Index;
	};
	TMap<uint32, FPlaylistTrackMetadata>	SelectedTrackMap;
	const EStreamType TypesOfSupportedTracks[] = { EStreamType::Video, EStreamType::Audio, EStreamType::Subtitle };
	{
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MP4_StreamReader);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, MP4_StreamReader);
		for(int32 nStrType=0; nStrType<UE_ARRAY_COUNT(TypesOfSupportedTracks); ++nStrType)
		{
			int32 NumAdapt = Request->MediaAsset->GetNumberOfAdaptationSets(TypesOfSupportedTracks[nStrType]);
			for(int32 nAdapt=0; nAdapt<NumAdapt; ++nAdapt)
			{
				TSharedPtrTS<IPlaybackAssetAdaptationSet> AdaptationSet = Request->MediaAsset->GetAdaptationSetByTypeAndIndex(TypesOfSupportedTracks[nStrType], nAdapt);
				const BCP47::FLanguageTag& LanguageTag = AdaptationSet->GetLanguageTag();
				FString AdaptID = AdaptationSet->GetUniqueIdentifier();
				int32 NumRepr = AdaptationSet->GetNumberOfRepresentations();
				for(int32 nRepr=0; nRepr<NumRepr; ++nRepr)
				{
					TSharedPtrTS<IPlaybackAssetRepresentation> Representation = AdaptationSet->GetRepresentationByIndex(nRepr);

					// Note: By definition the representations unique identifier is a string of the numeric track ID and can thus be parsed back into a number.
					FString ReprID = Representation->GetUniqueIdentifier();
					uint32 TrackId;
					LexFromString(TrackId, *ReprID);
					FPlaylistTrackMetadata tmd;
					tmd.Type			 = TypesOfSupportedTracks[nStrType];
					tmd.LanguageTag		 = LanguageTag;
					tmd.Kind			 = nRepr == 0 ? TEXT("main") : TEXT("translation");	// perhaps this should come from querying the metadata instead of doing this over here...
					tmd.PeriodID		 = Request->MediaAsset->GetUniqueIdentifier();
					tmd.AdaptationSetID  = AdaptID;
					tmd.RepresentationID = ReprID;
					tmd.Bitrate 		 = Representation->GetBitrate();
					tmd.Index			 = nAdapt;
					SelectedTrackMap.Emplace(TrackId, tmd);
				}
			}
		}
	}


	Metrics::FSegmentDownloadStats& ds = Request->DownloadStats;
	ds.StatsID = FMediaInterlockedIncrement(UniqueDownloadID);

	check(Request->PrimaryTrackIterator.IsValid() && Request->PrimaryTrackIterator->GetTrack());
	uint32 PrimaryTrackID = Request->PrimaryTrackIterator->GetTrack()->GetID();
	const FPlaylistTrackMetadata* PrimaryTrackMetadata = SelectedTrackMap.Find(PrimaryTrackID);
	check(PrimaryTrackMetadata);
	if (PrimaryTrackMetadata)
	{
		ds.MediaAssetID 	= PrimaryTrackMetadata->PeriodID;
		ds.AdaptationSetID  = PrimaryTrackMetadata->AdaptationSetID;
		ds.RepresentationID = PrimaryTrackMetadata->RepresentationID;
		ds.Bitrate  		= PrimaryTrackMetadata->Bitrate;
	}

	ds.FailureReason.Empty();
	ds.bWasSuccessful      = true;
	ds.bWasAborted  	   = false;
	ds.bDidTimeout  	   = false;
	ds.HTTPStatusCode      = 0;
	ds.StreamType   	   = Request->GetType();
	ds.SegmentType  	   = Metrics::ESegmentType::Media;
	ds.PresentationTime    = Request->FirstPTS.GetAsSeconds();
	ds.Duration 		   = Request->SegmentDuration.GetAsSeconds();
	ds.DurationDownloaded  = 0.0;
	ds.DurationDelivered   = 0.0;
	ds.TimeToFirstByte     = 0.0;
	ds.TimeToDownload      = 0.0;
	ds.ByteSize 		   = -1;
	ds.NumBytesDownloaded  = 0;
	ds.bInsertedFillerData = false;
	ds.URL.URL 			   = TimelineAsset->GetMediaURL();
	ds.bIsMissingSegment   = false;
	ds.bParseFailure	   = false;
	ds.RetryNumber  	   = Request->NumOverallRetries;

	Parameters.EventListener->OnFragmentOpen(Request);

	TSharedPtrTS<IElectraHttpManager::FProgressListener>	ProgressListener;
	ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateRaw(this, &FStreamReaderMP4::HTTPCompletionCallback);
	ProgressListener->ProgressDelegate   = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateRaw(this, &FStreamReaderMP4::HTTPProgressCallback);

	ReadBuffer.Reset();
	ReadBuffer.ReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
	ReadBuffer.SetCurrentPos(Request->FileStartOffset);

	TSharedPtrTS<IElectraHttpManager::FRequest> HTTP(new IElectraHttpManager::FRequest);
	HTTP->Parameters.URL				= TimelineAsset->GetMediaURL();
	HTTP->Parameters.Range.Start		= Request->FileStartOffset;
	HTTP->Parameters.Range.EndIncluding = Request->FileEndOffset;
	// No compression as this would not yield much with already compressed video/audio data.
	HTTP->Parameters.AcceptEncoding.Set(TEXT("identity"));
	// Timeouts
	HTTP->Parameters.ConnectTimeout = PlayerSessionServices->GetOptionValue(MP4::OptionKeyMP4LoadConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 8));
	HTTP->Parameters.NoDataTimeout = PlayerSessionServices->GetOptionValue(MP4::OptionKeyMP4LoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 6));

	// Explicit range?
	int64 NumRequestedBytes = HTTP->Parameters.Range.GetNumberOfBytes();

	HTTP->ReceiveBuffer = ReadBuffer.ReceiveBuffer;
	HTTP->ProgressListener = ProgressListener;
	HTTP->ResponseCache = PlayerSessionServices->GetHTTPResponseCache();
	PlayerSessionServices->GetHTTPManager()->AddRequest(HTTP, false);


	FTimeValue DurationSuccessfullyDelivered(FTimeValue::GetZero());
	FTimeValue DurationSuccessfullyRead(FTimeValue::GetZero());
	bool bDone = false;
	TSharedPtrTS<IParserISO14496_12::IAllTrackIterator> AllTrackIterator;
	{
		SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MP4_StreamReader);
		CSV_SCOPED_TIMING_STAT(ElectraPlayer, MP4_StreamReader);
		AllTrackIterator = TimelineAsset->GetMoovBoxParser()->CreateAllTrackIteratorByFilePos(Request->FileStartOffset);
	}

	// Encrypted?
	TSharedPtr<ElectraCDM::IMediaCDMDecrypter, ESPMode::ThreadSafe> Decrypter;
	if (Request->DrmClient.IsValid())
	{
		if (Request->DrmClient->CreateDecrypter(Decrypter, FString()) != ElectraCDM::ECDMError::Success)
		{
			bDone = true;
			bHasErrored = true;
			ds.FailureReason = FString::Printf(TEXT("Failed to create decrypter for segment. %s"), *Request->DrmClient->GetLastErrorMessage());
			LogMessage(IInfoLog::ELevel::Error, ds.FailureReason);
		}
	}

	uint32 PlaybackSequenceID = Request->GetPlaybackSequenceID();
	while(!bDone && !HasErrored() && !HasBeenAborted() && !bTerminate)
	{
		auto UpdateSelectedTrack = [&SelectedTrackMap, &PlaybackSequenceID](const IParserISO14496_12::ITrackIterator* trkIt, TMap<uint32, FSelectedTrackData>& ActiveTrks) -> FSelectedTrackData&
		{
			const IParserISO14496_12::ITrack* Track = trkIt->GetTrack();
			uint32 tkid = Track->GetID();

			// Check if this track ID is already in our map of active tracks.
			FSelectedTrackData& st = ActiveTrks.FindOrAdd(tkid);
			if (!st.CSD.IsValid())
			{
				TSharedPtrTS<FAccessUnit::CodecData> CSD(new FAccessUnit::CodecData);
				CSD->CodecSpecificData = Track->GetCodecSpecificData();
				CSD->RawCSD			   = Track->GetCodecSpecificDataRAW();
				CSD->ParsedInfo		   = Track->GetCodecInformation();
				st.CSD = MoveTemp(CSD);
			}
			if (!st.BufferSourceInfo.IsValid())
			{
				auto meta = MakeShared<FBufferSourceInfo, ESPMode::ThreadSafe>();

				// Check if this track is in the list of selected tracks.
				const FPlaylistTrackMetadata* SelectedTrackMetadata = SelectedTrackMap.Find(tkid);
				if (SelectedTrackMetadata)
				{
					st.bIsSelectedTrack = true;
					st.StreamType = SelectedTrackMetadata->Type;
					st.Bitrate = SelectedTrackMetadata->Bitrate;
					if (st.CSD.IsValid())
					{
						st.CSD->ParsedInfo.SetBitrate(st.Bitrate);
					}
					meta->Kind = SelectedTrackMetadata->Kind;
					meta->LanguageTag = SelectedTrackMetadata->LanguageTag;
					meta->Codec = st.CSD.IsValid() ? st.CSD->ParsedInfo.GetCodecName() : FString();
					meta->PeriodID = SelectedTrackMetadata->PeriodID;
					meta->PeriodAdaptationSetID = SelectedTrackMetadata->PeriodID + TEXT(".") + SelectedTrackMetadata->AdaptationSetID;
					meta->HardIndex = SelectedTrackMetadata->Index;
					meta->PlaybackSequenceID = PlaybackSequenceID;
				}
				st.BufferSourceInfo = MoveTemp(meta);
			}
			return st;
		};

		// Handle all the new tracks that have reached EOS while iterating. We do this first here to
		// handle the tracks that hit EOS before reaching the intended start position.
		TArray<const IParserISO14496_12::ITrackIterator*> TracksAtEOS;
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MP4_StreamReader);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, MP4_StreamReader);
			AllTrackIterator->GetNewEOSTracks(TracksAtEOS);
			AllTrackIterator->ClearNewEOSTracks();
			for(int32 nTrk=0; nTrk<TracksAtEOS.Num(); ++nTrk)
			{
				const IParserISO14496_12::ITrackIterator* TrackIt = TracksAtEOS[nTrk];
				FSelectedTrackData& SelectedTrack = UpdateSelectedTrack(TrackIt, ActiveTrackMap);
				// Is this a track that is selected and we are interested in?
				if (SelectedTrack.bIsSelectedTrack)
				{
					Parameters.EventListener->OnFragmentReachedEOS(SelectedTrack.StreamType, SelectedTrack.BufferSourceInfo);
				}
			}
		}

		// Handle current track iterator
		const IParserISO14496_12::ITrackIterator* TrackIt = AllTrackIterator->Current();
		if (TrackIt)
		{
			FSelectedTrackData& SelectedTrack = UpdateSelectedTrack(TrackIt, ActiveTrackMap);

			// Get the sample properties
			uint32 SampleNumber    = TrackIt->GetSampleNumber();
			int64 DTS   		   = TrackIt->GetDTS();
			int64 PTS   		   = TrackIt->GetPTS();
			int64 Duration  	   = TrackIt->GetDuration();
			uint32 Timescale	   = TrackIt->GetTimescale();
			bool bIsSyncSample     = TrackIt->IsSyncSample();
			int64 SampleSize	   = TrackIt->GetSampleSize();
			int64 SampleFileOffset = TrackIt->GetSampleFileOffset();

			// Remember at which file position we are currently at. In case of failure this is where we will retry.
			Request->CurrentIteratorBytePos = SampleFileOffset;

			// Do we need to skip over some data?
			if (SampleFileOffset > ReadBuffer.GetCurrentPos())
			{
				int32 NumBytesToSkip = SampleFileOffset - ReadBuffer.GetCurrentPos();
				int64 nr = ReadBuffer.ReadTo(nullptr, NumBytesToSkip);
				if (nr != NumBytesToSkip)
				{
					bDone = true;
					break;
				}
			}
			else if (SampleFileOffset < ReadBuffer.GetCurrentPos())
			{
				ds.bParseFailure  = true;
				ParsingErrorMessage = FString::Printf(TEXT("Segment parse error. Sample offset %lld for sample #%u in track %u is before the current read position at %lld"), (long long int)SampleFileOffset, SampleNumber, TrackIt->GetTrack()->GetID(), (long long int)ReadBuffer.GetCurrentPos());
				bHasErrored = true;
				break;
			}

			// Do we read the sample because the track is selected or do we discard it?
			if (SelectedTrack.bIsSelectedTrack)
			{
				// Create an access unit.
				FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
				if (AccessUnit)
				{
					AccessUnit->ESType = SelectedTrack.StreamType;
					AccessUnit->PTS.SetFromND(PTS, Timescale);
					AccessUnit->DTS.SetFromND(DTS, Timescale);
					AccessUnit->Duration.SetFromND(Duration, Timescale);

					AccessUnit->EarliestPTS = Request->EarliestPTS;
					AccessUnit->LatestPTS = Request->LastPTS;

					AccessUnit->AUSize = (uint32) SampleSize;
					AccessUnit->AUCodecData = SelectedTrack.CSD;
					// Set the sequence index member and update all timestamps with it as well.
					AccessUnit->SequenceIndex = Request->TimestampSequenceIndex;
					AccessUnit->DTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->PTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->EarliestPTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->LatestPTS.SetSequenceIndex(Request->TimestampSequenceIndex);

					AccessUnit->bIsFirstInSequence = SelectedTrack.bIsFirstInSequence;
					AccessUnit->bIsSyncSample = bIsSyncSample;
					AccessUnit->bIsDummyData = false;
					AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);

					// Set the associated stream metadata
					AccessUnit->BufferSourceInfo = SelectedTrack.BufferSourceInfo;

					SelectedTrack.bIsFirstInSequence = false;

					int64 nr = ReadBuffer.ReadTo(AccessUnit->AUData, SampleSize);
					if (nr == SampleSize)
					{
						SelectedTrack.DurationSuccessfullyRead += AccessUnit->Duration;

						//LogMessage(IInfoLog::ELevel::Info, FString::Printf("[%u] %4u: DTS=%lld PTS=%lld dur=%lld sync=%d; %lld bytes @ %lld", tkid, SampleNumber, (long long int)AccessUnit->mDTS.GetAsMicroseconds(), (long long int)AccessUnit->mPTS.GetAsMicroseconds(), (long long int)AccessUnit->mDuration.GetAsMicroseconds(), bIsSyncSample?1:0, (long long int)SampleSize, (long long int)SampleFileOffset));

						bool bSentOff = false;


						// Check if the AU is outside the time range we are allowed to read.
						// The last one (the one that is already outside the range, actually) is tagged as such and sent into the buffer.
						// The respective decoder has to handle this flag if necessary and/or drop the AU.
						// We need to send at least one AU down so the FMultiTrackAccessUnitBuffer does not stay empty for this period!
						// Already sent the last one?
						if (SelectedTrack.bReadPastLastPTS)
						{
							// Yes. Release this AU and do not forward it. Continue reading however.
							bSentOff = true;
							// Since we have skipped this access unit, if we are detecting an error now we need to then
							// retry on the _next_ AU and not this one again!
							Request->CurrentIteratorBytePos = SampleFileOffset + SampleSize;
						}
						else if (AccessUnit->DTS >= AccessUnit->LatestPTS && AccessUnit->PTS >= AccessUnit->LatestPTS)
						{
							// Tag the last one and send it off, but stop doing so for the remainder of the segment.
							// Note: we continue reading this segment all the way to the end on purpose in case there are further 'emsg' boxes.
							AccessUnit->bIsLastInPeriod = true;
							SelectedTrack.bReadPastLastPTS = true;
						}

						ElectraCDM::FMediaCDMSampleInfo SampleEncryptionInfo;
						bool bIsSampleEncrypted = TrackIt->GetEncryptionInfo(SampleEncryptionInfo);
						// If we need to decrypt we have to wait for the decrypter to become ready.
						if (bIsSampleEncrypted && Decrypter.IsValid())
						{
							while(!bTerminate && !HasBeenAborted() &&
								(Decrypter->GetState() == ElectraCDM::ECDMState::WaitingForKey || Decrypter->GetState() == ElectraCDM::ECDMState::Idle))
							{
								FMediaRunnable::SleepMilliseconds(100);
							}
							ElectraCDM::ECDMError DecryptResult = ElectraCDM::ECDMError::Failure;
							if (Decrypter->GetState() == ElectraCDM::ECDMState::Ready)
							{
								SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MP4_StreamReader);
								CSV_SCOPED_TIMING_STAT(ElectraPlayer, MP4_StreamReader);
								DecryptResult = Decrypter->DecryptInPlace((uint8*) AccessUnit->AUData, (int32) AccessUnit->AUSize, SampleEncryptionInfo);
							}
							if (DecryptResult != ElectraCDM::ECDMError::Success)
							{
								FAccessUnit::Release(AccessUnit);
								AccessUnit = nullptr;
								ds.FailureReason = FString::Printf(TEXT("Failed to decrypt segment with error %d (%s)"), (int32)DecryptResult, *Decrypter->GetLastErrorMessage());
								LogMessage(IInfoLog::ELevel::Error, ds.FailureReason);
								bHasErrored = true;
								bDone = true;
								break;
							}
						}

						while(!bSentOff && !HasBeenAborted() && !bTerminate)
						{
							if (Parameters.EventListener->OnFragmentAccessUnitReceived(AccessUnit))
							{
								SelectedTrack.DurationSuccessfullyDelivered += AccessUnit->Duration;
								bSentOff = true;
								AccessUnit = nullptr;

								// Since we have delivered this access unit, if we are detecting an error now we need to then
								// retry on the _next_ AU and not this one again!
								Request->CurrentIteratorBytePos = SampleFileOffset + SampleSize;
							}
							else
							{
								FMediaRunnable::SleepMicroseconds(1000 * 10);
							}
						}

						// Release the AU if we still have it.
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;

						// For error handling, if we managed to get additional data we reset the retry count.
						if (ds.RetryNumber && SelectedTrack.DurationSuccessfullyRead.GetAsSeconds() > 2.0)
						{
							ds.RetryNumber = 0;
							Request->NumOverallRetries = 0;
						}
					}
					else
					{
						// Did not get the number of bytes we needed. Either because of a read error or because we got aborted.
						FAccessUnit::Release(AccessUnit);
						AccessUnit = nullptr;
						bDone = true;
						break;
					}
				}
				else
				{
					// TODO: Throw OOM error
				}
			}
		}
		else
		{
			break;
		}
		AllTrackIterator->Next();
	}

	// Remove the download request.
	ProgressListener.Reset();
	PlayerSessionServices->GetHTTPManager()->RemoveRequest(HTTP, false);
	Request->ConnectionInfo = HTTP->ConnectionInfo;
	HTTP.Reset();

	// Set downloaded and delivered duration from the primary track.
	FSelectedTrackData& PrimaryTrack = ActiveTrackMap.FindOrAdd(PrimaryTrackID);
	DurationSuccessfullyRead	  = PrimaryTrack.DurationSuccessfullyRead;
	DurationSuccessfullyDelivered = PrimaryTrack.DurationSuccessfullyDelivered;

	// Set up download stat fields.
	ds.HTTPStatusCode = Request->ConnectionInfo.StatusInfo.HTTPStatus;
	ds.TimeToFirstByte = Request->ConnectionInfo.TimeUntilFirstByte;
	ds.TimeToDownload = (Request->ConnectionInfo.RequestEndTime - Request->ConnectionInfo.RequestStartTime).GetAsSeconds();
	ds.ByteSize = Request->ConnectionInfo.ContentLength;
	ds.NumBytesDownloaded = Request->ConnectionInfo.BytesReadSoFar;

	ds.FailureReason = Request->ConnectionInfo.StatusInfo.ErrorDetail.GetMessage();
	if (ParsingErrorMessage.Len())
	{
		ds.FailureReason = ParsingErrorMessage;
	}
	ds.bWasSuccessful     = !bHasErrored;
	ds.URL.URL 			  = Request->ConnectionInfo.EffectiveURL;
	ds.HTTPStatusCode     = Request->ConnectionInfo.StatusInfo.HTTPStatus;
	ds.DurationDownloaded = DurationSuccessfullyRead.GetAsSeconds();
	ds.DurationDelivered  = DurationSuccessfullyDelivered.GetAsSeconds();
	ds.TimeToFirstByte    = Request->ConnectionInfo.TimeUntilFirstByte;
	ds.TimeToDownload     = (Request->ConnectionInfo.RequestEndTime - Request->ConnectionInfo.RequestStartTime).GetAsSeconds();
	ds.ByteSize 		  = Request->ConnectionInfo.ContentLength;
	ds.NumBytesDownloaded = Request->ConnectionInfo.BytesReadSoFar;
	ds.bIsCachedResponse = Request->ConnectionInfo.bIsCachedResponse;

	ActiveTrackMap.Reset();

	// Reset the current request so another one can be added immediately when we call OnFragmentClose()
	CurrentRequest.Reset();
	PlayerSessionServices->GetStreamSelector()->ReportDownloadEnd(ds);
	Parameters.EventListener->OnFragmentClose(Request);
}

void FStreamReaderMP4::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);
	while(!bTerminate)
	{
		WorkSignal.WaitAndReset();
		if (bTerminate)
		{
			break;
		}
		TSharedPtrTS<FStreamSegmentRequestMP4>	Request = CurrentRequest;
		if (Request.IsValid())
		{
			HandleRequest();
		}
	}
}


int32 FStreamReaderMP4::FReadBuffer::ReadTo(void* IntoBuffer, int64 NumBytesToRead)
{
	FWaitableBuffer& SourceBuffer = *ReceiveBuffer;
	// Make sure the buffer will have the amount of data we need.
	while(1)
	{
		if (!SourceBuffer.WaitUntilSizeAvailable(ParsePos + NumBytesToRead, 1000 * 100))
		{
			if (bHasErrored || SourceBuffer.WasAborted() || bAbort)
			{
				return -1;
			}
		}
		else
		{
			SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MP4_StreamReader);
			CSV_SCOPED_TIMING_STAT(ElectraPlayer, MP4_StreamReader);
			SourceBuffer.Lock();
			if (SourceBuffer.Num() >= ParsePos + NumBytesToRead)
			{
				if (IntoBuffer)
				{
					FMemory::Memcpy(IntoBuffer, SourceBuffer.GetLinearReadData() + ParsePos, NumBytesToRead);
				}
				SourceBuffer.Unlock();
				ParsePos += NumBytesToRead;
				CurrentPos += NumBytesToRead;
				return NumBytesToRead;
			}
			else
			{
				// Return 0 at EOF and -1 on error.
				SourceBuffer.Unlock();
				return bHasErrored ? -1 : 0;
			}
		}
	}
}


} // namespace Electra


