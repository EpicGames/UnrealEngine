// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Demuxer/ParserISO14496-12.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMP4.h"
#include "Utilities/HashFunctions.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "StreamAccessUnitBuffer.h"
#include "Player/DRM/DRMManager.h"




namespace Electra
{


class FStreamSegmentRequestMP4 : public IStreamSegment
{
public:
	FStreamSegmentRequestMP4();
	virtual ~FStreamSegmentRequestMP4();

	void SetPlaybackSequenceID(uint32 InPlaybackSequenceID) override;
	uint32 GetPlaybackSequenceID() const override;

	void SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay) override;
	FTimeValue GetExecuteAtUTCTime() const override;

	EStreamType GetType() const override;

	void GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const override;
	void GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams) override;
	void GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams) override;

	FTimeValue GetFirstPTS() const override;
	FTimeRange GetTimeRange() const override;

	int32 GetQualityIndex() const override;
	int32 GetBitrate() const override;

	void GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const override;
	bool GetStartupDelay(FTimeValue& OutStartTime, FTimeValue& OutTimeIntoSegment, FTimeValue& OutSegmentDuration) const override
	{ return false; }



	TSharedPtrTS<ITimelineMediaAsset>					MediaAsset;					//!< The entire mp4 asset
	TSharedPtrTS<IParserISO14496_12::ITrackIterator>	PrimaryTrackIterator;
	TArray<EStreamType>									DependentStreamTypes;

	FTimeValue											FirstPTS;					//!< The PTS of the first sample
	FTimeValue											SegmentDuration;

	FTimeValue											EarliestPTS;				//!< PTS of the first sample to be presented.
	FTimeValue											LastPTS;					//!< PTS at which no further samples are to be presented.

	EStreamType											PrimaryStreamType;
	int64												FileStartOffset;			//!< Where to start in the file
	int64												FileEndOffset;				//!< Where to end in the file (for HTTP range GET requests)
	int64												SegmentInternalSize;		//!< Size of the segment as defined by internal structures.
	uint32												PlaybackSequenceID;
	int32												Bitrate;
	bool												bStartingOnMOOF;			//!< when using the 'sidx' a segment is expected to start on a 'moof', otherwise inside an 'mdat'.
	bool												bIsFirstSegment;			//!< true if this segment is the first to start with or the first after a seek.
	bool												bIsLastSegment;				//!< true if this segment is the last.

	bool												bAllTracksAtEOS;

	int64												TimestampSequenceIndex;		//!< Sequence index to set in all timestamp values of the decoded access unit.
	int32												NumOverallRetries;			//!< Number of retries

	Metrics::FSegmentDownloadStats						DownloadStats;
	HTTP::FConnectionInfo								ConnectionInfo;
	int64												CurrentIteratorBytePos;

	// Encryption
	TSharedPtrTS<ElectraCDM::IMediaCDMClient>			DrmClient;
	FString												DrmMimeType;
};


/**
 * This class implements an interface to read from an mp4 file.
 */
class FStreamReaderMP4 : public IStreamReader, public FMediaThread
{
public:
	FStreamReaderMP4();
	virtual ~FStreamReaderMP4();
	UEMediaError Create(IPlayerSessionServices* PlayerSessionService, const CreateParam &createParam) override;
	bool LoadsMultiplexedTracksSequentially() override;
	void Close() override;
	EAddResult AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> Request) override;
	void CancelRequest(EStreamType StreamType, bool bSilent) override;
	void CancelRequests() override;

private:
	void WorkerThread();
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);
	int32 HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest);
	void HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest);
	void HandleRequest();

	bool HasBeenAborted() const;
	bool HasErrored() const;

	struct FReadBuffer
	{
		FReadBuffer()
		{
			Reset();
		}
		void Reset()
		{
			ReceiveBuffer.Reset();
			CurrentPos = 0;
			ParsePos = 0;
			bAbort = false;
			bHasErrored = false;
		}
		void SetCurrentPos(int64 InPos)
		{
			CurrentPos = InPos;
		}
		int64 GetCurrentPos()
		{
			return CurrentPos;
		}
		void Abort()
		{
			bAbort = true;
			TSharedPtrTS<FWaitableBuffer> Buffer = ReceiveBuffer;
			if (Buffer.IsValid())
			{
				Buffer->Abort();
			}
		}
		void SetHasErrored()
		{
			bHasErrored = true;
		}
		int32 ReadTo(void* ToBuffer, int64 NumBytes);
		TSharedPtrTS<FWaitableBuffer>						ReceiveBuffer;
		int64												CurrentPos;
		int64												ParsePos;
		bool												bAbort;
		bool												bHasErrored;
	};

	struct FSelectedTrackData
	{
		FSelectedTrackData()
			: StreamType(EStreamType::Video), Bitrate(0), bIsSelectedTrack(false), bIsFirstInSequence(true), bReadPastLastPTS(false)
		{
			DurationSuccessfullyRead.SetToZero();
			DurationSuccessfullyDelivered.SetToZero();
		}
		TSharedPtrTS<FAccessUnit::CodecData>	CSD;
		TSharedPtrTS<FBufferSourceInfo>			BufferSourceInfo;
		EStreamType								StreamType;
		int32									Bitrate;
		bool									bIsSelectedTrack;
		bool									bIsFirstInSequence;
		bool									bReadPastLastPTS;
		FTimeValue								DurationSuccessfullyRead;
		FTimeValue								DurationSuccessfullyDelivered;
	};

	CreateParam									Parameters;
	IPlayerSessionServices*						PlayerSessionServices;
	bool										bIsStarted;
	bool										bTerminate;
	bool										bRequestCanceled;
	bool										bHasErrored;
	FErrorDetail								ErrorDetail;

	TSharedPtrTS<FStreamSegmentRequestMP4>		CurrentRequest;
	FMediaEvent									WorkSignal;
	FReadBuffer									ReadBuffer;

	TMap<uint32, FSelectedTrackData>			ActiveTrackMap;

	static uint32								UniqueDownloadID;
};



} // namespace Electra


