// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMPEGAudio.h"
#include "Utilities/Utilities.h"
#include "Utilities/HashFunctions.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/AdaptiveStreamingPlayerResourceRequest.h"
#include "Player/AdaptivePlayerOptionKeynames.h"
#include "Player/PlayerStreamFilter.h"
#include "Player/mpegaudio/StreamReaderMPEGAudio.h"
#include "Player/mpegaudio/ManifestMPEGAudio.h"


#define ERRCODE_MANIFEST_MPEGAUDIO_NOT_DECODABLE			1
#define ERRCODE_MANIFEST_MPEGAUDIO_STARTSEGMENT_NOT_FOUND	2


#define INTENDED_LIVE_LATENCY_IN_SECONDS	1.0

#define USE_LOW_LATENCY_DESCRIPTOR 0

DECLARE_CYCLE_STAT(TEXT("FPlayPeriodMPEGAudio::FindSegment"), STAT_ElectraPlayer_MPEGAudio_FindSegment, STATGROUP_ElectraPlayer);

namespace Electra
{

//-----------------------------------------------------------------------------
/**
 * CTOR
 */
FManifestMPEGAudioInternal::FManifestMPEGAudioInternal(IPlayerSessionServices* InPlayerSessionServices)
	: PlayerSessionServices(InPlayerSessionServices)
{
}


//-----------------------------------------------------------------------------
/**
 * DTOR
 */
FManifestMPEGAudioInternal::~FManifestMPEGAudioInternal()
{
}


//-----------------------------------------------------------------------------
/**
 * Builds the internal manifest.
 *
 * @param InHeader
 * @param URL
 *
 * @return
 */
FErrorDetail FManifestMPEGAudioInternal::Build(const FMPEGAudioInfoHeader& InHeader, const FString& URL)
{
	MPEGInfoHeader = InHeader;
	MediaAsset = MakeSharedTS<FTimelineAssetMPEGAudio>();
	FErrorDetail Result = MediaAsset->Build(PlayerSessionServices, InHeader, URL);
	FTimeRange PlaybackRange = GetPlaybackRange(IManifest::EPlaybackRangeType::TemporaryPlaystartRange);
	DefaultStartTime = PlaybackRange.Start;
	DefaultEndTime = PlaybackRange.End;

#if USE_LOW_LATENCY_DESCRIPTOR
	if (MPEGInfoHeader.bIsLive)
	{
		LatencyDescriptor = MakeSharedTS<FLowLatencyDescriptor>();
		LatencyDescriptor->Latency.Target.SetFromSeconds(INTENDED_LIVE_LATENCY_IN_SECONDS);
		LatencyDescriptor->Latency.Min.SetFromSeconds(0.5);	// needs to be less than INTENDED_LIVE_LATENCY_IN_SECONDS
		LatencyDescriptor->Latency.Max.SetFromSeconds(4.0);	// needs to be more than INTENDED_LIVE_LATENCY_IN_SECONDS
		LatencyDescriptor->PlayRate.Min.SetFromSeconds(0.9);
		LatencyDescriptor->PlayRate.Max.SetFromSeconds(1.05);
	}
#endif
	return Result;
}



//-----------------------------------------------------------------------------
/**
 * Logs a message.
 *
 * @param Level
 * @param Message
 */
void FManifestMPEGAudioInternal::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::MPEGAudioPlaylist, Level, Message);
	}
}



EMediaFormatType FManifestMPEGAudioInternal::GetMediaFormatType() const
{
	return EMediaFormatType::MPEGAudio;
}

bool FManifestMPEGAudioInternal::ReloadsPlaylistPeriodically() const
{
	return false;
}

bool FManifestMPEGAudioInternal::SupportsMultipleTracksPerKind() const
{
	return false;
}

bool FManifestMPEGAudioInternal::OrderOfTracksCanChangeDynamically() const
{
	return false;
}


//-----------------------------------------------------------------------------
/**
 * Returns the type of presentation.
 *
 * @return
 */
IManifest::EType FManifestMPEGAudioInternal::GetPresentationType() const
{
	return MediaAsset.IsValid() ? MediaAsset->GetDuration().IsPositiveInfinity() ? IManifest::EType::Live : IManifest::EType::OnDemand : IManifest::EType::OnDemand;
}


//-----------------------------------------------------------------------------
/**
 * Returns track metadata.
 *
 * @param OutMetadata
 * @param StreamType
 */
void FManifestMPEGAudioInternal::GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const
{
	if (MediaAsset.IsValid())
	{
		MediaAsset->GetMetaData(OutMetadata, StreamType);
	}
}


void FManifestMPEGAudioInternal::UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData)
{
	TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> md = InUpdatedMetaData->GetMediaStreamMetadata();
	if (md.IsValid() && MediaAsset.IsValid())
	{
		for(auto& it : *md)
		{
			if (it.Key == TEXT("Title") && it.Value.Num())
			{
				MediaAsset->UpdateRunningMetaData(it.Key, it.Value[0]->GetValue());
			}
		}
	}
}


//-----------------------------------------------------------------------------
/**
 * Returns the playback range on the timeline, which is a subset of the total
 * time range. This may be set through manifest internal means or by URL fragment
 * parameters where permissable (eg. example.mpa#t=22,50).
 * If start or end are not specified they will be set to invalid.
 *
 * @return Optionally set time range to which playback is restricted.
 */
FTimeRange FManifestMPEGAudioInternal::GetPlaybackRange(EPlaybackRangeType InRangeType) const
{
	FTimeRange FromTo;

	// We are interested in the 't' or 'r' fragment value here.
	FString Time;
	for(auto& Fragment : URLFragmentComponents)
	{
		if ((InRangeType == IManifest::EPlaybackRangeType::TemporaryPlaystartRange && Fragment.Name.Equals(TEXT("t"))) ||
			(InRangeType == IManifest::EPlaybackRangeType::LockedPlaybackRange && Fragment.Name.Equals(TEXT("r"))))
		{
			Time = Fragment.Value;
		}
	}
	if (!Time.IsEmpty())
	{
		FTimeRange TotalRange = GetTotalTimeRange();
		TArray<FString> TimeRange;
		const TCHAR* const TimeDelimiter = TEXT(",");
		Time.ParseIntoArray(TimeRange, TimeDelimiter, false);
		if (TimeRange.Num() && !TimeRange[0].IsEmpty())
		{
			RFC2326::ParseNPTTime(FromTo.Start, TimeRange[0]);
		}
		if (TimeRange.Num() > 1 && !TimeRange[1].IsEmpty())
		{
			RFC2326::ParseNPTTime(FromTo.End, TimeRange[1]);
		}
		// Need to clamp this into the total time range to prevent any issues.
		if (FromTo.Start.IsValid() && TotalRange.Start.IsValid() && FromTo.Start < TotalRange.Start)
		{
			FromTo.Start = TotalRange.Start;
		}
		if (FromTo.End.IsValid() && TotalRange.End.IsValid() && FromTo.End > TotalRange.End)
		{
			FromTo.End = TotalRange.End;
		}
	}
	return FromTo;
}


//-----------------------------------------------------------------------------
/**
 * Returns the minimum duration of content that must be buffered up before playback
 * will begin.
 *
 * @return
 */
FTimeValue FManifestMPEGAudioInternal::GetMinBufferTime() const
{
	if (MPEGInfoHeader.bIsLive)
	{
		return FTimeValue().SetFromSeconds(1.0);
	}
	// NOTE: This is an arbitrary choice.
	return FTimeValue().SetFromSeconds(1.0);
}

FTimeValue FManifestMPEGAudioInternal::GetDesiredLiveLatency() const
{
	return LatencyDescriptor.IsValid() ? LatencyDescriptor->Latency.Target : FTimeValue().SetFromSeconds(INTENDED_LIVE_LATENCY_IN_SECONDS);
}

IManifest::ELiveEdgePlayMode FManifestMPEGAudioInternal::GetLiveEdgePlayMode() const
{
	return IManifest::ELiveEdgePlayMode::Never;
}


TSharedPtrTS<const FLowLatencyDescriptor> FManifestMPEGAudioInternal::GetLowLatencyDescriptor() const
{
	return LatencyDescriptor;
}

FTimeValue FManifestMPEGAudioInternal::CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const
{
	FTimeValue LiveLatency;
	if (GetPresentationType() != IManifest::EType::OnDemand)
	{
		FTimeValue UTCNow = PlayerSessionServices->GetSynchronizedUTCTime()->GetTime();
		LiveLatency = UTCNow - InCurrentPlaybackPosition;

		if (bViaLatencyElement)
		{
			TSharedPtrTS<const FLowLatencyDescriptor> llDesc = GetLowLatencyDescriptor();
			if (llDesc.IsValid())
			{
				// Low latency Live
				TSharedPtrTS<IProducerReferenceTimeInfo> ProdRefTime = GetProducerReferenceTimeInfo(llDesc->Latency.ReferenceID);
				if (ProdRefTime.IsValid())
				{
					if (InEncoderLatency.IsValid())
					{
						LiveLatency += InEncoderLatency;
					}
				}
			}
		}
	}
	return LiveLatency;
}


TSharedPtrTS<IProducerReferenceTimeInfo> FManifestMPEGAudioInternal::GetProducerReferenceTimeInfo(int64 ID) const
{
	return nullptr;
}

TRangeSet<double> FManifestMPEGAudioInternal::GetPossiblePlaybackRates(EPlayRateType InForType) const
{
	TRangeSet<double> Ranges;
//	Ranges.Add(TRange<double>{1.0}); // normal (real-time) playback rate
	Ranges.Add(TRange<double>::Inclusive(0.5, 4.0));
	Ranges.Add(TRange<double>{0.0}); // and pause
	return Ranges;
}


void FManifestMPEGAudioInternal::UpdateDynamicRefetchCounter()
{
	// No-op.
}

void FManifestMPEGAudioInternal::TriggerClockSync(IManifest::EClockSyncType InClockSyncType)
{
	// No-op.
}

void FManifestMPEGAudioInternal::TriggerPlaylistRefresh()
{
	// No-op.
}


//-----------------------------------------------------------------------------
/**
 * Creates an instance of a stream reader.
 *
 * @return
 */
IStreamReader* FManifestMPEGAudioInternal::CreateStreamReaderHandler()
{
	return new FStreamReaderMPEGAudio;
}


//-----------------------------------------------------------------------------
/**
 * Returns the playback period for the given time.
 *
 * @param OutPlayPeriod
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMPEGAudioInternal::FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	if (MediaAsset.IsValid() && StartPosition.Time.IsValid() && (StartPosition.Time < MediaAsset->GetDuration() || MediaAsset->GetDuration().IsPositiveInfinity()))
	{
		OutPlayPeriod = MakeSharedTS<FPlayPeriodMPEGAudio>(MediaAsset);
		return IManifest::FResult(IManifest::FResult::EType::Found);
	}
	else
	{
		return IManifest::FResult(IManifest::FResult::EType::PastEOS);
	}
}

IManifest::FResult FManifestMPEGAudioInternal::FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment)
{
	// There is no following period.
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}





//-----------------------------------------------------------------------------
/**
 * Constructs a playback period.
 *
 * @param InMediaAsset
 */
FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::FPlayPeriodMPEGAudio(TSharedPtrTS<FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio> InMediaAsset)
	: MediaAsset(InMediaAsset)
	, CurrentReadyState(IManifest::IPlayPeriod::EReadyState::NotLoaded)
{
}


//-----------------------------------------------------------------------------
/**
 * Destroys a playback period.
 */
FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::~FPlayPeriodMPEGAudio()
{
}


//-----------------------------------------------------------------------------
/**
 * Sets stream playback preferences for this playback period.
 *
 * @param ForStreamType
 * @param StreamAttributes
 */
void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	if (ForStreamType == EStreamType::Audio)
	{
		AudioPreferences = StreamAttributes;
	}
}


//-----------------------------------------------------------------------------
/**
 * Returns the starting bitrate.
 *
 * This is merely informational and not strictly required.
 *
 * @return
 */
int64 FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetDefaultStartingBitrate() const
{
	TSharedPtrTS<FTimelineAssetMPEGAudio> ma = MediaAsset.Pin();
	if (ma.IsValid())
	{
		TArray<FTrackMetadata> Metadata;
		ma->GetMetaData(Metadata, EStreamType::Audio);
		if (Metadata.Num() && Metadata[0].StreamDetails.Num())
		{
			return Metadata[0].StreamDetails[0].Bandwidth;
		}
	}
	return 128000;
}

//-----------------------------------------------------------------------------
/**
 * Returns the ready state of this playback period.
 *
 * @return
 */
IManifest::IPlayPeriod::EReadyState FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetReadyState()
{
	return CurrentReadyState;
}


void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::Load()
{
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::Loaded;
}

//-----------------------------------------------------------------------------
/**
 * Prepares the playback period for playback.
 * We are actually always ready for playback, but we say we're not
 * one time to get here with any possible options.
 */
void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::PrepareForPlay()
{
	SelectedAudioMetadata.Reset();
	AudioBufferSourceInfo.Reset();
	SelectInitialStream(EStreamType::Audio);
	CurrentReadyState = IManifest::IPlayPeriod::EReadyState::IsReady;
}


TSharedPtrTS<FBufferSourceInfo> FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetSelectedStreamBufferSourceInfo(EStreamType StreamType)
{
	return StreamType == EStreamType::Audio ? AudioBufferSourceInfo : nullptr;
}

FString FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetSelectedAdaptationSetID(EStreamType StreamType)
{
	if (StreamType == EStreamType::Audio)
	{
		return SelectedAudioMetadata.IsValid() ? SelectedAudioMetadata->ID : FString();
	}
	return FString();
}


IManifest::IPlayPeriod::ETrackChangeResult FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::ChangeTrackStreamPreference(EStreamType StreamType, const FStreamSelectionAttributes& StreamAttributes)
{
	TSharedPtrTS<FTrackMetadata> Metadata = SelectMetadataForAttributes(StreamType, StreamAttributes);
	if (Metadata.IsValid())
	{
		if (StreamType == EStreamType::Audio)
		{
			if (!(SelectedAudioMetadata.IsValid() && Metadata->Equals(*SelectedAudioMetadata)))
			{
				SelectedAudioMetadata = Metadata;
				MakeBufferSourceInfoFromMetadata(StreamType, AudioBufferSourceInfo, SelectedAudioMetadata);
				return IManifest::IPlayPeriod::ETrackChangeResult::Changed;
			}
		}
	}
	return IManifest::IPlayPeriod::ETrackChangeResult::NotChanged;
}

void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::SelectInitialStream(EStreamType StreamType)
{
	if (StreamType == EStreamType::Audio)
	{
		SelectedAudioMetadata = SelectMetadataForAttributes(StreamType, AudioPreferences);
		MakeBufferSourceInfoFromMetadata(StreamType, AudioBufferSourceInfo, SelectedAudioMetadata);
	}
}

TSharedPtrTS<FTrackMetadata> FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::SelectMetadataForAttributes(EStreamType StreamType, const FStreamSelectionAttributes& InAttributes)
{
	TSharedPtrTS<FTimelineAssetMPEGAudio> Asset = MediaAsset.Pin();
	if (Asset.IsValid())
	{
		TArray<FTrackMetadata> Metadata;
		Asset->GetMetaData(Metadata, StreamType);
		// Is there a fixed index to be used?
		if (InAttributes.OverrideIndex.IsSet() && InAttributes.OverrideIndex.GetValue() >= 0 && InAttributes.OverrideIndex.GetValue() < Metadata.Num())
		{
			// Use this.
			return MakeSharedTS<FTrackMetadata>(Metadata[InAttributes.OverrideIndex.GetValue()]);
		}
		if (Metadata.Num())
		{
			// We do not look at the 'kind' or 'codec' here, only the language.
			// Set the first track as default in case we do not find the one we're looking for.
			if (InAttributes.Language_RFC4647.IsSet())
			{
				TArray<int32> CandidateIndices;
				TArray<BCP47::FLanguageTag> CandList;
				for(auto &Meta : Metadata)
				{
					CandList.Emplace(Meta.LanguageTagRFC5646);
				}
				CandidateIndices = BCP47::FindExtendedFilteringMatch(CandList, InAttributes.Language_RFC4647.GetValue());
				return MakeSharedTS<FTrackMetadata>(CandidateIndices.Num() ? Metadata[CandidateIndices[0]] : Metadata[0]);
			}
			return MakeSharedTS<FTrackMetadata>(Metadata[0]);
		}
	}
	return nullptr;
}

void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::MakeBufferSourceInfoFromMetadata(EStreamType StreamType, TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FTrackMetadata> InMetadata)
{
	if (InMetadata.IsValid())
	{
		OutBufferSourceInfo = MakeSharedTS<FBufferSourceInfo>();
		OutBufferSourceInfo->Kind = InMetadata->Kind;
		OutBufferSourceInfo->LanguageTag = InMetadata->LanguageTagRFC5646;
		OutBufferSourceInfo->Codec = InMetadata->HighestBandwidthCodec.GetCodecName();
		TSharedPtrTS<FTimelineAssetMPEGAudio> Asset = MediaAsset.Pin();
		OutBufferSourceInfo->PeriodID = Asset->GetUniqueIdentifier();
		OutBufferSourceInfo->PeriodAdaptationSetID = Asset->GetUniqueIdentifier() + TEXT(".") + InMetadata->ID;
		TArray<FTrackMetadata> Metadata;
		Asset->GetMetaData(Metadata, StreamType);
		for(int32 i=0; i<Metadata.Num(); ++i)
		{
			if (Metadata[i].Equals(*InMetadata))
			{
				OutBufferSourceInfo->HardIndex = i;
				break;
			}
		}
	}
}



//-----------------------------------------------------------------------------
/**
 * Returns the timeline media asset. We have a weak pointer to it only to
 * prevent any cyclic locks, so we need to lock it first.
 *
 * @return
 */
TSharedPtrTS<ITimelineMediaAsset> FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetMediaAsset() const
{
	TSharedPtrTS<ITimelineMediaAsset> ma = MediaAsset.Pin();
	return ma;
}


//-----------------------------------------------------------------------------
/**
 * Selects a particular stream (== internal track ID) for playback.
 *
 * @param AdaptationSetID
 * @param RepresentationID
 */
void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex)
{
	// Presently this method is only called by the ABR to switch between quality levels, of which there are none.
}

void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload)
{
	// No-op.
}

//-----------------------------------------------------------------------------
/**
 * Creates the starting segment request to start playback with.
 *
 * @param OutSegment
 * @param InSequenceState
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	TSharedPtrTS<FTimelineAssetMPEGAudio> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetStartingSegment(OutSegment, InSequenceState, StartPosition, SearchType, -1) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}

//-----------------------------------------------------------------------------
/**
 * Same as GetStartingSegment() except this is for a specific stream (video, audio, ...) only.
 * To be used when a track (language) change is made and a new segment is needed at the current playback position.
 */
IManifest::FResult FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType StreamType, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	// Not supported
	return IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Sets up a starting segment request to loop playback to.
 * The streams selected through SelectStream() will be used.
 *
 * @param OutSegment
 * @param SequenceState
 * @param StartPosition
 * @param SearchType
 *
 * @return
 */
IManifest::FResult FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	TSharedPtrTS<FTimelineAssetMPEGAudio> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetLoopingSegment(OutSegment, SequenceState, StartPosition, SearchType) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Called by the ABR to increase the delay in fetching the next segment in case the segment returned a 404 when fetched at
 * the announced availability time. This may reduce 404's on the next segment fetches.
 *
 * @param IncreaseAmount
 */
void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount)
{
	// No-op.
}


//-----------------------------------------------------------------------------
/**
 * Creates the next segment request.
 *
 * @param OutSegment
 * @param CurrentSegment
 * @param Options
 *
 * @return
 */
IManifest::FResult FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options)
{
	TSharedPtrTS<FTimelineAssetMPEGAudio> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetNextSegment(OutSegment, CurrentSegment, Options) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Creates a segment retry request.
 *
 * @param OutSegment
 * @param CurrentSegment
 * @param Options
 * @param bReplaceWithFillerData
 *
 * @return
 */
IManifest::FResult FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData)
{
	TSharedPtrTS<FTimelineAssetMPEGAudio> ma = MediaAsset.Pin();
	return ma.IsValid() ? ma->GetRetrySegment(OutSegment, CurrentSegment, Options, bReplaceWithFillerData) : IManifest::FResult(IManifest::FResult::EType::NotFound);
}


//-----------------------------------------------------------------------------
/**
 * Returns the average segment duration.
 */
void FManifestMPEGAudioInternal::FPlayPeriodMPEGAudio::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	TSharedPtrTS<FTimelineAssetMPEGAudio> ma = MediaAsset.Pin();
	if (ma.IsValid())
	{
		ma->GetAverageSegmentDuration(OutAverageSegmentDuration, AdaptationSetID, RepresentationID);
	}
}





//-----------------------------------------------------------------------------
/**
 * Builds the timeline asset.
 *
 * @param InPlayerSessionServices
 * @param InHeader
 * @param URL
 *
 * @return
 */
FErrorDetail FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio::Build(IPlayerSessionServices* InPlayerSessionServices, const FMPEGAudioInfoHeader& InHeader, const FString& URL)
{
	PlayerSessionServices = InPlayerSessionServices;
	MediaURL = URL;
	MPEGInfoHeader = InHeader;

	// Can we decode this track?
	bool bIsUsable = false;
	IPlayerStreamFilter* StreamFilter = InPlayerSessionServices->GetStreamFilter();
	if (StreamFilter && StreamFilter->CanDecodeStream(MPEGInfoHeader.CodecInfo))
	{
		FErrorDetail err;
		TSharedPtrTS<FAdaptationSetMPEGAudio> AdaptationSet = MakeSharedTS<FAdaptationSetMPEGAudio>();
		err = AdaptationSet->CreateFrom(MPEGInfoHeader, URL);
		if (err.IsOK())
		{
			AudioAdaptationSets.Add(AdaptationSet);
		}
		else
		{
			return err;
		}
	}
	else
	{
		FErrorDetail err;
		err.SetFacility(Facility::EFacility::MPEGAudioPlaylist);
		err.SetMessage("This stream cannot be decoded");
		err.SetCode(ERRCODE_MANIFEST_MPEGAUDIO_NOT_DECODABLE);
		return err;
	}

	// Convert metadata, if present.
	if (MPEGInfoHeader.ID3v2.IsValid())
	{
		TSharedPtr<FMetadataParser, ESPMode::ThreadSafe> mp = MakeShared<FMetadataParser, ESPMode::ThreadSafe>();

		for(auto& it : MPEGInfoHeader.ID3v2->GetTags())
		{
			if (it.Key == Utils::Make4CC('T','I','T','2'))
			{
				mp->AddItem(TEXT("Title"), it.Value.Value.GetValue<FString>());
			}
			else if (it.Key == Utils::Make4CC('T','A','L','B'))
			{
				mp->AddItem(TEXT("Album"), it.Value.Value.GetValue<FString>());
			}
			else if (it.Key == Utils::Make4CC('T','P','E','1'))
			{
				mp->AddItem(TEXT("Artist"), it.Value.Value.GetValue<FString>());
			}
			else if (it.Key == Utils::Make4CC('T','E','N','C'))
			{
				mp->AddItem(TEXT("Encoder"), it.Value.Value.GetValue<FString>());
			}
			else if (it.Key == Utils::Make4CC('T','C','O','N'))
			{
				mp->AddItem(TEXT("Genre"), it.Value.Value.GetValue<FString>());
			}
			else if (it.Key == Utils::Make4CC('T','Y','E','R'))
			{
				mp->AddItem(TEXT("Date"), it.Value.Value.GetValue<FString>());
			}
			else if (it.Key == Utils::Make4CC('A','P','I','C'))
			{
				mp->AddItem(TEXT("covr"), it.Value.MimeType, it.Value.Value.GetValue<TArray<uint8>>());
			}
			else if (it.Key == Utils::Make4CC('T','L','E','N') ||
					 it.Key == Utils::Make4CC('M','L','L','T') ||
					 0)
			{
				// ignore
			}
		}
		PlayerSessionServices->SendMessageToPlayer(FPlaylistMetadataUpdateMessage::Create(FTimeValue(), mp, false));
	}
	else if (MPEGInfoHeader.HTTPResponseHeaders.Contains(TEXT("icy-name")))
	{
		TSharedPtr<FMetadataParser, ESPMode::ThreadSafe> mp = MakeShared<FMetadataParser, ESPMode::ThreadSafe>();
		TArray<FString> HeaderValues;
		MPEGInfoHeader.HTTPResponseHeaders.MultiFind(TEXT("icy-name"), HeaderValues);
		mp->AddItem(TEXT("Album"), HeaderValues[0]);
		if (MPEGInfoHeader.HTTPResponseHeaders.Contains(TEXT("icy-genre")))
		{
			HeaderValues.Empty();
			MPEGInfoHeader.HTTPResponseHeaders.MultiFind(TEXT("icy-genre"), HeaderValues);
			mp->AddItem(TEXT("Genre"), HeaderValues[0]);
		}
		PlayerSessionServices->SendMessageToPlayer(FPlaylistMetadataUpdateMessage::Create(FTimeValue(), mp, false));
	}
	return FErrorDetail();
}



void FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::MPEGAudioPlaylist, Level, Message);
	}
}




IManifest::FResult FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio::GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType, int64 AtAbsoluteFilePos)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MPEGAudio_FindSegment);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, MPEGAudio_FindSegment);

	if (AudioAdaptationSets.Num())
	{
		// Live stream?
		if (MPEGInfoHeader.bIsLive)
		{
			TSharedPtrTS<FStreamSegmentRequestMPEGAudio> req(new FStreamSegmentRequestMPEGAudio);
			OutSegment = req;
			req->MediaAsset = SharedThis(this);
			FTimeValue StartTime = PlayerSessionServices && PlayerSessionServices->GetSynchronizedUTCTime() ? PlayerSessionServices->GetSynchronizedUTCTime()->GetTime() : MEDIAutcTime::Current();
			// Is this a known audio cast type?
			if (MPEGInfoHeader.HTTPResponseHeaders.Contains(TEXT("icy-name")))
			{
				req->CastType = FStreamSegmentRequestMPEGAudio::ECastType::IcyCast;
			}
			req->FirstPTS = StartTime;
			req->EarliestPTS = StartTime;
			req->LastPTS.SetToPositiveInfinity();
			req->Bitrate = MPEGInfoHeader.AverageBitrate ? MPEGInfoHeader.AverageBitrate : MPEGInfoHeader.Bitrate;
			req->bIsContinuationSegment = false;
			req->bIsFirstSegment = true;
			req->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();
			req->bIsLastSegment = false;
			req->MPEGHeaderMask = MPEGInfoHeader.MPEGHeaderMask;
			req->MPEGHeaderExpectedValue = MPEGInfoHeader.MPEGHeaderExpectedValue;
			req->CBRFrameSize = MPEGInfoHeader.CBRFrameSize;
			req->bIsAAC = MPEGInfoHeader.Type == FMPEGAudioInfoHeader::EType::ISO_14496_3;
			req->bIsVBR = MPEGInfoHeader.bIsVBR;
			req->bIsLive = true;
			req->CodecInfo = MPEGInfoHeader.CodecInfo;
			return IManifest::FResult(IManifest::FResult::EType::Found);
		}
		else
		{
			FTimeValue StartTime = StartPosition.Time;
			FTimeValue PlayRangeEnd = StartPosition.Options.PlaybackRange.End;
			check(PlayRangeEnd.IsValid());
			if (PlayRangeEnd > MPEGInfoHeader.EstimatedDuration)
			{
				PlayRangeEnd = MPEGInfoHeader.EstimatedDuration;
			}

			int64 ApproxByteOffset = -1;
			if (AtAbsoluteFilePos < 0)
			{
				check(MPEGInfoHeader.EstimatedDuration.IsValid());
				check(MPEGInfoHeader.SampleRate);
				check(MPEGInfoHeader.SamplesPerFrame);
				check(MPEGInfoHeader.FirstDataByte >= 0);
				check(MPEGInfoHeader.LastDataByte > 0);

				double Percentage = FMath::Clamp(StartTime.GetAsSeconds() / MPEGInfoHeader.EstimatedDuration.GetAsSeconds(), 0.0, 1.0);
				// Seek accuracy varies greatly depending on the (optional!) information provided.
				if (!MPEGInfoHeader.bIsVBR)
				{
					// CBR. We can calculate an approximate position based on frame size.
					check(MPEGInfoHeader.Bitrate);
					check(MPEGInfoHeader.CBRFrameSize);
					ApproxByteOffset = MPEGInfoHeader.FirstDataByte + (MPEGInfoHeader.LastDataByte - MPEGInfoHeader.FirstDataByte) * Percentage;
					ApproxByteOffset = FMath::Clamp(ApproxByteOffset, MPEGInfoHeader.FirstDataByte, MPEGInfoHeader.LastDataByte);
				}
				else
				{
					check(MPEGInfoHeader.AverageBitrate);

					// Is there an MLLT entry from the ID3v2 header?
					if (MPEGInfoHeader.MLLT.IsValid())
					{
						int64 StartTimeMillis = StartTime.GetAsMilliseconds();
						// Locate the entry
						int32 EntryIdx, LastEntryIdx=MPEGInfoHeader.MLLT->TimeAndOffsets.Num();
						for(EntryIdx=0; EntryIdx<LastEntryIdx-1; ++EntryIdx)
						{
							if ((int64)MPEGInfoHeader.MLLT->TimeAndOffsets[EntryIdx+1].Milliseconds > StartTimeMillis)
							{
								break;
							}
						}
						int64 NextEntryOffset = EntryIdx+1 < LastEntryIdx ? (int64)MPEGInfoHeader.MLLT->TimeAndOffsets[EntryIdx+1].Offset : MPEGInfoHeader.LastDataByte - MPEGInfoHeader.FirstDataByte;
						int64 NextEntryMillis = EntryIdx+1 < LastEntryIdx ? (int64)MPEGInfoHeader.MLLT->TimeAndOffsets[EntryIdx+1].Milliseconds : MPEGInfoHeader.EstimatedDuration.GetAsMilliseconds();
						double Frac = (double)(StartTimeMillis - MPEGInfoHeader.MLLT->TimeAndOffsets[EntryIdx].Milliseconds) / (double)(NextEntryMillis - MPEGInfoHeader.MLLT->TimeAndOffsets[EntryIdx].Milliseconds);
						ApproxByteOffset = MPEGInfoHeader.FirstDataByte + MPEGInfoHeader.MLLT->TimeAndOffsets[EntryIdx].Offset + (int64)(Frac * (NextEntryOffset - MPEGInfoHeader.MLLT->TimeAndOffsets[EntryIdx].Offset));
						ApproxByteOffset = FMath::Clamp(ApproxByteOffset, MPEGInfoHeader.FirstDataByte, MPEGInfoHeader.LastDataByte);
					}
					// Is there a VBRI seek table?
					else if (MPEGInfoHeader.SeekTable.IsValid() && MPEGInfoHeader.FramesPerSeekTableEntry)
					{
						FTimeValue DurPerEntry = MPEGInfoHeader.EstimatedDuration / MPEGInfoHeader.SeekTable->Num();
						FTimeValue DurSoFar(FTimeValue::GetZero());
						int32 SeekTablePos;
						ApproxByteOffset = MPEGInfoHeader.FirstDataByte;
						for(SeekTablePos=0; SeekTablePos<MPEGInfoHeader.SeekTable->Num() && DurSoFar <= StartTime; ++SeekTablePos)
						{
							DurSoFar += DurPerEntry;
							ApproxByteOffset += MPEGInfoHeader.SeekTable->operator[](SeekTablePos);
						}
						int32 Frac = FMath::Floor(((DurSoFar - StartTime).GetAsSeconds() / DurPerEntry.GetAsSeconds() + 1.0 / (MPEGInfoHeader.FramesPerSeekTableEntry * 2)) * MPEGInfoHeader.FramesPerSeekTableEntry);
						int64 Delta = FMath::Floor(MPEGInfoHeader.SeekTable->operator[](SeekTablePos-1) * (double)Frac / MPEGInfoHeader.FramesPerSeekTableEntry);
						ApproxByteOffset -= Delta;
						ApproxByteOffset = FMath::Clamp(ApproxByteOffset, MPEGInfoHeader.FirstDataByte, MPEGInfoHeader.LastDataByte);
					}
					// Is there a TOC?
					else if (MPEGInfoHeader.bHaveTOC)
 					{
						// The TOC has 100 entries, mapping each full 1% of seek position to a percentage (scaled by 2.56 to map into 0-255 range) within the file.
						// This makes seeking less precise the longer the file gets.
						Percentage *= 100.0;
						int32 Idx = FMath::Floor(Percentage);
						double a = MPEGInfoHeader.TOC[FMath::Clamp(Idx, 0, 99)];
						double b = Idx < 99 ? MPEGInfoHeader.TOC[Idx + 1] : 256.0;
						double x = a + (b - a) * (Percentage - Idx);
						Percentage = x / 256.0;
						ApproxByteOffset = MPEGInfoHeader.FirstDataByte + (MPEGInfoHeader.LastDataByte - MPEGInfoHeader.FirstDataByte) * Percentage;
						ApproxByteOffset = FMath::Clamp(ApproxByteOffset, MPEGInfoHeader.FirstDataByte, MPEGInfoHeader.LastDataByte);
					}
					else
					{
						// Short of any seek table we can only jump to somewhere in the file by percentage as we do with CBR.
						ApproxByteOffset = MPEGInfoHeader.FirstDataByte + (MPEGInfoHeader.LastDataByte - MPEGInfoHeader.FirstDataByte) * Percentage;
						ApproxByteOffset = FMath::Clamp(ApproxByteOffset, MPEGInfoHeader.FirstDataByte, MPEGInfoHeader.LastDataByte);
					}
				}
			}
			else
			{
				ApproxByteOffset = AtAbsoluteFilePos;
				// Going past the end?
				check(MPEGInfoHeader.LastDataByte > 0);
				if (ApproxByteOffset >= MPEGInfoHeader.LastDataByte)
				{
					ApproxByteOffset = -1;
				}
			}

			if (ApproxByteOffset >= 0)
			{
				TSharedPtrTS<FStreamSegmentRequestMPEGAudio> req(new FStreamSegmentRequestMPEGAudio);
				OutSegment = req;
				req->MediaAsset = SharedThis(this);
				check(StartTime.IsValid() && PlayRangeEnd.IsValid());
				req->FirstPTS = StartTime;
				req->EarliestPTS = StartTime;
				req->LastPTS = PlayRangeEnd;

				req->FileStartOffset = ApproxByteOffset;
				req->Bitrate = MPEGInfoHeader.AverageBitrate ? MPEGInfoHeader.AverageBitrate : MPEGInfoHeader.Bitrate;
				req->bIsContinuationSegment = false;
				req->bIsFirstSegment = true;
				req->TimestampSequenceIndex = InSequenceState.GetSequenceIndex();

				// Approximate how many bytes will equal about n seconds
				const double kSegmentDuration = 3.0;
				double FrameDuration = (double)MPEGInfoHeader.SamplesPerFrame / (double)MPEGInfoHeader.SampleRate;
				int32 ApproxSegmentSize = 0;
				if (!MPEGInfoHeader.bIsVBR)
				{
					ApproxSegmentSize = MPEGInfoHeader.CBRFrameSize * (kSegmentDuration / FrameDuration);
				}
				else
				{
					int32 br = MPEGInfoHeader.AverageBitrate ? MPEGInfoHeader.AverageBitrate : MPEGInfoHeader.Bitrate;
					ApproxSegmentSize = br * kSegmentDuration / 8.0;
				}

				req->FileEndOffset = FMath::Clamp(ApproxByteOffset + ApproxSegmentSize, 0, MPEGInfoHeader.LastDataByte);
				req->bIsLastSegment = req->FileEndOffset >= MPEGInfoHeader.LastDataByte;
				req->MPEGHeaderMask = MPEGInfoHeader.MPEGHeaderMask;
				req->MPEGHeaderExpectedValue = MPEGInfoHeader.MPEGHeaderExpectedValue;
				req->CBRFrameSize = MPEGInfoHeader.CBRFrameSize;
				req->bIsAAC = MPEGInfoHeader.Type == FMPEGAudioInfoHeader::EType::ISO_14496_3;
				req->bIsVBR = MPEGInfoHeader.bIsVBR;
				req->bIsLive = false;
				req->Duration = kSegmentDuration;
				req->CodecInfo = MPEGInfoHeader.CodecInfo;
				// How many bytes will we be reading?
				int64 SegmentInternalSize = req->FileEndOffset - req->FileStartOffset;
				if (StartTime < PlayRangeEnd && SegmentInternalSize >= 12)
				{
					return IManifest::FResult(IManifest::FResult::EType::Found);
				}
				else
				{
					return IManifest::FResult(IManifest::FResult::EType::PastEOS);
				}
			}
			else
			{
				return IManifest::FResult(IManifest::FResult::EType::PastEOS);
			}
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::NotFound).SetErrorDetail(FErrorDetail().SetError(UEMEDIA_ERROR_INSUFFICIENT_DATA)
					   .SetFacility(Facility::EFacility::MPEGAudioPlaylist)
					   .SetCode(ERRCODE_MANIFEST_MPEGAUDIO_STARTSEGMENT_NOT_FOUND)
					   .SetMessage(FString::Printf(TEXT("Could not find start segment for time %lld, no valid tracks"), (long long int)StartPosition.Time.GetAsHNS())));
}

IManifest::FResult FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio::GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options)
{
	const FStreamSegmentRequestMPEGAudio* Request = static_cast<const FStreamSegmentRequestMPEGAudio*>(CurrentSegment.Get());
	if (Request)
	{
		// Check if the current request did not already go up to the end of the stream. If so there is no next segment.
		if (MPEGInfoHeader.bIsLive || (Request->FileEndOffset >= 0 && Request->FileEndOffset < MPEGInfoHeader.LastDataByte))
		{
			FPlayStartPosition dummyPos;
			FPlayerSequenceState seqState;
			dummyPos.Options = Options;
			dummyPos.Time = Request->LastSuccessfullyUsedPTS;
			seqState.SetSequenceIndex(Request->TimestampSequenceIndex);
			IManifest::FResult res = GetStartingSegment(OutSegment, seqState, dummyPos, ESearchType::Same, Request->LastSuccessfullyUsedBytePos);
			if (res.GetType() == IManifest::FResult::EType::Found)
			{
				FStreamSegmentRequestMPEGAudio* NextRequest = static_cast<FStreamSegmentRequestMPEGAudio*>(OutSegment.Get());
				NextRequest->bIsContinuationSegment = true;
				if (!MPEGInfoHeader.bIsLive)
				{
					NextRequest->bIsFirstSegment = false;
					NextRequest->EarliestPTS = Request->EarliestPTS;
					NextRequest->LastPTS = Request->LastPTS;
				}
				else
				{
					NextRequest->NumOverallRetries = Request->NumOverallRetries + 1;
				}
				return res;
			}
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::PastEOS);
}

IManifest::FResult FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio::GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData)
{
	const FStreamSegmentRequestMPEGAudio* Request = static_cast<const FStreamSegmentRequestMPEGAudio*>(CurrentSegment.Get());
	if (Request)
	{
		FPlayStartPosition dummyPos;
		FPlayerSequenceState seqState;
		dummyPos.Options = Options;
		dummyPos.Time = Request->LastSuccessfullyUsedPTS.IsValid() ? Request->LastSuccessfullyUsedPTS : Request->FirstPTS;
		seqState.SetSequenceIndex(Request->TimestampSequenceIndex);
		IManifest::FResult res = GetStartingSegment(OutSegment, seqState, dummyPos, ESearchType::Same, Request->LastSuccessfullyUsedBytePos);
		if (res.GetType() == IManifest::FResult::EType::Found)
		{
			FStreamSegmentRequestMPEGAudio* RetryRequest = static_cast<FStreamSegmentRequestMPEGAudio*>(OutSegment.Get());
			RetryRequest->bIsContinuationSegment = true;
			RetryRequest->NumOverallRetries = Request->NumOverallRetries + 1;
			RetryRequest->EarliestPTS = Request->EarliestPTS;
			RetryRequest->LastPTS = Request->LastPTS;
			return res;
		}
	}
	return IManifest::FResult(IManifest::FResult::EType::NotFound);
}


IManifest::FResult FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio::GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& SequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType)
{
	return GetStartingSegment(OutSegment, SequenceState, StartPosition, SearchType, -1);
}


void FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio::GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID)
{
	// This is not expected to be called. And if it does we return a dummy entry.
	OutAverageSegmentDuration.SetFromSeconds(5.0);
}

FErrorDetail FManifestMPEGAudioInternal::FAdaptationSetMPEGAudio::CreateFrom(const FMPEGAudioInfoHeader& InHeader, const FString& URL)
{
	Representation = MakeSharedTS<FRepresentationMPEGAudio>();
	FErrorDetail err = Representation->CreateFrom(InHeader, URL);
	if (err.IsOK())
	{
		CodecRFC6381 = Representation->GetCodecInformation().GetCodecSpecifierRFC6381();
		UniqueIdentifier = Representation->GetUniqueIdentifier();
		BCP47::ParseRFC5646Tag(LanguageTag, Representation->GetLanguage());
		bIsLivecast = InHeader.bIsLive;
	}
	return err;
}

bool FManifestMPEGAudioInternal::FAdaptationSetMPEGAudio::IsLowLatencyEnabled() const
{
#if USE_LOW_LATENCY_DESCRIPTOR
	return bIsLivecast;
#else
	return false;
#endif
}


FErrorDetail FManifestMPEGAudioInternal::FRepresentationMPEGAudio::CreateFrom(const FMPEGAudioInfoHeader& InHeader, const FString& URL)
{
	CodecInformation = InHeader.CodecInfo;

	// NOTE: This *MUST* be just a number since it gets parsed back out from a string into a number later! Do *NOT* prepend/append any string literals!!
	UniqueIdentifier = TEXT("1");

	// Get name from metadata if it exists
	if (InHeader.ID3v2.IsValid() && InHeader.ID3v2->HaveTag(Utils::Make4CC('T','I','T','2')))
	{
		MPEG::FID3V2Metadata::FItem Tit2;
		InHeader.ID3v2->GetTag(Tit2, Utils::Make4CC('T','I','T','2'));
		Name = Tit2.Value.GetValue<FString>();
	}
	else
	{
		Name = TEXT("Unknown");
	}

	// Language?
	if (InHeader.ID3v2.IsValid() && InHeader.ID3v2->HaveTag(Utils::Make4CC('T','L','A','N')))
	{
		MPEG::FID3V2Metadata::FItem TLan;
		InHeader.ID3v2->GetTag(TLan, Utils::Make4CC('T','L','A','N'));
		Language639_2 = TLan.Value.GetValue<FString>();
	}
	else
	{
		Language639_2 = TEXT("und");
	}

	Bitrate = InHeader.AverageBitrate ? InHeader.AverageBitrate : InHeader.Bitrate;
	Bitrate = Bitrate <= 0 ? 64000 : Bitrate;

	if (!CodecInformation.GetBitrate())
	{
		CodecInformation.SetBitrate(Bitrate);
	}
	return FErrorDetail();
}


} // namespace Electra
