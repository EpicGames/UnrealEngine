// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "Player/PlaybackTimeline.h"
#include "HTTP/HTTPManager.h"
#include "SynchronizedClock.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "Utilities/Utilities.h"
#include "Utilities/URLParser.h"
#include "Utilities/UtilsMPEG.h"
#include "MediaStreamMetadata.h"

namespace Electra
{

struct FMPEGAudioInfoHeader
{
	enum class EType
	{
		Unknown,
		ISO_11172_3,		// MPEG1/2 layer 1,2 and 3
		ISO_14496_3			// AAC
	};
	TSharedPtrTS<MPEG::FID3V2Metadata> ID3v2;
	EType Type = EType::Unknown;
	uint32 MPEGHeaderMask = 0;
	uint32 MPEGHeaderExpectedValue = 0;
	int32 MPEGVersion = 0;
	int32 MPEGLayer = 0;
	int32 SampleRate = 0;
	int32 NumChannels = 0;
	bool bIsVBR = false;
	int32 Bitrate = 0;
	int32 AverageBitrate = 0;
	int32 CBRFrameSize = 0;
	int32 SamplesPerFrame = 0;
	uint32 NumFrames = 0;
	int32 EncoderDelayStart = 0;
	int32 EncoderDelayEnd = 0;
	int64 FirstDataByte = -1;
	int64 LastDataByte = -1;
	bool bIsLive = false;
	bool bHaveTOC = false;
	TArray<uint8> TOC;
	int32 FramesPerSeekTableEntry = 0;
	TSharedPtrTS<TArray<uint32>> SeekTable;
	struct FMLLT
	{
		struct FTimeAndOffset
		{
			uint32 Offset = 0;
			uint32 Milliseconds = 0;
		};
		uint32 FramesBetweenReferences = 0;
		TArray<FTimeAndOffset> TimeAndOffsets;
	};
	TSharedPtrTS<FMLLT> MLLT;
	// Generated codec info from above values.
	FStreamCodecInformation CodecInfo;
	// Estimated duration
	FTimeValue EstimatedDuration;
	// HTTP response headers
	TMultiMap<FString, FString> HTTPResponseHeaders;
};

/**
 * This class represents the internal "manifest" or "playlist" of an MPEG audio file.
 */
class FManifestMPEGAudioInternal : public IManifest, public TSharedFromThis<FManifestMPEGAudioInternal, ESPMode::ThreadSafe>
{
public:
	FManifestMPEGAudioInternal(IPlayerSessionServices* InPlayerSessionServices);
	virtual ~FManifestMPEGAudioInternal();

	FErrorDetail Build(const FMPEGAudioInfoHeader& InHeader, const FString& URL);

	EMediaFormatType GetMediaFormatType() const override;
	bool ReloadsPlaylistPeriodically() const override;
	bool SupportsMultipleTracksPerKind() const override;
	bool OrderOfTracksCanChangeDynamically() const override;
	EType GetPresentationType() const override;
	EReplayEventType GetReplayType() const override
	{ return IManifest::EReplayEventType::NoReplay; }
	TSharedPtrTS<const FLowLatencyDescriptor> GetLowLatencyDescriptor() const override;
	FTimeValue CalculateCurrentLiveLatency(const FTimeValue& InCurrentPlaybackPosition, const FTimeValue& InEncoderLatency, bool bViaLatencyElement) const override;
	FTimeValue GetAnchorTime() const override
	{ return FTimeValue::GetZero(); }
	FTimeRange GetTotalTimeRange() const override
	{ return MediaAsset.IsValid() ? MediaAsset->GetTimeRange() : FTimeRange(); }
	FTimeRange GetSeekableTimeRange() const override
	{ return GetTotalTimeRange(); }
	FTimeRange GetPlaybackRange(EPlaybackRangeType InRangeType) const override;
	FTimeValue GetDuration() const override
	{ return MediaAsset.IsValid() ? MediaAsset->GetDuration() : FTimeValue(); }
	FTimeValue GetDefaultStartTime() const override
	{ return DefaultStartTime; }
	void ClearDefaultStartTime() override
	{ DefaultStartTime.SetToInvalid(); }
	FTimeValue GetDefaultEndTime() const override
	{ return DefaultEndTime; }
	void ClearDefaultEndTime() override
	{ DefaultEndTime.SetToInvalid(); }
	void GetTrackMetadata(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const override;
	void UpdateRunningMetaData(TSharedPtrTS<FMetadataParser> InUpdatedMetaData) override;
	FTimeValue GetMinBufferTime() const override;
	FTimeValue GetDesiredLiveLatency() const override;
	ELiveEdgePlayMode GetLiveEdgePlayMode() const override;

	TRangeSet<double> GetPossiblePlaybackRates(EPlayRateType InForType) const override;
	TSharedPtrTS<IProducerReferenceTimeInfo> GetProducerReferenceTimeInfo(int64 ID) const override;
	void UpdateDynamicRefetchCounter() override;
	void PrepareForLooping(int32 InNumLoopsToAdd) override
	{ }
	void TriggerClockSync(EClockSyncType InClockSyncType) override;
	void TriggerPlaylistRefresh() override;
	void ReachedStableBuffer() override
	{ }

	IStreamReader* CreateStreamReaderHandler() override;
	FResult FindPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
	FResult FindNextPlayPeriod(TSharedPtrTS<IPlayPeriod>& OutPlayPeriod, TSharedPtrTS<const IStreamSegment> CurrentSegment) override;

	class FRepresentationMPEGAudio : public IPlaybackAssetRepresentation
	{
	public:
		virtual ~FRepresentationMPEGAudio() = default;

		FErrorDetail CreateFrom(const FMPEGAudioInfoHeader& InHeader, const FString& URL);

		FString GetUniqueIdentifier() const override
		{ return UniqueIdentifier; }
		const FStreamCodecInformation& GetCodecInformation() const override
		{ return CodecInformation; }
		int32 GetBitrate() const override
		{ return Bitrate; }
		int32 GetQualityIndex() const override
		{ return 0; }
		bool CanBePlayed() const override
		{ return true; }
		const FString& GetName() const
		{ return Name; }
		void SetName(const FString& InNewName)
		{ Name = InNewName; }
		const FString& GetLanguage() const
		{ return Language639_2; }
		void SetLanguage(const FString& InNewName)
		{ Language639_2 = InNewName; }
	private:
		FStreamCodecInformation CodecInformation;
		FString UniqueIdentifier;
		FString Name;
		FString Language639_2;
		int32 Bitrate = 0;
	};

	class FAdaptationSetMPEGAudio : public IPlaybackAssetAdaptationSet
	{
	public:
		virtual ~FAdaptationSetMPEGAudio() = default;

		FErrorDetail CreateFrom(const FMPEGAudioInfoHeader& InHeader, const FString& URL);

		FString GetUniqueIdentifier() const override
		{ return UniqueIdentifier; }
		FString GetListOfCodecs() const override
		{ return CodecRFC6381; }
		const BCP47::FLanguageTag& GetLanguageTag() const override
		{ return LanguageTag; }
		int32 GetNumberOfRepresentations() const override
		{ return Representation.IsValid() ? 1 : 0; }
		bool IsLowLatencyEnabled() const override;
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByIndex(int32 RepresentationIndex) const override
		{ return RepresentationIndex == 0 ? Representation : TSharedPtrTS<IPlaybackAssetRepresentation>(); }
		TSharedPtrTS<IPlaybackAssetRepresentation> GetRepresentationByUniqueIdentifier(const FString& InUniqueIdentifier) const override
		{ return Representation.IsValid() && Representation->GetUniqueIdentifier() == InUniqueIdentifier ? Representation : TSharedPtrTS<IPlaybackAssetRepresentation>(); }
	private:
		TSharedPtrTS<FRepresentationMPEGAudio> Representation;
		BCP47::FLanguageTag LanguageTag;
		FString CodecRFC6381;
		FString UniqueIdentifier;
		bool bIsLivecast = false;
	};

	class FTimelineAssetMPEGAudio : public ITimelineMediaAsset, public TSharedFromThis<FTimelineAssetMPEGAudio, ESPMode::ThreadSafe>
	{
	public:
		FTimelineAssetMPEGAudio()
			: PlayerSessionServices(nullptr)
		{ }

		virtual ~FTimelineAssetMPEGAudio() = default;

		FErrorDetail Build(IPlayerSessionServices* InPlayerSessionServices, const FMPEGAudioInfoHeader& InHeader, const FString& URL);

		FTimeRange GetTimeRange() const override
		{
			FTimeRange tr;
			tr.Start.SetToZero();
			tr.End = GetDuration();
			return tr;
		}

		FTimeValue GetDuration() const override
		{
			return MPEGInfoHeader.EstimatedDuration;
		}

		FString GetAssetIdentifier() const override
		{ return FString("mpegaudio-asset.0"); }
		FString GetUniqueIdentifier() const override
		{ return FString("mpegaudio-media.0"); }
		int32 GetNumberOfAdaptationSets(EStreamType OfStreamType) const override
		{
			return OfStreamType == EStreamType::Audio ? AudioAdaptationSets.Num() : 0;
		}
		TSharedPtrTS<IPlaybackAssetAdaptationSet> GetAdaptationSetByTypeAndIndex(EStreamType OfStreamType, int32 AdaptationSetIndex) const override
		{
			return OfStreamType == EStreamType::Audio ? AdaptationSetIndex < AudioAdaptationSets.Num() ? AudioAdaptationSets[AdaptationSetIndex] : TSharedPtrTS<IPlaybackAssetAdaptationSet>() : TSharedPtrTS<IPlaybackAssetAdaptationSet>();
		}

		void GetMetaData(TArray<FTrackMetadata>& OutMetadata, EStreamType StreamType) const override
		{
			for(int32 i=0, iMax=GetNumberOfAdaptationSets(StreamType); i<iMax; ++i)
			{
				TSharedPtrTS<FAdaptationSetMPEGAudio> AdaptSet = StaticCastSharedPtr<FAdaptationSetMPEGAudio>(GetAdaptationSetByTypeAndIndex(StreamType, i));
				if (AdaptSet.IsValid())
				{
					FTrackMetadata tm;
					tm.ID = AdaptSet->GetUniqueIdentifier();
					tm.LanguageTagRFC5646 = AdaptSet->GetLanguageTag();
					tm.Kind = i==0 ? TEXT("main") : TEXT("translation");

					for(int32 j=0, jMax=AdaptSet->GetNumberOfRepresentations(); j<jMax; ++j)
					{
						TSharedPtrTS<FRepresentationMPEGAudio> Repr = StaticCastSharedPtr<FRepresentationMPEGAudio>(AdaptSet->GetRepresentationByIndex(j));
						if (Repr.IsValid())
						{
							FStreamMetadata sd;
							sd.Bandwidth = Repr->GetBitrate();
							sd.CodecInformation = Repr->GetCodecInformation();
							sd.ID = Repr->GetUniqueIdentifier();
							// There is only 1 "stream" per "track" so we can set the highest bitrate and codec info the same as the track.
							tm.HighestBandwidth = sd.Bandwidth;
							tm.HighestBandwidthCodec = sd.CodecInformation;
							tm.Label = Repr->GetName();
							tm.StreamDetails.Emplace(MoveTemp(sd));
						}
					}
					OutMetadata.Emplace(MoveTemp(tm));
				}
			}
		}

		void UpdateRunningMetaData(const FString& InKindOfValue, const FVariant& InNewValue) override
		{
			if (InKindOfValue.Equals(TEXT("Title")))
			{
				if (GetNumberOfAdaptationSets(EStreamType::Audio))
				{
					TSharedPtrTS<FAdaptationSetMPEGAudio> AdaptSet = StaticCastSharedPtr<FAdaptationSetMPEGAudio>(GetAdaptationSetByTypeAndIndex(EStreamType::Audio, 0));
					if (AdaptSet.IsValid() && AdaptSet->GetNumberOfRepresentations())
					{
						TSharedPtrTS<FRepresentationMPEGAudio> Repr = StaticCastSharedPtr<FRepresentationMPEGAudio>(AdaptSet->GetRepresentationByIndex(0));
						if (Repr.IsValid())
						{
							Repr->SetName(InNewValue.GetValue<FString>());
						}
					}
				}
				if (!MPEGInfoHeader.ID3v2.IsValid())
				{
					MPEGInfoHeader.ID3v2 = MakeSharedTS<MPEG::FID3V2Metadata>();
				}
				MPEGInfoHeader.ID3v2->GetTags().Add(Utils::Make4CC('T','I','T','2'), MPEG::FID3V2Metadata::FItem({.Value = InNewValue }));
			}
		}

		FResult GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType, int64 AtAbsoluteFilePos);
		FResult GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options);
		FResult GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData);
		FResult GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType);

		void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID);
		const FString& GetMediaURL() const
		{ return MediaURL; }

	private:
		void LogMessage(IInfoLog::ELevel Level, const FString& Message);
		IPlayerSessionServices* PlayerSessionServices = nullptr;
		FString MediaURL;
		TArray<TSharedPtrTS<FAdaptationSetMPEGAudio>> AudioAdaptationSets;
		FMPEGAudioInfoHeader MPEGInfoHeader;
	};


	class FPlayPeriodMPEGAudio : public IManifest::IPlayPeriod
	{
	public:
		FPlayPeriodMPEGAudio(TSharedPtrTS<FTimelineAssetMPEGAudio> InMediaAsset);
		virtual ~FPlayPeriodMPEGAudio();
		void SetStreamPreferences(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) override;
		EReadyState GetReadyState() override;
		void Load() override;
		void PrepareForPlay() override;
		int64 GetDefaultStartingBitrate() const override;
		TSharedPtrTS<FBufferSourceInfo> GetSelectedStreamBufferSourceInfo(EStreamType StreamType) override;
		FString GetSelectedAdaptationSetID(EStreamType StreamType) override;
		ETrackChangeResult ChangeTrackStreamPreference(EStreamType ForStreamType, const FStreamSelectionAttributes& StreamAttributes) override;
		TSharedPtrTS<ITimelineMediaAsset> GetMediaAsset() const override;
		void SelectStream(const FString& AdaptationSetID, const FString& RepresentationID, int32 QualityIndex, int32 MaxQualityIndex) override;
		void TriggerInitSegmentPreload(const TArray<FInitSegmentPreload>& InitSegmentsToPreload) override;
		FResult GetStartingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
		FResult GetContinuationSegment(TSharedPtrTS<IStreamSegment>& OutSegment, EStreamType StreamType, const FPlayerSequenceState& LoopState, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
		FResult GetNextSegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options) override;
		FResult GetRetrySegment(TSharedPtrTS<IStreamSegment>& OutSegment, TSharedPtrTS<const IStreamSegment> CurrentSegment, const FPlayStartOptions& Options, bool bReplaceWithFillerData) override;
		FResult GetLoopingSegment(TSharedPtrTS<IStreamSegment>& OutSegment, const FPlayerSequenceState& InSequenceState, const FPlayStartPosition& StartPosition, ESearchType SearchType) override;
		void IncreaseSegmentFetchDelay(const FTimeValue& IncreaseAmount) override;
		void GetAverageSegmentDuration(FTimeValue& OutAverageSegmentDuration, const FString& AdaptationSetID, const FString& RepresentationID) override;

	private:
		void SelectInitialStream(EStreamType StreamType);
		TSharedPtrTS<FTrackMetadata> SelectMetadataForAttributes(EStreamType StreamType, const FStreamSelectionAttributes& InAttributes);
		void MakeBufferSourceInfoFromMetadata(EStreamType StreamType, TSharedPtrTS<FBufferSourceInfo>& OutBufferSourceInfo, TSharedPtrTS<FTrackMetadata> InMetadata);

		TWeakPtrTS<FTimelineAssetMPEGAudio> MediaAsset;
		FStreamSelectionAttributes AudioPreferences;
		TSharedPtrTS<FTrackMetadata> SelectedAudioMetadata;
		TSharedPtrTS<FBufferSourceInfo> AudioBufferSourceInfo;
		EReadyState CurrentReadyState;
	};

	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

	const TArray<FURL_RFC3986::FQueryParam>& GetURLFragmentComponents() const
	{
		return URLFragmentComponents;
	}

	void SetURLFragmentComponents(TArray<FURL_RFC3986::FQueryParam> InURLFragmentComponents)
	{
		URLFragmentComponents = MoveTemp(InURLFragmentComponents);
	}

	const TArray<FURL_RFC3986::FQueryParam>& GetURLFragmentComponents()
	{
		return URLFragmentComponents;
	}

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	TSharedPtrTS<FTimelineAssetMPEGAudio> MediaAsset;
	TArray<FURL_RFC3986::FQueryParam> URLFragmentComponents;
	FTimeValue DefaultStartTime;
	FTimeValue DefaultEndTime;
	FMPEGAudioInfoHeader MPEGInfoHeader;
	TSharedPtrTS<FLowLatencyDescriptor> LatencyDescriptor;
};


} // namespace Electra
