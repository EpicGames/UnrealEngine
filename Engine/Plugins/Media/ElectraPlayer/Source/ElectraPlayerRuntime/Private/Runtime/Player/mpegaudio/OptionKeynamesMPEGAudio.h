// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{

namespace MPEGAudio
{
const FName OptionKeyMPEGAudioLoadConnectTimeout(TEXT("mpegaudio_connection_timeout"));
const FName OptionKeyMPEGAudioLoadNoDataTimeout(TEXT("mpegaudio_nodata_timeout"));
const FName OptionKeyMPEGAudioLoadTimeout(TEXT("mpegaudio_load_timeout"));
const FName OptionKeyMPEGAudioMaxPreloadBufferDuration(TEXT("mpegaudio_max_preload_buffer_duration"));
}

} // namespace Electra
