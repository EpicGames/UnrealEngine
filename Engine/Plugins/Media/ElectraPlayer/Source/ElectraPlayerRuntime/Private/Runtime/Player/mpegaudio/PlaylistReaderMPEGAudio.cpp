// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "HTTP/HTTPResponseCache.h"
#include "Http.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Serialization/Archive.h"
#include "Serialization/ArrayReader.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMPEGAudio.h"
#include "Utilities/Utilities.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/URLParser.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/mpegaudio/ManifestMPEGAudio.h"
#include "Player/mpegaudio/OptionKeynamesMPEGAudio.h"
#include "Utilities/ElectraBitstream.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"

#define ERRCODE_MPEGAUDIO_INVALID_FILE		1
#define ERRCODE_MPEGAUDIO_DOWNLOAD_ERROR	2


DECLARE_CYCLE_STAT(TEXT("FPlaylistReaderMPEGAudio_WorkerThread"), STAT_ElectraPlayer_MPEGAudio_PlaylistWorker, STATGROUP_ElectraPlayer);


namespace Electra
{

class FPlaylistReaderMPEGAudio : public IPlaylistReaderMPEGAudio, public FMediaThread
{
public:
	FPlaylistReaderMPEGAudio();
	void Initialize(IPlayerSessionServices* PlayerSessionServices);

	virtual ~FPlaylistReaderMPEGAudio();

	EMediaFormatType GetMediaFormatType() const override
	{ return EMediaFormatType::MPEGAudio; }
	void Close() override;
	void HandleOnce() override;

	/**
	 * Returns the type of playlist format.
	 *
	 * @return "mpegaudio"
	 */
	const FString& GetPlaylistType() const override
	{
		static FString Type("mpegaudio");
		return Type;
	}

	/**
	 * Loads and parses the playlist.
	 *
	 * @param URL URL of the playlist to load
	 * @param InReplayEventParams URL fragments pertaining to replay event which have been removed from the source URL
	 */
	void LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams) override;

	/**
	 * Returns the URL from which the playlist was loaded (or supposed to be loaded).
	 *
	 * @return The playlist URL
	 */
	FString GetURL() const override;

	/**
	 * Returns an interface to the manifest.
	 *
	 * @return A shared manifest interface pointer.
	 */
	TSharedPtrTS<IManifest> GetManifest() override;

private:
	void StartWorkerThread();
	void StopWorkerThread();
	void WorkerThread();

	void PostError(const FString& Message, uint16 Code, UEMediaError Error = UEMEDIA_ERROR_OK);
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);

	struct FRequest : public TSharedFromThis<FRequest, ESPMode::ThreadSafe>
	{
		static constexpr int32 GetMinProbeSize()
		{
			return 32 << 10;
		}
		static constexpr int32 GetMaxProbeSize()
		{
			return 1 << 20;
		}
		static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
		{
			return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
		}
		static uint32 GetUINT32BE(const uint8* Data)
		{
			return (static_cast<uint32>(Data[0]) << 24) | (static_cast<uint32>(Data[1]) << 16) | (static_cast<uint32>(Data[2]) << 8) | static_cast<uint32>(Data[3]);
		}

		FRequest()
		{
			Events.Resize(4);
		}
		void OnProcessRequestComplete(FHttpRequestPtr InSourceHttpRequest, FHttpResponsePtr InHttpResponse, bool bInSucceeded);
		void OnHeaderReceived(FHttpRequestPtr InSourceHttpRequest, const FString& InHeaderName, const FString& InHeaderValue);
		void OnStatusCodeReceived(FHttpRequestPtr InSourceHttpRequest, int32 InHttpStatusCode);
		void OnProcessRequestStream(void* InDataPtr, int64& InLength);
		void FindSyncMarkers();
		void Cancel();
		bool Validate(FMPEGAudioInfoHeader& InOutInfoHeader, FString& OutError, TArray<FString>& OutWarnings);

		enum class EEvent
		{
			None,
			Abort,
			HaveProbeData,
			Finished,
		};
		TMediaMessageQueueWithTimeout<EEvent> Events;
		enum class EResult
		{
			Running,
			Succeeded,
			Aborted,
			Failed,
		};
		EResult Result = EResult::Running;

		TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> Handle;
		TMultiMap<FString, FString> Headers;
		TArray<uint8> Buffer;
		TArray<int32> SyncMarkerOffsets;
		int64 ContentLength = -1;
		int32 StatusCode = -1;
		int32 ProbeSize = -1;
		int32 CurrentSyncMarkerCheckPos = 0;
		bool bSentHaveProbeDataMsg = false;
	};

	IPlayerSessionServices* PlayerSessionServices = nullptr;
	FString PlaylistURL;
	FString URLFragment;
	FMediaEvent WorkerThreadQuitSignal;
	bool bIsWorkerThreadStarted = false;
	HTTP::FConnectionInfo ConnectionInfo;
	volatile bool bAbort = false;
	FErrorDetail LastErrorDetail;

	FMPEGAudioInfoHeader InfoHeader;
	TSharedPtrTS<FManifestMPEGAudioInternal> Manifest;
};


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

TSharedPtrTS<IPlaylistReader> IPlaylistReaderMPEGAudio::Create(IPlayerSessionServices* PlayerSessionServices)
{
	TSharedPtrTS<FPlaylistReaderMPEGAudio> PlaylistReader = MakeSharedTS<FPlaylistReaderMPEGAudio>();
	if (PlaylistReader)
	{
		PlaylistReader->Initialize(PlayerSessionServices);
	}
	return PlaylistReader;
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


FPlaylistReaderMPEGAudio::FPlaylistReaderMPEGAudio()
	: FMediaThread("ElectraPlayer::MPEGAudio Playlist")
{
}

FPlaylistReaderMPEGAudio::~FPlaylistReaderMPEGAudio()
{
	Close();
}

FString FPlaylistReaderMPEGAudio::GetURL() const
{
	return PlaylistURL;
}

TSharedPtrTS<IManifest> FPlaylistReaderMPEGAudio::GetManifest()
{
	return Manifest;
}

void FPlaylistReaderMPEGAudio::Initialize(IPlayerSessionServices* InPlayerSessionServices)
{
	PlayerSessionServices = InPlayerSessionServices;
}

void FPlaylistReaderMPEGAudio::Close()
{
	bAbort = true;
	StopWorkerThread();
}

void FPlaylistReaderMPEGAudio::HandleOnce()
{
	// No-op. This class is using a dedicated thread to read data from the stream
	// which can stall at any moment and thus not lend itself to a tickable instance.
}

void FPlaylistReaderMPEGAudio::StartWorkerThread()
{
	check(!bIsWorkerThreadStarted);
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FPlaylistReaderMPEGAudio::WorkerThread));
	bIsWorkerThreadStarted = true;
}

void FPlaylistReaderMPEGAudio::StopWorkerThread()
{
	if (bIsWorkerThreadStarted)
	{
		WorkerThreadQuitSignal.Signal();
		ThreadWaitDone();
		ThreadReset();
		bIsWorkerThreadStarted = false;
	}
}

void FPlaylistReaderMPEGAudio::PostError(const FString& InMessage, uint16 InCode, UEMediaError InError)
{
	LastErrorDetail.Clear();
	LastErrorDetail.SetError(InError != UEMEDIA_ERROR_OK ? InError : UEMEDIA_ERROR_DETAIL);
	LastErrorDetail.SetFacility(Facility::EFacility::MPEGAudioPlaylistReader);
	LastErrorDetail.SetCode(InCode);
	LastErrorDetail.SetMessage(InMessage);
	check(PlayerSessionServices);
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostError(LastErrorDetail);
	}
}

void FPlaylistReaderMPEGAudio::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	if (PlayerSessionServices)
	{
		PlayerSessionServices->PostLog(Facility::EFacility::MPEGAudioPlaylistReader, Level, Message);
	}
}

void FPlaylistReaderMPEGAudio::LoadAndParse(const FString& InURL, const Playlist::FReplayEventParams& InReplayEventParams)
{
	PlaylistURL = InURL;
	StartWorkerThread();
}


void FPlaylistReaderMPEGAudio::FRequest::OnProcessRequestComplete(FHttpRequestPtr InSourceHttpRequest, FHttpResponsePtr InHttpResponse, bool bInSucceeded)
{
	Result = bInSucceeded ? EResult::Succeeded : Result != EResult::Aborted ? EResult::Failed : Result;
	Events.SendMessage(EEvent::Finished);
}

void FPlaylistReaderMPEGAudio::FRequest::OnHeaderReceived(FHttpRequestPtr InSourceHttpRequest, const FString& InHeaderName, const FString& InHeaderValue)
{
	if (InHeaderName.Len())
	{
		// Headers are treated as case insensitive, so for simpliciy in later comparisons convert it to all lowercase here.
		Headers.Add(InHeaderName.ToLower(), InHeaderValue);
		// Content length?
		if (InHeaderName.Equals(TEXT("content-length"), ESearchCase::IgnoreCase))
		{
			LexFromString(ContentLength, *InHeaderValue);
		}
	}
	else
	{
		Headers.Add(TEXT("_"), InHeaderValue);
	}
}

void FPlaylistReaderMPEGAudio::FRequest::OnStatusCodeReceived(FHttpRequestPtr InSourceHttpRequest, int32 InHttpStatusCode)
{
	if (InHttpStatusCode > 0 && InHttpStatusCode < 600)
	{
		StatusCode = InHttpStatusCode;
	}
}

void FPlaylistReaderMPEGAudio::FRequest::FindSyncMarkers()
{
	if (Buffer.Num() && CurrentSyncMarkerCheckPos < ProbeSize)
	{
		const uint8* Base = Buffer.GetData() + CurrentSyncMarkerCheckPos;
		const uint8* End = Buffer.GetData() + Buffer.Num() - 2;
		for(; Base < End; ++Base)
		{
			if (*Base != 0xff)
			{
				continue;
			}
			// Check for ISO 11172-3 header
			if (Base+2 < End && (Base[1] & 0xe0) == 0xe0		// sync marker (11 1-bits)
				&& ((Base[1] >> 3) & 3) >= 2					// audio version 1 or 2 (2.5 not supported)
				&& ((Base[1] >> 1) & 3) != 0					// layer index 1, 2 or 3
				&& (Base[2] >> 4) != 15							// bitrate index not 15
				&& (Base[2] & 0x0c) != 0x0c)					// sample rate index not 3
			{
				SyncMarkerOffsets.Add(Base - Buffer.GetData());
				Base += 2;
			}
			// Check for ISO 14496-3 ADTS header
			else if (Base+2 < End && (Base[1] & 0xf0) == 0xf0		// sync marker (12 1-bits)
				&& ((Base[1] & 0x8) == 0)							// MPEG version must be MPEG-4
				&& ((Base[1] & 0x6) == 0)							// layer must be 0
				&& ((Base[2] >> 2) & 7) < 13)						// sampling frequency index not a reserved value
			{
				SyncMarkerOffsets.Add(Base - Buffer.GetData());
				Base += 2;
			}
		}
		CurrentSyncMarkerCheckPos = Buffer.Num() - 1;
	}
}

void FPlaylistReaderMPEGAudio::FRequest::OnProcessRequestStream(void *InDataPtr, int64& InLength)
{
	if (StatusCode < 200 || StatusCode >= 300)
	{
		return;
	}

	// Add new data unconditionally. This won't be overly much so not to worry.
	if (InDataPtr && InLength > 0)
	{
		int32 MaxToCopy = ProbeSize < 0 ? static_cast<int32>(InLength) : Utils::Min(ProbeSize - Buffer.Num(), static_cast<int32>(InLength));
		Buffer.Append(reinterpret_cast<const uint8*>(InDataPtr), MaxToCopy);
	}

	// When we receive the first chunk of data we check if it starts with an ID3v2 tag.
	if (ProbeSize < 0 && Buffer.Num() >= 10)
	{
		int32 ID3HeaderSize = 0;
		const uint8* const HeaderData = Buffer.GetData();
		if (HeaderData[0] == 'I' && HeaderData[1] == 'D' && HeaderData[2] == '3' &&
			HeaderData[3] != 0xff && HeaderData[4] != 0xff && HeaderData[6] < 0x80 && HeaderData[7] < 0x80 && HeaderData[8] < 0x80 && HeaderData[9] < 0x80)
		{
			ID3HeaderSize = (int32) (10U + (HeaderData[6] << 21U) + (HeaderData[7] << 14U) + (HeaderData[8] << 7U) + HeaderData[9]);
			CurrentSyncMarkerCheckPos = ID3HeaderSize;
		}

		ProbeSize = GetMinProbeSize() + ID3HeaderSize;
		if (ContentLength >= 0)
		{
			ProbeSize = Utils::Min(ProbeSize, (int32)ContentLength);
		}
	}

	if (!bSentHaveProbeDataMsg && Buffer.Num() >= ProbeSize)
	{
		bSentHaveProbeDataMsg = true;
		Events.SendMessage(EEvent::HaveProbeData);
	}
}

void FPlaylistReaderMPEGAudio::FRequest::Cancel()
{
	Result = Result == EResult::Running ? EResult::Aborted : Result;
	if (Handle.IsValid())
	{
		Handle->CancelRequest();
	}
}

bool FPlaylistReaderMPEGAudio::FRequest::Validate(FMPEGAudioInfoHeader& InOutInfoHeader, FString& OutError, TArray<FString>& OutWarnings)
{
	check(Buffer.Num() >= 8192);
	if (Buffer.Num() < 8192)
	{
		OutError = FString::Printf(TEXT("Insufficient buffer data to validate file contents."));
		return false;
	}

	const uint8* const HeaderData = Buffer.GetData();

	// Check if there is an ID3 header.
	if (SyncMarkerOffsets.Num() && SyncMarkerOffsets[0] && HeaderData[0] == 'I' && HeaderData[1] == 'D' && HeaderData[2] == '3' &&
		HeaderData[3] != 0xff && HeaderData[4] != 0xff && HeaderData[6] < 0x80 && HeaderData[7] < 0x80 && HeaderData[8] < 0x80 && HeaderData[9] < 0x80)
	{
		int32 ID3HeaderSize = (int32) (10U + (HeaderData[6] << 21U) + (HeaderData[7] << 14U) + (HeaderData[8] << 7U) + HeaderData[9]);
		if (ID3HeaderSize >= ProbeSize)
		{
			// ID3v2 header is larger than we have probe data for, so this data cannot be validated.
			OutError = FString::Printf(TEXT("ID3v2 header larger than buffer data. Cannot validate file contents."));
			return false;
		}
		// Parse the header.
		InOutInfoHeader.ID3v2 = MakeSharedTS<MPEG::FID3V2Metadata>();
		if (!InOutInfoHeader.ID3v2->Parse(HeaderData, ID3HeaderSize))
		{
			InOutInfoHeader.ID3v2.Reset();
			OutWarnings.Emplace(FString::Printf(TEXT("Could not parse the ID3v2 header, ignoring.")));
		}
		while(SyncMarkerOffsets.Num() && SyncMarkerOffsets[0] < ID3HeaderSize)
		{
			SyncMarkerOffsets.RemoveAt(0);
		}
		// Is there an MLLT entry?
		if (InOutInfoHeader.ID3v2.IsValid() && InOutInfoHeader.ID3v2->HaveTag(Utils::Make4CC('M','L','L','T')))
		{
			MPEG::FID3V2Metadata::FItem MLLTBlob;
			InOutInfoHeader.ID3v2->GetTag(MLLTBlob, Utils::Make4CC('M','L','L','T'));
			TArray<uint8> MLLT = MLLTBlob.Value.GetValue<TArray<uint8>>();
			FBitstreamReader br(MLLT.GetData(), MLLT.Num());
			InOutInfoHeader.MLLT = MakeSharedTS<FMPEGAudioInfoHeader::FMLLT>();
			InOutInfoHeader.MLLT->FramesBetweenReferences = br.GetBits(16);
			uint32 BytesBetweenReference = br.GetBits(24);
			uint32 MillisecondsBetweenReference = br.GetBits(24);
			uint32 nbDevBytes = br.GetBits(8);
			uint32 nbDevMillis = br.GetBits(8);
			check(nbDevBytes <= 32);
			check(nbDevMillis <= 32);
			check((nbDevBytes + nbDevMillis) % 4 == 0);
			int32 NumEntries = br.GetRemainingBits() / (nbDevBytes + nbDevMillis);
			InOutInfoHeader.MLLT->TimeAndOffsets.AddDefaulted(NumEntries + 1);
			uint32 Offset = 0;
			uint32 Milliseconds = 0;
			for(int32 i=1; i<=NumEntries; ++i)
			{
				Offset += BytesBetweenReference + br.GetBits(nbDevBytes);
				Milliseconds += MillisecondsBetweenReference + br.GetBits(nbDevMillis);
				InOutInfoHeader.MLLT->TimeAndOffsets[i].Offset = Offset;
				InOutInfoHeader.MLLT->TimeAndOffsets[i].Milliseconds = Milliseconds;
			}
			check(br.GetRemainingBits() == 0);
		}
	}

	// Try to locate one of the special info headers
	auto IsXingHeader = [](const uint32 InHeaderValue, const uint8* InData, int32& InOutOffset, uint32& OutHeader) -> bool
	{
		const int32 Version = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetVersion(InHeaderValue);
		const int32 NumChannels = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetChannelCount(InHeaderValue);
		int32 Off = InOutOffset + 4 + (Version == 1 ? NumChannels == 1 ? 17 : 32 : NumChannels == 1 ? 9 : 17);
		const uint32 Hdr = GetUINT32BE(InData + Off);
		if (Hdr == Make4CC('X','i','n','g') || Hdr == Make4CC('I','n','f','o'))
		{
			OutHeader = Hdr;
			InOutOffset = Off;
			return true;
		}
		return false;
	};

	auto IsVBRIHeader = [](const uint32 InHeaderValue, const uint8* InData, int32& InOutOffset, uint32& OutHeader) -> bool
	{
		int32 Off = 4+ InOutOffset + 32;
		const uint32 Hdr = GetUINT32BE(InData + Off);
		if (Hdr == Make4CC('V','B','R','I'))
		{
			OutHeader = Hdr;
			InOutOffset = Off;
			return true;
		}
		return false;
	};

	int32 InfoHeaderOffset = -1;
	for(int32 nSyncMarker=0; InfoHeaderOffset < 0 && nSyncMarker<SyncMarkerOffsets.Num()-2; ++nSyncMarker)
	{
		int32 Off = SyncMarkerOffsets[nSyncMarker];
		const uint32 HeaderValue = GetUINT32BE(HeaderData + Off);

		// These exist only in Layer III
		const int32 Layer = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetLayer(HeaderValue);
		if (Layer == 3)
		{
			uint32 HeaderName = 0;
			// Test Xing header
			if (IsXingHeader(HeaderValue, HeaderData, Off, HeaderName))
			{
				const int32 Version = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetVersion(HeaderValue);
				const int32 NumChannels = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetChannelCount(HeaderValue);
				InfoHeaderOffset = SyncMarkerOffsets[nSyncMarker];
				InOutInfoHeader.MPEGHeaderExpectedValue = HeaderValue & 0xfffe0c00;
				InOutInfoHeader.MPEGVersion = Version;
				InOutInfoHeader.MPEGLayer = Layer;
				InOutInfoHeader.SampleRate = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplingRate(HeaderValue);
				InOutInfoHeader.NumChannels = NumChannels;
				InOutInfoHeader.Bitrate = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetBitrate(HeaderValue);

				uint32 NumFrames = 0;
				uint32 NumBytes = 0;
				int32 VBRScale = -1;
				InOutInfoHeader.bIsVBR = HeaderName == Make4CC('X','i','n','g');
				if (!InOutInfoHeader.bIsVBR)
				{
					InOutInfoHeader.CBRFrameSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetFrameSize(HeaderValue, 1);
				}
				InOutInfoHeader.SamplesPerFrame = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplesPerFrame(HeaderValue);

				uint32 Flags = GetUINT32BE(HeaderData + Off + 4);
				Off += 8;
				if ((Flags & 1) != 0)
				{
					NumFrames = GetUINT32BE(HeaderData + Off);
					Off += 4;
					InOutInfoHeader.NumFrames = NumFrames;
				}
				if ((Flags & 2) != 0)
				{
					NumBytes = GetUINT32BE(HeaderData + Off);
					Off += 4;
				}
				InOutInfoHeader.bHaveTOC = (Flags & 4) != 0;
				if (InOutInfoHeader.bHaveTOC)
				{
					InOutInfoHeader.TOC.AddUninitialized(100);
					FMemory::Memcpy(InOutInfoHeader.TOC.GetData(), HeaderData + Off, 100);
					Off += 100;
				}
				if ((Flags & 8) != 0)
				{
					VBRScale = (int32) GetUINT32BE(HeaderData + Off);
					Off += 4;
				}
				// Check if there is a `LAME` header following.
				if (GetUINT32BE(HeaderData + Off) == Make4CC('L','A','M','E'))
				{
					uint32 VersionMajor = HeaderData[Off + 4] - 0x30;
					uint32 VersionMinor = (HeaderData[Off + 5] == '.' ? 0 : HeaderData[Off + 5] - 0x30) * 100 + (HeaderData[Off + 6] - 0x30) * 10 + (HeaderData[Off + 7] - 0x30);
					// Additional info added with version 3.90
					if (VersionMajor > 3 || (VersionMajor == 3 && VersionMinor >= 90))
					{
						FBitstreamReader br(HeaderData + Off, 208, 9, 0);
						uint32 InfoTagRevision = br.GetBits(4);			// 15=reserved
						uint32 VBRMethod = br.GetBits(4);				// 0=unknown, 1=CBR, 2=ABR, 3-6=VBR method 1-4, 8=CBR 2 pass, 9=ABR 2 pass, 15=reserved
						uint32 LowPassFreq = br.GetBits(8) * 100;		// 0=unknown
						union UReplayGain { uint32 u; float f; } rg;
						rg.u = br.GetBits(32);
						float PeakSignalAmplitude = rg.f;				// 0.0 = unknown
						uint32 RadioReplayGain = br.GetBits(16);		// AAAAAAAAASOOONNN, A=Absolute gain adjustment, S=sign bit, O=originator (0=not set, 1=artist, 2=user, 3=model, 4=RMS), N=name (0=not set, 1=radio, 2=audiophile)
						uint32 AudiophileReplayGain = br.GetBits(16);	// AAAAAAAAASOOONNN
						uint32 ATHType = br.GetBits(4);
						uint32 NoGapContPrevTrack = br.GetBits(1);
						uint32 NoGapContNextTrack = br.GetBits(1);
						uint32 NsSafeJoint = br.GetBits(1);
						uint32 NsPsyTune = br.GetBits(1);
						uint32 ABR = br.GetBits(8);						// Average bitrate if VBRMethod==2|9, CBR rate for 1|8, min VBR rate otherwise, 0=unknown, 255= >=255 kbps
						uint32 EncoderDelayS = br.GetBits(12);			// Number of prepended silent samples
						uint32 EncoderDelayE = br.GetBits(12);			// Number of appended silent samples
						uint32 SourceSampleRate = br.GetBits(2);		// 0= <=32kHz, 1=44.1kHz, 2=48kHz, 3= >48kHz
						uint32 UnwiseSettingsUsed = br.GetBits(1);
						uint32 StereoMode = br.GetBits(3);				// 0=mono, 1=stereo, 2=dual, 3=joint, 4=force, 5=auto, 6=intensity, 7=other
						uint32 NoiseShaping = br.GetBits(2);			// 0-3 = noise shaping mode 0-3
						uint32 MP3GainSign = br.GetBits(1);				// Sign of gain (value is not 2's complement!)
						uint32 MP3Gain = br.GetBits(7);					// Gain that was applied (without clipping/clamping). Factor was 2^(x/4.0). Used to undo scaling if necessary.
						br.SkipBits(2);	// unused
						uint32 SurroundInfo = br.GetBits(3);			// 0=no info, 1=DPL, 2=DPL2, 3=Ambisonic
						uint32 Preset = br.GetBits(11);					// 0=unknown, LAME internal preset number otherwise
						uint32 MusicLength = br.GetBits(32);			// number of bytes in file including this block, excluding any earlier blocks and ID3v2 tag and ID3v tag at the end. If 0 file >4GiB or unknown.
						uint32 MusicCRC = br.GetBits(16);				// CRC of the music data, starting with the next block after this one and up to the last block excluding IDv3 tag at the end
						uint32 InfoTagCRC = br.GetBits(16);				// CRC of this header

						// VBR mismatch?
						if ((InOutInfoHeader.bIsVBR && (VBRMethod == 1 || VBRMethod == 8)) ||						// VBR via Xing, but CBR here?
							(!InOutInfoHeader.bIsVBR && ((VBRMethod >= 2 && VBRMethod <= 6) || VBRMethod == 9)))	// CBR via Info, but VBR here?
						{
							OutWarnings.Emplace(FString::Printf(TEXT("VBR/CBR mismatch")));
						}
						InOutInfoHeader.AverageBitrate = VBRMethod == 1 || VBRMethod == 8 ? 0 : ABR * 1000;
						if (MusicLength)
						{
							// Mismatch?
							if (NumBytes && NumBytes != MusicLength)
							{
								OutWarnings.Emplace(FString::Printf(TEXT("Mismatching music length in LAME tag")));
							}
							// Trust this info more than the one above.
							NumBytes = MusicLength;
						}
						InOutInfoHeader.EncoderDelayStart = EncoderDelayS;
						InOutInfoHeader.EncoderDelayEnd = EncoderDelayE;
					}
				}

				InOutInfoHeader.FirstDataByte = SyncMarkerOffsets[nSyncMarker];
				if (NumBytes)
				{
					InOutInfoHeader.LastDataByte = InOutInfoHeader.FirstDataByte + (int64)NumBytes;
				}
			}
			// Test VBRI header
			else if (IsVBRIHeader(HeaderValue, HeaderData, Off, HeaderName))
			{
				FBitstreamReader br(HeaderData + Off + 4, Utils::Max(0, ProbeSize-Off));
				int32 VersionId = (int32)br.GetBits(16);
				int32 Delay = (int32)br.GetBits(16);
				int32 QualityIndicator = (int32)br.GetBits(16);
				uint32 NumBytes = br.GetBits(32);
				uint32 NumFrames = br.GetBits(32);
				int32 NumTOCEntries = (int32)br.GetBits(16);
				uint16 TOCTableScale = (uint16)br.GetBits(16);
				int32 TableEntrySize = (int32)br.GetBits(16);
				int32 FramesPerTableEntry = (int32)br.GetBits(16);
				if (br.GetRemainingByteLength() >= TableEntrySize * NumTOCEntries)
				{
					if (VersionId == 1 && TOCTableScale && TableEntrySize <= 4 && FramesPerTableEntry)
					{
						InOutInfoHeader.SeekTable = MakeSharedTS<TArray<uint32>>();
						InOutInfoHeader.FramesPerSeekTableEntry = FramesPerTableEntry;
						InOutInfoHeader.SeekTable->AddUninitialized(NumTOCEntries);
						const int32 nb = 8 * TableEntrySize;
						for(int32 j=0; j<NumTOCEntries; ++j)
						{
							InOutInfoHeader.SeekTable->operator[](j) = br.GetBits(nb) * TOCTableScale;
						}
					}
					else
					{
						OutError = FString::Printf(TEXT("Unsupported VBRI header. Check values!"));
						return false;
					}
				}
				else
				{
					OutError = FString::Printf(TEXT("VBRI seek table larger than available probe data"));
					return false;
				}
				InOutInfoHeader.NumFrames = NumFrames;
				InOutInfoHeader.bIsVBR = true;

				int32 VBRIHeaderEndOffset = reinterpret_cast<const uint8*>(br.GetRemainingData()) - HeaderData;
				while(SyncMarkerOffsets.Num() && SyncMarkerOffsets[0] < VBRIHeaderEndOffset)
				{
					SyncMarkerOffsets.RemoveAt(0);
					--nSyncMarker;
				}
				if (SyncMarkerOffsets.Num())
				{
					InOutInfoHeader.FirstDataByte = SyncMarkerOffsets[0];
					if (NumBytes)
					{
						InOutInfoHeader.LastDataByte = SyncMarkerOffsets[0] + (int64)NumBytes;
					}
				}
				else
				{
					OutError = FString::Printf(TEXT("No further sync marker found after VBRI header. Probe data too small?"));
					return false;
				}
			}
		}
	}



	// Iterate all the headers that resemble an mpeg audio header and count those that are identical
	// except for slight differences (like padding, private, copyright bit etc.)
	TMap<uint32, int32> HeaderMap1, HeaderMap2;
	for(int32 i=0; i<SyncMarkerOffsets.Num()-2; ++i)
	{
		int32 Off = SyncMarkerOffsets[i];
		uint32 HeaderValue = GetUINT32BE(HeaderData + Off);
		++HeaderMap1.FindOrAdd(HeaderValue & 0xfffe0c00, 0);
		++HeaderMap2.FindOrAdd(HeaderValue & 0xfffefd00, 0);
	}
	HeaderMap1.ValueSort([](const int32& e1, const int32& e2) { return e1 > e2; });
	HeaderMap2.ValueSort([](const int32& e1, const int32& e2) { return e1 > e2; });

	int32 FirstHeaderOffset1 = InfoHeaderOffset;
	int32 FirstHeaderOffset2 = InfoHeaderOffset;
	bool bProbe11172_3 = false;
	bool bProbe14496_3 = false;
	ElectraDecodersUtil::MPEG::AACUtils::ADTSheader FirstADTSHeader;
	// Check ISO 11172-3
	if (HeaderMap1.Num())
	{
		InOutInfoHeader.MPEGHeaderMask = 0xfffe0c00;
		if (!InOutInfoHeader.MPEGHeaderExpectedValue)
		{
			InOutInfoHeader.MPEGHeaderExpectedValue = HeaderMap1.CreateConstIterator().Key() & InOutInfoHeader.MPEGHeaderMask;
		}
		// Locate the first matching header.
		FirstHeaderOffset1 = InfoHeaderOffset;
		for(int32 i=0; FirstHeaderOffset1<0 && i<SyncMarkerOffsets.Num()-2; ++i)
		{
			const int32 Off = SyncMarkerOffsets[i];
			if ((GetUINT32BE(HeaderData + Off) & InOutInfoHeader.MPEGHeaderMask) == InOutInfoHeader.MPEGHeaderExpectedValue)
			{
				FirstHeaderOffset1 = Off;
				break;
			}
		}

		const int32 LastHeaderOffset = SyncMarkerOffsets.Num() >= 2 ? SyncMarkerOffsets[SyncMarkerOffsets.Num()-2] : 0;
		bProbe11172_3 = true;
		int32 Off = FirstHeaderOffset1;
		// Probing 10 headers should be fine
		for(int32 i=0; i<10; ++i)
		{
			const uint32 HeaderValue = GetUINT32BE(HeaderData + Off);
			int32 FrameSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetFrameSize(HeaderValue);
			if (FrameSize <= 0)
			{
				bProbe11172_3 = false;
				break;
			}
			else if (Off + FrameSize + 4 <= LastHeaderOffset)
			{
				const uint32 HeaderValue2 = GetUINT32BE(HeaderData + Off + FrameSize);
				if ((HeaderValue2 & InOutInfoHeader.MPEGHeaderMask) != (HeaderValue & InOutInfoHeader.MPEGHeaderMask))
				{
					bProbe11172_3 = false;
					break;
				}
				Off += FrameSize;
			}
			else
			{
				break;
			}
		}
	}

	// Check for ISO 14496-3 if 11172-3 was no success.
	if (!bProbe11172_3 && HeaderMap2.Num())
	{
		InOutInfoHeader.MPEGHeaderMask = 0xfffefd00;
		InOutInfoHeader.MPEGHeaderExpectedValue = HeaderMap2.CreateConstIterator().Key() & InOutInfoHeader.MPEGHeaderMask;
		// Locate the first matching header.
		FirstHeaderOffset2 = InfoHeaderOffset;
		for(int32 i=0; FirstHeaderOffset2<0 && i<SyncMarkerOffsets.Num()-2; ++i)
		{
			const int32 Off = SyncMarkerOffsets[i];
			if ((GetUINT32BE(HeaderData + Off) & InOutInfoHeader.MPEGHeaderMask) == InOutInfoHeader.MPEGHeaderExpectedValue)
			{
				FirstHeaderOffset2 = Off;
				break;
			}
		}

		const int32 LastHeaderOffset = SyncMarkerOffsets.Num() >= 2 ? SyncMarkerOffsets[SyncMarkerOffsets.Num()-2] : 0;
		bProbe14496_3 = true;
		int32 Off = FirstHeaderOffset2;
		// Probing 10 headers should be fine
		for(int32 i=0; i<10 && LastHeaderOffset > Off; ++i)
		{
			ElectraDecodersUtil::MPEG::AACUtils::ADTSheader adts;
			if (!ElectraDecodersUtil::MPEG::AACUtils::ParseADTSHeader(adts, MakeConstArrayView(HeaderData + Off, LastHeaderOffset - Off)))
			{
				bProbe14496_3 = false;
				break;
			}
			else if (i == 0)
			{
				FirstADTSHeader = adts;
			}
			Off += adts.FrameLength;
		}
	}

	if (bProbe11172_3 && bProbe14496_3)
	{
		OutError = FString::Printf(TEXT("Data parses as both MPEG 1 or 2 and AAC. Cannot determine precise format!"));
		return false;
	}


	if (bProbe11172_3)
	{
		InOutInfoHeader.Type = FMPEGAudioInfoHeader::EType::ISO_11172_3;
		// Fill in values that haven't been set up by any of the special headers yet.
		if (FirstHeaderOffset1 >= 0)
		{
			const uint32 HeaderValue = GetUINT32BE(HeaderData + FirstHeaderOffset1);
			InOutInfoHeader.MPEGVersion = InOutInfoHeader.MPEGVersion ? InOutInfoHeader.MPEGVersion : ElectraDecodersUtil::MPEG::UtilsMPEG123::GetVersion(HeaderValue);
			InOutInfoHeader.MPEGLayer = InOutInfoHeader.MPEGLayer ? InOutInfoHeader.MPEGLayer : ElectraDecodersUtil::MPEG::UtilsMPEG123::GetLayer(HeaderValue);
			InOutInfoHeader.SampleRate = InOutInfoHeader.SampleRate ? InOutInfoHeader.SampleRate : ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplingRate(HeaderValue);
			InOutInfoHeader.NumChannels = InOutInfoHeader.NumChannels ? InOutInfoHeader.NumChannels : ElectraDecodersUtil::MPEG::UtilsMPEG123::GetChannelCount(HeaderValue);
			InOutInfoHeader.Bitrate = InOutInfoHeader.Bitrate ? InOutInfoHeader.Bitrate : ElectraDecodersUtil::MPEG::UtilsMPEG123::GetBitrate(HeaderValue);
			InOutInfoHeader.CBRFrameSize = InOutInfoHeader.bIsVBR ? 0 : InOutInfoHeader.CBRFrameSize ? InOutInfoHeader.CBRFrameSize : ElectraDecodersUtil::MPEG::UtilsMPEG123::GetFrameSize(HeaderValue, 1);
			InOutInfoHeader.SamplesPerFrame = InOutInfoHeader.SamplesPerFrame ? InOutInfoHeader.SamplesPerFrame : ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplesPerFrame(HeaderValue);
			InOutInfoHeader.FirstDataByte = InOutInfoHeader.FirstDataByte >= 0 ? InOutInfoHeader.FirstDataByte : FirstHeaderOffset1;
			InOutInfoHeader.LastDataByte = InOutInfoHeader.LastDataByte >= 0 ? InOutInfoHeader.LastDataByte : -1;
		}

		InOutInfoHeader.CodecInfo.SetStreamType(EStreamType::Audio);
		InOutInfoHeader.CodecInfo.SetMimeType(TEXT("audio/mpeg"));
		InOutInfoHeader.CodecInfo.SetCodec(FStreamCodecInformation::ECodec::Audio4CC);
		InOutInfoHeader.CodecInfo.SetCodec4CC(Utils::Make4CC('m','p','g','a'));
		InOutInfoHeader.CodecInfo.SetProfile(InOutInfoHeader.MPEGVersion);
		InOutInfoHeader.CodecInfo.SetProfileLevel(InOutInfoHeader.MPEGLayer);
		InOutInfoHeader.CodecInfo.SetCodecSpecifierRFC6381(TEXT("mp4a.6b"));
		InOutInfoHeader.CodecInfo.SetSamplingRate(InOutInfoHeader.SampleRate);
		InOutInfoHeader.CodecInfo.SetNumberOfChannels(InOutInfoHeader.NumChannels);
		InOutInfoHeader.CodecInfo.SetBitrate(InOutInfoHeader.AverageBitrate ? InOutInfoHeader.AverageBitrate : InOutInfoHeader.Bitrate ? InOutInfoHeader.Bitrate : 0);
	}
	else if (bProbe14496_3)
	{
		InOutInfoHeader.Type = FMPEGAudioInfoHeader::EType::ISO_14496_3;
		InOutInfoHeader.bIsVBR = true;
		InOutInfoHeader.SampleRate = ElectraDecodersUtil::MPEG::AACUtils::GetSampleRateFromFrequenceIndex(FirstADTSHeader.SamplingFrequencyIndex);
		InOutInfoHeader.NumChannels = ElectraDecodersUtil::MPEG::AACUtils::GetNumberOfChannelsFromChannelConfiguration(FirstADTSHeader.ChannelConfiguration);
		InOutInfoHeader.AverageBitrate = 64000;		// we need some value to guesstimate the duration....
		InOutInfoHeader.SamplesPerFrame = 1024;
		InOutInfoHeader.FirstDataByte = InOutInfoHeader.FirstDataByte >= 0 ? InOutInfoHeader.FirstDataByte : FirstHeaderOffset2;
		InOutInfoHeader.LastDataByte = InOutInfoHeader.LastDataByte >= 0 ? InOutInfoHeader.LastDataByte : -1;

		InOutInfoHeader.CodecInfo.SetStreamType(EStreamType::Audio);
		InOutInfoHeader.CodecInfo.SetMimeType(TEXT("audio/aac"));
		InOutInfoHeader.CodecInfo.SetCodec(FStreamCodecInformation::ECodec::AAC);
		InOutInfoHeader.CodecInfo.SetCodecSpecifierRFC6381(TEXT("mp4a.40.2"));
		InOutInfoHeader.CodecInfo.SetSamplingRate(InOutInfoHeader.SampleRate);
		InOutInfoHeader.CodecInfo.SetNumberOfChannels(InOutInfoHeader.NumChannels);
		InOutInfoHeader.CodecInfo.SetChannelConfiguration(FirstADTSHeader.ChannelConfiguration);
		InOutInfoHeader.CodecInfo.SetBitrate(InOutInfoHeader.AverageBitrate ? InOutInfoHeader.AverageBitrate : InOutInfoHeader.Bitrate ? InOutInfoHeader.Bitrate : 0);

		TArray<uint8> CSD;
		CSD.SetNumUninitialized(2);
		uint32 csd = (uint32)(FirstADTSHeader.Profile + 1) << 11;
		csd |= (uint32)FirstADTSHeader.SamplingFrequencyIndex << 7;
		csd |= (uint32)FirstADTSHeader.ChannelConfiguration << 3;
		CSD[0] = (uint8)(csd >> 8);
		CSD[1] = (uint8)(csd & 255);
		InOutInfoHeader.CodecInfo.SetCodecSpecificData(CSD);
	}
	if (bProbe11172_3 || bProbe14496_3)
	{
		if (InOutInfoHeader.bIsLive)
		{
			InOutInfoHeader.EstimatedDuration.SetToPositiveInfinity();
		}
		else
		{
			auto CalculateAverageBitrate = [&InOutInfoHeader]() -> void
			{
				// If VBR we can recalculate the average bitrate if we know the file size.
				if (InOutInfoHeader.bIsVBR && InOutInfoHeader.FirstDataByte >= 0 && InOutInfoHeader.LastDataByte > InOutInfoHeader.FirstDataByte)
				{
					InOutInfoHeader.AverageBitrate = (int32)((InOutInfoHeader.LastDataByte - InOutInfoHeader.FirstDataByte) * 8 / InOutInfoHeader.EstimatedDuration.GetAsSeconds());
				}
			};
			InOutInfoHeader.EstimatedDuration.SetToInvalid();
			// Is there an ID3v2 tag giving the duration?
			if (InOutInfoHeader.ID3v2.IsValid())
			{
				MPEG::FID3V2Metadata::FItem v;
				if (InOutInfoHeader.ID3v2->GetTag(v, Utils::Make4CC('T','L','E','N')))
				{
					InOutInfoHeader.EstimatedDuration.SetFromTimespan(v.Value.GetValue<FTimespan>());
					CalculateAverageBitrate();
				}
			}
			// Duration not valid, try to calculate an estimate from the pieces of information we have.
			// Is there a total frame number given?
			if (!InOutInfoHeader.EstimatedDuration.IsValid() && InOutInfoHeader.NumFrames && InOutInfoHeader.SamplesPerFrame && InOutInfoHeader.SampleRate)
			{
				InOutInfoHeader.EstimatedDuration.SetFromTimeFraction(FTimeFraction((int64)(InOutInfoHeader.NumFrames * (uint32)InOutInfoHeader.SamplesPerFrame - (uint32)(InOutInfoHeader.EncoderDelayStart + InOutInfoHeader.EncoderDelayEnd)), (uint32)InOutInfoHeader.SampleRate));
				CalculateAverageBitrate();
			}
			// For CBR we can guess based on filesize, if that is known.
			if (!InOutInfoHeader.EstimatedDuration.IsValid() && !InOutInfoHeader.bIsVBR && InOutInfoHeader.Bitrate && InOutInfoHeader.FirstDataByte >= 0 && InOutInfoHeader.LastDataByte > InOutInfoHeader.FirstDataByte)
			{
				if (InOutInfoHeader.CBRFrameSize && InOutInfoHeader.SamplesPerFrame && InOutInfoHeader.SampleRate)
				{
					int64 NumFrames = (InOutInfoHeader.LastDataByte - InOutInfoHeader.FirstDataByte) / InOutInfoHeader.CBRFrameSize;
					InOutInfoHeader.EstimatedDuration.SetFromTimeFraction(FTimeFraction(NumFrames * InOutInfoHeader.SamplesPerFrame - (InOutInfoHeader.EncoderDelayStart + InOutInfoHeader.EncoderDelayEnd), (uint32)InOutInfoHeader.SampleRate));
					InOutInfoHeader.NumFrames = (uint32)NumFrames;
				}
				else
				{
					// This is very likely to overshoot the actual duration.
					InOutInfoHeader.EstimatedDuration.SetFromTimeFraction(FTimeFraction((InOutInfoHeader.LastDataByte - InOutInfoHeader.FirstDataByte) * 8, (uint32)InOutInfoHeader.Bitrate));
				}
			}
			// For VBR without a dedicated info header the estimate will be extremely rough.
			if (!InOutInfoHeader.EstimatedDuration.IsValid() && InOutInfoHeader.bIsVBR && (InOutInfoHeader.Bitrate || InOutInfoHeader.AverageBitrate) && InOutInfoHeader.FirstDataByte >= 0 && InOutInfoHeader.LastDataByte > InOutInfoHeader.FirstDataByte)
			{
				InOutInfoHeader.EstimatedDuration.SetFromTimeFraction(FTimeFraction((InOutInfoHeader.LastDataByte - InOutInfoHeader.FirstDataByte) * 8, (uint32)(InOutInfoHeader.AverageBitrate ? InOutInfoHeader.AverageBitrate : InOutInfoHeader.Bitrate)));
				CalculateAverageBitrate();
			}
		}
		// Copy the HTTP response headers across. This might be useful for Casting (ie. Icy Cast)
		InOutInfoHeader.HTTPResponseHeaders = Headers;
	}

	return bProbe11172_3 || bProbe14496_3;
}


void FPlaylistReaderMPEGAudio::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);

	SCOPE_CYCLE_COUNTER(STAT_ElectraPlayer_MPEGAudio_PlaylistWorker);
	CSV_SCOPED_TIMING_STAT(ElectraPlayer, MPEGAudio_PlaylistWorker);

	FURL_RFC3986 UrlParser;
	UrlParser.Parse(PlaylistURL);
	PlaylistURL = UrlParser.Get(true, false);
	URLFragment = UrlParser.GetFragment();

	TSharedPtr<FRequest, ESPMode::ThreadSafe> Req = MakeShared<FRequest, ESPMode::ThreadSafe>();

	// We have to distinguish between local file playback and internet here.
	ConnectionInfo.RequestStartTime = MEDIAutcTime::Current();
	if (UrlParser.GetScheme().Equals(TEXT("file")))
	{
		FString Filename;
		FURL_RFC3986::UrlDecode(Filename, UrlParser.Get(false, false));
		Filename.MidInline(7);	// remove "file://"
		TSharedPtr<FArchive, ESPMode::ThreadSafe> Archive = MakeShareable(IFileManager::Get().CreateFileReader(*Filename));
		ConnectionInfo.EffectiveURL = Filename;
		// Set a content mime type. Does not correctly identify the type yet (might be `audio/aac`)
		ConnectionInfo.ContentType = TEXT("audio/mpeg");
		if (Archive.IsValid())
		{
			Req->StatusCode = 200;
			Req->ContentLength = Archive->TotalSize();
			int32 AlreadyRead = 0;
			if (Req->ContentLength > 10)
			{
				AlreadyRead = 10;
				Req->Buffer.AddUninitialized(10);
				Archive->Serialize(Req->Buffer.GetData(), 10);
				int64 Length = 0;
				Req->OnProcessRequestStream(nullptr, Length);
				check(Req->ProbeSize > 0);
			}
			Req->ProbeSize = Req->ProbeSize < 0 ? FRequest::GetMinProbeSize() : Req->ProbeSize;
			Req->Buffer.AddUninitialized(Req->ProbeSize - AlreadyRead);
			Archive->Serialize(Req->Buffer.GetData() + AlreadyRead, Req->ProbeSize - AlreadyRead);
			Req->FindSyncMarkers();
			Req->Result = FRequest::EResult::Succeeded;
		}
		else
		{
			Req->StatusCode = 404;
			Req->Result = FRequest::EResult::Failed;
		}
	}
	else
	{
		FHttpRequestStreamDelegateV2 StreamDelegate;
		StreamDelegate.BindThreadSafeSP(Req.ToSharedRef(), &FPlaylistReaderMPEGAudio::FRequest::OnProcessRequestStream);
		Req->Handle = FHttpModule::Get().CreateRequest();
		Req->Handle->SetVerb(TEXT("GET"));
		Req->Handle->SetURL(PlaylistURL);
		Req->Handle->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);
		Req->Handle->OnProcessRequestComplete().BindThreadSafeSP(Req.ToSharedRef(), &FPlaylistReaderMPEGAudio::FRequest::OnProcessRequestComplete);
		Req->Handle->OnHeaderReceived().BindThreadSafeSP(Req.ToSharedRef(), &FPlaylistReaderMPEGAudio::FRequest::OnHeaderReceived);
		Req->Handle->OnStatusCodeReceived().BindThreadSafeSP(Req.ToSharedRef(), &FPlaylistReaderMPEGAudio::FRequest::OnStatusCodeReceived);
		Req->Handle->SetResponseBodyReceiveStreamDelegateV2(StreamDelegate);
		Req->Handle->SetHeader(TEXT("User-Agent"), IElectraHttpManager::GetDefaultUserAgent());
		Req->Handle->SetHeader(TEXT("Accept-Encoding"), TEXT("identity"));
		Req->Handle->SetTimeout(PlayerSessionServices->GetOptionValue(MPEGAudio::OptionKeyMPEGAudioLoadTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 60)).GetAsSeconds());
		Req->Handle->SetActivityTimeout(PlayerSessionServices->GetOptionValue(MPEGAudio::OptionKeyMPEGAudioLoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 8)).GetAsSeconds());
		Req->Handle->ProcessRequest();

		bool bDone = false;
		bool bDidAbort = false;
		while(!bDone)
		{
			FRequest::EEvent evt = FRequest::EEvent::None;
			if (Req->Events.ReceiveMessage(evt, 20*1000))
			{
				switch(evt)
				{
					case FRequest::EEvent::Abort:
					{
						Req->Cancel();
						break;
					}
					case FRequest::EEvent::HaveProbeData:
					{
						if (!bAbort)
						{
							Req->Cancel();
							Req->FindSyncMarkers();
						}
						break;
					}
					case FRequest::EEvent::Finished:
					{
						bDone = true;
						break;
					}
				}
			}
			if (bAbort)
			{
				if (!bDidAbort)
				{
					bDidAbort = true;
					Req->Events.SendMessage(FRequest::EEvent::Abort);
				}
			}
		}
		Req->Handle.Reset();
	}

	ConnectionInfo.RequestEndTime = MEDIAutcTime::Current();
	ConnectionInfo.bHasFinished = !bAbort;
	ConnectionInfo.bWasAborted = bAbort;
	ConnectionInfo.StatusInfo.HTTPStatus = Req->StatusCode;
	ConnectionInfo.StatusInfo.bReadError = Req->Result == FRequest::EResult::Failed;

	// There are currently no retries, so this is the first (and only) attempt we make.
	const int32 Attempt = 1;
	// Notify the download of the "main playlist". This indicates the download only, not the parsing thereof.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistDownloadMessage::Create(&ConnectionInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempt));
	// Notify that the "main playlist" has been parsed, successfully or not.
	PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ConnectionInfo, Playlist::EListType::Main, Playlist::ELoadType::Initial, Attempt));
	if (!bAbort)
	{
		// Error?
		if (Req->Result == FRequest::EResult::Failed || Req->StatusCode != 200)
		{
			PostError(FString::Printf(TEXT("Error while downloading \"%s\""), *PlaylistURL), ERRCODE_MPEGAUDIO_DOWNLOAD_ERROR, UEMEDIA_ERROR_READ_ERROR);
		}
		else
		{
			FString OutError;
			TArray<FString> OutWarnings;
			InfoHeader.bIsLive = Req->ContentLength < 0;
			InfoHeader.LastDataByte = Req->ContentLength;
			bool bIsValid = Req->Validate(InfoHeader, OutError, OutWarnings);
			if (bIsValid)
			{
				for(auto& Warn : OutWarnings)
				{
					LogMessage(IInfoLog::ELevel::Info, FString::Printf(TEXT("%s"), *Warn));
				}

				Manifest = MakeSharedTS<FManifestMPEGAudioInternal>(PlayerSessionServices);

				TArray<FURL_RFC3986::FQueryParam> URLFragmentComponents;
				FURL_RFC3986::GetQueryParams(URLFragmentComponents, URLFragment, false);	// The fragment is already URL escaped, so no need to do it again.
				Manifest->SetURLFragmentComponents(MoveTemp(URLFragmentComponents));

				LastErrorDetail = Manifest->Build(InfoHeader, PlaylistURL);

				// Let the external registry know that we have no properties with an end-of-properties call.
				PlayerSessionServices->ValidateMainPlaylistCustomProperty(GetPlaylistType(), PlaylistURL, TArray<FElectraHTTPStreamHeader>(), IPlayerSessionServices::FPlaylistProperty());

				// Notify that the "variant playlists" are ready. There are no variants in an mp4, but this is the trigger that the playlists are all set up and are good to go now.
				PlayerSessionServices->SendMessageToPlayer(IPlaylistReader::PlaylistLoadedMessage::Create(LastErrorDetail, &ConnectionInfo, Playlist::EListType::Variant, Playlist::ELoadType::Initial, Attempt));
			}
			else
			{
				// Not valid. File might have an ID3v2 tag larger than the probe amount or it is actually not a valid file.
				PostError(FString::Printf(TEXT("File \"%s\" does not appear to be a valid MPEG audio file."), *PlaylistURL), ERRCODE_MPEGAUDIO_INVALID_FILE, UEMEDIA_ERROR_FORMAT_ERROR);
			}
		}
	}

	// This thread's work is done. We only wait for termination now.
	WorkerThreadQuitSignal.Wait();
}

} // namespace Electra
