// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "Http.h"
#include "Stats/Stats.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMPEGAudio.h"
#include "Utilities/Utilities.h"
#include "Utilities/TimeUtilities.h"
#include "Utilities/BCP47-Helpers.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "Player/AdaptiveStreamingPlayerABR.h"
#include "Player/mpegaudio/ManifestMPEGAudio.h"
#include "Player/mpegaudio/StreamReaderMPEGAudio.h"
#include "Player/mpegaudio/OptionKeynamesMPEGAudio.h"
#include "Player/PlayerSessionServices.h"
#include "Player/PlayerStreamFilter.h"
#include "MediaStreamMetadata.h"


DECLARE_CYCLE_STAT(TEXT("FStreamReaderMPEGAudio_HandleRequest"), STAT_ElectraPlayer_MPEGAudio_StreamReader, STATGROUP_ElectraPlayer);


namespace Electra
{

void FStreamSegmentRequestMPEGAudio::GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams)
{
	OutRequestedStreams.Empty();
	OutRequestedStreams.Emplace(SharedThis(this));
}

void FStreamSegmentRequestMPEGAudio::GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams)
{
	OutAlreadyEndedStreams.Empty();
}

FTimeRange FStreamSegmentRequestMPEGAudio::GetTimeRange() const
{
	FTimeRange tr;
	tr.Start = FirstPTS;
	tr.End = FirstPTS + FTimeValue(Duration);
	tr.Start.SetSequenceIndex(TimestampSequenceIndex);
	tr.End.SetSequenceIndex(TimestampSequenceIndex);
	return tr;
}


struct FStreamReaderMPEGAudio::FLiveRequest : public TSharedFromThis<FStreamReaderMPEGAudio::FLiveRequest, ESPMode::ThreadSafe>
{
	FLiveRequest()
	{
		Events.Resize(4);
	}

	void OnProcessRequestComplete(FHttpRequestPtr InSourceHttpRequest, FHttpResponsePtr InHttpResponse, bool bInSucceeded);
	void OnHeaderReceived(FHttpRequestPtr InSourceHttpRequest, const FString& InHeaderName, const FString& InHeaderValue);
	void OnStatusCodeReceived(FHttpRequestPtr InSourceHttpRequest, int32 InHttpStatusCode);
	void OnProcessRequestStream(void* InDataPtr, int64& InLength);
	void Cancel();
	void WaitUntilFinished();
	enum class EEvent
	{
		None,
		Failed,
		Finished
	};
	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> Handle;
	FHttpRequestStreamDelegateV2 StreamDelegate;
	int32 StatusCode = 0;
	TMediaMessageQueueWithTimeout<EEvent> Events;
	TMultiMap<FString, FString> Headers;
	TWeakPtrTS<FWaitableBuffer> ReceiveBuffer;
	bool bCanceled = false;
	int32 MetaDataEveryNBytes = 0;
	int32 MetaDataBytesToGo = 0;
	int32 BytesUntilNextMetadata = 0;
	int64 TotalDataBytePos = 0;
	int32 MaxDataBytes = 0;
	bool bIsReceivingMetadata = false;
	volatile bool bHasFailed = false;
	TArray<uint8> MetadataBuffer;
	TMap<int64, TArray<uint8>> MetadataBufferMap;
};



uint32	FStreamReaderMPEGAudio::UniqueDownloadID = 1;

FStreamReaderMPEGAudio::FStreamReaderMPEGAudio()
{
}

FStreamReaderMPEGAudio::~FStreamReaderMPEGAudio()
{
	Close();
}

UEMediaError FStreamReaderMPEGAudio::Create(IPlayerSessionServices* InPlayerSessionService, const CreateParam &InCreateParam)
{
	if (!InCreateParam.MemoryProvider || !InCreateParam.EventListener)
	{
		return UEMEDIA_ERROR_BAD_ARGUMENTS;
	}

	PlayerSessionServices = InPlayerSessionService;
	Parameters = InCreateParam;
	bTerminate = false;
	bIsStarted = true;

	ThreadSetName("ElectraPlayer::MPEGAudio streamer");
	ThreadStart(FMediaRunnable::FStartDelegate::CreateRaw(this, &FStreamReaderMPEGAudio::WorkerThread));

	return UEMEDIA_ERROR_OK;
}

bool FStreamReaderMPEGAudio::LoadsMultiplexedTracksSequentially()
{
	return false;
}

void FStreamReaderMPEGAudio::Close()
{
	if (bIsStarted)
	{
		bIsStarted = false;
		CancelRequests();
		bTerminate = true;
		WorkSignal.Signal();
		ThreadWaitDone();
		ThreadReset();
		CurrentRequest.Reset();
	}
}

void FStreamReaderMPEGAudio::LogMessage(IInfoLog::ELevel Level, const FString& Message)
{
	PlayerSessionServices->PostLog(Facility::EFacility::MPEGAudioStreamReader, Level, Message);
}

IStreamReader::EAddResult FStreamReaderMPEGAudio::AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> InRequest)
{
	TSharedPtrTS<FStreamSegmentRequestMPEGAudio> Request = CurrentRequest;
	if (Request.IsValid())
	{
		check(!"why is the handler busy??");
		return IStreamReader::EAddResult::TryAgainLater;
	}
	Request = StaticCastSharedPtr<FStreamSegmentRequestMPEGAudio>(InRequest);
	Request->SetPlaybackSequenceID(CurrentPlaybackSequenceID);
	bRequestCanceled = false;
	bHasErrored = false;
	// Only add the request if it is not an all-EOS one!
	if (!Request->bIsEOSRequest)
	{
		CurrentRequest = Request;
		WorkSignal.Signal();
	}
	return EAddResult::Added;
}

void FStreamReaderMPEGAudio::CancelRequest(EStreamType StreamType, bool bSilent)
{
}

void FStreamReaderMPEGAudio::CancelRequests()
{
	bRequestCanceled = true;
	TSharedPtrTS<FWaitableBuffer> RcvBuf = ReceiveBuffer;
	if (RcvBuf.IsValid())
	{
		RcvBuf->Abort();
	}
	TSharedPtrTS<FLiveRequest> lr(LiveRequest);
	if (lr.IsValid())
	{
		lr->Cancel();
	}
}

bool FStreamReaderMPEGAudio::HasBeenAborted() const
{
	TSharedPtrTS<FWaitableBuffer> RcvBuf = ReceiveBuffer;
	return bRequestCanceled || (RcvBuf.IsValid() && RcvBuf->WasAborted());
}

bool FStreamReaderMPEGAudio::HasErrored() const
{
	return bHasErrored;
}

int32 FStreamReaderMPEGAudio::HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest)
{
	// Aborted?
	return HasBeenAborted() ? 1 : 0;
}

void FStreamReaderMPEGAudio::HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest)
{
	bHasErrored = bHasErrored || InRequest->ConnectionInfo.StatusInfo.ErrorDetail.IsError();
}

void FStreamReaderMPEGAudio::FLiveRequest::Cancel()
{
	if (!bCanceled)
	{
		bCanceled = true;
		if (Handle.IsValid())
		{
			Handle->CancelRequest();
		}
	}
}

void FStreamReaderMPEGAudio::FLiveRequest::WaitUntilFinished()
{
	EEvent evt = EEvent::None;
	while(1)
	{
		if (Events.ReceiveMessage(evt, 20*1000))
		{
			break;
		}
	}
}

void FStreamReaderMPEGAudio::FLiveRequest::OnProcessRequestComplete(FHttpRequestPtr InSourceHttpRequest, FHttpResponsePtr InHttpResponse, bool bInSucceeded)
{
	bHasFailed = true;
	Events.SendMessage(bInSucceeded ? EEvent::Finished : EEvent::Failed);
}

void FStreamReaderMPEGAudio::FLiveRequest::OnHeaderReceived(FHttpRequestPtr InSourceHttpRequest, const FString& InHeaderName, const FString& InHeaderValue)
{
	if (InHeaderName.Len())
	{
		Headers.Add(InHeaderName.ToLower(), InHeaderValue);

		// Icecast metadata interval?
		if (InHeaderName.Equals(TEXT("icy-metaint"), ESearchCase::IgnoreCase))
		{
			LexFromString(MetaDataEveryNBytes, *InHeaderValue);
			BytesUntilNextMetadata = MetaDataEveryNBytes > 0 ? MetaDataEveryNBytes : -1;
		}
	}
}

void FStreamReaderMPEGAudio::FLiveRequest::OnStatusCodeReceived(FHttpRequestPtr InSourceHttpRequest, int32 InHttpStatusCode)
{
	if (InHttpStatusCode > 0 && InHttpStatusCode < 600)
	{
		StatusCode = InHttpStatusCode;
	}
}

void FStreamReaderMPEGAudio::FLiveRequest::OnProcessRequestStream(void* InDataPtr, int64& InOutLength)
{
	if (StatusCode == 200)
	{
		int64 InLength = InOutLength;
		while(InLength > 0)
		{
			// Are we receiving metadata right now?
			if (bIsReceivingMetadata)
			{
				// How many more bytes of metadata to receive?
				if (MetaDataBytesToGo <= 0)
				{
					MetaDataBytesToGo = 16 * *reinterpret_cast<const uint8*>(InDataPtr);
					InLength -= 1;
					InDataPtr = (void*)((uint8*)InDataPtr + 1);
					MetadataBuffer.Empty();
				}
				int32 MetaBytesNow = Utils::Min((int32) InLength, MetaDataBytesToGo);
				MetadataBuffer.Append(reinterpret_cast<const uint8*>(InDataPtr), MetaBytesNow);
				InLength -= MetaBytesNow;
				InDataPtr = (void*)((uint8*)InDataPtr + MetaBytesNow);
				bIsReceivingMetadata = (MetaDataBytesToGo -= MetaBytesNow) > 0;
				if (!bIsReceivingMetadata)
				{
					BytesUntilNextMetadata = MetaDataEveryNBytes;
					while(MetadataBuffer.Num() && MetadataBuffer[MetadataBuffer.Num() - 1] == 0)
					{
						MetadataBuffer.RemoveAt(MetadataBuffer.Num() - 1);
					}
					MetadataBufferMap.Emplace(TotalDataBytePos, MoveTemp(MetadataBuffer));
				}
			}
			else
			{
				int32 DataBytesNow = BytesUntilNextMetadata > 0 ? Utils::Min((int32) InLength, BytesUntilNextMetadata) : InLength;
				TSharedPtrTS<FWaitableBuffer> rb = ReceiveBuffer.Pin();
				if (rb.IsValid())
				{
					FScopeLock Lock(rb->GetLock());
					int64 BufSizeRequired = rb->Num() + DataBytesNow;
					if (!rb->EnlargeTo(BufSizeRequired))
					{
						InOutLength = 0;
						return;
					}
					if (!rb->PushData(reinterpret_cast<const uint8*>(InDataPtr), DataBytesNow))
					{
						InOutLength = 0;
						return;
					}

					// Sanity check that we are not reading excessive data, which is the case when the
					// player has been paused for instance.
					if (MaxDataBytes && rb->Num() > MaxDataBytes)
					{
						bHasFailed = true;
						InOutLength = 0;
						return;
					}
				}
				TotalDataBytePos += DataBytesNow;
				InLength -= DataBytesNow;
				InDataPtr = (void*)((uint8*)InDataPtr + DataBytesNow);
				bIsReceivingMetadata = MetaDataEveryNBytes && (BytesUntilNextMetadata -= DataBytesNow) <= 0;
				if (bIsReceivingMetadata)
				{
					BytesUntilNextMetadata = MetaDataEveryNBytes;
					MetaDataBytesToGo = 0;
				}
			}
		}
	}
}


void FStreamReaderMPEGAudio::HandleRequest()
{
	TSharedPtrTS<FStreamSegmentRequestMPEGAudio> Request = CurrentRequest;
	FString ParsingErrorMessage;

	Metrics::FSegmentDownloadStats& ds = Request->DownloadStats;
	ds.StatsID = FMediaInterlockedIncrement(UniqueDownloadID);
	ds.MediaAssetID = TEXT("1");
	ds.AdaptationSetID = TEXT("1");
	ds.RepresentationID = TEXT("1");
	ds.Bitrate = Request->GetBitrate();
	FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio* TimelineAsset = static_cast<FManifestMPEGAudioInternal::FTimelineAssetMPEGAudio*>(Request->MediaAsset.Get());
	if (TimelineAsset->GetNumberOfAdaptationSets(EStreamType::Audio))
	{
		TSharedPtrTS<IPlaybackAssetAdaptationSet> AdaptationSet = Request->MediaAsset->GetAdaptationSetByTypeAndIndex(EStreamType::Audio, 0);
		FString AdaptID = AdaptationSet->GetUniqueIdentifier();
		if (AdaptationSet->GetNumberOfRepresentations())
		{
			TSharedPtrTS<IPlaybackAssetRepresentation> Representation = AdaptationSet->GetRepresentationByIndex(0);
			ds.MediaAssetID = TimelineAsset->GetUniqueIdentifier();
			ds.AdaptationSetID = AdaptID;
			ds.RepresentationID = Representation->GetUniqueIdentifier();
			ds.Bitrate = Representation->GetBitrate();
		}
	}
	ds.FailureReason.Empty();
	ds.bWasSuccessful = true;
	ds.bWasAborted = false;
	ds.bDidTimeout = false;
	ds.HTTPStatusCode = 0;
	ds.StreamType = Request->GetType();
	ds.SegmentType = Metrics::ESegmentType::Media;
	ds.PresentationTime = Request->FirstPTS.GetAsSeconds();
	ds.Duration = Request->Duration;
	ds.DurationDownloaded = 0.0;
	ds.DurationDelivered = 0.0;
	ds.TimeToFirstByte = 0.0;
	ds.TimeToDownload = 0.0;
	ds.ByteSize = -1;
	ds.NumBytesDownloaded  = 0;
	ds.bInsertedFillerData = false;
	ds.URL.URL = TimelineAsset->GetMediaURL();
	ds.bIsMissingSegment = false;
	ds.bParseFailure = false;
	ds.RetryNumber = Request->NumOverallRetries;

	Parameters.EventListener->OnFragmentOpen(Request);

	// We need to handle Live playback (ie an Icecast) differently.
	const bool bIsLivePlayback = Request->bIsLive;

	TSharedPtrTS<IElectraHttpManager::FRequest> HTTP(new IElectraHttpManager::FRequest);
	TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener;
	ReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
	if (!bIsLivePlayback)
	{
		ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
		ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateRaw(this, &FStreamReaderMPEGAudio::HTTPCompletionCallback);
		ProgressListener->ProgressDelegate   = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateRaw(this, &FStreamReaderMPEGAudio::HTTPProgressCallback);

		HTTP->Parameters.URL = TimelineAsset->GetMediaURL();
		HTTP->Parameters.Range.Start = Request->FileStartOffset;
		HTTP->Parameters.Range.EndIncluding = Request->FileEndOffset;
		// No compression as this would not yield much with already compressed data.
		HTTP->Parameters.AcceptEncoding.Set(TEXT("identity"));
		// Timeouts
		HTTP->Parameters.ConnectTimeout = PlayerSessionServices->GetOptionValue(MPEGAudio::OptionKeyMPEGAudioLoadConnectTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 8));
		HTTP->Parameters.NoDataTimeout = PlayerSessionServices->GetOptionValue(MPEGAudio::OptionKeyMPEGAudioLoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 6));

		// Explicit range?
		int64 NumRequestedBytes = HTTP->Parameters.Range.GetNumberOfBytes();

		HTTP->ReceiveBuffer = ReceiveBuffer;
		HTTP->ProgressListener = ProgressListener;
		HTTP->ResponseCache = PlayerSessionServices->GetHTTPResponseCache();
		PlayerSessionServices->GetHTTPManager()->AddRequest(HTTP, false);
	}
	else
	{
		LiveRequest = MakeSharedTS<FLiveRequest>();
		LiveRequest->ReceiveBuffer = ReceiveBuffer;
		LiveRequest->StreamDelegate.BindThreadSafeSP(LiveRequest.ToSharedRef(), &FStreamReaderMPEGAudio::FLiveRequest::OnProcessRequestStream);
		LiveRequest->Handle = FHttpModule::Get().CreateRequest();
		LiveRequest->Handle->SetVerb(TEXT("GET"));
		LiveRequest->Handle->SetURL(TimelineAsset->GetMediaURL());
		LiveRequest->Handle->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);
		LiveRequest->Handle->OnProcessRequestComplete().BindThreadSafeSP(LiveRequest.ToSharedRef(), &FStreamReaderMPEGAudio::FLiveRequest::OnProcessRequestComplete);
		LiveRequest->Handle->OnHeaderReceived().BindThreadSafeSP(LiveRequest.ToSharedRef(), &FStreamReaderMPEGAudio::FLiveRequest::OnHeaderReceived);
		LiveRequest->Handle->OnStatusCodeReceived().BindThreadSafeSP(LiveRequest.ToSharedRef(), &FStreamReaderMPEGAudio::FLiveRequest::OnStatusCodeReceived);
		LiveRequest->Handle->SetResponseBodyReceiveStreamDelegateV2(LiveRequest->StreamDelegate);
		LiveRequest->Handle->SetHeader(TEXT("User-Agent"), IElectraHttpManager::GetDefaultUserAgent());
		LiveRequest->Handle->SetHeader(TEXT("Accept-Encoding"), TEXT("identity"));
		// If this is an Icycast, we ask for period metadata.
		if (Request->CastType == FStreamSegmentRequestMPEGAudio::ECastType::IcyCast)
		{
			LiveRequest->Handle->SetHeader(TEXT("Icy-Metadata"), TEXT("1"));
		}
		LiveRequest->MaxDataBytes = Utils::Min((int32) (PlayerSessionServices->GetOptionValue(MPEGAudio::OptionKeyMPEGAudioMaxPreloadBufferDuration).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 180)).GetAsSeconds() * Request->GetBitrate() / 8), (int32)(8<<20));
		LiveRequest->Handle->SetActivityTimeout(PlayerSessionServices->GetOptionValue(MPEGAudio::OptionKeyMPEGAudioLoadNoDataTimeout).SafeGetTimeValue(FTimeValue().SetFromMilliseconds(1000 * 5)).GetAsSeconds());
		LiveRequest->Handle->ProcessRequest();
	}

	FTimeValue DurationSuccessfullyDelivered(FTimeValue::GetZero());
	FTimeValue DurationSuccessfullyRead(FTimeValue::GetZero());
	bool bDone = false;

	uint32 PlaybackSequenceID = Request->GetPlaybackSequenceID();

	// We have to probe the data for sync markers as the file has no framing whatsoever.
	// Other than perhaps at the beginning of the file it would be purely coincidental to start on a frame.
	auto GetUINT32BE = [](const uint8* InData) -> uint32
	{
		return (static_cast<uint32>(InData[0]) << 24) | (static_cast<uint32>(InData[1]) << 16) | (static_cast<uint32>(InData[2]) << 8) | static_cast<uint32>(InData[3]);
	};
	int32 SyncMarkerCheckPos = 0;
	const int32 FrameCheckSize = 4096;
	const int32 NumFramesToProbe = 10;
	const uint32 FrameSyncMask = Request->MPEGHeaderMask;
	const uint32 FrameSyncValue = Request->MPEGHeaderExpectedValue;
	TArray<int32> SyncMarkerOffsets;
	while(!bDone && !HasErrored() && !HasBeenAborted() && !bTerminate)
	{
		if (ReceiveBuffer->WaitUntilSizeAvailable(SyncMarkerCheckPos + FrameCheckSize, 1000 * 20))
		{
			FScopeLock Lock(ReceiveBuffer->GetLock());
			const uint8* BufferBaseData = ReceiveBuffer->GetLinearReadData();
			int64 BufferDataSize = ReceiveBuffer->GetLinearReadSize();
			if (!BufferBaseData)
			{
				bDone = true;
				break;
			}

			const uint8* Base = BufferBaseData + SyncMarkerCheckPos;
			const uint8* End = BufferBaseData + BufferDataSize - 4;
			for(; Base < End; ++Base)
			{
				if (*Base != 0xff)
				{
					continue;
				}
				// Check for validity. This works for MPEG1 and AAC audio.
				uint32 hdr = GetUINT32BE(Base);
				if ((hdr & FrameSyncMask) == FrameSyncValue)
				{
					SyncMarkerOffsets.Add(Base - BufferBaseData);
					Base += 3;
					if (SyncMarkerOffsets.Num() >= NumFramesToProbe)
					{
						bDone = true;
					}
				}
			}
			SyncMarkerCheckPos = BufferDataSize - 1;
			if (ReceiveBuffer->GetEOD())
			{
				bDone = true;
			}
		}
	}
	int32 FirstAUBufferOffset = -1;
	int32 FirstAUFrameSize = 0;
	// Probe that we are properly locked on to the frames.
	if (!HasErrored() && !HasBeenAborted() && !bTerminate)
	{
		FScopeLock Lock(ReceiveBuffer->GetLock());
		const uint8* BufferBaseData = ReceiveBuffer->GetLinearReadData();
		if (BufferBaseData)
		{
			const int32 MaxCheckFrames = FMath::Min(10, SyncMarkerOffsets.Num());
			const uint8* MaxCheckAddr = BufferBaseData + (SyncMarkerOffsets.Num() ? SyncMarkerOffsets[MaxCheckFrames-1] : 0);
			for(int32 i=0; i<MaxCheckFrames; ++i)
			{
				const uint8* CheckBaseAddr = BufferBaseData + SyncMarkerOffsets[i];
				bool bRunOk = true;
				int32 StartFrameSize = 0;
				if (!Request->bIsAAC)
				{
					uint32 HeaderValue = GetUINT32BE(CheckBaseAddr);
					StartFrameSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetFrameSize(HeaderValue);
					for(int32 j=i+1; j<MaxCheckFrames; ++j)
					{
						int32 FrameSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetFrameSize(HeaderValue);
						CheckBaseAddr += FrameSize;
						if (CheckBaseAddr >= MaxCheckAddr)
						{
							break;
						}
						HeaderValue = GetUINT32BE(CheckBaseAddr);
						if ((HeaderValue & FrameSyncMask) != FrameSyncValue)
						{
							bRunOk = false;
							break;
						}
					}
				}
				else
				{
					ElectraDecodersUtil::MPEG::AACUtils::ADTSheader adts;
					if (CheckBaseAddr >= MaxCheckAddr - 12)
					{
						break;
					}
					if (ElectraDecodersUtil::MPEG::AACUtils::ParseADTSHeader(adts, MakeConstArrayView(CheckBaseAddr, MaxCheckAddr - CheckBaseAddr)))
					{
						StartFrameSize = adts.FrameLength;
						for(int32 j=i+1; j<MaxCheckFrames; ++j)
						{
							CheckBaseAddr += adts.FrameLength;
							if (CheckBaseAddr >= MaxCheckAddr - 12)
							{
								break;
							}
							if (!ElectraDecodersUtil::MPEG::AACUtils::ParseADTSHeader(adts, MakeConstArrayView(CheckBaseAddr, MaxCheckAddr - CheckBaseAddr)))
							{
								bRunOk = false;
								break;
							}
						}
					}
					else
					{
						bRunOk = false;
						break;
					}
					CheckBaseAddr += adts.FrameLength;
				}
				if (bRunOk)
				{
					FirstAUBufferOffset = SyncMarkerOffsets[i];
					FirstAUFrameSize = StartFrameSize;
					break;
				}
			}
		}
	}

	bDone = FirstAUBufferOffset < 0;
	// If we did not find any sync marker to start on we are done.
	if (bDone)
	{
		bDone = true;
		Request->LastSuccessfullyUsedBytePos = Request->FileEndOffset;
		Request->LastSuccessfullyUsedPTS = Request->FirstPTS;
	}
	FTimeValue NextAUPTS = Request->FirstPTS;
	NextAUPTS.SetSequenceIndex(Request->TimestampSequenceIndex);
	int32 NextAUBufferOffset = FirstAUBufferOffset;
	int32 NextAUFrameSize = FirstAUFrameSize;
	int32 NextHeaderSkipBytes = 0;
	bool bIsFirstInSequence = true;
	bool bReadPastLastPTS = false;
	FTimeValue AUDuration;
	TSharedPtrTS<FAccessUnit::CodecData> AUCodecData = MakeSharedTS<FAccessUnit::CodecData>();
	AUCodecData->ParsedInfo = Request->CodecInfo;
	AUCodecData->CodecSpecificData = Request->CodecInfo.GetCodecSpecificData();

	TSharedPtrTS<FBufferSourceInfo> BufferSourceInfo = MakeSharedTS<FBufferSourceInfo>();
	BufferSourceInfo->PeriodID = ds.MediaAssetID;
	BufferSourceInfo->PeriodAdaptationSetID = ds.MediaAssetID + TEXT(".") + ds.AdaptationSetID;
	BCP47::ParseRFC5646Tag(BufferSourceInfo->LanguageTag, FString(TEXT("und")));
	BufferSourceInfo->Codec = Request->CodecInfo.GetCodecSpecifierRFC6381();
	BufferSourceInfo->HardIndex = 0;
	BufferSourceInfo->PlaybackSequenceID = Request->GetPlaybackSequenceID();

	int64 NumTotalBytesRead = 0;
	// Remove data we already processed every 4 seconds. Do this in blocks to reduce the overall
	// amount of memory being moved around.
	int32 NumLiveStreamBytesToRemove = Request->GetBitrate() * 4 / 8;

	const int32 kNumNextHeaderBytes = Request->bIsAAC ? 7 : 4;
	ElectraDecodersUtil::MPEG::AACUtils::ADTSheader adts;
	while(!bDone && !HasErrored() && !HasBeenAborted() && !bTerminate)
	{
		if (!NextAUFrameSize)
		{
			break;
		}

		// With Live playback we have to remove the data bytes we already passed along from the start of the buffer.
		if (LiveRequest.IsValid() && NumLiveStreamBytesToRemove && NextAUBufferOffset >= NumLiveStreamBytesToRemove)
		{
			ReceiveBuffer->RemoveFromBeginning(NextAUBufferOffset);
			NextAUBufferOffset = 0;
		}

		// Wait until we get the next AU's data plus the following bytes that are the header of the following frame.
		const int64 TotalNumNeeded = NextAUBufferOffset + NextAUFrameSize + kNumNextHeaderBytes;
		if (ReceiveBuffer->WaitUntilSizeAvailable(TotalNumNeeded, 1000 * 20))
		{
			FScopeLock Lock(ReceiveBuffer->GetLock());

			const uint8* BufferBaseData = ReceiveBuffer->GetLinearReadData();
			if (!BufferBaseData)
			{
				break;
			}
			int64 BufferDataSize = ReceiveBuffer->GetLinearReadSize();

			int64 NumGot = BufferDataSize - NextAUBufferOffset;
			// Did we get the next frame's worth?
			if (NumGot >= NextAUFrameSize)
			{
				BufferBaseData += NextAUBufferOffset;
				uint32 HeaderValue = GetUINT32BE(BufferBaseData);
				// Safety check. Works for MPEG1 and AAC audio.
				if ((HeaderValue & FrameSyncMask) != FrameSyncValue)
				{
					ParsingErrorMessage = FString::Printf(TEXT("Frame sync marker not found. Corrupt file?"));
					LogMessage(IInfoLog::ELevel::Error, ParsingErrorMessage);
					bDone = true;
					bHasErrored = true;
				}

				if (Request->bIsAAC)
				{
					if (ElectraDecodersUtil::MPEG::AACUtils::ParseADTSHeader(adts, MakeConstArrayView(BufferBaseData, NextAUFrameSize)))
					{
						NextHeaderSkipBytes = adts.HeaderSize;
					}
					else
					{
						ParsingErrorMessage = FString::Printf(TEXT("Not a valid ADTS frame. Corrupt file?"));
						LogMessage(IInfoLog::ELevel::Error, ParsingErrorMessage);
						bDone = true;
						bHasErrored = true;
					}
				}

				if (!AUDuration.IsValid())
				{
					if (!Request->bIsAAC)
					{
						int32 FrameSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplesPerFrame(HeaderValue);
						int32 SampleRate = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplingRate(HeaderValue);
						AUDuration.SetFromND(FrameSize, (uint32)SampleRate);
					}
					else
					{
						int32 FrameSize = 1024;
						int32 SampleRate = ElectraDecodersUtil::MPEG::AACUtils::GetSampleRateFromFrequenceIndex(adts.SamplingFrequencyIndex);
						if (SampleRate)
						{
							AUDuration.SetFromND(FrameSize, (uint32)SampleRate);
						}
					}
				}

				// Keep track of how many bytes we have consumed so far. This correlates the Live stream metadata to this time.
				NumTotalBytesRead += NextAUFrameSize;
				if (LiveRequest.IsValid())
				{
					if (LiveRequest->bHasFailed)
					{
						bHasErrored = true;
					}

					TArray<int64> MetaPositions;
					LiveRequest->MetadataBufferMap.GenerateKeyArray(MetaPositions);
					for(int32 nm=0; nm<MetaPositions.Num(); ++nm)
					{
						if (MetaPositions[nm] <= NumTotalBytesRead)
						{
							TArray<uint8> mda = LiveRequest->MetadataBufferMap[MetaPositions[nm]];
							LiveRequest->MetadataBufferMap.Remove(MetaPositions[nm]);
							if (mda.Num())
							{
								const TCHAR* const Delimiter = TEXT(";");
								TArray<FString> md;
								StringHelpers::ArrayToString(mda).ParseIntoArray(md, Delimiter, true);
								for(int32 k=0; k<md.Num(); ++k)
								{
									if (md[k].StartsWith(TEXT("StreamTitle=")))
									{
										FString Title = md[k].Mid(12);
										if (Title.Len()>1 && Title[0]==TCHAR('\'') && Title[Title.Len()-1]==TCHAR('\''))
										{
											Title.MidInline(1, Title.Len()-2);
										}

										TSharedPtr<FMetadataParser, ESPMode::ThreadSafe> mp = MakeShared<FMetadataParser, ESPMode::ThreadSafe>();
										mp->AddItem(TEXT("Title"), Title);
										PlayerSessionServices->SendMessageToPlayer(FPlaylistMetadataUpdateMessage::Create(NextAUPTS, mp, true));
									}
								}
							}
						}
					}
				}

				// Create an access unit.
				FAccessUnit *AccessUnit = FAccessUnit::Create(Parameters.MemoryProvider);
				if (AccessUnit)
				{
					AccessUnit->ESType = EStreamType::Audio;
					AccessUnit->PTS = NextAUPTS;
					AccessUnit->DTS = NextAUPTS;
					AccessUnit->Duration = AUDuration;

					AccessUnit->EarliestPTS = Request->EarliestPTS;
					AccessUnit->LatestPTS = Request->LastPTS;

					AccessUnit->AUSize = (uint32) NextAUFrameSize - NextHeaderSkipBytes;
					AccessUnit->AUCodecData = AUCodecData;
					// Set the sequence index member and update all timestamps with it as well.
					AccessUnit->SequenceIndex = Request->TimestampSequenceIndex;
					AccessUnit->DTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->PTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->EarliestPTS.SetSequenceIndex(Request->TimestampSequenceIndex);
					AccessUnit->LatestPTS.SetSequenceIndex(Request->TimestampSequenceIndex);

					AccessUnit->bIsFirstInSequence = bIsFirstInSequence;
					AccessUnit->bIsSyncSample = true;
					AccessUnit->bIsDummyData = false;
					AccessUnit->AUData = AccessUnit->AllocatePayloadBuffer(AccessUnit->AUSize);

					// Set the associated stream metadata
					AccessUnit->BufferSourceInfo = BufferSourceInfo;

					bIsFirstInSequence = false;

					FMemory::Memcpy(AccessUnit->AUData, BufferBaseData + NextHeaderSkipBytes, NextAUFrameSize - NextHeaderSkipBytes);

					// Unlock the receive buffer now so the reader won't be blocked.
					Lock.Unlock();

					DurationSuccessfullyRead += AccessUnit->Duration;

					bool bSentOff = false;

					// Check if the AU is outside the time range we are allowed to read.
					// The last one (the one that is already outside the range, actually) is tagged as such and sent into the buffer.
					// The respective decoder has to handle this flag if necessary and/or drop the AU.
					// We need to send at least one AU down so the FMultiTrackAccessUnitBuffer does not stay empty for this period!
					// Already sent the last one?
					if (bReadPastLastPTS)
					{
						// Yes. Release this AU and be done.
						bSentOff = true;
						bDone = true;
					}
					else if (AccessUnit->DTS >= AccessUnit->LatestPTS && AccessUnit->PTS >= AccessUnit->LatestPTS)
					{
						// Tag the last one and send it off.
						AccessUnit->bIsLastInPeriod = true;
						bReadPastLastPTS = true;
					}

					while(!bSentOff && !HasBeenAborted() && !bTerminate)
					{
						if (Parameters.EventListener->OnFragmentAccessUnitReceived(AccessUnit))
						{
							DurationSuccessfullyDelivered += AccessUnit->Duration;
							bSentOff = true;
							AccessUnit = nullptr;

							// Since we have delivered this access unit, if we are detecting an error now we need to then
							// retry on the _next_ AU and not this one again!
							Request->LastSuccessfullyUsedBytePos = Request->FileStartOffset + NextAUBufferOffset + NextAUFrameSize;
							Request->LastSuccessfullyUsedPTS = NextAUPTS + AUDuration;
						}
						else
						{
							FMediaRunnable::SleepMicroseconds(1000 * 10);
						}
					}

					// Release the AU if we still have it.
					FAccessUnit::Release(AccessUnit);
					AccessUnit = nullptr;

					// For error handling, if we managed to get additional data we reset the retry count.
					if (ds.RetryNumber && DurationSuccessfullyRead.GetAsSeconds() > 1.0)
					{
						ds.RetryNumber = 0;
						Request->NumOverallRetries = 0;
					}
				}

				// Advance to next
				NextAUPTS += AUDuration;
				NextAUBufferOffset += NextAUFrameSize;
				NumGot -= NextAUFrameSize;
				NextAUFrameSize = 0;
				// Did we also get the next header bytes?
				if (NumGot >= kNumNextHeaderBytes)
				{
					FScopeLock Lock2(ReceiveBuffer->GetLock());
					if (!Request->bIsAAC)
					{
						HeaderValue = GetUINT32BE(ReceiveBuffer->GetLinearReadData() + NextAUBufferOffset + NextAUFrameSize);
						NextAUFrameSize = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetFrameSize(HeaderValue);
					}
					else
					{
						if (ElectraDecodersUtil::MPEG::AACUtils::ParseADTSHeader(adts, MakeConstArrayView(ReceiveBuffer->GetLinearReadData() + NextAUBufferOffset + NextAUFrameSize, kNumNextHeaderBytes)))
						{
							NextAUFrameSize = adts.FrameLength;
							NextHeaderSkipBytes = adts.HeaderSize;
						}
					}
				}
				else
				{
					bDone = true;
				}
			}
			else
			{
				bDone = true;
			}
		}
	}

	// Remove the download request.
	if (!bIsLivePlayback)
	{
		ProgressListener.Reset();
		PlayerSessionServices->GetHTTPManager()->RemoveRequest(HTTP, false);
		Request->ConnectionInfo = HTTP->ConnectionInfo;
		HTTP.Reset();
	}
	else
	{
		LiveRequest->Cancel();
		LiveRequest->WaitUntilFinished();

		Request->ConnectionInfo.ContentType = TEXT("audio/mpeg");
		Request->ConnectionInfo.BytesReadSoFar = LiveRequest->TotalDataBytePos;
		Request->ConnectionInfo.HTTPVersionReceived = 11;
		Request->ConnectionInfo.bWasAborted = bRequestCanceled;
		Request->ConnectionInfo.bHasFinished = true;
		Request->ConnectionInfo.StatusInfo.HTTPStatus = LiveRequest->StatusCode;
		Request->ConnectionInfo.StatusInfo.bReadError = true;
		// Unless the server responded with a bad status code, set the retry count to a negative
		// value to prevent the ABR from checking against it. A live cast has no end, so getting
		// here with a good status code (unless aborted) means there was a connection error,
		// which can be retried indefinitely.
		if (!bRequestCanceled && LiveRequest->StatusCode < 300)
		{
			ds.RetryNumber = -1;
		}

		LiveRequest.Reset();
	}

	// Set up download stat fields.
	ds.HTTPStatusCode = Request->ConnectionInfo.StatusInfo.HTTPStatus;
	ds.TimeToFirstByte = Request->ConnectionInfo.TimeUntilFirstByte;
	ds.TimeToDownload = (Request->ConnectionInfo.RequestEndTime - Request->ConnectionInfo.RequestStartTime).GetAsSeconds();
	ds.ByteSize = Request->ConnectionInfo.ContentLength;
	ds.NumBytesDownloaded = Request->ConnectionInfo.BytesReadSoFar;

	ds.FailureReason = Request->ConnectionInfo.StatusInfo.ErrorDetail.GetMessage();
	if (ParsingErrorMessage.Len())
	{
		ds.FailureReason = ParsingErrorMessage;
	}
	ds.bWasSuccessful = !bHasErrored;
	ds.URL.URL = Request->ConnectionInfo.EffectiveURL;
	ds.HTTPStatusCode = Request->ConnectionInfo.StatusInfo.HTTPStatus;
	ds.DurationDownloaded = DurationSuccessfullyRead.GetAsSeconds();
	ds.DurationDelivered = DurationSuccessfullyDelivered.GetAsSeconds();
	ds.TimeToFirstByte = Request->ConnectionInfo.TimeUntilFirstByte;
	ds.TimeToDownload = (Request->ConnectionInfo.RequestEndTime - Request->ConnectionInfo.RequestStartTime).GetAsSeconds();
	ds.ByteSize = Request->ConnectionInfo.ContentLength;
	ds.NumBytesDownloaded = Request->ConnectionInfo.BytesReadSoFar;
	ds.bIsCachedResponse = Request->ConnectionInfo.bIsCachedResponse;

	// Reset the current request so another one can be added immediately when we call OnFragmentClose()
	CurrentRequest.Reset();
	PlayerSessionServices->GetStreamSelector()->ReportDownloadEnd(ds);
	Parameters.EventListener->OnFragmentClose(Request);
}

void FStreamReaderMPEGAudio::WorkerThread()
{
	LLM_SCOPE(ELLMTag::ElectraPlayer);
	while(!bTerminate)
	{
		WorkSignal.WaitAndReset();
		if (bTerminate)
		{
			break;
		}
		TSharedPtrTS<FStreamSegmentRequestMPEGAudio> Request = CurrentRequest;
		if (Request.IsValid())
		{
			HandleRequest();
		}
	}
}

} // namespace Electra
