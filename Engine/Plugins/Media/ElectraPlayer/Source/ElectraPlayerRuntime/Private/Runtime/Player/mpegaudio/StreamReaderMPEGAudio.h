// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Player/Manifest.h"
#include "HTTP/HTTPManager.h"
#include "SynchronizedClock.h"
#include "PlaylistReaderMPEGAudio.h"
#include "Utilities/TimeUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "ElectraPlayerPrivate.h"
#include "Player/PlayerStreamReader.h"
#include "StreamAccessUnitBuffer.h"



namespace Electra
{


class FStreamSegmentRequestMPEGAudio : public IStreamSegment
{
public:
	virtual ~FStreamSegmentRequestMPEGAudio() = default;

	void SetPlaybackSequenceID(uint32 InPlaybackSequenceID) override
	{ PlaybackSequenceID = InPlaybackSequenceID; }
	uint32 GetPlaybackSequenceID() const override
	{ return PlaybackSequenceID; }
	void SetExecutionDelay(const FTimeValue& UTCNow, const FTimeValue& ExecutionDelay) override
	{ }
	FTimeValue GetExecuteAtUTCTime() const override
	{ return FTimeValue::GetInvalid(); }
	EStreamType GetType() const override
	{ return EStreamType::Audio; }
	void GetDependentStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutDependentStreams) const override
	{ }
	void GetRequestedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutRequestedStreams) override;
	void GetEndedStreams(TArray<TSharedPtrTS<IStreamSegment>>& OutAlreadyEndedStreams) override;
	FTimeValue GetFirstPTS() const override
	{ 
		FTimeValue First(FirstPTS);
		First.SetSequenceIndex(TimestampSequenceIndex);
		return First;
	}
	FTimeRange GetTimeRange() const override;

	int32 GetQualityIndex() const override
	{ return 0; }
	int32 GetBitrate() const override
	{ return Bitrate; }
	void GetDownloadStats(Metrics::FSegmentDownloadStats& OutStats) const override
	{ OutStats = DownloadStats; }
	bool GetStartupDelay(FTimeValue& OutStartTime, FTimeValue& OutTimeIntoSegment, FTimeValue& OutSegmentDuration) const override
	{ return false; }


	TSharedPtrTS<ITimelineMediaAsset> MediaAsset;

	FTimeValue FirstPTS;
	FTimeValue EarliestPTS;					//!< PTS of the first sample to be presented.
	FTimeValue LastPTS;						//!< PTS at which no further samples are to be presented.
	int64 FileStartOffset = -1;				//!< Where to start in the file
	int64 FileEndOffset = -1;				//!< Where to end in the file (for HTTP range GET requests)
	enum class ECastType
	{
		None,
		IcyCast
	};
	ECastType CastType = ECastType::None;

	uint32 MPEGHeaderMask = 0;
	uint32 MPEGHeaderExpectedValue = 0;
	int32 CBRFrameSize = 0;
	bool bIsAAC = false;
	bool bIsVBR = false;
	bool bIsLive = false;
	bool bIsEOSRequest = false;
	double Duration = 0.0;
	FStreamCodecInformation CodecInfo;

	uint32 PlaybackSequenceID = ~0U;
	int32 Bitrate = 0;
	bool bIsContinuationSegment = false;		//!< true if this segment continues where the previous left off and no sync samples should be expected.
	bool bIsFirstSegment = false;				//!< true if this segment is the first to start with or the first after a seek.
	bool bIsLastSegment = false;				//!< true if this segment is the last.

	int64 TimestampSequenceIndex = 0;		//!< Sequence index to set in all timestamp values of the decoded access unit.
	int32 NumOverallRetries = 0;			//!< Number of retries

	int64 LastSuccessfullyUsedBytePos = -1;
	FTimeValue LastSuccessfullyUsedPTS;

	Metrics::FSegmentDownloadStats DownloadStats;
	HTTP::FConnectionInfo ConnectionInfo;
};


class FStreamReaderMPEGAudio : public IStreamReader, public FMediaThread
{
public:
	FStreamReaderMPEGAudio();
	virtual ~FStreamReaderMPEGAudio();
	UEMediaError Create(IPlayerSessionServices* PlayerSessionService, const CreateParam &createParam) override;
	bool LoadsMultiplexedTracksSequentially() override;
	void Close() override;
	EAddResult AddRequest(uint32 CurrentPlaybackSequenceID, TSharedPtrTS<IStreamSegment> Request) override;
	void CancelRequest(EStreamType StreamType, bool bSilent) override;
	void CancelRequests() override;

private:
	void WorkerThread();
	void LogMessage(IInfoLog::ELevel Level, const FString& Message);
	int32 HTTPProgressCallback(const IElectraHttpManager::FRequest* InRequest);
	void HTTPCompletionCallback(const IElectraHttpManager::FRequest* InRequest);
	void HandleRequest();

	bool HasBeenAborted() const;
	bool HasErrored() const;

	CreateParam Parameters;
	IPlayerSessionServices* PlayerSessionServices = nullptr;
	bool bIsStarted = false;
	bool bTerminate = false;
	bool bRequestCanceled = false;
	bool bHasErrored = false;
	FErrorDetail ErrorDetail;

	struct FLiveRequest;
	TSharedPtrTS<FLiveRequest> LiveRequest;
	TSharedPtrTS<FStreamSegmentRequestMPEGAudio> CurrentRequest;
	TSharedPtrTS<FWaitableBuffer> ReceiveBuffer;
	FMediaEvent WorkSignal;

	static uint32 UniqueDownloadID;
};

} // namespace Electra
