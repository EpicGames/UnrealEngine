// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "PlayerTime.h"

namespace Electra
{
	class IMediaRenderClock
	{
	public:
		virtual ~IMediaRenderClock() = default;

		enum class ERendererType
		{
			Video,
			Audio,
			Subtitles,
		};

		/**
		 * Called by the renderer to set the time of the most sample that has been output most recently.
		 *
		 * @param ForRenderer
		 *               Identifies the type of renderer from which the time is set.
		 * @param CurrentRenderTime
		 *               The time of the sample most recently output.
		 */
		virtual void SetCurrentTime(ERendererType ForRenderer, const FTimeValue& CurrentRenderTime) = 0;

		/**
		 * Gets the current _interpolated_ sample output time from the last
		 * time SetCurrentTime() was called plus the elapsed time since then.
		 *
		 * @param FromRenderer
		 *               Identifies the type of renderer for which to get the time.
		 *
		 * @return Interpolated render time
		 */
		virtual FTimeValue GetInterpolatedRenderTime(ERendererType FromRenderer) = 0;
	};

} // namespace Electra
