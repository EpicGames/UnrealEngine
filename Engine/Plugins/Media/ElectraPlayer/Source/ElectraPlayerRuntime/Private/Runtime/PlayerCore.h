// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/MediaTypes.h"
#include "Core/MediaMacros.h"
#include "Core/MediaNoncopyable.h"

#include "Core/MediaThreads.h"
#include "Core/MediaQueue.h"
#include "Core/MediaSemaphore.h"
#include "Core/MediaEventSignal.h"
#include "Core/MediaInterlocked.h"
#include "Core/MediaMessageQueue.h"
