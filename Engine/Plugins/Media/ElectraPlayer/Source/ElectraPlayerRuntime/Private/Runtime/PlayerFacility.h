// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"


namespace Electra
{

	namespace Facility
	{

		enum class EFacility
		{
			Unknown,
			Utility,
			Player,
			ABR,
			HTTPReader,
			BufferedDataReader,
			HLSPlaylistHandler,
			HLSParser,
			HLSPlaylistBuilder,
			MP4Parser,
			VideoRenderer,
			AudioRenderer,
			SubtitleRenderer,
			MP4PlaylistReader,
			MP4Playlist,
			MP4StreamReader,
			ISOBMFFPlaylistReader,
			ISOBMFFPlaylist,
			ISOBMFFStreamReader,
			MKVPlaylistReader,
			MKVPlaylist,
			MKVStreamReader,
			MKVParser,
			MPEGTSParser,
			LicenseKey,
			DASHMPDReader,
			DASHMPDParser,
			DASHMPDBuilder,
			DASHManifest,
			DASHStreamReader,
			MPEGAudioPlaylistReader,
			MPEGAudioPlaylist,
			MPEGAudioStreamReader,
			CommonSegmentReader,
			EntityCache,
			DRM,
			SubtitleDecoder,
			AudioDecoder,
			VideoDecoder,
			LastEntry = 255
		};

		static const TCHAR* GetName(EFacility InFacility)
		{
			switch (InFacility)
			{
				case EFacility::LastEntry:
					return TEXT("???");
				case EFacility::Unknown:
					return TEXT("Unknown");
				case EFacility::Utility:
					return TEXT("Utility");
				case EFacility::Player:
					return TEXT("Player");
				case EFacility::ABR:
					return TEXT("ABR");
				case EFacility::HTTPReader:
					return TEXT("HTTP reader");
				case EFacility::BufferedDataReader:
					return TEXT("Buffered data reader");
				case EFacility::HLSPlaylistHandler:
					return TEXT("HLS playlist handler");
				case EFacility::HLSParser:
					return TEXT("HLS parser");
				case EFacility::HLSPlaylistBuilder:
					return TEXT("HLS playlist builder");
				case EFacility::MP4Parser:
					return TEXT("MP4 parser");
				case EFacility::VideoRenderer:
					return TEXT("Video renderer");
				case EFacility::AudioRenderer:
					return TEXT("Audio renderer");
				case EFacility::SubtitleRenderer:
					return TEXT("Subtitle renderer");
				case EFacility::MP4PlaylistReader:
					return TEXT("MP4 playlist reader");
				case EFacility::MP4Playlist:
					return TEXT("MP4 playlist");
				case EFacility::MP4StreamReader:
					return TEXT("MP4 reader");
				case EFacility::ISOBMFFPlaylistReader:
					return TEXT("ISOBMFF playlist reader");
				case EFacility::ISOBMFFPlaylist:
					return TEXT("ISOBMFF playlist");
				case EFacility::ISOBMFFStreamReader:
					return TEXT("ISOBMFF reader");
				case EFacility::LicenseKey:
					return TEXT("License key");
				case EFacility::DASHMPDReader:
					return TEXT("DASH MPD reader");
				case EFacility::DASHMPDParser:
					return TEXT("DASH MPD parser");
				case EFacility::DASHMPDBuilder:
					return TEXT("DASH MPD builder");
				case EFacility::DASHManifest:
					return TEXT("DASH manifest");
				case EFacility::DASHStreamReader:
					return TEXT("DASH stream reader");
				case EFacility::MPEGAudioPlaylistReader:
					return TEXT("MPEG audio playlist reader");
				case EFacility::MPEGAudioPlaylist:
					return TEXT("MPEG audio playlist");
				case EFacility::MPEGAudioStreamReader:
					return TEXT("MPEG audio stream reader");
				case EFacility::EntityCache:
					return TEXT("Entity cache");
				case EFacility::DRM:
					return TEXT("DRM");
				case EFacility::SubtitleDecoder:
					return TEXT("Subtitle decoder");
				case EFacility::AudioDecoder:
					return TEXT("Audio decoder");
				case EFacility::VideoDecoder:
					return TEXT("Video decoder");
				case EFacility::MKVPlaylistReader:
					return TEXT("MKV playlist reader");
				case EFacility::MKVPlaylist:
					return TEXT("MKV playlist");
				case EFacility::MKVStreamReader:
					return TEXT("MKV reader");
				case EFacility::MKVParser:
					return TEXT("MKV parser");
				case EFacility::MPEGTSParser:
					return TEXT("MPEG TS parser");
				case EFacility::CommonSegmentReader:
					return TEXT("Common stream reader");
			}
			return TEXT("???");
		}

	} // namespace Facility

} // namespace Electra

