// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "PlayerRuntimeGlobal.h"
#include "PlayerPlatform.h"
#include "HAL/PlatformProperties.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "String/LexFromString.h"
#include "Templates/Tuple.h"
#include "Modules/ModuleManager.h"

#ifndef ELECTRA_PRECREATE_HTTP_MANAGER
#define ELECTRA_PRECREATE_HTTP_MANAGER 0
#endif

#if ELECTRA_PRECREATE_HTTP_MANAGER
#include "HTTP/HTTPManager.h"
#endif



namespace Electra
{
	namespace Global
	{
		enum ESuspendOn
		{
			Background = 1 << 0,
			Deactivate = 1 << 1
		};

		FString ThisPlatformName;
		TMap<FString, bool> EnabledAnalyticsEvents;
		FDelegateHandle ApplicationSuspendedDelegate;
		FDelegateHandle ApplicationResumeDelegate;
		FDelegateHandle ApplicationDeactivatedDelegate;
		FDelegateHandle ApplicationReactivatedDelegate;
		FDelegateHandle ApplicationTerminatingDelegate;
		FCriticalSection ApplicationHandlerLock;
		TArray<TWeakPtrTS<FApplicationTerminationHandler>> ApplicationTerminationHandlers;
		TArray<TPair<TWeakPtrTS<FFGBGNotificationHandlers>, int32>>	ApplicationBGFGHandlers;
		int32 NotifyOnMask = ESuspendOn::Background;
		bool bPlatformDecodersMustSuspendFromINI = false;
		bool bPlatformDecodersMustSuspendFromCVar = false;
		volatile bool bIsInBackground = false;
		volatile bool bIsDeactivated = false;
		volatile bool bAppIsTerminating = false;

#if ELECTRA_PRECREATE_HTTP_MANAGER
		TSharedPtrTS<IElectraHttpManager> HttpManager;
#endif
	}
	using namespace Global;

	static void UpdateSuspendFromPlatformList(bool& bOut, int32& OutModeMask, const FString& InPlatformList, int32 InCurrentMask)
	{
		using KV = TPair<FString, int32>;
		auto SplitKV = [](const FString& InKV, int32 InDefaultV) -> KV
		{
			int32 EqPos = INDEX_NONE;
			if (InKV.FindChar(TCHAR('='), EqPos))
			{
				FString K = InKV.Left(EqPos);
				FString V = InKV.Mid(EqPos+1);
				int32 VV = InDefaultV;
				LexFromString(VV, V);
				if (VV < 0 || VV > 3)
				{
					VV = InDefaultV;
				}
				return KV(K, VV);
			}
			return KV(InKV, InDefaultV);
		};
		bOut = false;
		auto AllPlInfs = FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos();
		TArray<FString> PlatformIDs;
		InPlatformList.ParseIntoArray(PlatformIDs, TEXT(","), true);
		for(auto pfId : PlatformIDs)
		{
			const KV platKV = SplitKV(pfId, InCurrentMask);
			// Does this parse as a Guid?
			FGuid guid;
			if (FGuid::Parse(platKV.Key, guid))
			{
				for(auto& plInf : AllPlInfs)
				{
					if (plInf.Value.GlobalIdentifier == guid && Global::ThisPlatformName.Equals(plInf.Value.IniPlatformName.ToString(), ESearchCase::IgnoreCase))
					{
						bOut = true;
						OutModeMask = platKV.Value;
						return;
					}
				}
			}
			else if (Global::ThisPlatformName.Equals(platKV.Key, ESearchCase::IgnoreCase))
			{
				bOut = true;
				OutModeMask = platKV.Value;
				return;
			}
			else if (platKV.Key.Equals(TEXT("*")))
			{
				bOut = true;
				OutModeMask = platKV.Value;
				return;
			}
		}
	}

	static TAutoConsoleVariable<bool> CVarElectraPlayerBackgroundSuspended(
		TEXT("ElectraPlayer.bBackgroundSuspendPlatform"), false,
		TEXT("Whether or not this platform must suspend decoders in background"),
		ECVF_Default);

	static TAutoConsoleVariable<bool> CVarElectraPlayerBackgroundNotSuspended(
		TEXT("ElectraPlayer.bBackgroundNotSuspendPlatform"), false,
		TEXT("If true this platform will never suspend decoders in background"),
		ECVF_Default);

	static TAutoConsoleVariable<FString> CVarElectraPlayerBackgroundSuspendedPlatformsStr(
		TEXT("ElectraPlayer.BackgroundSuspendedPlatforms"), "",
		TEXT("Comma separated platform identifiers requiring decoders to be suspended in background"),
		ECVF_Default);

	static void UpdatePlatformSuspendFromCVar(IConsoleVariable* Var)
	{
		UpdateSuspendFromPlatformList(Global::bPlatformDecodersMustSuspendFromCVar, Global::NotifyOnMask, Var->GetString(), Global::NotifyOnMask);
	}


	static void HandleApplicationWillTerminate()
	{
		ApplicationHandlerLock.Lock();
		TArray<TWeakPtrTS<FApplicationTerminationHandler>>	CurrentHandlers(ApplicationTerminationHandlers);
		bAppIsTerminating = true;
		ApplicationHandlerLock.Unlock();
		for(auto &Handler : CurrentHandlers)
		{
			TSharedPtrTS<FApplicationTerminationHandler> Hdlr = Handler.Pin();
			if (Hdlr.IsValid())
			{
				Hdlr->Terminate();
			}
		}
	}

	static void HandleApplicationWillEnterBackground()
	{
		ApplicationHandlerLock.Lock();
		TArray<TPair<TWeakPtrTS<FFGBGNotificationHandlers>, int32>>	CurrentHandlers(ApplicationBGFGHandlers);
		bIsInBackground = true;
		ApplicationHandlerLock.Unlock();
		for(auto &Handler : CurrentHandlers)
		{
			if ((Handler.Value & Global::ESuspendOn::Background) != 0)
			{
				TSharedPtrTS<FFGBGNotificationHandlers> Hdlr = Handler.Key.Pin();
				if (Hdlr.IsValid())
				{
					Hdlr->WillEnterBackground();
				}
			}
		}
	}

	static void HandleApplicationHasEnteredForeground()
	{
		ApplicationHandlerLock.Lock();
		TArray<TPair<TWeakPtrTS<FFGBGNotificationHandlers>, int32>>	CurrentHandlers(ApplicationBGFGHandlers);
		bIsInBackground = false;
		ApplicationHandlerLock.Unlock();
		for(auto &Handler : CurrentHandlers)
		{
			if ((Handler.Value & Global::ESuspendOn::Background) != 0)
			{
				TSharedPtrTS<FFGBGNotificationHandlers> Hdlr = Handler.Key.Pin();
				if (Hdlr.IsValid())
				{
					Hdlr->HasEnteredForeground();
				}
			}
		}
	}

	static void HandleApplicationWillDeactivate()
	{
		ApplicationHandlerLock.Lock();
		TArray<TPair<TWeakPtrTS<FFGBGNotificationHandlers>, int32>>	CurrentHandlers(ApplicationBGFGHandlers);
		bIsDeactivated = true;
		ApplicationHandlerLock.Unlock();
		for(auto &Handler : CurrentHandlers)
		{
			if ((Handler.Value & Global::ESuspendOn::Deactivate) != 0)
			{
				TSharedPtrTS<FFGBGNotificationHandlers> Hdlr = Handler.Key.Pin();
				if (Hdlr.IsValid())
				{
					Hdlr->WillEnterBackground();
				}
			}
		}
	}

	static void HandleApplicationHasReactivated()
	{
		ApplicationHandlerLock.Lock();
		TArray<TPair<TWeakPtrTS<FFGBGNotificationHandlers>, int32>>	CurrentHandlers(ApplicationBGFGHandlers);
		bIsDeactivated = false;
		ApplicationHandlerLock.Unlock();
		for(auto &Handler : CurrentHandlers)
		{
			if ((Handler.Value & Global::ESuspendOn::Deactivate) != 0)
			{
				TSharedPtrTS<FFGBGNotificationHandlers> Hdlr = Handler.Key.Pin();
				if (Hdlr.IsValid())
				{
					Hdlr->HasEnteredForeground();
				}
			}
		}
	}


	//-----------------------------------------------------------------------------
	/**
	 * Initializes core service functionality. Memory hooks must have been registered before calling this function.
	 *
	 * @param configuration
	 *
	 * @return
	 */
	bool Startup(const Configuration& InConfiguration)
	{
		Global::ThisPlatformName = FPlatformProperties::IniPlatformName();

		// Load the modules we depend on. They may have been loaded already, but we do it explicitly here to ensure that
		// they will not be unloaded on shutdown before this module here, otherwise there could be crashes.
		FModuleManager::Get().LoadModule(TEXT("ElectraBase"));
		FModuleManager::Get().LoadModule(TEXT("ElectraSamples"));
		FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));
		FModuleManager::Get().LoadModule(TEXT("ElectraHTTPStream"));
		FModuleManager::Get().LoadModule(TEXT("ElectraSubtitles"));
		FModuleManager::Get().LoadModule(TEXT("ElectraCDM"));

		EnabledAnalyticsEvents = InConfiguration.EnabledAnalyticsEvents;

		// Dummy get the platform ID once so any potential statics are created right away.
		Platform::GetPlatformID();

		// Get the list of platforms for which all decoders need to be suspended when the application
		// goes into background/suspend mode.
		FString BgSuspendPlatformsFromIni;
		if (GConfig->GetString(TEXT("ElectraPlayer"), TEXT("BackgroundSuspendedPlatforms"), BgSuspendPlatformsFromIni, GEngineIni))
		{
			UpdateSuspendFromPlatformList(Global::bPlatformDecodersMustSuspendFromINI, Global::NotifyOnMask, BgSuspendPlatformsFromIni, Global::NotifyOnMask);
		}
		CVarElectraPlayerBackgroundSuspendedPlatformsStr.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&UpdatePlatformSuspendFromCVar));
		UpdatePlatformSuspendFromCVar(CVarElectraPlayerBackgroundSuspendedPlatformsStr.AsVariable());

		if (!ApplicationTerminatingDelegate.IsValid())
		{
			ApplicationTerminatingDelegate = FCoreDelegates::GetApplicationWillTerminateDelegate().AddStatic(&HandleApplicationWillTerminate);
		}
		if (!ApplicationSuspendedDelegate.IsValid())
		{
			ApplicationSuspendedDelegate = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddStatic(&HandleApplicationWillEnterBackground);
		}
		if (!ApplicationDeactivatedDelegate.IsValid())
		{
			ApplicationDeactivatedDelegate = FCoreDelegates::ApplicationWillDeactivateDelegate.AddStatic(&HandleApplicationWillDeactivate);
		}
		if (!ApplicationResumeDelegate.IsValid())
		{
			ApplicationResumeDelegate = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddStatic(&HandleApplicationHasEnteredForeground);
		}
		if (!ApplicationReactivatedDelegate.IsValid())
		{
			ApplicationReactivatedDelegate = FCoreDelegates::ApplicationHasReactivatedDelegate.AddStatic(&HandleApplicationHasReactivated);
		}


#if ELECTRA_PRECREATE_HTTP_MANAGER
		HttpManager = IElectraHttpManager::Create();
		if (!HttpManager.IsValid())
		{
			return false;
		}
#endif

		return true;
	}


	//-----------------------------------------------------------------------------
	/**
	 * Shuts down core services.
	 */
	void Shutdown(void)
	{
#if ELECTRA_PRECREATE_HTTP_MANAGER
		HttpManager.Reset();
#endif

		if (ApplicationSuspendedDelegate.IsValid())
		{
			FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(ApplicationSuspendedDelegate);
			ApplicationSuspendedDelegate.Reset();
		}
		if (ApplicationDeactivatedDelegate.IsValid())
		{
			FCoreDelegates::ApplicationWillDeactivateDelegate.Remove(ApplicationDeactivatedDelegate);
			ApplicationDeactivatedDelegate.Reset();
		}
		if (ApplicationResumeDelegate.IsValid())
		{
			FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ApplicationResumeDelegate);
			ApplicationResumeDelegate.Reset();
		}
		if (ApplicationReactivatedDelegate.IsValid())
		{
			FCoreDelegates::ApplicationHasReactivatedDelegate.Remove(ApplicationReactivatedDelegate);
			ApplicationReactivatedDelegate.Reset();
		}
		if (ApplicationTerminatingDelegate.IsValid())
		{
			FCoreDelegates::GetApplicationWillTerminateDelegate().Remove(ApplicationTerminatingDelegate);
		}
	}


	void AddTerminationNotificationHandler(TSharedPtrTS<FApplicationTerminationHandler> InHandler)
	{
		FScopeLock lock(&ApplicationHandlerLock);
		ApplicationTerminationHandlers.Add(InHandler);
	}

	void RemoveTerminationNotificationHandler(TSharedPtrTS<FApplicationTerminationHandler> InHandler)
	{
		FScopeLock lock(&ApplicationHandlerLock);
		ApplicationTerminationHandlers.Remove(InHandler);
	}

	bool AddBGFGNotificationHandler(TSharedPtrTS<FFGBGNotificationHandlers> InHandlers)
	{
		FScopeLock lock(&ApplicationHandlerLock);
		ApplicationBGFGHandlers.Add(TPair<TWeakPtrTS<FFGBGNotificationHandlers>, int32>(InHandlers, Global::NotifyOnMask));
		return (bIsInBackground && ((Global::NotifyOnMask & Global::ESuspendOn::Background) != 0)) ||
			   (bIsDeactivated && ((Global::NotifyOnMask & Global::ESuspendOn::Deactivate) != 0));
	}

	void RemoveBGFGNotificationHandler(TSharedPtrTS<FFGBGNotificationHandlers> InHandlers)
	{
		FScopeLock lock(&ApplicationHandlerLock);
		for(int32 i=0; i<ApplicationBGFGHandlers.Num(); ++i)
		{
			if (ApplicationBGFGHandlers[i].Key == InHandlers)
			{
				ApplicationBGFGHandlers.RemoveAt(i);
				--i;
			}
		}
	}


	/**
	 * Check if the provided analytics event is enabled
	 *
	 * @param AnalyticsEventName of event to check
	 * @return true if event is found and is set to true
	 */
	bool IsAnalyticsEventEnabled(const FString& AnalyticsEventName)
	{
		const bool* bEventEnabled = EnabledAnalyticsEvents.Find(AnalyticsEventName);
		return bEventEnabled && *bEventEnabled;
	}


	/**
	 * Checks if this platform needs to suspend decoding when backgrounded.
	 * This is either set via engine ini file or cvars.
	 * There is an additional cvar to never suspend if it is conceptually simpler to
	 * unblock the current platform via hotfix.
	 */
	bool MustSuspendDecodersInBackground(bool bInDecoderPreference)
	{
		// Is this platform forcing no suspend?
		return CVarElectraPlayerBackgroundNotSuspended.GetValueOnAnyThread() ? false :
			(bInDecoderPreference || Global::bPlatformDecodersMustSuspendFromINI || Global::bPlatformDecodersMustSuspendFromCVar || CVarElectraPlayerBackgroundSuspended.GetValueOnAnyThread());
	}


	class PendingTaskCounter
	{
	public:
		PendingTaskCounter() : AllDoneSignal(nullptr), NumPendingTasks(0)
		{
			// Note: we only initialize the done signal on first adding a task etc.
			// to avoid a signal to be used during the global constructor phase
			// (too early for UE)
		}

		~PendingTaskCounter()
		{
		}

		//! Adds a pending task.
		void AddTask()
		{
			Init();
			if (FMediaInterlockedIncrement(NumPendingTasks) == 0)
			{
				AllDoneSignal->Reset();
			}
		}

		//! Removes a pending task when it's done. Returns true when this was the last task, false otherwise.
		bool RemoveTask()
		{
			Init();
			if (FMediaInterlockedDecrement(NumPendingTasks) == 1)
			{
				AllDoneSignal->Signal();
				return true;
			}
			else
			{
				return false;
			}
		}

		//! Waits for all pending tasks to have finished. Once all are done new tasks cannot be added.
		bool WaitAllFinished(int32 TimeoutMs)
		{
			Init();
			if (TimeoutMs <= 0)
			{
				AllDoneSignal->Wait();
				return true;
			}
			return AllDoneSignal->WaitTimeout(TimeoutMs * 1000);
		}

		void Reset()
		{
			delete AllDoneSignal;
			AllDoneSignal = nullptr;
		}

	private:
		void Init()
		{
			// Initialize our signal event if we don't have it already...
			if (!AllDoneSignal)
			{
				FMediaEvent* NewSignal = new FMediaEvent();
				if (FMediaInterlockedCompareExchangePointer((void* volatile&)AllDoneSignal, (void*)NewSignal, (void*)nullptr) != nullptr)
				{
					delete NewSignal;
				}
				// The new signal must be set initially to allow for WaitAllFinished() to leave
				// without any task having been added. It gets cleared on the first AddTask().
				AllDoneSignal->Signal();
			}
		}

		FMediaEvent* AllDoneSignal;
		int32		NumPendingTasks;
	};




	static PendingTaskCounter NumActivePlayers;


	bool WaitForAllPlayersToHaveTerminated()
	{
		bool bOk = NumActivePlayers.WaitAllFinished(1000);	// bAppIsTerminating ? 1000 : 0);
		if (bOk)
		{
			// Explicitly shutdown anything in the counter class that may use the engine (as it might shutdown after this)
			NumActivePlayers.Reset();
		}
		return bOk;
	}

	void AddActivePlayerInstance()
	{
		NumActivePlayers.AddTask();
	}

	void RemoveActivePlayerInstance()
	{
		NumActivePlayers.RemoveTask();
	}


};


