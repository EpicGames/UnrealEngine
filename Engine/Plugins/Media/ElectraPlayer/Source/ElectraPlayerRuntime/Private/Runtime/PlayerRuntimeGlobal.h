// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "Templates/Function.h"

namespace Electra
{
	struct Configuration
	{
		TMap<FString, bool>	EnabledAnalyticsEvents;
	};

	//! Initializes core service functionality. Memory hooks must have been registered before calling this function.
	bool Startup(const Configuration& Configuration);

	//! Shuts down core services.
	void Shutdown();

	//! Waits until all player instances have terminated, which may happen asynchronously.
	bool WaitForAllPlayersToHaveTerminated();


	void AddActivePlayerInstance();
	void RemoveActivePlayerInstance();

	//! Check if an analytics event is enabled
	bool IsAnalyticsEventEnabled(const FString& AnalyticsEventName);

	//! Returns if all decoders must be suspended when the application goes into background mode,
	//! either because of platform requirements or because of ini settings.
	bool MustSuspendDecodersInBackground(bool bInDecoderPreference);

	//! Application termination
	struct FApplicationTerminationHandler
	{
		TFunction<void()> Terminate;
	};
	void AddTerminationNotificationHandler(TSharedPtrTS<FApplicationTerminationHandler> InHandler);
	void RemoveTerminationNotificationHandler(TSharedPtrTS<FApplicationTerminationHandler> InHandler);

	//! Background / foreground handling
	struct FFGBGNotificationHandlers
	{
		TFunction<void()> WillEnterBackground;
		TFunction<void()> HasEnteredForeground;
	};
	bool AddBGFGNotificationHandler(TSharedPtrTS<FFGBGNotificationHandlers> InHandlers);
	void RemoveBGFGNotificationHandler(TSharedPtrTS<FFGBGNotificationHandlers> InHandlers);
};

