// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamAccessUnitBuffer.h"

namespace Electra
{
	FMultiTrackAccessUnitBuffer::FMultiTrackAccessUnitBuffer(EStreamType InForType)
	{
		Type = InForType;
		EmptyBuffer = CreateNewBuffer();
		LastPoppedDTS.SetToInvalid();
		LastPoppedPTS.SetToInvalid();
		PlayableDurationPushedSinceEOT.SetToZero();
		bEndOfData = false;
		bEndOfTrack = false;
		bLastPushWasBlocked = false;
		bIsParallelTrackMode = false;
		bPopAsDummyUntilSyncFrame = true;
	}

	TSharedPtrTS<FAccessUnitBuffer> FMultiTrackAccessUnitBuffer::CreateNewBuffer()
	{
		TSharedPtrTS<FAccessUnitBuffer> NewBuffer = MakeSharedTS<FAccessUnitBuffer>();
		return NewBuffer;
	}

	FMultiTrackAccessUnitBuffer::~FMultiTrackAccessUnitBuffer()
	{
		Clear();
	}

	void FMultiTrackAccessUnitBuffer::SetParallelTrackMode()
	{
		bIsParallelTrackMode = true;
	}


	TSharedPtrTS<FAccessUnitBuffer> FMultiTrackAccessUnitBuffer::FindOrCreateBufferFor(TSharedPtrTS<const FBufferSourceInfo>& OutBufferSourceInfo, const TSharedPtrTS<const FBufferSourceInfo>& InBufferInfo, bool bCreateIfNotExist)
	{
		// Note: The access mutex must have been locked already!
		for(int32 i=0; i<BufferList.Num(); ++i)
		{
			if (BufferList[i].Info->PlaybackSequenceID == InBufferInfo->PlaybackSequenceID)
			{
				if (bIsParallelTrackMode)
				{
					if (BufferList[i].Info->HardIndex == InBufferInfo->HardIndex)
					{
						OutBufferSourceInfo = BufferList[i].Info;
						return BufferList[i].Buffer;
					}
				}
				else
				{
					OutBufferSourceInfo = BufferList[i].Info;
					return BufferList[i].Buffer;
				}
			}
		}
		TSharedPtrTS<FAccessUnitBuffer> Buffer;
		if (bCreateIfNotExist)
		{
			FBufferByInfoType bit;
			OutBufferSourceInfo = bit.Info = InBufferInfo;
			Buffer = bit.Buffer = CreateNewBuffer();
			BufferList.Emplace(MoveTemp(bit));
		}
		return Buffer;
	}

	bool FMultiTrackAccessUnitBuffer::Push(FAccessUnit*& AU, const FAccessUnitBuffer::FConfiguration* BufferConfiguration, const FAccessUnitBuffer::FExternalBufferInfo* InCurrentTotalBufferUtilization)
	{
		check(AU->BufferSourceInfo.IsValid());

		AccessLock.Lock();

		TSharedPtrTS<const FBufferSourceInfo> BufferSourceInfo;
		TSharedPtrTS<FAccessUnitBuffer> Buffer = FindOrCreateBufferFor(BufferSourceInfo, AU->BufferSourceInfo, true);

		ActivateInitialBuffer();

		// Pushing data to either buffer means that there is data and we are not at EOD here any more.
		// This does not necessarily mean that the selected track is not at EOD. Just that some track is not.
		bEndOfData = false;
		// Keep track of how much data since the last end-of-track was signaled has been added.
		// This is useful when looking at available content to detect underruns which may not be done if
		// a newly enabled track did not have enough time to collect new data. Only when sufficient data
		// was available might it make sense to check for underruns.
		if (bEndOfTrack)
		{
			bEndOfTrack = false;
			PlayableDurationPushedSinceEOT.SetToZero();
		}
		check(AU->EarliestPTS.IsValid());
		check(AU->LatestPTS.IsValid());
		const bool bIsPlayableAU = AU->PTS >= AU->EarliestPTS && (AU->PTS + AU->Duration) < AU->LatestPTS;
		if (bIsPlayableAU)
		{
			PlayableDurationPushedSinceEOT += AU->Duration;
		}

		AccessLock.Unlock();

		if (!Buffer.IsValid())
		{
			return false;
		}
		bool bWasPushed = Buffer->Push(AU, BufferConfiguration, InCurrentTotalBufferUtilization);
		bLastPushWasBlocked = Buffer->WasLastPushBlocked();
		return bWasPushed;
	}

	void FMultiTrackAccessUnitBuffer::PushEndOfDataFor(TSharedPtrTS<const FBufferSourceInfo> InStreamSourceInfo)
	{
		AccessLock.Lock();
		TSharedPtrTS<const FBufferSourceInfo> BufferSourceInfo;
		TSharedPtrTS<FAccessUnitBuffer> Buffer = FindOrCreateBufferFor(BufferSourceInfo, InStreamSourceInfo, true);
		ActivateInitialBuffer();
		AccessLock.Unlock();
		if (Buffer.IsValid())
		{
			Buffer->PushEndOfData();
		}
	}

	void FMultiTrackAccessUnitBuffer::PushEndOfDataAll()
	{
		FScopeLock lock(&AccessLock);
		bEndOfData = true;
		// Push an end-of-data into all tracks.
		for(auto& It : BufferList)
		{
			It.Buffer->PushEndOfData();
		}
	}

	void FMultiTrackAccessUnitBuffer::SetEndOfTrackFor(TSharedPtrTS<const FBufferSourceInfo> InStreamSourceInfo)
	{
		AccessLock.Lock();
		TSharedPtrTS<const FBufferSourceInfo> BufferSourceInfo;
		TSharedPtrTS<FAccessUnitBuffer> Buffer = FindOrCreateBufferFor(BufferSourceInfo, InStreamSourceInfo, true);
		ActivateInitialBuffer();
		AccessLock.Unlock();
		if (Buffer.IsValid())
		{
			Buffer->SetEndOfTrack();
		}
	}

	void FMultiTrackAccessUnitBuffer::SetEndOfTrackAll()
	{
		FScopeLock lock(&AccessLock);
		bEndOfTrack = true;
		// Push an end-of-data into all tracks.
		for(auto& It : BufferList)
		{
			It.Buffer->SetEndOfTrack();
		}
	}

	void FMultiTrackAccessUnitBuffer::Clear()
	{
		BufferList.Empty();
		PendingBufferSwitch.Reset();
		ActiveBuffer.Reset();
		ActiveOutputBufferInfo.Reset();
		LastPoppedBufferInfo.Reset();
		LastPoppedDTS.SetToInvalid();
		LastPoppedPTS.SetToInvalid();
		PlayableDurationPushedSinceEOT.SetToZero();
		bEndOfData = false;
		bEndOfTrack = false;
		bLastPushWasBlocked = false;
		bPopAsDummyUntilSyncFrame = true;
	}

	void FMultiTrackAccessUnitBuffer::Flush()
	{
		FScopeLock lock(&AccessLock);
		Clear();
	}

	void FMultiTrackAccessUnitBuffer::SelectTrackWhenAvailable(uint32 PlaybackSequenceID, TSharedPtrTS<FBufferSourceInfo> InBufferSourceInfo)
	{
		FScopeLock lock(&AccessLock);
		if (InBufferSourceInfo.IsValid())
		{
			PendingBufferSwitch.BufferInfo = MakeSharedTS<FBufferSourceInfo>(*InBufferSourceInfo);
			PendingBufferSwitch.BufferInfo->PlaybackSequenceID = PlaybackSequenceID;
		}
		else
		{
			PendingBufferSwitch.Reset();
		}
	}

	void FMultiTrackAccessUnitBuffer::ActivateInitialBuffer()
	{
		// Note: The access mutex must have been locked already!
		if (!ActiveBuffer.IsValid() && PendingBufferSwitch.IsSet())
		{
			TSharedPtrTS<const FBufferSourceInfo> BufferSourceInfo;
			TSharedPtrTS<FAccessUnitBuffer> Buffer = FindOrCreateBufferFor(BufferSourceInfo, PendingBufferSwitch.BufferInfo, false);
			if (Buffer.IsValid())
			{
				ActiveBuffer = MoveTemp(Buffer);
				ActiveOutputBufferInfo = MoveTemp(BufferSourceInfo);
				PendingBufferSwitch.BufferInfo.Reset();
			}
		}
	}

	void FMultiTrackAccessUnitBuffer::HandlePendingSwitch()
	{
		// Note: The access mutex must have been locked already!
		if (PendingBufferSwitch.IsSet())
		{
			TSharedPtrTS<const FBufferSourceInfo> BufferSourceInfo;
			TSharedPtrTS<FAccessUnitBuffer> Buffer = FindOrCreateBufferFor(BufferSourceInfo, PendingBufferSwitch.BufferInfo, false);
			// Switching to the same buffer we are already on means we're already done.
			if (ActiveBuffer.IsValid() && ActiveBuffer == Buffer)
			{
				PendingBufferSwitch.BufferInfo.Reset();
				return;
			}

			if (Buffer.IsValid() && LastPoppedDTS.IsValid())
			{
				FTimeValue dd, dp;
				Buffer->DiscardUntil(LastPoppedDTS, FTimeValue(), dd, dp);

				// We switch on keyframes only. If there is none we keep the current buffer even if this may mean that
				// we run out of data altogether.
				FAccessUnit* NextAU = nullptr;
				if (Buffer->PeekAndAddRef(NextAU))
				{
					if (NextAU->bIsSyncSample)
					{
						ActiveBuffer = MoveTemp(Buffer);
						ActiveOutputBufferInfo = MoveTemp(BufferSourceInfo);
						PendingBufferSwitch.BufferInfo.Reset();
					}
				}
			}
		}
		RemoveOutdatedBuffers();
	}

	void FMultiTrackAccessUnitBuffer::RemoveOutdatedBuffers()
	{
		if (ActiveOutputBufferInfo.IsValid())
		{
			uint32 PlaybackSequenceID = ActiveOutputBufferInfo->PlaybackSequenceID;
			for(int32 i=0; i<BufferList.Num(); ++i)
			{
				if (BufferList[i].Info->PlaybackSequenceID < PlaybackSequenceID)
				{
					BufferList.RemoveAt(i);
					--i;
				}
			}
		}
	}


	FTimeValue FMultiTrackAccessUnitBuffer::GetLastPoppedPTS()
	{
		FScopeLock lock(&AccessLock);
		return LastPoppedPTS;
	}

	FTimeValue FMultiTrackAccessUnitBuffer::GetLastPoppedDTS()
	{
		FScopeLock lock(&AccessLock);
		return LastPoppedDTS;
	}

	FTimeValue FMultiTrackAccessUnitBuffer::GetPlayableDurationPushedSinceEOT()
	{
		FScopeLock lock(&AccessLock);
		return PlayableDurationPushedSinceEOT;
	}

	TSharedPtrTS<FAccessUnitBuffer> FMultiTrackAccessUnitBuffer::GetSelectedTrackBuffer()
	{
		FScopeLock lock(&AccessLock);
		return ActiveBuffer.IsValid() ? ActiveBuffer : EmptyBuffer;
	}

	void FMultiTrackAccessUnitBuffer::GetStats(FAccessUnitBufferInfo& OutStats)
	{
		FScopeLock lock(&AccessLock);
		ActivateInitialBuffer();
		TSharedPtrTS<const FAccessUnitBuffer> Buf = GetSelectedTrackBuffer();
		Buf->GetStats(OutStats);
		if (Buf == EmptyBuffer)
		{
			OutStats.bEndOfData = bEndOfData;
			OutStats.bEndOfTrack = bEndOfTrack;
		}
	}

	TSharedPtrTS<FAccessUnitBuffer> FMultiTrackAccessUnitBuffer::FindBufferFor(const TSharedPtrTS<const FBufferSourceInfo>& InBufferInfo)
	{
		FScopeLock lock(&AccessLock);
		for(int32 i=0,iMax=BufferList.Num(); i<iMax; ++i)
		{
			if (BufferList[i].Info->PeriodAdaptationSetID.Equals(InBufferInfo->PeriodAdaptationSetID))
			{
				return BufferList[i].Buffer;
			}
		}
		return nullptr;
	}

	bool FMultiTrackAccessUnitBuffer::PeekAndAddRef(FAccessUnit*& OutAU)
	{
		FScopedLock lock(AsShared());
		ActivateInitialBuffer();
		HandlePendingSwitch();
		TSharedPtrTS<FAccessUnitBuffer> Buf = FMultiTrackAccessUnitBuffer::GetSelectedTrackBuffer();
		if (Buf->Num())
		{
			return Buf->PeekAndAddRef(OutAU);
		}
		else
		{
			OutAU = nullptr;
			return false;
		}
	}

	bool FMultiTrackAccessUnitBuffer::Pop(FAccessUnit*& OutAU)
	{
		// Note: We assume the access lock is held by the caller!
		ActivateInitialBuffer();
		HandlePendingSwitch();
		TSharedPtrTS<FAccessUnitBuffer> Buf = FMultiTrackAccessUnitBuffer::GetSelectedTrackBuffer();
		if (Buf.IsValid() && Buf->Num())
		{
			bool bDidPop = Buf->Pop(OutAU);
			if (bDidPop && OutAU)
			{
				// Did we just pop from a different buffer than last time?
				if (LastPoppedBufferInfo != ActiveOutputBufferInfo)
				{
					if (LastPoppedBufferInfo)
					{
						OutAU->bTrackChangeDiscontinuity = true;
					}

					// Remember from which buffer we popped.
					LastPoppedBufferInfo = ActiveOutputBufferInfo;
				}

				// Do we need to wait for a sync frame and tag everything else as a dummy until then?
				if (bPopAsDummyUntilSyncFrame)
				{
					bPopAsDummyUntilSyncFrame = !OutAU->bIsSyncSample;
					OutAU->bIsDummyData = !OutAU->bIsSyncSample || OutAU->bIsDummyData;
					OutAU->bTrackChangeDiscontinuity = OutAU->bIsSyncSample || OutAU->bTrackChangeDiscontinuity;
				}

				LastPoppedDTS = OutAU->DTS;
				LastPoppedPTS = OutAU->PTS;
				// With this AU being popped off now we will also pop off all AUs that are now obsolete in the other tracks.
				for(auto& It : BufferList)
				{
					if (It.Buffer != Buf)
					{
						FTimeValue PoppedDTS, PoppedPTS;
						It.Buffer->DiscardUntil(LastPoppedDTS, LastPoppedPTS, PoppedDTS, PoppedPTS);
					}
				}
			}
			return bDidPop;
		}
		else
		{
			OutAU = nullptr;
			return false;
		}
	}

	void FMultiTrackAccessUnitBuffer::PopDiscardUntil(FTimeValue UntilTime)
	{
		// Note: We assume the access lock is held by the caller!
		for(auto& It : BufferList)
		{
			FTimeValue PoppedDTS, PoppedPTS;
			It.Buffer->DiscardUntil(UntilTime, UntilTime, PoppedDTS, PoppedPTS);
			if (PoppedDTS.IsValid() && LastPoppedDTS.IsValid() && PoppedDTS > LastPoppedDTS)
			{
				LastPoppedDTS = PoppedDTS;
			}
			if (PoppedPTS.IsValid() && LastPoppedPTS.IsValid() && PoppedPTS > LastPoppedPTS)
			{
				LastPoppedPTS = PoppedPTS;
			}
		}
		// Discarding from the buffer means that the next popping of an access unit needs to return
		// a sync sample. Anything that is not will instead be returned as a dummy sample.
		bPopAsDummyUntilSyncFrame  = true;
	}

	bool FMultiTrackAccessUnitBuffer::IsEODFlagSet()
	{
		// Note: We assume the access lock is held by the caller!
		TSharedPtrTS<FAccessUnitBuffer> Buf = GetSelectedTrackBuffer();
		return Buf == EmptyBuffer ? bEndOfData : Buf->IsEODFlagSet();
	}

	bool FMultiTrackAccessUnitBuffer::IsEndOfTrack()
	{
		// Note: We assume the access lock is held by the caller!
		TSharedPtrTS<FAccessUnitBuffer> Buf = GetSelectedTrackBuffer();
		return Buf == EmptyBuffer ? bEndOfTrack : Buf->IsEndOfTrack();
	}

	int32 FMultiTrackAccessUnitBuffer::Num()
	{
		// Note: We assume the access lock is held by the caller!
		ActivateInitialBuffer();
		return GetSelectedTrackBuffer()->Num();
	}

	bool FMultiTrackAccessUnitBuffer::WasLastPushBlocked()
	{
		return bLastPushWasBlocked;
	}

	bool FMultiTrackAccessUnitBuffer::HasPendingTrackSwitch()
	{
		FScopeLock lock(&AccessLock);
		return PendingBufferSwitch.IsSet();
	}


} // namespace Electra


