// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "StreamTypes.h"
#include "Containers/Queue.h"
#include "HAL/PlatformMisc.h"



namespace Electra
{
	namespace DynamicSidebandData
	{
		const FName ITU_T_35(TEXT("itu-t-35"));
		const FName VPxAlpha(TEXT("vpx-alpha"));
	}


	class IAccessUnitMemoryProvider
	{
	public:
		enum class EDataType
		{
			AU,
			Payload,
			GenericData
		};
		virtual ~IAccessUnitMemoryProvider() = default;

		virtual void* AUAllocate(EDataType type, SIZE_T size, SIZE_T alignment = 0) = 0;
		virtual void AUDeallocate(EDataType type, void* pAddr) = 0;
	};



	/**
	 * Information into which buffer the AU data needs to be placed.
	 */
	struct FBufferSourceInfo
	{
		// The period the data comes from. Necessary to track period transitions.
		FString		PeriodID;
		// Identifies the period and track (adaptation set) this data is originating from.
		FString		PeriodAdaptationSetID;
		// Partial track metadata. See FTrackMetadata.
		BCP47::FLanguageTag LanguageTag;
		FString		Kind;
		FString		Codec;

		// Internal hard index, used for multiplexed streams.
		int32		HardIndex = -1;

		// To which playback sequence this belongs.
		uint32		PlaybackSequenceID = 0;
	};

	struct FAccessUnit
	{
		struct CodecData
		{
			TArray<uint8>					CodecSpecificData;
			TArray<uint8>					RawCSD;
			FStreamCodecInformation			ParsedInfo;
		};

		TSharedPtrTS<CodecData>		AUCodecData;					//!< If set, points to constant sideband data for this access unit.
		TSharedPtrTS<const FBufferSourceInfo>	BufferSourceInfo;
		TUniquePtr<TMap<FName, TArray<uint8>>> DynamicSidebandData;	//!< If set, contains a map of dynamically changing sideband data for just this access unit.
		EStreamType					ESType;							//!< Type of elementary stream this is an access unit of.
		FTimeValue					PTS;							//!< PTS
		FTimeValue					DTS;							//!< DTS
		FTimeValue					Duration;						//!< Duration
		FTimeValue					EarliestPTS;					//!< Earliest PTS at which to present samples. If this is larger than PTS the sample is not to be presented.
		FTimeValue					LatestPTS;						//!< Latest PTS at which to present samples. If this is less than PTS the sample is not to be presented.
		FTimeValue					ProducerReferenceTime;			//!< If set, the wallclock time of the producer when this AU was encoded or captured
		int64						SequenceIndex;
		void*						AUData;							//!< Access unit data
		uint32						AUSize;							//!< Size of this access unit
		bool						bIsFirstInSequence;				//!< true for the first AU in a segment
		bool						bIsLastInPeriod;				//!< true if this is the last AU in the playing period.
		bool						bIsSyncSample;					//!< true if this is a sync sample (keyframe)
		bool						bIsDummyData;					//!< True if this is not actual data but empty filler data due to some segment problem.
		bool						bTrackChangeDiscontinuity;		//!< True if this is the first AU after a track change.
		bool						bIsSideloaded;					//!< True if the payload is not streamed but loaded from a sidecar file.


		static FAccessUnit* Create(IAccessUnitMemoryProvider* MemProvider)
		{
			void* NewBuffer = MemProvider->AUAllocate(IAccessUnitMemoryProvider::EDataType::AU, sizeof(FAccessUnit));
			FAccessUnit* NewAU = new (NewBuffer) FAccessUnit;
			NewAU->AUMemoryProvider = MemProvider;
			return NewAU;
		}

		int64 TotalMemSize() const
		{
			return sizeof(FAccessUnit) + AUSize + (AUCodecData.IsValid() ? AUCodecData->CodecSpecificData.Num() : 0);
		}

		void AddRef()
		{
			FMediaInterlockedIncrement(RefCount);
		}

		void SetCodecSpecificData(const TArray<uint8>& Csd)
		{
			AUCodecData = MakeSharedTS<CodecData>();
			AUCodecData->CodecSpecificData = Csd;
		}

		void SetCodecSpecificData(const TSharedPtrTS<CodecData>& Csd)
		{
			AUCodecData = Csd;
		}

		void* AllocatePayloadBuffer(SIZE_T Num)
		{
			check(AUMemoryProvider);
			return AUMemoryProvider ? AUMemoryProvider->AUAllocate(IAccessUnitMemoryProvider::EDataType::Payload, Num) : nullptr;
		}

		void AdoptNewPayloadBuffer(void* Buffer, SIZE_T Num)
		{
			if (AUData)
			{
				AUMemoryProvider->AUDeallocate(IAccessUnitMemoryProvider::EDataType::Payload, AUData);
			}
			AUData = Buffer;
			AUSize = Num;
		}

		static void Release(FAccessUnit* AccessUnit)
		{
			if (AccessUnit)
			{
				check(AccessUnit->RefCount > 0);
				if (FMediaInterlockedDecrement(AccessUnit->RefCount) == 1)
				{
					IAccessUnitMemoryProvider* MemoryProvider = AccessUnit->AUMemoryProvider;
					AccessUnit->~FAccessUnit();
					MemoryProvider->AUDeallocate(IAccessUnitMemoryProvider::EDataType::AU, AccessUnit);
				}
			}
		}

	private:
		IAccessUnitMemoryProvider*		AUMemoryProvider;			//!< Interface to use to delete the allocated AU
		uint32							RefCount;

		FAccessUnit()
		{
			RefCount = 1;
			PTS.SetToInvalid();
			DTS.SetToInvalid();
			Duration.SetToInvalid();
			EarliestPTS.SetToInvalid();
			LatestPTS.SetToInvalid();
			SequenceIndex = 0;
			ESType = EStreamType::Unsupported;
			AUSize = 0;
			AUData = nullptr;
			AUMemoryProvider = nullptr;
			bIsFirstInSequence = false;
			bIsLastInPeriod = false;
			bIsSyncSample = false;
			bIsDummyData = false;
			bTrackChangeDiscontinuity = false;
			bIsSideloaded = false;
		}

		~FAccessUnit()
		{
			if (AUData)
			{
				check(AUMemoryProvider);
				AUMemoryProvider->AUDeallocate(IAccessUnitMemoryProvider::EDataType::Payload, AUData);
				AUData = nullptr;
			}
		}
	};






	struct FAccessUnitBufferInfo
	{
		FAccessUnitBufferInfo()
		{
			Clear();
		}
		void Clear()
		{
			FrontDTS.SetToInvalid();
			SmallestPTS.SetToInvalid();
			LargestPTSPlusDur.SetToInvalid();
			PlayableDuration.SetToZero();
			CurrentMemInUse = 0;
			NumCurrentAccessUnits = 0;
			bEndOfData = false;
			bEndOfTrack = false;
			bLastPushWasBlocked = false;
		}

		FTimeValue FrontDTS;
		FTimeValue SmallestPTS;
		FTimeValue LargestPTSPlusDur;
		FTimeValue PlayableDuration;
		int64 CurrentMemInUse;
		int64 NumCurrentAccessUnits;
		bool bEndOfData;
		bool bEndOfTrack;
		bool bLastPushWasBlocked;
	};


	/**
	 * This class implements a decoder input data FIFO.
	**/
	class FAccessUnitBuffer
	{
	public:
		struct FConfiguration
		{
			FConfiguration(double InMaxSeconds = 0.0)
			{
				MaxDuration.SetFromSeconds(InMaxSeconds);
			}
			FTimeValue	MaxDuration;
		};

		struct FExternalBufferInfo
		{
			FTimeValue	Duration = FTimeValue::GetZero();
		};

		FAccessUnitBuffer()
			: FrontDTS(FTimeValue::GetInvalid())
			, SmallestPTS(FTimeValue::GetInvalid())
			, LargestPTSPlusDur(FTimeValue::GetInvalid())
			, PlayableDuration(FTimeValue::GetZero())
			, CurrentMemInUse(0)
			, bEndOfData(false)
			, bEndOfTrack(false)
			, bLastPushWasBlocked(false)
		{
		}

		~FAccessUnitBuffer()
		{
			while(AccessUnits.Num())
			{
				FAccessUnit::Release(AccessUnits.Pop());
			}
		}

		//! Returns the number of access units currently in the FIFO.
		int64 Num() const
		{
			FScopeLock Lock(&AccessLock);
			return AccessUnits.Num();
		}

		//! Returns the amount of memory currently allocated.
		int64 AllocatedSize() const
		{
			FScopeLock Lock(&AccessLock);
			return CurrentMemInUse;
		}

		//! Returns all vital statistics.
		void GetStats(FAccessUnitBufferInfo& OutStats) const
		{
			FScopeLock Lock(&AccessLock);
			OutStats.FrontDTS = FrontDTS;
			OutStats.SmallestPTS = SmallestPTS;
			OutStats.LargestPTSPlusDur = LargestPTSPlusDur;
			OutStats.PlayableDuration = PlayableDuration;
			OutStats.CurrentMemInUse = CurrentMemInUse;
			OutStats.NumCurrentAccessUnits = AccessUnits.Num();
			OutStats.bEndOfData = bEndOfData;
			OutStats.bEndOfTrack = bEndOfTrack;
			OutStats.bLastPushWasBlocked = bLastPushWasBlocked;
		}

		//! Adds an access unit to the FIFO. Returns true if successful, false if the FIFO has insufficient free space.
		bool Push(FAccessUnit*& AU, const FConfiguration* Limit = nullptr, const FExternalBufferInfo* ExternalInfo = nullptr)
		{
			FScopeLock Lock(&AccessLock);
			// Pushing new data unconditionally clears the EOD flag even if the buffer is currently full.
			// The attempt to push implies there will be more data.
			bEndOfData = false;
			bEndOfTrack = false;
			ExternalInfo = ExternalInfo ? ExternalInfo : &ZeroExternalInfo;
			check(AU->EarliestPTS.IsValid());
			check(AU->LatestPTS.IsValid());
			const bool bIsPlayableAU = AU->PTS >= AU->EarliestPTS && (AU->PTS + AU->Duration) < AU->LatestPTS;
			if (!bIsPlayableAU || CanPush(AU, Limit, ExternalInfo))
			{
				bLastPushWasBlocked = false;
				AccessUnits.Push(AU);
				int64 memSize = AU->TotalMemSize();
				CurrentMemInUse += memSize;

				if (bIsPlayableAU)
				{
					if (!FrontDTS.IsValid())
					{
						FrontDTS = AU->DTS;
					}
					if (!SmallestPTS.IsValid() || AU->PTS < SmallestPTS)
					{
						SmallestPTS = AU->PTS;
					}
					FTimeValue End = AU->PTS + AU->Duration;
					End.SetSequenceIndex(AU->PTS.GetSequenceIndex());
					if (!LargestPTSPlusDur.IsValid() || End > LargestPTSPlusDur)
					{
						LargestPTSPlusDur = End;
					}
					PlayableDuration += AU->Duration;
				}
				NumInSemaphore.Release();
				return true;
			}
			else
			{
				bLastPushWasBlocked = true;
				return false;
			}
		}

		//! "Pushes" an end-of-data marker signaling that no further data will be pushed. May be called more than once. Flushing or pushing new data clears the flag.
		void PushEndOfData()
		{
			bEndOfData = true;
		}

		void SetEndOfTrack()
		{
			bEndOfTrack = true;
		}
		void ClearEndOfTrack()
		{
			bEndOfTrack = false;
		}
		bool IsEndOfTrack() const
		{
			return bEndOfTrack;
		}

		//! Removes and returns the oldest access unit from the FIFO. Returns false if the FIFO is empty.
		bool Pop(FAccessUnit*& OutAU)
		{
			FScopeLock Lock(&AccessLock);
			if (Num())
			{
				OutAU = AccessUnits.Pop();
				int64 nMemSize = OutAU->TotalMemSize();
				CurrentMemInUse -= nMemSize;
				NumInSemaphore.TryToObtain();

				if (!AccessUnits.IsEmpty())
				{
					const bool bIsPlayableAU = AccessUnits.FrontRef()->PTS >= AccessUnits.FrontRef()->EarliestPTS && (AccessUnits.FrontRef()->PTS + AccessUnits.FrontRef()->Duration) < AccessUnits.FrontRef()->LatestPTS;
					if (bIsPlayableAU)
					{
						FrontDTS = AccessUnits.FrontRef()->DTS;
					}
					else
					{
						FrontDTS.SetToInvalid();
						for(int32 i=1; i<AccessUnits.Num(); ++i)
						{
							const bool bNextIsPlayableAU = AccessUnits[i]->PTS >= AccessUnits[i]->EarliestPTS && (AccessUnits[i]->PTS + AccessUnits[i]->Duration) < AccessUnits[i]->LatestPTS;
							if (bNextIsPlayableAU)
							{
								FrontDTS = AccessUnits[i]->DTS;
								break;
							}
						}
					}

					SmallestPTS.SetToPositiveInfinity();
					for(int32 i=0,iMax=AccessUnits.Num(),j=0; i<iMax; ++i)
					{
						const bool bNextIsPlayableAU = AccessUnits[i]->PTS >= AccessUnits[i]->EarliestPTS && (AccessUnits[i]->PTS + AccessUnits[i]->Duration) < AccessUnits[i]->LatestPTS;
						if (bNextIsPlayableAU)
						{
							if (AccessUnits[i]->PTS < SmallestPTS)
							{
								SmallestPTS = AccessUnits[i]->PTS;
							}
							// Look only at the first couple of AU's. The smallest one is going to be among them
							// unless there is a huge amount of reordered samples.
							if (++j >= 10)
							{
								break;
							}
						}
					}
					const bool bWasPlayableAU = OutAU->PTS >= OutAU->EarliestPTS && (OutAU->PTS + OutAU->Duration) < OutAU->LatestPTS;
					if (bWasPlayableAU)
					{
						PlayableDuration -= OutAU->Duration;
					}
				}
				else
				{
					FrontDTS.SetToInvalid();
					SmallestPTS.SetToInvalid();
					LargestPTSPlusDur.SetToInvalid();
					PlayableDuration.SetToZero();
				}
				return true;
			}
			else
			{
				OutAU = nullptr;
				return false;
			}
		}

		//!
		bool PeekAndAddRef(FAccessUnit*& OutAU)
		{
			FScopeLock Lock(&AccessLock);
			if (Num())
			{
				OutAU = AccessUnits.Front();
				OutAU->AddRef();
				return true;
			}
			else
			{
				OutAU = nullptr;
				return false;
			}
		}

		//!
		bool ContainsPTS(const FTimeValue& InPTS) const
		{
			FScopeLock Lock(&AccessLock);
			if (AccessUnits.Num())
			{
				return AccessUnits.FrontRef()->PTS <= InPTS && InPTS < AccessUnits.BackRef()->PTS + AccessUnits.BackRef()->Duration;
			}
			return false;
		}

		bool ContainsFuturePTS(const FTimeValue& InPTS) const
		{
			FScopeLock Lock(&AccessLock);
			if (AccessUnits.Num())
			{
				return InPTS <= AccessUnits.BackRef()->PTS + AccessUnits.BackRef()->Duration;
			}
			return false;
		}


		//! Discards data that has both its DTS and PTS less than the provided ones.
		void DiscardUntil(const FTimeValue& NextValidDTS, const FTimeValue& NextValidPTS, FTimeValue& OutPoppedDTS, FTimeValue& OutPoppedPTS)
		{
			while(true)
			{
				FAccessUnit* NextAU = nullptr;
				FAccessUnit* PeekedAU = nullptr;
				if (PeekAndAddRef(PeekedAU))
				{
					if (PeekedAU)
					{
						bool bDTS = NextValidDTS.IsValid() ? PeekedAU->DTS < NextValidDTS : true;
						bool bPTS = NextValidPTS.IsValid() ? PeekedAU->PTS < NextValidPTS : true;
						if (bDTS && bPTS)
						{
							OutPoppedDTS = PeekedAU->DTS;
							OutPoppedPTS = PeekedAU->PTS;
							Pop(NextAU);
							FAccessUnit::Release(NextAU);
							NextAU = nullptr;
						}
						else
						{
							FAccessUnit::Release(PeekedAU);
							PeekedAU = nullptr;
							break;
						}
						FAccessUnit::Release(PeekedAU);
						PeekedAU = nullptr;
					}
					else
					{
						break;
					}
				}
				else
				{
					break;
				}
			}
		}

		//! Waits for data to arrive. Returns true if data is present. False if not and timeout expired.
		bool WaitForData(int64 waitForMicroseconds = -1)
		{
			bool bHave = NumInSemaphore.Obtain(waitForMicroseconds);
			if (bHave)
			{
				NumInSemaphore.Release();
			}
			return bHave;
		}

		//! Removes all elements from the FIFO
		void Flush()
		{
			FScopeLock Lock(&AccessLock);
			while(AccessUnits.Num())
			{
				FAccessUnit::Release(AccessUnits.Pop());
				NumInSemaphore.TryToObtain();
			}
			CurrentMemInUse = 0;
			bEndOfData = false;
			bEndOfTrack = false;
			bLastPushWasBlocked = false;
			FrontDTS.SetToInvalid();
			PlayableDuration.SetToZero();
		}

		//! Checks if the buffer has reached the end-of-data marker (marker is set and no more data is in the buffer).
		bool IsEndOfData() const
		{
			FScopeLock Lock(&AccessLock);
			return bEndOfData && AccessUnits.IsEmpty();
		}

		// Checks if the end-of-data flag has been set. There may still be data in the buffer though!
		bool IsEODFlagSet() const
		{
			FScopeLock Lock(&AccessLock);
			return bEndOfData;
		}

		// Was the last push blocked because the buffer limits were reached?
		bool WasLastPushBlocked() const
		{
			return bLastPushWasBlocked;
		}

		// Helper class to lock the AU buffer
		class FScopedLock
		{
		public:
			explicit FScopedLock(const FAccessUnitBuffer& owner)
				: mOwner(owner)
			{
				mOwner.AccessLock.Lock();
			}
			~FScopedLock()
			{
				mOwner.AccessLock.Unlock();
			}
		private:
			FScopedLock() = delete;
			FScopedLock(const FScopedLock&) = delete;
			FScopedLock& operator= (const FScopedLock&) = delete;
			const FAccessUnitBuffer& mOwner;
		};

	private:
		//! Checks if an access unit can be pushed to the FIFO. Returns true if successful, false if the FIFO has insufficient free space.
		bool CanPush(const FAccessUnit* AU, const FConfiguration* Limit, const FExternalBufferInfo* ExternalInfo)
		{
			check(Limit);
			if (!Limit)
			{
				return false;
			}
			check(Limit->MaxDuration > FTimeValue::GetZero());
			check(AU->Duration.IsValid() && !AU->Duration.IsInfinity());
			if (ExternalInfo == nullptr)
			{
				// Max allowed duration ok?
				return PlayableDuration.IsValid() && PlayableDuration + AU->Duration > Limit->MaxDuration ? false : true;
			}
			else
			{
				// Max allowed duration ok?
				return (AU->Duration + ExternalInfo->Duration) > Limit->MaxDuration ? false : true;
			}
		}

		mutable FCriticalSection AccessLock;
		FExternalBufferInfo ZeroExternalInfo;
		TMediaQueueDynamicNoLock<FAccessUnit*> AccessUnits;
		FMediaSemaphore NumInSemaphore;
		FTimeValue FrontDTS;
		FTimeValue SmallestPTS;
		FTimeValue LargestPTSPlusDur;
		FTimeValue PlayableDuration;
		int64 CurrentMemInUse = 0;
		bool bEndOfData = false;
		bool bEndOfTrack = false;
		bool bLastPushWasBlocked = false;
	};



	/**
	 * A multi-track access unit buffer keeps access units from several tracks in individual buffers,
	 * one of which is selected to return AUs to the decoder from. The other unselected tracks will
	 * discard their AUs as the play position progresses.
	 */
	class FMultiTrackAccessUnitBuffer : public TSharedFromThis<FMultiTrackAccessUnitBuffer, ESPMode::ThreadSafe>
	{
	public:
		FMultiTrackAccessUnitBuffer(EStreamType InForType);
		~FMultiTrackAccessUnitBuffer();
		void SetParallelTrackMode();
		void SelectTrackWhenAvailable(uint32 PlaybackSequenceID, TSharedPtrTS<FBufferSourceInfo> InBufferSourceInfo);
		bool Push(FAccessUnit*& AU, const FAccessUnitBuffer::FConfiguration* BufferConfiguration, const FAccessUnitBuffer::FExternalBufferInfo* InCurrentTotalBufferUtilization);
		void PushEndOfDataFor(TSharedPtrTS<const FBufferSourceInfo> InStreamSourceInfo);
		void PushEndOfDataAll();
		void SetEndOfTrackFor(TSharedPtrTS<const FBufferSourceInfo> InStreamSourceInfo);
		void SetEndOfTrackAll();
		void Flush();
		void GetStats(FAccessUnitBufferInfo& OutStats);
		FTimeValue GetLastPoppedDTS();
		FTimeValue GetLastPoppedPTS();
		FTimeValue GetPlayableDurationPushedSinceEOT();

		// Helper class to lock the AU buffer
		class FScopedLock
		{
		public:
			explicit FScopedLock(TSharedPtrTS<FMultiTrackAccessUnitBuffer> Self)
				: LockedSelf(MoveTemp(Self))
			{
				LockedSelf->AccessLock.Lock();
			}
			~FScopedLock()
			{
				LockedSelf->AccessLock.Unlock();
			}
		private:
			FScopedLock();
			FScopedLock(const FScopedLock&) = delete;
			FScopedLock& operator= (const FScopedLock&) = delete;

			TSharedPtrTS<FMultiTrackAccessUnitBuffer> LockedSelf;
		};

		TSharedPtrTS<FAccessUnitBuffer> FindBufferFor(const TSharedPtrTS<const FBufferSourceInfo>& InBufferInfo);

		bool PeekAndAddRef(FAccessUnit*& OutAU);
		bool Pop(FAccessUnit*& OutAU);
		void PopDiscardUntil(FTimeValue UntilTime);
		bool IsEODFlagSet();
		bool IsEndOfTrack();
		int32 Num();
		bool WasLastPushBlocked();
		bool HasPendingTrackSwitch();

	private:

		struct FSwitchToBuffer
		{
			void Reset()
			{
				BufferInfo.Reset();
			}
			bool IsSet() const
			{
				return BufferInfo.IsValid();
			}
			TSharedPtrTS<FBufferSourceInfo> BufferInfo;
		};

		struct FBufferByInfoType
		{
			TSharedPtrTS<const FBufferSourceInfo> Info;
			TSharedPtrTS<FAccessUnitBuffer> Buffer;
		};

		void Clear();

		TSharedPtrTS<FAccessUnitBuffer> CreateNewBuffer();
		TSharedPtrTS<FAccessUnitBuffer> FindOrCreateBufferFor(TSharedPtrTS<const FBufferSourceInfo>& OutBufferSourceInfo, const TSharedPtrTS<const FBufferSourceInfo>& InBufferInfo, bool bCreateIfNotExist);
		void ActivateInitialBuffer();
		void HandlePendingSwitch();
		void RemoveOutdatedBuffers();

		TSharedPtrTS<FAccessUnitBuffer> GetSelectedTrackBuffer();

		FCriticalSection								AccessLock;
		EStreamType										Type;
		TArray<FBufferByInfoType>						BufferList;
		FSwitchToBuffer									PendingBufferSwitch;
		TSharedPtrTS<FAccessUnitBuffer>					EmptyBuffer;
		TSharedPtrTS<FAccessUnitBuffer>					ActiveBuffer;
		TSharedPtrTS<const FBufferSourceInfo>			ActiveOutputBufferInfo;
		TSharedPtrTS<const FBufferSourceInfo>			LastPoppedBufferInfo;
		FTimeValue										LastPoppedDTS;
		FTimeValue										LastPoppedPTS;
		FTimeValue										PlayableDurationPushedSinceEOT;
		bool											bEndOfData;
		bool											bEndOfTrack;
		bool											bLastPushWasBlocked;
		bool											bPopAsDummyUntilSyncFrame;
		bool											bIsParallelTrackMode;
	};




	/**
	 * Base class for any decoder receiving data in "access units".
	 * An access unit (AU) is considered a data packet that can be sent into a decoder without
	 * the decoder stalling. For a h.264 decoder this is usually a single NALU.
	**/
	class IAccessUnitBufferInterface
	{
	public:
		virtual ~IAccessUnitBufferInterface() = default;

		//! Pushes an access unit to the decoder. Ownership of the access unit is transferred to the decoder.
		virtual void AUdataPushAU(FAccessUnit* AccessUnit) = 0;
		//! Notifies the decoder that there will be no further access units.
		virtual void AUdataPushEOD() = 0;
		//! Notifies the decoder that there may be further access units.
		virtual void AUdataClearEOD() = 0;
		//! Instructs the decoder to flush all pending input and all already decoded output.
		virtual void AUdataFlushEverything() = 0;
	};



	//---------------------------------------------------------------------------------------------------------------------
	/**
	 * A decoder input buffer listener callback to monitor the current state of decoder input buffer levels.
	 *
	 * The decoder will invoke this listener right before it wants to get an access unit from its input buffer,
	 * whether the buffer already contains data or is empty.
	**/
	class IAccessUnitBufferListener
	{
	public:
		virtual ~IAccessUnitBufferListener() = default;

		struct FBufferStats
		{
			FBufferStats()
			{
				Clear();
			}
			void Clear()
			{
				bEODSignaled = false;
				bEODReached = false;
			}
			bool	bEODSignaled;			//!< Set after PushEndOfData() has been called
			bool	bEODReached;			//!< Set after PushEndOfData() has been called AND the last AU was taken from the buffer.
		};

		//! Called right before the decoder wants to get an access unit from its input buffer, regardless if it already has data or not.
		virtual void DecoderInputNeeded(const FBufferStats& CurrentInputBufferStats) = 0;
	};



	//---------------------------------------------------------------------------------------------------------------------
	/**
	 * A decoder ready listener callback to monitor the decoder activity.
	 *
	 * The decoder will invoke this listener right before it needs a buffer from its output buffer prior to decoding.
	 * This is called whether or not the output buffer has room for new decoded data or not.
	**/
	class IDecoderOutputBufferListener
	{
	public:
		virtual ~IDecoderOutputBufferListener() = default;

		struct FDecodeReadyStats
		{
			FDecodeReadyStats()
			{
				Clear();
			}
			void Clear()
			{
				InDecoderTimeRangePTS.Reset();
				OutputBufferPoolSize = 0;
				NumElementsInDecoder = 0;
				bOutputStalled = false;
				bEODreached = false;
			}
			FTimeRange				InDecoderTimeRangePTS;		//!< Time range of elements in the decoder pipeline by PTS.
			int64					OutputBufferPoolSize;		//!< Maximum number of decoded elements the output pool can hold.
			int64					NumElementsInDecoder;		//!< Number of elements currently in the decoder pipeline.
			bool					bOutputStalled;				//!< true if the output is full and decoding is delayed until there's room again.
			bool					bEODreached;				//!< true when the final decoded element has been passed on (but may still be in the queue).
		};

		virtual void DecoderOutputReady(const FDecodeReadyStats& CurrentReadyStats) = 0;
	};





template <typename T>
class TAccessUnitQueue
{
public:
	TAccessUnitQueue() = default;

	~TAccessUnitQueue()
	{
		Empty();
	}

	void Enqueue(const T& InElement)
	{
		Elements.Enqueue(InElement);
		AvailSema.Release();
	}

	void Enqueue(T&& InElement)
	{
		Elements.Enqueue(MoveTemp(InElement));
		AvailSema.Release();
	}

	int32 Num()
	{
		return AvailSema.CurrentCount();
	}

	bool IsEmpty()
	{
		return Num() == 0;
	}

	void Empty()
	{
		while(AvailSema.TryToObtain())
		{
		}
		Elements.Empty();
		bIsEOD = false;
	}

	bool Wait(int64 InWaitForMicroseconds)
	{
		if (AvailSema.Obtain(InWaitForMicroseconds))
		{
			AvailSema.Release();
			return true;
		}
		if (GetEOD())
		{
			bReachedEOD = true;
		}
		return false;
	}

	bool Dequeue(T& OutElement)
	{
		if (AvailSema.Obtain())
		{
			bool bGot = Elements.Dequeue(OutElement);
			check(bGot);
			return bGot;
		}
		if (GetEOD())
		{
			bReachedEOD = true;
		}
		return false;
	}

	bool Dequeue(T& OutElement, int64 InWaitForMicroseconds)
	{
		if (AvailSema.Obtain(InWaitForMicroseconds))
		{
			bool bGot = Elements.Dequeue(OutElement);
			check(bGot);
			return bGot;
		}
		if (GetEOD())
		{
			bReachedEOD = true;
		}
		return false;
	}

	void SetEOD()
	{
		bIsEOD = true;
	}

	void ClearEOD()
	{
		bIsEOD = false;
		FPlatformMisc::MemoryBarrier();
		bReachedEOD = false;
	}

	bool GetEOD() const
	{
		return bIsEOD;
	}

	bool ReachedEOD() const
	{
		return bReachedEOD;
	}
private:
	TAccessUnitQueue(const TAccessUnitQueue&) = delete;
	TAccessUnitQueue& operator = (const TAccessUnitQueue&) = delete;

	FMediaSemaphore AvailSema;
	TQueue<T> Elements;
	volatile bool bIsEOD = false;
	volatile bool bReachedEOD = false;
};


} // namespace Electra


