// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{
	/**
	 * A buffer with waiting capability.
	**/
	class FWaitableBuffer
	{
	public:
		FWaitableBuffer() = default;
		FWaitableBuffer(const FWaitableBuffer&) = delete;
		FWaitableBuffer& operator=(const FWaitableBuffer&) = delete;

		~FWaitableBuffer()
		{
			FScopeLock Lock(&AccessLock);
			Deallocate();
		}

		void SetExternalBuffer(uint8* InExternalBuffer, int64 InExternalBufferSize)
		{
			FScopeLock Lock(&AccessLock);
			Deallocate();
			ExternalBuffer = InExternalBuffer;
			DataSize = (uint64) InExternalBufferSize;
		}

		// Allocates buffer of the specified capacity, destroying any previous buffer.
		bool Reserve(int64 InNumBytes)
		{
			FScopeLock Lock(&AccessLock);
			if (ExternalBuffer)
			{
				return false;
			}
			Deallocate();
			if (Allocate(InNumBytes))
			{
				Reset();
				return true;
			}
			return false;
		}

		// Enlarges the buffer to the new capacity, retaining the current content.
		bool EnlargeTo(int64 InNewNumBytes)
		{
			FScopeLock Lock(&AccessLock);
			if (ExternalBuffer)
			{
				return InNewNumBytes <= Capacity();
			}
			// Do we need a bigger capacity?
			if (InNewNumBytes > Capacity())
			{
				return IsEmpty() ? Reserve(InNewNumBytes) : InternalGrowTo(InNewNumBytes);
			}
			return true;
		}

		// Clears the buffer
		void Reset()
		{
			FScopeLock Lock(&AccessLock);
			WritePos = 0;
			bEOD = false;
			bWasAborted = false;
			bHasErrored = false;
			// Do not modify the "waiter" members. When waiting for data to arrive it needs to continue
			// doing so even when we are resetting the buffer.
				//WaitingForSize = 0;
				//SizeAvailableSignal.Signal();
		}

		// Returns the buffer capacity
		int64 Capacity() const
		{
			FScopeLock Lock(&AccessLock);
			return DataSize;
		}

		// Returns the number of bytes in the buffer
		int64 Num() const
		{
			FScopeLock Lock(&AccessLock);
			return WritePos;
		}

		// Returns the number of free bytes in the buffer (amount that can be pushed)
		int64 Avail() const
		{
			FScopeLock Lock(&AccessLock);
			return DataSize - WritePos;
		}

		// Checks if the buffer is empty.
		bool IsEmpty() const
		{
			return Num() == 0;
		}

		// Checks if the buffer is full.
		bool IsFull() const
		{
			return Avail() == 0;
		}

		// Checks if the buffer has reached the end-of-data marker (marker is set and no more data is in the buffer).
		bool IsEndOfData() const
		{
			return IsEmpty() && bEOD;
		}

		// Checks if the end-of-data flag has been set. There may still be data in the buffer though!
		bool GetEOD() const
		{
			return bEOD;
		}

		/*
			Waits until the specified number of bytes has arrived in the buffer.
			Note: This method is somewhat dangerous in that there is no guarantee the required amount will ever arrive.
			      You must also never wait for more data than is the capcacity of the buffer!
		*/
		bool WaitUntilSizeAvailable(int64 SizeNeeded, int32 TimeoutMicroseconds)
		{
			// Only wait if not at EOD and more data than presently available is asked for.
			// Otherwise return enough data to be present even if that is not actually the case.
			if (!bEOD && SizeNeeded > Num())
			{
				AccessLock.Lock();
				// Repeat the size check inside the mutex lock in case we enter this block
				// while new data is being pushed from another thread.
				if (SizeNeeded > Num())
				{
					SizeAvailableSignal.Reset();
					WaitingForSize = SizeNeeded;
				}
				else
				{
					SizeAvailableSignal.Signal();
					WaitingForSize = 0;
				}
				AccessLock.Unlock();
				if (TimeoutMicroseconds > 0)
				{
					return SizeAvailableSignal.WaitTimeout(TimeoutMicroseconds);
				}
				else
				{
					// No infinite waiting by specifying negative timeouts!
					check(TimeoutMicroseconds == 0);
					return SizeAvailableSignal.IsSignaled();
				}
			}
			return true;
		}


		// Inserts elements into the buffer. Returns true if successful, false if there is no room.
		bool PushData(const uint8* InData, int64 NumElements)
		{
			FScopeLock Lock(&AccessLock);
			check(!bEOD);

			// Zero elements can always be pushed...
			if (NumElements == 0)
			{
				return true;
			}
			if (Avail() >= NumElements)
			{
				if (InData)
				{
					CopyData(GetBufferBase() + WritePos, InData, NumElements);
				}
				WritePos += NumElements;
				if (WritePos >= WaitingForSize)
				{
					SizeAvailableSignal.Signal();
				}
				return true;
			}
			else
			{
				return false;
			}
		}

		// "Pushes" an end-of-data marker signaling that no further data will be pushed. May be called more than once. Buffer must be Reset() before next use.
		void SetEOD()
		{
			FScopeLock Lock(&AccessLock);
			bEOD = true;
			// Signal that data is present to wake any waiters on WaitForData() even though there may be no data in the buffer anymore.
			SizeAvailableSignal.Signal();
		}

		void RemoveFromBeginning(int64 InNumBytesToRemove)
		{
			FScopeLock Lock(&AccessLock);
			check(ExternalBuffer == nullptr);
			check(InNumBytesToRemove >= 0);
			check(InNumBytesToRemove <= (int64)WritePos);
			const int64 InNow = WritePos;
			check(InNow - InNumBytesToRemove >= 0);
			if (InNumBytesToRemove > 0)
			{
				uint8 *Base = GetBufferBase();
				FMemory::Memmove(Base, Base + InNumBytesToRemove, InNow - InNumBytesToRemove);
				WritePos -= (uint64)InNumBytesToRemove;
			}
		}

		void Lock()
		{
			AccessLock.Lock();
		}

		void Unlock()
		{
			AccessLock.Unlock();
		}

		// For use with an external FScopeLock
		FCriticalSection* GetLock()
		{
			return &AccessLock;
		}

		int64 GetLinearReadSize() const
		{
			FScopeLock Lock(&AccessLock);
			return WritePos;
		}

		void SetLinearReadSize(int64 InNewSize)
		{
			FScopeLock Lock(&AccessLock);
			WritePos = InNewSize;
		}

		// Must control Lock()/Unlock() externally!
		const uint8* GetLinearReadData() const
		{
			return GetBufferBase() ? GetBufferBase() : nullptr;
		}
		uint8* GetLinearReadData()
		{
			return GetBufferBase() ? GetBufferBase() : nullptr;
		}

		uint8* GetLinearWriteData(int64 InNumBytesToAppend)
		{
			FScopeLock Lock(&AccessLock);
			int64 Av = Avail();
			if (InNumBytesToAppend > Av)
			{
				bool bOk = IsEmpty() ? Allocate(InNumBytesToAppend) : InternalGrowTo(DataSize + InNumBytesToAppend - Av);
				check(bOk); (void)bOk;
			}
			return GetBufferBase() ? GetBufferBase() + WritePos : nullptr;
		}
		void AppendedNewData(int64 InNumAppended)
		{
			FScopeLock Lock(&AccessLock);
			check(!bEOD);
			if (InNumAppended > 0)
			{
				WritePos += InNumAppended;
				if (WritePos >= WaitingForSize)
				{
					SizeAvailableSignal.Signal();
				}
			}
		}

		void SetExternalData(TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> InExternalBuffer)
		{
			FScopeLock Lock(&AccessLock);
			Deallocate();
			Buffer = MoveTemp(InExternalBuffer);
			if (Buffer.IsValid())
			{
				DataSize = Buffer->Num();
				WritePos += DataSize;
				if (WritePos >= WaitingForSize)
				{
					SizeAvailableSignal.Signal();
				}
			}
		}

		void Abort()
		{
			FScopeLock Lock(&AccessLock);
			bWasAborted = true;
			SizeAvailableSignal.Signal();
		}

		bool WasAborted() const
		{
			return bWasAborted;
		}

		void SetHasErrored()
		{
			bHasErrored = true;
		}

		bool HasErrored() const
		{
			return bHasErrored;
		}

	protected:
		uint8* GetBufferBase() const
		{
			return Buffer.IsValid() ? Buffer->GetData() : ExternalBuffer ? ExternalBuffer : nullptr;
		}

		bool Allocate(int64 InSize)
		{
			if (ExternalBuffer)
			{
				return false;
			}
			if (InSize)
			{
				DataSize = InSize;
				Buffer.Reset();
				Buffer = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();
				Buffer->AddUninitialized(InSize);
				WritePos = 0;
			}
			return true;
		}

		void Deallocate()
		{
			Buffer.Reset();
			DataSize = 0;
			WritePos = 0;
			ExternalBuffer = nullptr;
		}

		bool InternalGrowTo(int64 InNewNumBytes)
		{
			// Note: The access mutex must already be held here!
			check(Buffer.IsValid() && InNewNumBytes);
			// Resize the buffer
			Buffer->Reserve(InNewNumBytes);
			DataSize = InNewNumBytes;
			return true;
		}

		void CopyData(uint8* CopyTo, const uint8* CopyFrom, int64 NumElements)
		{
			if (NumElements && CopyTo && CopyFrom)
			{
				FMemory::Memcpy(CopyTo, CopyFrom, NumElements);
			}
		}

		mutable FCriticalSection AccessLock;
		// Signal which gets set when at least `WaitingForSize` amount of data is present
		FMediaEvent SizeAvailableSignal;
		// The buffer
		TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> Buffer;
		// Allocated buffer size
		uint64 DataSize = 0;
		// Offset into buffer where to add new data
		uint64 WritePos = 0;
		// Amount of data necessary to be present for `SizeAvailableSignal` to get set.
		uint64 WaitingForSize = 0;
		// If set a buffer is provided externally to read into directly.
		uint8* ExternalBuffer = nullptr;
		// Flag indicating that no additional data will be added to the buffer.
		volatile bool bEOD = false;
		// Flag indicating that reading into the buffer has been aborted.
		volatile bool bWasAborted = false;
		// Flag indicating that filling the buffer from the source has encountered an error.
		volatile bool bHasErrored = false;
	};

} // namespace Electra
