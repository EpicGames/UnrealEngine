// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "StreamTypes.h"
#include "Utilities/StringHelpers.h"
#include "Utilities/Utilities.h"
#include "ElectraDecodersUtils.h"

#include "Misc/TVariant.h"
#include "CodecTypeFormat.h"
#include "MP4Utilities.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "Utils/AOMedia/ElectraUtilsAV1Video.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"

namespace Electra
{

	namespace
	{
		FString Printable4CC(const uint32 In4CC)
		{
			FString Out;
			// Not so much just printable as alphanumeric.
			for(uint32 i=0, Atom=In4CC; i<4; ++i, Atom<<=8)
			{
				int32 v = Atom >> 24;
				if ((v >= 'A' && v <= 'Z') || (v >= 'a' && v <= 'z') || (v >= '0' && v <= '9') || v == '_'|| v == '.')
				{
					Out.AppendChar(v);
				}
				else
				{
					// Not alphanumeric, return it as a hex string.
					return FString::Printf(TEXT("%08x"), In4CC);
				}
			}
			return Out;
		}
	}


	FCodecTypeFormat FStreamCodecInformation::ToCodecTypeFormat() const
	{
		FCodecTypeFormat fmt;
		auto SetCommon = [&](FCodecTypeFormat& InOut) -> void
		{
			InOut.LanguageTag = StreamLanguageTag;
			InOut.HumanReadableFormatInfo = GetHumanReadableCodecName();
			InOut.RFC6381 = GetCodecSpecifierRFC6381();
			InOut.MimeType = GetMimeType();

			TMap<FString, FVariant> Boxes;
			GetExtras().ConvertTo(Boxes, FString());
			for(auto &It : Boxes)
			{
				if (It.Key.EndsWith(TEXT("_box")) && It.Value.GetType() == EVariantTypes::ByteArray)
				{
					TArray<uint8> BoxData = It.Value.GetValue<TArray<uint8>>();
					if (BoxData.Num())
					{
						FString BoxName(It.Key);
						BoxName.LeftChopInline(4);
						if (BoxName.Len() == 4)
						{
							uint32 Box4CC = ElectraDecodersUtil::Make4CC((uint8)BoxName[0], (uint8)BoxName[1], (uint8)BoxName[2], (uint8)BoxName[3]);
							fmt.ExtraBoxes.Add(Box4CC, BoxData);
						}
					}
				}
			}
			InOut.DCR = DCR;
			InOut.CSD = CSD;
			InOut.FourCC = GetCodec4CC();
			InOut.Bitrate = InOut.AverageBitrate = GetBitrate();
		};
		switch(GetStreamType())
		{
			case EStreamType::Video:
			{
				fmt.Type = FCodecTypeFormat::EType::Video;
				fmt.Properties.Emplace<FCodecTypeFormat::FVideo>();
				FCodecTypeFormat::FVideo& vid = fmt.Properties.Get<FCodecTypeFormat::FVideo>();
				vid.Width = (uint32) Resolution.Width;
				vid.Height = (uint32) Resolution.Height;
				vid.BitDepth = 0;
				vid.AspectRatioW = AspectRatio.IsSet() ? (uint32) AspectRatio.Width : 1U;
				vid.AspectRatioH = AspectRatio.IsSet() ? (uint32) AspectRatio.Height : 1U;
				vid.Profile.Tier = (uint32) GetProfileTier();
				vid.Profile.ProfileSpace = (uint32) GetProfileSpace();
				vid.Profile.Profile = (uint32) GetProfile();
				vid.Profile.Level = (uint32) GetProfileLevel();
				vid.Profile.Constraints = GetProfileConstraints();
				vid.Profile.CompatibilityFlags = GetProfileCompatibilityFlags();
				vid.FrameRate.Numerator = (int32) FrameRate.GetNumerator();
				vid.FrameRate.Denominator = (int32) FrameRate.GetDenominator();
				SetCommon(fmt);
				if (!fmt.FourCC && GetCodec() == ECodec::H264)
				{
					fmt.FourCC = ElectraDecodersUtil::Make4CC('a','v','c','1');
				}
				else if (!fmt.FourCC && GetCodec() == ECodec::H265)
				{
					fmt.FourCC = ElectraDecodersUtil::Make4CC('h','v','c','1');
				}
				ElectraDecodersUtil::PrepareCodecTypeFormat(fmt);
				break;
			}
			case EStreamType::Audio:
			{
				fmt.Type = FCodecTypeFormat::EType::Audio;
				fmt.Properties.Emplace<FCodecTypeFormat::FAudio>();
				FCodecTypeFormat::FAudio& aud = fmt.Properties.Get<FCodecTypeFormat::FAudio>();
				aud.NumChannels = (uint32) GetNumberOfChannels();
				aud.ChannelConfiguration = GetChannelConfiguration();
				aud.SampleRate = (uint32) GetSamplingRate();
				SetCommon(fmt);
				if (!fmt.FourCC && GetCodec() == ECodec::AAC)
				{
					fmt.FourCC = ElectraDecodersUtil::Make4CC('m','p','4','a');
				}
				ElectraDecodersUtil::PrepareCodecTypeFormat(fmt);
				break;
			}
		}
		return fmt;
	}


	FStreamCodecInformation::FStreamCodecInformation(const FCodecTypeFormat& InFromCodecTypeFormat)
	{
		Clear();
		auto SetCommon = [&]() -> void
		{
			SetMimeType(InFromCodecTypeFormat.MimeType);
			SetCodec4CC(InFromCodecTypeFormat.FourCC);
			SetCodecSpecifierRFC6381(InFromCodecTypeFormat.RFC6381);
			SetBitrate(InFromCodecTypeFormat.Bitrate);
			SetCodecSpecificData(InFromCodecTypeFormat.CSD);
			SetDecoderConfigRecord(InFromCodecTypeFormat.DCR);
			SetStreamLanguageTag(InFromCodecTypeFormat.LanguageTag);
			for(auto &It : InFromCodecTypeFormat.ExtraBoxes)
			{
				FString BoxName = Printable4CC(It.Key);
				BoxName.Append(TEXT("_box"));
				GetExtras().Set(FName(BoxName), FVariantValue(It.Value));
			}
		};
		switch(InFromCodecTypeFormat.Type)
		{
			case FCodecTypeFormat::EType::Video:
			{
				const FCodecTypeFormat::FVideo& cvid = InFromCodecTypeFormat.Properties.Get<FCodecTypeFormat::FVideo>();

				SetStreamType(EStreamType::Video);
				if (InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('a','v','c','1') || InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('a','v','c','3'))
				{
					SetCodec(ECodec::H264);
				}
				else if (InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('h','v','c','1') || InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('h','e','v','1'))
				{
					SetCodec(ECodec::H265);
				}
				else
				{
					SetCodec(ECodec::Video4CC);
				}
				SetCommon();

				SetResolution(FResolution((int32)cvid.Width, (int32)cvid.Height));
				SetFrameRate(FTimeFraction(cvid.FrameRate.Numerator, cvid.FrameRate.Denominator));
				SetAspectRatio(FAspectRatio(cvid.AspectRatioW, cvid.AspectRatioH));

				SetProfileTier((int32) cvid.Profile.Tier);
				SetProfileSpace((int32) cvid.Profile.ProfileSpace);
				SetProfile((int32) cvid.Profile.Profile);
				SetProfileLevel((int32) cvid.Profile.Level);
				SetProfileConstraints(cvid.Profile.Constraints);
				SetProfileCompatibilityFlags((uint32) cvid.Profile.CompatibilityFlags);

				SetHumanReadableCodecName(InFromCodecTypeFormat.HumanReadableFormatInfo);
				break;
			}
			case FCodecTypeFormat::EType::Audio:
			{
				const FCodecTypeFormat::FAudio& caud = InFromCodecTypeFormat.Properties.Get<FCodecTypeFormat::FAudio>();

				SetStreamType(EStreamType::Audio);
				if (InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('m','p','4','a'))
				{
					SetCodec(ECodec::AAC);
				}
				else if (InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('e','c','-','3'))
				{
					SetCodec(ECodec::EAC3);
				}
				else if (InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('a','c','-','3'))
				{
					SetCodec(ECodec::AC3);
				}
				else
				{
					SetCodec(ECodec::Audio4CC);
				}
				SetCommon();

				SetSamplingRate((int32) caud.SampleRate);
				SetNumberOfChannels((int32) caud.NumChannels);
				SetChannelConfiguration(caud.ChannelConfiguration);

				SetHumanReadableCodecName(InFromCodecTypeFormat.HumanReadableFormatInfo);
				break;
			}
			case FCodecTypeFormat::EType::Subtitle:
			{
				const FCodecTypeFormat::FSubtitle& csub = InFromCodecTypeFormat.Properties.Get<FCodecTypeFormat::FSubtitle>();

				SetStreamType(EStreamType::Subtitle);
				if (InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('w','v','t','t'))
				{
					SetCodec(ECodec::WebVTT);
				}
				else if (InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('s','t','p','p') || InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('t','t','m','l'))
				{
					SetCodec(ECodec::TTML);
				}
				else if (InFromCodecTypeFormat.FourCC == ElectraDecodersUtil::Make4CC('t','x','3','g'))
				{
					SetCodec(ECodec::TX3G);
				}
				else
				{
					SetCodec(ECodec::OtherSubtitle);
				}
				SetCommon();

				SetHumanReadableCodecName(InFromCodecTypeFormat.HumanReadableFormatInfo);
				break;
			}
			default:
			{
				break;
			}
		}
	}





	FString FStreamCodecInformation::GetMimeType() const
	{
		if (!MimeType.IsEmpty())
		{
			return MimeType;
		}
		switch(GetCodec())
		{
			case ECodec::H264:
			{
				return FString(TEXT("video/mp4"));
			}
			case ECodec::H265:
			{
				return FString(TEXT("video/mp4"));
			}
			case ECodec::AAC:
			{
				return FString(TEXT("audio/mp4"));
			}
			case ECodec::EAC3:
			{
				return FString(TEXT("audio/mp4"));
			}
			case ECodec::WebVTT:
			case ECodec::TTML:
			case ECodec::TX3G:
			case ECodec::OtherSubtitle:
			{
				return FString(TEXT("application/mp4"));
			}
			default:
			{
				return FString(TEXT("application/octet-stream"));
			}
		}
	}

	FString FStreamCodecInformation::GetMimeTypeWithCodec() const
	{
		return GetMimeType() + FString::Printf(TEXT("; codecs=\"%s\""), *GetCodecSpecifierRFC6381());
	}

	FString FStreamCodecInformation::GetMimeTypeWithCodecAndFeatures() const
	{
		if (GetStreamType() == EStreamType::Video && GetResolution().Width && GetResolution().Height)
		{
			return GetMimeTypeWithCodec() + FString::Printf(TEXT("; resolution=%dx%d"), GetResolution().Width, GetResolution().Height);
		}
		return GetMimeTypeWithCodec();
	}


	bool FStreamCodecInformation::ParseFromRFC6381(const FString& CodecOTI)
	{
		FCodecTypeFormat fmt;
		fmt.RFC6381 = CodecOTI;
		fmt.DCR = DCR;
		TMap<FString, FVariant> Boxes;
		GetExtras().ConvertTo(Boxes, FString());
		for(auto &It : Boxes)
		{
			if (It.Key.EndsWith(TEXT("_box")) && It.Value.GetType() == EVariantTypes::ByteArray)
			{
				TArray<uint8> BoxData = It.Value.GetValue<TArray<uint8>>();
				if (BoxData.Num())
				{
					FString BoxName(It.Key);
					BoxName.LeftChopInline(4);
					if (BoxName.Len() == 4)
					{
						uint32 Box4CC = ElectraDecodersUtil::Make4CC((uint8)BoxName[0], (uint8)BoxName[1], (uint8)BoxName[2], (uint8)BoxName[3]);
						fmt.ExtraBoxes.Add(Box4CC, BoxData);
					}
				}
			}
		}
		if (!ElectraDecodersUtil::PrepareCodecTypeFormat(fmt))
		{
			StreamType = EStreamType::Unsupported;
			CodecSpecifier = CodecOTI;
			Codec = ECodec::Unknown;
			return false;
		}
		SetMimeType(fmt.MimeType);
		SetCodec4CC(fmt.FourCC);
		SetCodecSpecifierRFC6381(fmt.RFC6381);
		SetCodecSpecificData(fmt.CSD);
		switch(fmt.Type)
		{
			case FCodecTypeFormat::EType::Video:
			{
				SetStreamType(EStreamType::Video);
				if (fmt.FourCC == ElectraDecodersUtil::Make4CC('a','v','c','1') || fmt.FourCC == ElectraDecodersUtil::Make4CC('a','v','c','3'))
				{
					SetCodec(ECodec::H264);
				}
				else if (fmt.FourCC == ElectraDecodersUtil::Make4CC('h','v','c','1') || fmt.FourCC == ElectraDecodersUtil::Make4CC('h','e','v','1'))
				{
					SetCodec(ECodec::H265);
				}
				else
				{
					SetCodec(ECodec::Video4CC);
				}
				const FCodecTypeFormat::FVideo& cvid = fmt.Properties.Get<FCodecTypeFormat::FVideo>();
				SetResolution(FResolution((int32)cvid.Width, (int32)cvid.Height));
				SetFrameRate(FTimeFraction(cvid.FrameRate.Numerator, cvid.FrameRate.Denominator));
				SetAspectRatio(FAspectRatio(cvid.AspectRatioW, cvid.AspectRatioH));
				SetProfileTier((int32) cvid.Profile.Tier);
				SetProfileSpace((int32) cvid.Profile.ProfileSpace);
				SetProfile((int32) cvid.Profile.Profile);
				SetProfileLevel((int32) cvid.Profile.Level);
				SetProfileConstraints(cvid.Profile.Constraints);
				SetProfileCompatibilityFlags((uint32) cvid.Profile.CompatibilityFlags);
				SetHumanReadableCodecName(fmt.HumanReadableFormatInfo);
				break;
			}
			case FCodecTypeFormat::EType::Audio:
			{
				SetStreamType(EStreamType::Audio);
				if (fmt.FourCC == ElectraDecodersUtil::Make4CC('m','p','4','a'))
				{
					SetCodec(ECodec::AAC);
				}
				else if (fmt.FourCC == ElectraDecodersUtil::Make4CC('e','c','-','3'))
				{
					SetCodec(ECodec::EAC3);
				}
				else if (fmt.FourCC == ElectraDecodersUtil::Make4CC('a','c','-','3'))
				{
					SetCodec(ECodec::AC3);
				}
				else
				{
					SetCodec(ECodec::Audio4CC);
				}
				const FCodecTypeFormat::FAudio& caud = fmt.Properties.Get<FCodecTypeFormat::FAudio>();
				SetSamplingRate((int32) caud.SampleRate);
				SetNumberOfChannels((int32) caud.NumChannels);
				SetChannelConfiguration(caud.ChannelConfiguration);
				if (caud.ObjectType.IsType<FCodecTypeFormat::FAudio::FMPEGObjectType>())
				{
					const FCodecTypeFormat::FAudio::FMPEGObjectType& ot(caud.ObjectType.Get<FCodecTypeFormat::FAudio::FMPEGObjectType>());
					if (ot.MPEG == 0)
					{
						SetProfile(ot.AudioObjectType);
					}
					else
					{
						SetProfile(1);
						SetProfileLevel(ot.Layer);
					}
				}
				SetHumanReadableCodecName(fmt.HumanReadableFormatInfo);
				break;
			}
			case FCodecTypeFormat::EType::Subtitle:
			{
				SetStreamType(EStreamType::Subtitle);
				if (fmt.FourCC == ElectraDecodersUtil::Make4CC('w','v','t','t'))
				{
					SetCodec(ECodec::WebVTT);
				}
				else if (fmt.FourCC == ElectraDecodersUtil::Make4CC('s','t','p','p') || fmt.FourCC == ElectraDecodersUtil::Make4CC('t','t','m','l'))
				{
					SetCodec(ECodec::TTML);
				}
				else if (fmt.FourCC == ElectraDecodersUtil::Make4CC('t','x','3','g'))
				{
					SetCodec(ECodec::TX3G);
				}
				else
				{
					SetCodec(ECodec::OtherSubtitle);
				}
				//const FCodecTypeFormat::FSubtitle& csub = fmt.Properties.Get<FCodecTypeFormat::FSubtitle>();
				SetHumanReadableCodecName(fmt.HumanReadableFormatInfo);
				break;
			}
			default:
			{
				StreamType = EStreamType::Unsupported;
				CodecSpecifier = CodecOTI;
				Codec = ECodec::Unknown;
				return false;
			}
		}
		return true;
	}

	void FStreamCodecInformation::UpdateFromExtraBoxes()
	{
		switch(GetCodec4CC())
		{
			case ElectraDecodersUtil::Make4CC('v','p','0','8'):
			case ElectraDecodersUtil::Make4CC('v','p','0','9'):
			{
				if (Extras.HaveKey(StreamCodecInformationOptions::VPccBox))
				{
					ParseFromRFC6381(CodecSpecifier);
				}
				break;
			}
			case ElectraDecodersUtil::Make4CC('a','v','0','1'):
			{
				if (Extras.HaveKey(StreamCodecInformationOptions::AV1cBox))
				{
					ParseFromRFC6381(CodecSpecifier);
				}
				break;
			}
			default:
			{
				break;
			}
		}
	}

	FString FStreamCodecInformation::GetCodecName() const
	{
		switch(Codec)
		{
			case FStreamCodecInformation::ECodec::H264:
			{
				return FString(TEXT("avc"));
			}
			case FStreamCodecInformation::ECodec::H265:
			{
				return FString(TEXT("hevc"));
			}
			case FStreamCodecInformation::ECodec::AAC:
			{
				return FString(TEXT("aac"));
			}
			case FStreamCodecInformation::ECodec::EAC3:
			{
				return FString(TEXT("eac3"));
			}
			case FStreamCodecInformation::ECodec::WebVTT:
			{
				return FString(TEXT("wvtt"));
			}
			case FStreamCodecInformation::ECodec::TTML:
			{
				return FString(TEXT("stpp"));
			}
			case FStreamCodecInformation::ECodec::TX3G:
			{
				return FString(TEXT("tx3g"));
			}
			case FStreamCodecInformation::ECodec::OtherSubtitle:
			{
				return FString(TEXT("subt"));
			}
			case FStreamCodecInformation::ECodec::Video4CC:
			case FStreamCodecInformation::ECodec::Audio4CC:
			{
				return Printable4CC(Codec4CC);
			}
			default:
			{
				return FString(TEXT("unknown"));
			}
		}
	}

	const FString& FStreamCodecInformation::GetHumanReadableCodecName() const
	{
		if (HumanReadableCodecName.IsEmpty())
		{
			if (!TryConstructHumanReadableCodecName())
			{
				HumanReadableCodecName = CodecSpecifier;
			}
		}
		return HumanReadableCodecName;
	}

	bool FStreamCodecInformation::TryConstructHumanReadableCodecName() const
	{
		switch(GetCodec())
		{
			case ECodec::H264:
			{
				HumanReadableCodecName = TEXT("AVC, ");
				HumanReadableCodecName.Append(ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet::GetFormatInfo(ProfileLevel.Profile, ProfileLevel.Level, ProfileLevel.Constraints));
				return true;
			}
			case ECodec::H265:
			{
				HumanReadableCodecName = TEXT("HEVC, ");
				HumanReadableCodecName.Append(ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet::GetFormatInfo(ProfileLevel.Profile, ProfileLevel.Level, ProfileLevel.Constraints));
				return true;
			}
			case ECodec::Video4CC:
			{
				switch(GetCodec4CC())
				{
					case ElectraDecodersUtil::Make4CC('v','p','0','8'):
					{
						HumanReadableCodecName = TEXT("VP8");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('v','p','0','9'):
					{
						if (ProfileLevel.Level)
						{
							HumanReadableCodecName = TEXT("VP9, ");
							HumanReadableCodecName.Append(ElectraDecodersUtil::VPxVideo::FVPCodecConfigurationRecord::GetFormatInfo(ProfileLevel.Profile, ProfileLevel.Level));
						}
						else
						{
							HumanReadableCodecName = TEXT("VP9");
						}
						return true;
					}
					case ElectraDecodersUtil::Make4CC('a','v','0','1'):
					{
						HumanReadableCodecName = TEXT("AV1, ");
						HumanReadableCodecName.Append(ElectraDecodersUtil::AV1Video::FAV1CodecConfigurationRecord::GetFormatInfo(ProfileLevel.Profile, ProfileLevel.Level));
						return true;
					}

					case ElectraDecodersUtil::Make4CC('a','p','c','h'):
					{
						HumanReadableCodecName = TEXT("Apple ProRes 422 High Quality");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('a','p','c','n'):
					{
						HumanReadableCodecName = TEXT("Apple ProRes 422 Standard Definition");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('a','p','c','s'):
					{
						HumanReadableCodecName = TEXT("Apple ProRes 422 LT");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('a','p','c','o'):
					{
						HumanReadableCodecName = TEXT("Apple ProRes 422 Proxy");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('a','p','4','h'):
					{
						HumanReadableCodecName = TEXT("Apple ProRes 4444");
						return true;
					}

					case ElectraDecodersUtil::Make4CC('H','a','p','1'):
					{
						HumanReadableCodecName = TEXT("Hap");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('H','a','p','5'):
					{
						HumanReadableCodecName = TEXT("Hap Alpha");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('H','a','p','Y'):
					{
						HumanReadableCodecName = TEXT("Hap Q");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('H','a','p','M'):
					{
						HumanReadableCodecName = TEXT("Hap Q Alpha");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('H','a','p','7'):
					{
						HumanReadableCodecName = TEXT("Hap R");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('H','a','p','H'):
					{
						HumanReadableCodecName = TEXT("Hap HDR");
						return true;
					}

					case ElectraDecodersUtil::Make4CC('A','V','d','h'):
					{
						HumanReadableCodecName = TEXT("Avid DNxHD");
						return true;
					}

					case ElectraDecodersUtil::Make4CC('a','p','v','1'):
					{
						HumanReadableCodecName = TEXT("Advanced Professional Video (APV)");
						return true;
					}
				}
				HumanReadableCodecName = Printable4CC(GetCodec4CC());
				return true;
			}
			case ECodec::AAC:
			{
				HumanReadableCodecName = TEXT("MPEG AAC");
				return true;
			}
			case ECodec::EAC3:
			{
				HumanReadableCodecName = TEXT("Dolby Digital");
				return true;
			}
			case ECodec::Audio4CC:
			{
				switch(GetCodec4CC())
				{
					case ElectraDecodersUtil::Make4CC('O','p','u','s'):
					{
						HumanReadableCodecName = TEXT("Opus");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('f','L','a','C'):
					{
						HumanReadableCodecName = TEXT("Free Lossless Audio Codec (FLAC)");
						return true;
					}
					case ElectraDecodersUtil::Make4CC('m','p','g','a'):
					{
						if (GetProfileLevel())
						{
							HumanReadableCodecName = FString::Printf(TEXT("MPEG%d Layer %d"), GetProfile(), GetProfileLevel());
						}
						else
						{
							HumanReadableCodecName = FString::Printf(TEXT("MPEG%d audio"), GetProfile());
						}
						return true;
					}
				}
				HumanReadableCodecName = Printable4CC(GetCodec4CC());
				return true;
			}
			case ECodec::WebVTT:
			{
				HumanReadableCodecName = TEXT("WebVTT");
				return true;
			}
			case ECodec::TTML:
			{
				HumanReadableCodecName = TEXT("TTML");
				return true;
			}
			case ECodec::TX3G:
			{
				HumanReadableCodecName = TEXT("SRT/TX3G");
				return true;
			}
		}
		return false;
	}




	bool FCodecSelectionPriorities::Initialize(const FString& ConfigurationString)
	{
		ClassPriorities.Empty();
		if (ConfigurationString.Len() && !ParseInternal(ConfigurationString))
		{
			ClassPriorities.Empty();
			return false;
		}
		return true;
	}
	bool FCodecSelectionPriorities::ParseInternal(const FString& ConfigurationString)
	{
		auto SkipWhiteSpaces = [](StringHelpers::FStringIterator& it) -> void
		{
			while(it && TChar<TCHAR>::IsWhitespace(*it))
			{
				++it;
			}
		};

		auto ParsePriority = [](int32& OutPrio, StringHelpers::FStringIterator& it, bool bInClass) -> bool
		{
			int64 Prio = 0;
			bool bEmpty = true;
			while(it && TChar<TCHAR>::IsDigit(*it))
			{
				bEmpty = false;
				Prio *= 10;
				Prio += *it - TCHAR('0');
				++it;
			}
			while(it && TChar<TCHAR>::IsWhitespace(*it))
			{
				++it;
			}
			// Did we end the priority properly?
			if (!bEmpty && (!it || *it == TCHAR(',') || (bInClass && *it == TCHAR('{'))))
			{
				OutPrio = Prio;
				return true;
			}
			// Unexpected next character. Fail!
			return false;
		};

		const TCHAR* const CommaDelimiter = TEXT(",");
		StringHelpers::FStringIterator it(ConfigurationString);
		while(it)
		{
			FClassPriority ClassPriority;
			SkipWhiteSpaces(it);
			while(it && *it != TCHAR('=') && *it != TCHAR('{') && *it != TCHAR(','))
			{
				ClassPriority.Prefix += *it++;
			}
			if (ClassPriority.Prefix.Len() == 0)
			{
				return false;
			}

			// Is the next char assigning a priority?
			if (it && *it == TCHAR('='))
			{
				// Get the class priority
				++it;
				if (!ParsePriority(ClassPriority.Priority, it, true))
				{
					return false;
				}
			}
			// If no priority then there must now be a group for stream specific priorities.
			else if (!it || *it != TCHAR('{'))
			{
				return false;
			}
			// Do stream specific priorities follow?
			if (it && *it == TCHAR('{'))
			{
				int32 GroupStart = it.GetIndex();
				// Look for the end of the group.
				while(it && *it != TCHAR('}'))
				{
					++it;
				}
				if (!it || *it != TCHAR('}'))
				{
					return false;
				}
				++it;
				FString Group = ConfigurationString.Mid(GroupStart+1, it.GetIndex()-GroupStart-2);
				TArray<FString> StreamPriorities;
				Group.ParseIntoArray(StreamPriorities, CommaDelimiter, true);
				if (StreamPriorities.Num() == 0)
				{
					return false;
				}
				for(auto &sp : StreamPriorities)
				{
					FStreamPriority StreamPriority;
					StringHelpers::FStringIterator spIt(sp);
					while(spIt)
					{
						SkipWhiteSpaces(spIt);
						while(spIt && *spIt != TCHAR('=') && *spIt != TCHAR('{') && *spIt != TCHAR(','))
						{
							StreamPriority.Prefix += *spIt++;
						}
						if (StreamPriority.Prefix.Len() == 0)
						{
							return false;
						}
						if (spIt && *spIt == TCHAR('='))
						{
							++spIt;
							if (!ParsePriority(StreamPriority.Priority, spIt, false))
							{
								return false;
							}
						}
						else
						{
							return false;
						}
					}
					ClassPriority.StreamPriorities.Emplace(MoveTemp(StreamPriority));
				}
			}
			// Either there's a comma separating successive entries or we are done.
			SkipWhiteSpaces(it);
			if (it && *it != TCHAR(','))
			{
				return false;
			}
			++it;
			ClassPriorities.Emplace(MoveTemp(ClassPriority));
		}
		return true;
	}

	int32 FCodecSelectionPriorities::GetClassPriority(const FString& CodecSpecifierRFC6381) const
	{
		// If no priorities are given then all have the same priority of 0.
		if (!ClassPriorities.Num())
		{
			return 0;
		}
		// Otherwise apply the priority filter. If no match then return -1.
		for(auto &CodecClass : ClassPriorities)
		{
			if (CodecSpecifierRFC6381.StartsWith(CodecClass.Prefix, ESearchCase::IgnoreCase))
			{
				return CodecClass.Priority;
			}
		}
		return -1;
	}

	int32 FCodecSelectionPriorities::GetStreamPriority(const FString& CodecSpecifierRFC6381) const
	{
		for(auto &CodecClass : ClassPriorities)
		{
			if (CodecSpecifierRFC6381.StartsWith(CodecClass.Prefix, ESearchCase::IgnoreCase))
			{
				for(auto &CodecStream : CodecClass.StreamPriorities)
				{
					if (CodecSpecifierRFC6381.StartsWith(CodecStream.Prefix, ESearchCase::IgnoreCase))
					{
						return CodecStream.Priority;
					}
				}
			}
		}
		return -1;
	}

} // namespace Electra
