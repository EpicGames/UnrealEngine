// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"
#include "ParameterDictionary.h"
#include "MediaVideoDecoderOutput.h"
#include "MediaURLType.h"
#include "Utilities/BCP47-Helpers.h"
#include "CodecTypeFormat.h"

namespace Electra
{

	enum class EStreamType
	{
		Unsupported,
		Video,
		Audio,
		Subtitle,
	};

	static inline const TCHAR* GetStreamTypeName(EStreamType StreamType)
	{
		switch(StreamType)
		{
			case EStreamType::Video:
				return TEXT("Video");
			case EStreamType::Audio:
				return TEXT("Audio");
			case EStreamType::Subtitle:
				return TEXT("Subtitle");
			case EStreamType::Unsupported:
				return TEXT("Unsupported");
		}
		return TEXT("n/a");
	}

	static inline constexpr int32 StreamTypeToArrayIndex(EStreamType StreamType)
	{
		return StreamType == EStreamType::Video ? 0 :
			   StreamType == EStreamType::Audio ? 1 :
			   StreamType == EStreamType::Subtitle ? 2 : 3;
	}

	static inline constexpr int32 StreamTypeToArrayIndex0(EStreamType StreamType)
	{
		check(StreamType != EStreamType::Unsupported);
		return StreamType == EStreamType::Audio ? 1 : StreamType == EStreamType::Subtitle ? 2 : 0;
	}

	static inline constexpr EStreamType StreamArrayIndexToType(int32 InIndex)
	{
		return InIndex == 0 ? EStreamType::Video :
			   InIndex == 1 ? EStreamType::Audio :
			   InIndex == 2 ? EStreamType::Subtitle : EStreamType::Unsupported;
	}

	class FStreamCodecInformation
	{
	public:
		FStreamCodecInformation()
		{
			Clear();
		}
		explicit FStreamCodecInformation(const FCodecTypeFormat& InFromCodecTypeFormat);
		FCodecTypeFormat ToCodecTypeFormat() const;

		enum class ECodec
		{
			// --- Unknown ---
			Unknown = 0,
			// --- Video ---
			H264 = 1,
			H265,
			Video4CC,
			// --- Audio ---
			AAC = 100,
			EAC3,
			AC3,
			Audio4CC,
			// --- Subtitle / Caption ---
			WebVTT = 200,
			TTML,
			TX3G,
			OtherSubtitle
		};

		EStreamType GetStreamType() const
		{
			return StreamType;
		}

		void SetMimeType(const FString& InMimeType)
		{
			MimeType = InMimeType;
		}
		FString GetMimeType() const;
		FString GetMimeTypeWithCodec() const;
		FString GetMimeTypeWithCodecAndFeatures() const;

		bool ParseFromRFC6381(const FString& CodecOTI);

		void SetStreamType(EStreamType InStreamType)
		{
			StreamType = InStreamType;
		}

		uint32 GetCodec4CC() const
		{
			return Codec4CC;
		}

		void SetCodec4CC(uint32 In4CC)
		{
			Codec4CC = In4CC;
			ResetHumanReadableCodecName();
		}

		ECodec GetCodec() const
		{
			return Codec;
		}

		void SetCodec(ECodec InCodec)
		{
			Codec = InCodec;
			ResetHumanReadableCodecName();
		}

		FString GetCodecName() const;

		void UpdateFromExtraBoxes();

		bool IsVideoCodec() const
		{
			switch(GetCodec())
			{
				case ECodec::H264:
				case ECodec::H265:
				case ECodec::Video4CC:
					return true;
				default:
					return false;
			}
		}

		bool IsAudioCodec() const
		{
			switch(GetCodec())
			{
				case ECodec::AAC:
				case ECodec::EAC3:
				case ECodec::AC3:
				case ECodec::Audio4CC:
					return true;
				default:
					return false;
			}
		}

		bool IsSubtitleCodec() const
		{
			switch(GetCodec())
			{
				case ECodec::WebVTT:
				case ECodec::TTML:
				case ECodec::TX3G:
				case ECodec::OtherSubtitle:
					return true;
				default:
					return false;
			}
		}

		bool IsCodec(EStreamType InType) const
		{
			switch(InType)
			{
				case EStreamType::Video:
				{
					return IsVideoCodec();
				}
				case EStreamType::Audio:
				{
					return IsAudioCodec();
				}
				case EStreamType::Subtitle:
				{
					return IsSubtitleCodec();
				}
			}
			return false;
		}

		const FString& GetCodecSpecifierRFC6381() const
		{
			return CodecSpecifier;
		}

		void SetCodecSpecifierRFC6381(const FString& InCodecSpecifier)
		{
			CodecSpecifier = InCodecSpecifier;
			ResetHumanReadableCodecName();
		}

		const FString& GetHumanReadableCodecName() const;

		void SetHumanReadableCodecName(const FString& InHumanReadableCodecName)
		{
			HumanReadableCodecName = InHumanReadableCodecName;
		}

		struct FResolution
		{
			FResolution(int32 w = 0, int32 h = 0)
				: Width(w)
				, Height(h)
			{
			}
			void Clear()
			{
				Width = Height = 0;
			}
			bool IsSet() const
			{
				return Width != 0 || Height != 0;
			}
			bool ExceedsLimit(int32 LimitWidth, int32 LimitHeight) const
			{
				if (LimitWidth && Width > LimitWidth)
				{
					return true;
				}
				if (LimitHeight && Height > LimitHeight)
				{
					return true;
				}
				return false;
			}
			bool ExceedsLimit(const FResolution& Limit) const
			{
				return ExceedsLimit(Limit.Width, Limit.Height);
			}

			bool operator == (const FResolution& rhs) const
			{
				return Width == rhs.Width && Height == rhs.Height;
			}

			bool operator != (const FResolution& rhs) const
			{
				return !(*this == rhs);
			}

			int32	Width;
			int32	Height;
		};

		const FResolution& GetResolution() const
		{
			return Resolution;
		}

		void SetResolution(const FResolution& InResolution)
		{
			Resolution = InResolution;
			ResetHumanReadableCodecName();
		}


		struct FTranslation
		{
			FTranslation(int32 x = 0, int32 y = 0)
				: Left(x)
				, Top(y)
			{
			}
			void Clear()
			{
				Left = Top = 0;
			}
			int32 GetX() const
			{
				return Left;
			}
			int32 GetY() const
			{
				return Top;
			}

			bool operator == (const FTranslation& rhs) const
			{
				return Left == rhs.Left && Top == rhs.Top;
			}

			bool operator != (const FTranslation& rhs) const
			{
				return !(*this == rhs);
			}

			int32 Left;
			int32 Top;
		};

		const FTranslation& GetTranslation() const
		{
			return Translation;
		}

		void SetTranslation(const FTranslation& InTranslation)
		{
			Translation = InTranslation;
		}


		struct FAspectRatio
		{
			FAspectRatio(int32 w = 0, int32 h = 0)
				: Width(w)
				, Height(h)
			{
			}
			void Clear()
			{
				Width = Height = 0;
			}
			bool IsSet() const
			{
				return Width != 0 || Height != 0;
			}

			bool operator == (const FAspectRatio& rhs) const
			{
				return Width == rhs.Width && Height == rhs.Height;
			}

			bool operator != (const FAspectRatio& rhs) const
			{
				return !(*this == rhs);
			}

			int32	Width;
			int32	Height;
		};

		struct FCrop
		{
			FCrop(int32 InLeft = 0, int32 InTop = 0, int32 InRight = 0, int32 InBottom = 0)
				: Left(InLeft)
				, Top(InTop)
				, Right(InRight)
				, Bottom(InBottom)
			{
			}

			void Clear()
			{
				Left = Top = Right = Bottom = 0;
			}

			bool operator == (const FCrop& rhs) const
			{
				return Left == rhs.Left && Top == rhs.Top && Right == rhs.Right && Bottom == rhs.Bottom;
			}

			bool operator != (const FCrop& rhs) const
			{
				return !(*this == rhs);
			}

			int32	Left;
			int32	Top;
			int32	Right;
			int32	Bottom;
		};

		const FCrop& GetCrop() const
		{
			return Crop;
		}

		void SetCrop(const FCrop& InCrop)
		{
			Crop = InCrop;
		}

		const FAspectRatio& GetAspectRatio() const
		{
			return AspectRatio;
		}

		void SetAspectRatio(const FAspectRatio& InAspectRatio)
		{
			AspectRatio = InAspectRatio;
		}

		const FTimeFraction& GetFrameRate() const
		{
			return FrameRate;
		}

		void SetFrameRate(const FTimeFraction& InFrameRate)
		{
		// Temporary hack to avoid some malformed streams to send in ridiculous frame rates.
			if (InFrameRate.IsValid() && InFrameRate.GetAsDouble() <= 120.5)
			{
				FrameRate = InFrameRate;
			}
		}

		void SetProfileSpace(int32 InProfileSpace)
		{
			ProfileLevel.ProfileSpace = InProfileSpace;
			ResetHumanReadableCodecName();
		}

		int32 GetProfileSpace() const
		{
			return ProfileLevel.ProfileSpace;
		}

		void SetProfile(int32 InProfile)
		{
			ProfileLevel.Profile = InProfile;
			ResetHumanReadableCodecName();
		}

		int32 GetProfile() const
		{
			return ProfileLevel.Profile;
		}

		void SetProfileLevel(int32 InLevel)
		{
			ProfileLevel.Level = InLevel;
			ResetHumanReadableCodecName();
		}

		int32 GetProfileLevel() const
		{
			return ProfileLevel.Level;
		}

		void SetProfileCompatibilityFlags(uint32 InCompatibilityFlags)
		{
			ProfileLevel.CompatibilityFlags = InCompatibilityFlags;
			ResetHumanReadableCodecName();
		}

		uint32 GetProfileCompatibilityFlags() const
		{
			return ProfileLevel.CompatibilityFlags;
		}

		void SetProfileConstraints(uint64 InConstraints)
		{
			ProfileLevel.Constraints = InConstraints;
			ResetHumanReadableCodecName();
		}

		uint64 GetProfileConstraints() const
		{
			return ProfileLevel.Constraints;
		}

		void SetProfileTier(int32 InTier)
		{
			ProfileLevel.Tier = InTier;
			ResetHumanReadableCodecName();
		}

		int32 GetProfileTier() const
		{
			return ProfileLevel.Tier;
		}

		void SetSamplingRate(int32 InSamplingRate)
		{
			SampleRate = InSamplingRate;
			ResetHumanReadableCodecName();
		}

		int32 GetSamplingRate() const
		{
			return SampleRate;
		}

		void SetNumberOfChannels(int32 InNumberOfChannels)
		{
			NumChannels = InNumberOfChannels;
			ResetHumanReadableCodecName();
		}

		int32 GetNumberOfChannels() const
		{
			return NumChannels;
		}

		void SetChannelConfiguration(uint32 InChannelConfiguration)
		{
			ChannelConfiguration = InChannelConfiguration;
			ResetHumanReadableCodecName();
		}

		uint32 GetChannelConfiguration() const
		{
			return ChannelConfiguration;
		}

		void SetAudioDecodingComplexity(int InAudioDecodingComplexity)
		{
			AudioDecodingComplexity = InAudioDecodingComplexity;
			ResetHumanReadableCodecName();
		}

		int32 GetAudioDecodingComplexity() const
		{
			return AudioDecodingComplexity;
		}

		void SetAudioAccessibility(int32 InAudioAccessibility)
		{
			AudioAccessibility = InAudioAccessibility;
			ResetHumanReadableCodecName();
		}

		int32 GetAudioAccessibility() const
		{
			return AudioAccessibility;
		}

		void SetNumberOfAudioObjects(int32 InNumberOfAudioObjects)
		{
			NumberOfAudioObjects = InNumberOfAudioObjects;
			ResetHumanReadableCodecName();
		}

		int32 GetNumberOfAudioObjects() const
		{
			return NumberOfAudioObjects;
		}

		void SetStreamLanguageTag(const BCP47::FLanguageTag& InStreamLanguageTag)
		{
			StreamLanguageTag = InStreamLanguageTag;
		}

		void SetCodecSpecificData(const TArray<uint8>& InCSD)
		{
			CSD = InCSD;
		}

		const TArray<uint8>& GetCodecSpecificData() const
		{
			return CSD;
		}

		void SetDecoderConfigRecord(const TArray<uint8>& InDCR)
		{
			DCR = InDCR;
		}

		const TArray<uint8>& GetDecoderConfigRecord() const
		{
			return DCR;
		}

		void SetBitrate(int32 InBitrate)
		{
			Bitrate = InBitrate;
		}

		int32 GetBitrate() const
		{
			return Bitrate;
		}

		FParamDict& GetExtras()
		{
			return Extras;
		}

		const FParamDict& GetExtras() const
		{
			return Extras;
		}

		void Clear()
		{
			StreamType = EStreamType::Unsupported;
			CodecSpecifier.Empty();
			MimeType.Empty();
			Codec = ECodec::Unknown;
			Resolution.Clear();
			Crop.Clear();
			Translation.Clear();
			AspectRatio.Clear();
			FrameRate = FTimeFraction::GetInvalid();
			ProfileLevel.Clear();
			StreamLanguageTag.Empty();
			Bitrate = 0;
			SampleRate = 0;
			NumChannels = 0;
			ChannelConfiguration = 0;
			AudioDecodingComplexity = 0;
			AudioAccessibility = 0;
			NumberOfAudioObjects = 0;
			Codec4CC = 0;
			Extras.Clear();
			CSD.Empty();
			DCR.Empty();
			ResetHumanReadableCodecName();
		}

		bool Equals(const FStreamCodecInformation& Other) const
		{
			if (StreamType == Other.StreamType && Codec == Other.Codec && CodecSpecifier.Equals(Other.CodecSpecifier) && StreamLanguageTag.Get(true, true, true, false, false, false).Equals(Other.StreamLanguageTag.Get(true, true, true, false, false, false)))
			{
				if (StreamType == EStreamType::Video)
				{
					return Resolution == Other.Resolution &&
						   Crop == Other.Crop &&
						   AspectRatio == Other.AspectRatio &&
						   FrameRate == Other.FrameRate &&
						   ProfileLevel == Other.ProfileLevel;
				}
				else if (StreamType == EStreamType::Audio)
				{
					return SampleRate == Other.SampleRate &&
						   NumChannels == Other.NumChannels &&
						   ChannelConfiguration == Other.ChannelConfiguration &&
						   AudioDecodingComplexity == Other.AudioDecodingComplexity &&
						   AudioAccessibility == Other.AudioAccessibility &&
						   NumberOfAudioObjects == Other.NumberOfAudioObjects;
				}
				else if (StreamType == EStreamType::Subtitle)
				{
					return Resolution == Other.Resolution &&
						   Translation == Other.Translation &&
						   FrameRate == Other.FrameRate;
				}
			}
			return false;
		}

	private:
		void ResetHumanReadableCodecName()
		{
			HumanReadableCodecName.Empty();
		}
		bool TryConstructHumanReadableCodecName() const;

		struct FProfileLevel
		{
			FProfileLevel()
			{
				Clear();
			}
			void Clear()
			{
				ProfileSpace = 0;
				Profile = 0;
				Level = 0;
				CompatibilityFlags = 0;
				Constraints = 0;
				Tier = 0;
			}
			int32		ProfileSpace;
			int32		Profile;
			int32		Level;
			uint32		CompatibilityFlags;
			uint64		Constraints;
			int32		Tier;

			bool operator == (const FProfileLevel& rhs) const
			{
				return ProfileSpace == rhs.ProfileSpace && Profile == rhs.Profile && Level == rhs.Level && Constraints == rhs.Constraints && Tier == rhs.Tier;
			}

			bool operator != (const FProfileLevel& rhs) const
			{
				return !(*this == rhs);
			}
		};

		EStreamType		StreamType;
		FString			CodecSpecifier;				//!< Codec specifier as per RFC 6381
		FString			MimeType;					//!< Explicitly set mime type if it cannot be inferred.
		ECodec			Codec;
		FResolution		Resolution;					//!< Resolution, if this is a video stream
		FCrop			Crop;						//!< Cropping, if this is a video stream
		FTranslation	Translation;				//!< Top-left corner offset for subtitles
		FAspectRatio	AspectRatio;				//!< Aspect ratio, if this is a video stream
		FTimeFraction	FrameRate;					//!< Frame rate, if this is a video stream
		FProfileLevel	ProfileLevel;
		BCP47::FLanguageTag StreamLanguageTag;
		int32			Bitrate;
		int32			SampleRate;					//!< Decoded sample rate, if this is an audio stream.
		int32			NumChannels;				//!< Number of decoded channels, if this is an audio stream.
		uint32			ChannelConfiguration;		//!< Format specific audio channel configuration
		int32			AudioDecodingComplexity;	//!< Format specific audio decoding complexity
		int32			AudioAccessibility;			//!< Format specific audio accessibility
		int32			NumberOfAudioObjects;		//!< Format specific number of audio objects
		uint32			Codec4CC;
		FParamDict		Extras;						//!< Additional details/properties, depending on playlist.
		TArray<uint8>	CSD;						//!< Codec specific data, if available.
		TArray<uint8>	DCR;						//!< Decoder configuration record, if available.
		mutable FString	HumanReadableCodecName;
	};


	namespace StreamCodecInformationOptions
	{
		static const FName DecoderConfigurationRecord(TEXT("dcr"));
		static const FName VPccBox(TEXT("vpcC_box"));
		static const FName AV1cBox(TEXT("av1C_box"));
		static const FName ColrBox(TEXT("colr_box"));
		static const FName DOpsBox(TEXT("dOps_box"));
		static const FName ChanBox(TEXT("chan_box"));
		static const FName SamplesPerBlock(TEXT("samples_per_block"));
		static const FName FormatSpecificFlags(TEXT("FormatSpecificFlags"));
		static const FName SampleSize(TEXT("SampleSize"));
		static const FName ConstBytesPerAudioPacket(TEXT("ConstBytesPerAudioPacket"));
		static const FName ConstLPCMFramesPerAudioPacket(TEXT("ConstLPCMFramesPerAudioPacket"));
		static const FName PresentationTimeOffset(TEXT("PresentationTimeOffset"));
	}



	/**
	 * Metadata of an elementary stream as specified by the playlist/manifest.
	 * This is only as correct as the information in the manifest. If any part is not listed there
	 * the information in here will be incomplete.
	 */
	struct FStreamMetadata
	{
		FStreamCodecInformation		CodecInformation;					//!< Stream codec information
		FString						ID;									//!< ID of this stream
		int32						Bandwidth;							//!< Bandwidth required for this stream in bits per second
		int32						QualityIndex = 0;
		bool Equals(const FStreamMetadata& Other) const
		{
			return ID == Other.ID && Bandwidth == Other.Bandwidth && QualityIndex == Other.QualityIndex && CodecInformation.Equals(Other.CodecInformation);
		}
	};

	/**
	 * Metadata per track type.
	 *
	 * See: https://dev.w3.org/html5/html-sourcing-inband-tracks/
	 *
	 * While this may not represent every possible "role" the presentation format may offer this is a
	 * representation that covers most of the use cases and can be applied to a variety of formats.
	 */
	struct FTrackMetadata
	{
		BCP47::FLanguageTag LanguageTagRFC5646;
		FString ID;
		FString Kind;
		FString Label;
		TOptional<FTimespan> Duration;

		TArray<FStreamMetadata> StreamDetails;
		FStreamCodecInformation HighestBandwidthCodec;
		int32 HighestBandwidth = 0;
		bool Equals(const FTrackMetadata& Other) const
		{
			bool bEquals = ID.Equals(Other.ID) && Kind.Equals(Other.Kind) && LanguageTagRFC5646.Get(true, true, true, false, false, false).Equals(Other.LanguageTagRFC5646.Get(true, true, true, false, false, false));
			if (bEquals)
			{
				if (StreamDetails.Num() == Other.StreamDetails.Num())
				{
					for(int32 i=0; i<StreamDetails.Num(); ++i)
					{
						if (!StreamDetails[i].Equals(Other.StreamDetails[i]))
						{
							return false;
						}
					}
					return true;
				}
			}
			return false;
		}
	};


	/**
	 * Stream selection attributes.
	 * See FTrackMetadata comments.
	 */
	struct FStreamSelectionAttributes
	{
		// Used for video, audio and subtitles or captions. This is a comma separated string of RFC-4647 language ranges
		// to match the RFC-5646 language tag provided by the stream metadata.
		TOptional<FString> Language_RFC4647;

		// Primarily used for audio selection. Should typically be set to "main" or left unset.
		TOptional<FString> Kind;

		// Preferred codec. Typically set to ensure the same track remains selected after a seek in case the same content
		// is provided with different formats. See GetCodecName()
		TOptional<FString> Codec;

		// Rarely used. Unconditionally selects a track by its index where the index is a sequential numbering from [0..n)
		// of the tracks as they are found. If the index is invalid the selection rules for kind and language are applied.
		TOptional<int32> OverrideIndex;

		bool IsSet() const
		{
			return Kind.IsSet() || Language_RFC4647.IsSet() || Codec.IsSet() || OverrideIndex.IsSet();
		}

		bool IsCompatibleWith(const FStreamSelectionAttributes& Other)
		{
			if (OverrideIndex.IsSet() && Other.OverrideIndex.IsSet() && OverrideIndex.GetValue() >= 0 && Other.OverrideIndex.GetValue() >= 0 && OverrideIndex.GetValue() != Other.OverrideIndex.GetValue())
			{
				return false;
			}
			FString Kind1 = Kind.Get(FString());
			FString Kind2 = Other.Kind.Get(FString());

			FString Lang1 = Language_RFC4647.Get(FString());
			FString Lang2 = Other.Language_RFC4647.Get(FString());

			FString Codec1 = Codec.Get(FString());
			FString Codec2 = Other.Codec.Get(FString());

			if (Kind1.IsEmpty() || Kind2.IsEmpty() || Kind1.Equals(Kind2))
			{
				return Lang1.Equals(Lang2) && Codec1.Equals(Codec2);
			}
			return false;
		}

		void Reset()
		{
			Kind.Reset();
			Language_RFC4647.Reset();
			Codec.Reset();
			OverrideIndex.Reset();
		}
		void UpdateWith(const FString& InKind, const BCP47::FLanguageTag& InLanguageTag, const FString& InCodec, int32 InOverrideIndex)
		{
			Reset();
			if (!InKind.IsEmpty())
			{
				Kind = InKind;
			}
			FString Lang5646 = InLanguageTag.Get();
			if (!Lang5646.IsEmpty())
			{
				Language_RFC4647 = MoveTemp(Lang5646);
			}
			else
			{
				Language_RFC4647.Reset();
			}
			if (!InCodec.IsEmpty())
			{
				Codec = InCodec;
			}
			if (InOverrideIndex >= 0)
			{
				OverrideIndex = InOverrideIndex;
			}
		}
		void UpdateIfOverrideSet(const FString& InKind, const BCP47::FLanguageTag& InLanguageTag, const FString& InCodec)
		{
			if (OverrideIndex.IsSet())
			{
				ClearOverrideIndex();
				Kind = InKind;
				Language_RFC4647 = InLanguageTag.Get();
				Codec = InCodec;
			}
		}
		void ClearOverrideIndex()
		{
			OverrideIndex.Reset();
		}
	};



	class FCodecSelectionPriorities
	{
	public:
		/**
		 * Initializes this selector with a priority string in the following FORMAT:
		 *   FORMAT = CLASSPRIO 0*[COMMA CLASSPRIO]
		 *   COMMA = ,
		 *   EQ = =
		 *   PRIO = 1*DIGIT
		 *   PREFIX = 1*VCHAR    ; except , = { }
		 *   CLASS = PREFIX
		 *   CODECPRIO = CLASS EQ PRIO
		 *   CLASSWITHPRIO = CODECPRIO 0*[ { CODECPRIO 0*[ COMMA CODECPRIO ] } ]
		 *   CLASSWITHOUTPRIO = CLASS 1*[ { CODECPRIO 0*[ COMMA CODECPRIO ] } ]
		 *   CLASSPRIO = CLASSWITHPRIO / CLASSWITHOUTPRIO
		 *
		 * Examples: hvc=2,hev=2,avc=1
		 *           mp4a{mp4a.40.5=0,mp4a.40.2=1}
		 *
		 * First codec priorities are given for an entire codec class (eg. "hvc").
		 * Within each class, where it makes sense, individual streams can be prioritized.
		 * Say within a class "mp4a" there are two AAC streams. One LC and one HE.
		 * To use the LC over the HE stream the "mp4a" class gives more detailed codec
		 * prefixes and their priorities like the above example.
		 *
		 * If used with DASH streams the class priority can be thought of the priority
		 * of an AdaptationSet and the stream priority of that of a Representation.
		 * User defined priorities override the @selectionPriority attribute of a
		 * DASH AdaptationSet or Representation.
		 */
		bool Initialize(const FString& ConfigurationString);
		int32 GetClassPriority(const FString& CodecSpecifierRFC6381) const;
		int32 GetStreamPriority(const FString& CodecSpecifierRFC6381) const;
	private:
		bool ParseInternal(const FString& ConfigurationString);
		struct FStreamPriority
		{
			FString Prefix;
			int32 Priority = -1;
		};
		struct FClassPriority
		{
			FString Prefix;
			int32 Priority = -1;
			TArray<FStreamPriority> StreamPriorities;
		};
		TArray<FClassPriority> ClassPriorities;
	};



	struct FPlayerSequenceState
	{
		FPlayerSequenceState()
		{
			Reset();
		}
		void Reset()
		{
			PrimaryIndex = 0;
			SecondaryIndex = 0;
		}
		int64 GetSequenceIndex() const
		{
			return (int64(PrimaryIndex) << 32) + int64(SecondaryIndex);
		}
		void SetSequenceIndex(int64 SequenceIndex)
		{
			check(SequenceIndex >= 0);	// should be the case in Electra at all times
			PrimaryIndex = int32(SequenceIndex >> 32);
			SecondaryIndex = int32(SequenceIndex);
		}

		int32	PrimaryIndex;
		int32	SecondaryIndex;
	};



} // namespace Electra


