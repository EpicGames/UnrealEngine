// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerCore.h"
#include "SynchronizedClock.h"


namespace Electra
{

	class FSynchronizedUTCTime : public ISynchronizedUTCTime
	{
	public:
		FSynchronizedUTCTime();
		virtual ~FSynchronizedUTCTime();

		virtual void SetTime(const FTimeValue& TimeNow) override;
		virtual void SetTime(const FTimeValue& LocalTime, const FTimeValue& UTCTime) override;
		virtual FTimeValue GetTime() override;
		virtual FTimeValue MapToSyncTime(const FTimeValue& InTimeToMap) override;

	private:
		FCriticalSection Lock;
		FTimeValue BaseUTCTime;
		int64 BaseSystemTime;
	};


	ISynchronizedUTCTime* ISynchronizedUTCTime::Create()
	{
		return new FSynchronizedUTCTime;
	}



	FSynchronizedUTCTime::FSynchronizedUTCTime()
	{
		BaseSystemTime = MEDIAutcTime::CurrentMSec();
	}

	FSynchronizedUTCTime::~FSynchronizedUTCTime()
	{
		Lock.Lock();
		Lock.Unlock();
	}

	void FSynchronizedUTCTime::SetTime(const FTimeValue& TimeNow)
	{
		int64 NowSystem = MEDIAutcTime::CurrentMSec();
		Lock.Lock();
		BaseUTCTime = TimeNow;
		BaseSystemTime = NowSystem;
		Lock.Unlock();
	}

	void FSynchronizedUTCTime::SetTime(const FTimeValue& LocalTime, const FTimeValue& UTCTime)
	{
		Lock.Lock();
		BaseUTCTime = UTCTime;
		BaseSystemTime = LocalTime.GetAsMilliseconds();
		Lock.Unlock();
	}

	FTimeValue FSynchronizedUTCTime::GetTime()
	{
		int64 NowSystem = MEDIAutcTime::CurrentMSec();
		Lock.Lock();
		FTimeValue 	LastBaseUTC = BaseUTCTime;
		int64   	LastBaseSystem = BaseSystemTime;
		Lock.Unlock();
		return LastBaseUTC + FTimeValue(FTimeValue::MillisecondsToHNS(NowSystem - LastBaseSystem));
	}

	FTimeValue FSynchronizedUTCTime::MapToSyncTime(const FTimeValue& InTimeToMap)
	{
		Lock.Lock();
		FTimeValue 	LastBaseUTC = BaseUTCTime;
		int64   	LastBaseSystem = BaseSystemTime;
		Lock.Unlock();
		return LastBaseUTC + FTimeValue(FTimeValue::MillisecondsToHNS(InTimeToMap.GetAsMilliseconds() - LastBaseSystem));
	}


} // namespace Electra
