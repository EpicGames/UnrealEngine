// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"
#include "PlayerTime.h"

namespace Electra
{

	/**
	 * This implements a UTC clock synchronized to an external time source.
	 */
	class ISynchronizedUTCTime
	{
	public:
		static ISynchronizedUTCTime* Create();
		virtual ~ISynchronizedUTCTime() = default;

		/**
		 * Establishes a relationship between the current local system time and actual UTC time.
		 */
		virtual void SetTime(const FTimeValue& TimeNow) = 0;

		virtual void SetTime(const FTimeValue& LocalTime, const FTimeValue& UTCTime) = 0;

		/**
		 * Gets the current system time as synchronized UTC time.
		 */
		virtual FTimeValue GetTime() = 0;

		/**
		 * Returns the synchronized UTC time for a given time.
		 */
		virtual FTimeValue MapToSyncTime(const FTimeValue& InTimeToMap) = 0;
	};


	/**
	 * System wallclock time (UTC)
	**/
	class MEDIAutcTime
	{
	public:
		static Electra::FTimeValue Current()
		{
			return Electra::FTimeValue(Electra::FTimeValue::MillisecondsToHNS(CurrentMSec()));
		}
		static int64 CurrentMSec();
	};

} // namespace Electra


