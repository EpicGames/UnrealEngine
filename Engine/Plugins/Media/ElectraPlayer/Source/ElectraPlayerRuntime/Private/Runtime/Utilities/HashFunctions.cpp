// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/HashFunctions.h"

namespace Electra
{
	namespace HashFunctions
	{


		namespace CRC
		{
			static const uint32 kCrc32Table[256] =
			{
				0x00000000, 0x77073096, 0xee0e612c, 0x990951ba, 0x076dc419, 0x706af48f, 0xe963a535, 0x9e6495a3,
				0x0edb8832, 0x79dcb8a4, 0xe0d5e91e, 0x97d2d988, 0x09b64c2b, 0x7eb17cbd, 0xe7b82d07, 0x90bf1d91,
				0x1db71064, 0x6ab020f2, 0xf3b97148, 0x84be41de, 0x1adad47d, 0x6ddde4eb, 0xf4d4b551, 0x83d385c7,
				0x136c9856, 0x646ba8c0, 0xfd62f97a, 0x8a65c9ec, 0x14015c4f, 0x63066cd9, 0xfa0f3d63, 0x8d080df5,
				0x3b6e20c8, 0x4c69105e, 0xd56041e4, 0xa2677172, 0x3c03e4d1, 0x4b04d447, 0xd20d85fd, 0xa50ab56b,
				0x35b5a8fa, 0x42b2986c, 0xdbbbc9d6, 0xacbcf940, 0x32d86ce3, 0x45df5c75, 0xdcd60dcf, 0xabd13d59,
				0x26d930ac, 0x51de003a, 0xc8d75180, 0xbfd06116, 0x21b4f4b5, 0x56b3c423, 0xcfba9599, 0xb8bda50f,
				0x2802b89e, 0x5f058808, 0xc60cd9b2, 0xb10be924, 0x2f6f7c87, 0x58684c11, 0xc1611dab, 0xb6662d3d,
				0x76dc4190, 0x01db7106, 0x98d220bc, 0xefd5102a, 0x71b18589, 0x06b6b51f, 0x9fbfe4a5, 0xe8b8d433,
				0x7807c9a2, 0x0f00f934, 0x9609a88e, 0xe10e9818, 0x7f6a0dbb, 0x086d3d2d, 0x91646c97, 0xe6635c01,
				0x6b6b51f4, 0x1c6c6162, 0x856530d8, 0xf262004e, 0x6c0695ed, 0x1b01a57b, 0x8208f4c1, 0xf50fc457,
				0x65b0d9c6, 0x12b7e950, 0x8bbeb8ea, 0xfcb9887c, 0x62dd1ddf, 0x15da2d49, 0x8cd37cf3, 0xfbd44c65,
				0x4db26158, 0x3ab551ce, 0xa3bc0074, 0xd4bb30e2, 0x4adfa541, 0x3dd895d7, 0xa4d1c46d, 0xd3d6f4fb,
				0x4369e96a, 0x346ed9fc, 0xad678846, 0xda60b8d0, 0x44042d73, 0x33031de5, 0xaa0a4c5f, 0xdd0d7cc9,
				0x5005713c, 0x270241aa, 0xbe0b1010, 0xc90c2086, 0x5768b525, 0x206f85b3, 0xb966d409, 0xce61e49f,
				0x5edef90e,	0x29d9c998,	0xb0d09822,	0xc7d7a8b4,	0x59b33d17,	0x2eb40d81,	0xb7bd5c3b,	0xc0ba6cad,
				0xedb88320,	0x9abfb3b6,	0x03b6e20c,	0x74b1d29a,	0xead54739,	0x9dd277af,	0x04db2615,	0x73dc1683,
				0xe3630b12,	0x94643b84,	0x0d6d6a3e,	0x7a6a5aa8,	0xe40ecf0b,	0x9309ff9d,	0x0a00ae27,	0x7d079eb1,
				0xf00f9344,	0x8708a3d2,	0x1e01f268,	0x6906c2fe,	0xf762575d,	0x806567cb,	0x196c3671,	0x6e6b06e7,
				0xfed41b76,	0x89d32be0,	0x10da7a5a,	0x67dd4acc,	0xf9b9df6f,	0x8ebeeff9,	0x17b7be43,	0x60b08ed5,
				0xd6d6a3e8,	0xa1d1937e,	0x38d8c2c4,	0x4fdff252,	0xd1bb67f1,	0xa6bc5767,	0x3fb506dd,	0x48b2364b,
				0xd80d2bda,	0xaf0a1b4c,	0x36034af6,	0x41047a60,	0xdf60efc3,	0xa867df55,	0x316e8eef,	0x4669be79,
				0xcb61b38c,	0xbc66831a,	0x256fd2a0,	0x5268e236,	0xcc0c7795,	0xbb0b4703,	0x220216b9,	0x5505262f,
				0xc5ba3bbe,	0xb2bd0b28,	0x2bb45a92,	0x5cb36a04,	0xc2d7ffa7,	0xb5d0cf31,	0x2cd99e8b,	0x5bdeae1d,
				0x9b64c2b0,	0xec63f226,	0x756aa39c,	0x026d930a,	0x9c0906a9,	0xeb0e363f,	0x72076785,	0x05005713,
				0x95bf4a82,	0xe2b87a14,	0x7bb12bae,	0x0cb61b38,	0x92d28e9b,	0xe5d5be0d,	0x7cdcefb7,	0x0bdbdf21,
				0x86d3d2d4,	0xf1d4e242,	0x68ddb3f8,	0x1fda836e,	0x81be16cd,	0xf6b9265b,	0x6fb077e1,	0x18b74777,
				0x88085ae6,	0xff0f6a70,	0x66063bca,	0x11010b5c,	0x8f659eff,	0xf862ae69,	0x616bffd3,	0x166ccf45,
				0xa00ae278,	0xd70dd2ee,	0x4e048354,	0x3903b3c2,	0xa7672661,	0xd06016f7,	0x4969474d,	0x3e6e77db,
				0xaed16a4a,	0xd9d65adc,	0x40df0b66,	0x37d83bf0,	0xa9bcae53,	0xdebb9ec5,	0x47b2cf7f,	0x30b5ffe9,
				0xbdbdf21c,	0xcabac28a,	0x53b39330,	0x24b4a3a6,	0xbad03605,	0xcdd70693,	0x54de5729,	0x23d967bf,
				0xb3667a2e,	0xc4614ab8,	0x5d681b02,	0x2a6f2b94,	0xb40bbe37,	0xc30c8ea1,	0x5a05df1b,	0x2d02ef8d,
				/*
				static void _makeCrc32Table(void)
				{
					for(uint32 n=0; n<256; ++n)
					{
						uint32 c = n;
						for(uint32 k=0; k<8; ++k)
						{
							if (c & 1)
								c = 0xedb88320L ^ (c >> 1);
							else
								c = c >> 1;
						}
						_crc32Table[n] = c;
					}
				}
				*/
			};

			static const uint64 kCrc64Table[256] =
			{
				0x0000000000000000ULL,	0xb32e4cbe03a75f6fULL,	0xf4843657a840a05bULL,	0x47aa7ae9abe7ff34ULL,	0x7bd0c384ff8f5e33ULL,	0xc8fe8f3afc28015cULL,	0x8f54f5d357cffe68ULL,	0x3c7ab96d5468a107ULL,
				0xf7a18709ff1ebc66ULL,	0x448fcbb7fcb9e309ULL,	0x0325b15e575e1c3dULL,	0xb00bfde054f94352ULL,	0x8c71448d0091e255ULL,	0x3f5f08330336bd3aULL,	0x78f572daa8d1420eULL,	0xcbdb3e64ab761d61ULL,
				0x7d9ba13851336649ULL,	0xceb5ed8652943926ULL,	0x891f976ff973c612ULL,	0x3a31dbd1fad4997dULL,	0x064b62bcaebc387aULL,	0xb5652e02ad1b6715ULL,	0xf2cf54eb06fc9821ULL,	0x41e11855055bc74eULL,
				0x8a3a2631ae2dda2fULL,	0x39146a8fad8a8540ULL,	0x7ebe1066066d7a74ULL,	0xcd905cd805ca251bULL,	0xf1eae5b551a2841cULL,	0x42c4a90b5205db73ULL,	0x056ed3e2f9e22447ULL,	0xb6409f5cfa457b28ULL,
				0xfb374270a266cc92ULL,	0x48190ecea1c193fdULL,	0x0fb374270a266cc9ULL,	0xbc9d3899098133a6ULL,	0x80e781f45de992a1ULL,	0x33c9cd4a5e4ecdceULL,	0x7463b7a3f5a932faULL,	0xc74dfb1df60e6d95ULL,
				0x0c96c5795d7870f4ULL,	0xbfb889c75edf2f9bULL,	0xf812f32ef538d0afULL,	0x4b3cbf90f69f8fc0ULL,	0x774606fda2f72ec7ULL,	0xc4684a43a15071a8ULL,	0x83c230aa0ab78e9cULL,	0x30ec7c140910d1f3ULL,
				0x86ace348f355aadbULL,	0x3582aff6f0f2f5b4ULL,	0x7228d51f5b150a80ULL,	0xc10699a158b255efULL,	0xfd7c20cc0cdaf4e8ULL,	0x4e526c720f7dab87ULL,	0x09f8169ba49a54b3ULL,	0xbad65a25a73d0bdcULL,
				0x710d64410c4b16bdULL,	0xc22328ff0fec49d2ULL,	0x85895216a40bb6e6ULL,	0x36a71ea8a7ace989ULL,	0x0adda7c5f3c4488eULL,	0xb9f3eb7bf06317e1ULL,	0xfe5991925b84e8d5ULL,	0x4d77dd2c5823b7baULL,
				0x64b62bcaebc387a1ULL,	0xd7986774e864d8ceULL,	0x90321d9d438327faULL,	0x231c512340247895ULL,	0x1f66e84e144cd992ULL,	0xac48a4f017eb86fdULL,	0xebe2de19bc0c79c9ULL,	0x58cc92a7bfab26a6ULL,
				0x9317acc314dd3bc7ULL,	0x2039e07d177a64a8ULL,	0x67939a94bc9d9b9cULL,	0xd4bdd62abf3ac4f3ULL,	0xe8c76f47eb5265f4ULL,	0x5be923f9e8f53a9bULL,	0x1c4359104312c5afULL,	0xaf6d15ae40b59ac0ULL,
				0x192d8af2baf0e1e8ULL,	0xaa03c64cb957be87ULL,	0xeda9bca512b041b3ULL,	0x5e87f01b11171edcULL,	0x62fd4976457fbfdbULL,	0xd1d305c846d8e0b4ULL,	0x96797f21ed3f1f80ULL,	0x2557339fee9840efULL,
				0xee8c0dfb45ee5d8eULL,	0x5da24145464902e1ULL,	0x1a083bacedaefdd5ULL,	0xa9267712ee09a2baULL,	0x955cce7fba6103bdULL,	0x267282c1b9c65cd2ULL,	0x61d8f8281221a3e6ULL,	0xd2f6b4961186fc89ULL,
				0x9f8169ba49a54b33ULL,	0x2caf25044a02145cULL,	0x6b055fede1e5eb68ULL,	0xd82b1353e242b407ULL,	0xe451aa3eb62a1500ULL,	0x577fe680b58d4a6fULL,	0x10d59c691e6ab55bULL,	0xa3fbd0d71dcdea34ULL,
				0x6820eeb3b6bbf755ULL,	0xdb0ea20db51ca83aULL,	0x9ca4d8e41efb570eULL,	0x2f8a945a1d5c0861ULL,	0x13f02d374934a966ULL,	0xa0de61894a93f609ULL,	0xe7741b60e174093dULL,	0x545a57dee2d35652ULL,
				0xe21ac88218962d7aULL,	0x5134843c1b317215ULL,	0x169efed5b0d68d21ULL,	0xa5b0b26bb371d24eULL,	0x99ca0b06e7197349ULL,	0x2ae447b8e4be2c26ULL,	0x6d4e3d514f59d312ULL,	0xde6071ef4cfe8c7dULL,
				0x15bb4f8be788911cULL,	0xa6950335e42fce73ULL,	0xe13f79dc4fc83147ULL,	0x521135624c6f6e28ULL,	0x6e6b8c0f1807cf2fULL,	0xdd45c0b11ba09040ULL,	0x9aefba58b0476f74ULL,	0x29c1f6e6b3e0301bULL,
				0xc96c5795d7870f42ULL,	0x7a421b2bd420502dULL,	0x3de861c27fc7af19ULL,	0x8ec62d7c7c60f076ULL,	0xb2bc941128085171ULL,	0x0192d8af2baf0e1eULL,	0x4638a2468048f12aULL,	0xf516eef883efae45ULL,
				0x3ecdd09c2899b324ULL,	0x8de39c222b3eec4bULL,	0xca49e6cb80d9137fULL,	0x7967aa75837e4c10ULL,	0x451d1318d716ed17ULL,	0xf6335fa6d4b1b278ULL,	0xb199254f7f564d4cULL,	0x02b769f17cf11223ULL,
				0xb4f7f6ad86b4690bULL,	0x07d9ba1385133664ULL,	0x4073c0fa2ef4c950ULL,	0xf35d8c442d53963fULL,	0xcf273529793b3738ULL,	0x7c0979977a9c6857ULL,	0x3ba3037ed17b9763ULL,	0x888d4fc0d2dcc80cULL,
				0x435671a479aad56dULL,	0xf0783d1a7a0d8a02ULL,	0xb7d247f3d1ea7536ULL,	0x04fc0b4dd24d2a59ULL,	0x3886b22086258b5eULL,	0x8ba8fe9e8582d431ULL,	0xcc0284772e652b05ULL,	0x7f2cc8c92dc2746aULL,
				0x325b15e575e1c3d0ULL,	0x8175595b76469cbfULL,	0xc6df23b2dda1638bULL,	0x75f16f0cde063ce4ULL,	0x498bd6618a6e9de3ULL,	0xfaa59adf89c9c28cULL,	0xbd0fe036222e3db8ULL,	0x0e21ac88218962d7ULL,
				0xc5fa92ec8aff7fb6ULL,	0x76d4de52895820d9ULL,	0x317ea4bb22bfdfedULL,	0x8250e80521188082ULL,	0xbe2a516875702185ULL,	0x0d041dd676d77eeaULL,	0x4aae673fdd3081deULL,	0xf9802b81de97deb1ULL,
				0x4fc0b4dd24d2a599ULL,	0xfceef8632775faf6ULL,	0xbb44828a8c9205c2ULL,	0x086ace348f355aadULL,	0x34107759db5dfbaaULL,	0x873e3be7d8faa4c5ULL,	0xc094410e731d5bf1ULL,	0x73ba0db070ba049eULL,
				0xb86133d4dbcc19ffULL,	0x0b4f7f6ad86b4690ULL,	0x4ce50583738cb9a4ULL,	0xffcb493d702be6cbULL,	0xc3b1f050244347ccULL,	0x709fbcee27e418a3ULL,	0x3735c6078c03e797ULL,	0x841b8ab98fa4b8f8ULL,
				0xadda7c5f3c4488e3ULL,	0x1ef430e13fe3d78cULL,	0x595e4a08940428b8ULL,	0xea7006b697a377d7ULL,	0xd60abfdbc3cbd6d0ULL,	0x6524f365c06c89bfULL,	0x228e898c6b8b768bULL,	0x91a0c532682c29e4ULL,
				0x5a7bfb56c35a3485ULL,	0xe955b7e8c0fd6beaULL,	0xaeffcd016b1a94deULL,	0x1dd181bf68bdcbb1ULL,	0x21ab38d23cd56ab6ULL,	0x9285746c3f7235d9ULL,	0xd52f0e859495caedULL,	0x6601423b97329582ULL,
				0xd041dd676d77eeaaULL,	0x636f91d96ed0b1c5ULL,	0x24c5eb30c5374ef1ULL,	0x97eba78ec690119eULL,	0xab911ee392f8b099ULL,	0x18bf525d915feff6ULL,	0x5f1528b43ab810c2ULL,	0xec3b640a391f4fadULL,
				0x27e05a6e926952ccULL,	0x94ce16d091ce0da3ULL,	0xd3646c393a29f297ULL,	0x604a2087398eadf8ULL,	0x5c3099ea6de60cffULL,	0xef1ed5546e415390ULL,	0xa8b4afbdc5a6aca4ULL,	0x1b9ae303c601f3cbULL,
				0x56ed3e2f9e224471ULL,	0xe5c372919d851b1eULL,	0xa26908783662e42aULL,	0x114744c635c5bb45ULL,	0x2d3dfdab61ad1a42ULL,	0x9e13b115620a452dULL,	0xd9b9cbfcc9edba19ULL,	0x6a978742ca4ae576ULL,
				0xa14cb926613cf817ULL,	0x1262f598629ba778ULL,	0x55c88f71c97c584cULL,	0xe6e6c3cfcadb0723ULL,	0xda9c7aa29eb3a624ULL,	0x69b2361c9d14f94bULL,	0x2e184cf536f3067fULL,	0x9d36004b35545910ULL,
				0x2b769f17cf112238ULL,	0x9858d3a9ccb67d57ULL,	0xdff2a94067518263ULL,	0x6cdce5fe64f6dd0cULL,	0x50a65c93309e7c0bULL,	0xe388102d33392364ULL,	0xa4226ac498dedc50ULL,	0x170c267a9b79833fULL,
				0xdcd7181e300f9e5eULL,	0x6ff954a033a8c131ULL,	0x28532e49984f3e05ULL,	0x9b7d62f79be8616aULL,	0xa707db9acf80c06dULL,	0x14299724cc279f02ULL,	0x5383edcd67c06036ULL,	0xe0ada17364673f59ULL,
				/*
				static void _makeCrc64Table(void)
				{
					for(uint32 n=0; n<256; ++n)
					{
						uint64 c = n;
						for (uint32 k=0; k<8; ++k)
						{
							if (c & 1)
								c = 0xC96C5795D7870F42ULL ^ (c >> 1); // ECMA-182, reflected form
							else
								c = c >> 1;
						}
						_crc64Table[n] = c;
					}
				}
				*/
			};


			/**
			 * Update a running CRC with the bytes Data[0 .. NumBytes-1].
			 *
			 * @param InitialCrc
			 * @param Data
			 * @param NumBytes
			 *
			 * @return 32 bit CRC
			 */
			uint32 Update32(uint32 InitialCrc, const void* Data, int64 NumBytes)
			{
				const uint8* Buf = reinterpret_cast<const uint8*>(Data);
				uint32	c = InitialCrc;
				for (uint32 n = 0; n < NumBytes; ++n)
				{
					c = kCrc32Table[(c ^ Buf[n]) & 0xff] ^ (c >> 8);
				}
				return c;
			}

			/**
			 * Return the CRC of the bytes Data[0 .. NumBytes-1].
			 *
			 * @param Data
			 * @param NumBytes
			 *
			 * @return 32 bit CRC
			 */
			uint32 Calc32(const void* Data, int64 NumBytes)
			{
				return ~Update32(uint32(~0U), Data, NumBytes);
			}



			/**
			 * Update a running CRC with the bytes Data[0 .. NumBytes-1].
			 *
			 * @param InitialCrc
			 * @param Data
			 * @param NumBytes
			 *
			 * @return 64 bit CRC
			 */
			uint64 Update64(uint64 InitialCrc, const void* Data, int64 NumBytes)
			{
				const uint8* Buf = reinterpret_cast<const uint8*>(Data);
				uint64	c = InitialCrc;
				for (uint32 n = 0; n < NumBytes; ++n)
				{
					c = kCrc64Table[(uint8(c) & 0xff) ^ Buf[n]] ^ (c >> 8);
				}
				return c;
			}

			/**
			 * Return the CRC of the bytes buf[0 .. NumBytes-1].
			 *
			 * @param Data
			 * @param NumBytes
			 *
			 * @return 64 bit CRC
			 */
			uint64 Calc64(const void* Data, int64 NumBytes)
			{
				return ~Update64(uint64(~(0LLU)), Data, NumBytes);
			}

		} // namespace CRC

	} // namespace HashFunctions
} // namespace Electra


