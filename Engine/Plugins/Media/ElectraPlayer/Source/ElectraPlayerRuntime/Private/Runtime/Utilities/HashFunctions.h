// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerCore.h"

namespace Electra
{
	namespace HashFunctions
	{

		namespace CRC
		{
			uint32 Update32(uint32 InitialCrc, const void* Data, int64 NumBytes);
			uint32 Calc32(const void* Data, int64 NumBytes);

			uint64 Update64(uint64 InitialCrc, const void* Data, int64 NumBytes);
			uint64 Calc64(const void* Data, int64 NumBytes);

		} // namespace CRC



	} // namespace HashFunctions
} // namespace Electra

