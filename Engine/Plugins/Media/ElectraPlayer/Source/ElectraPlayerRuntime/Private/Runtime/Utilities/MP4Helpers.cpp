// Copyright Epic Games, Inc. All Rights Reserved.

#include "MP4Helpers.h"
#include "MP4Utilities.h"
#include "Utilities/URLParser.h"
#include "Utilities/BCP47-Helpers.h"

namespace Electra
{

FMP4DataLoader::~FMP4DataLoader()
{
	Close();
}

void FMP4DataLoader::Open(const FString& InURL, const FHttpParams& InHttpParams)
{
	URL = InURL;
	// Check if this is a local file.
	if (URL.StartsWith(TEXT("file://")))
	{
		FString Filename;
		FURL_RFC3986::UrlDecode(Filename, InURL);
		Filename.RightChopInline(7);
		ConnectionInfo.bIsConnected = true;
		ConnectionInfo.bHaveResponseHeaders = true;
		ConnectionInfo.ContentType = TEXT("application/octet-stream");
		ConnectionInfo.EffectiveURL = InURL;
		ConnectionInfo.HTTPVersionReceived = 11;
		ConnectionInfo.bIsChunked = false;

		Archive = IFileManager::Get().CreateFileReader(*Filename, 0);
		if (Archive)
		{
			FileSize = Archive->TotalSize();
			ConnectionInfo.ContentLength = FileSize;
			ConnectionInfo.StatusInfo.HTTPStatus = 200;
			ConnectionInfo.ContentLengthHeader = FString::Printf(TEXT("Content-Length: %lld"), (long long int)FileSize);
		}
		else
		{
			bHasErrored = true;
			LastError = FString::Printf(TEXT("Failed to open file \"%s\""), *URL);
			ConnectionInfo.StatusInfo.HTTPStatus = 404;	// File not found
			ConnectionInfo.StatusInfo.ErrorDetail.SetMessage(FString::Printf(TEXT("HTTP returned status 404")));
			ConnectionInfo.StatusInfo.ErrorCode = HTTP::EStatusErrorCode::ERRCODE_HTTP_RETURNED_ERROR;
		}
	}
	else
	{
		HttpParams = InHttpParams;
		ProgressListener = MakeShared<IElectraHttpManager::FProgressListener, ESPMode::ThreadSafe>();
		ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateThreadSafeSP(AsShared(), &FMP4DataLoader::HTTPCompletionHandle);
	}
}

void FMP4DataLoader::Close()
{
	if (Archive)
	{
		Archive->Close();
		delete Archive;
		Archive = nullptr;
	}
	HttpParams.HTTPManager.Reset();
	HttpParams.HTTPResponseCache.Reset();
	ProgressListener.Reset();
}

void FMP4DataLoader::HTTPCompletionHandle(const IElectraHttpManager::FRequest* InRequest)
{
	bool bFailed = InRequest->ConnectionInfo.StatusInfo.ErrorDetail.IsError();
	ConnectionInfo = InRequest->ConnectionInfo;
	if (!bFailed)
	{
		// Set the size of the resource if we don't have it yet.
		if (FileSize < 0)
		{
			ElectraHTTPStream::FHttpRange crh;
			if (crh.ParseFromContentRangeResponse(InRequest->ConnectionInfo.ContentRangeHeader))
			{
				FileSize = crh.GetDocumentSize();
			}
		}
		if (ConnectionInfo.EffectiveURL.Len())
		{
			URL = ConnectionInfo.EffectiveURL;
		}
	}
	bHasErrored = bFailed;
	ReadCompleted.Signal();
}


int64 FMP4DataLoader::ReadData(void* InOutBuffer, int64 InNumBytes, int64 InFromOffset, FCancellationCheckDelegate InCheckCancellationDelegate)
{
	if (bHasErrored)
	{
		return EResult::ReadError;
	}
	if (InNumBytes <= 0)
	{
		return 0;
	}
	if (Archive)
	{
		if (CurrentOffset != InFromOffset)
		{
			CurrentOffset = InFromOffset >= 0 ? InFromOffset <= FileSize ? InFromOffset : FileSize : 0;
			Archive->Seek(CurrentOffset);
		}
		if (InNumBytes + CurrentOffset > FileSize)
		{
			InNumBytes = FileSize - CurrentOffset;
		}
		if (InOutBuffer)
		{
			Archive->Serialize(InOutBuffer, InNumBytes);
		}
		else
		{
			Archive->Seek(CurrentOffset + InNumBytes);
		}
		CurrentOffset += InNumBytes;
		return InNumBytes;
	}
	else
	{
		// Skip data?
		if (!InOutBuffer)
		{
			CurrentOffset = InFromOffset + InNumBytes;
			return InNumBytes;
		}

		ProgressListener->ProgressDelegate = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateLambda([&](const IElectraHttpManager::FRequest* InRequest)->int32
		{
			bWasAborted = InCheckCancellationDelegate.Execute();
			return bWasAborted ? 1 : 0;
		});


		constexpr int32 kSmallReadChunkSize = 4096;
		// Loop the read attempt in case the first request fails with a "416 - Range Not Satisfiable"
		bool bForceFirstReadAsSmall = false;
		while(1)
		{
			TSharedPtr<FWaitableBuffer, ESPMode::ThreadSafe> TargetBuffer = MakeShared<FWaitableBuffer, ESPMode::ThreadSafe>();
			bool bIsSmallRead = !bForceFirstReadAsSmall && InNumBytes < kSmallReadChunkSize;
			bool bReadViaTempBuffer = false;
			int64 NumBytesToRead = InNumBytes;
			if (bIsSmallRead)
			{
				// Did we read past what we have to read now from an earlier read?
				if (InFromOffset + InNumBytes <= CurrentEndOffsetIncluding)
				{
					TargetBuffer->SetExternalBuffer(static_cast<uint8*>(InOutBuffer), NumBytesToRead);
				}
				else
				{
					NumBytesToRead = kSmallReadChunkSize;
					TargetBuffer->Reserve(NumBytesToRead);
					bReadViaTempBuffer = true;
				}
			}
			else
			{
				TargetBuffer->SetExternalBuffer(static_cast<uint8*>(InOutBuffer), NumBytesToRead);
			}

			TSharedPtr<IElectraHttpManager::FRequest, ESPMode::ThreadSafe> Request = MakeShared<IElectraHttpManager::FRequest, ESPMode::ThreadSafe>();
			Request->Parameters.URL = URL;
			Request->Parameters.Range.SetStart(InFromOffset);
			int64 LastByte = InFromOffset + NumBytesToRead - 1;
			if (FileSize >= 0 && LastByte > FileSize-1)
			{
				LastByte = FileSize - 1;
			}
			CurrentEndOffsetIncluding = LastByte > CurrentEndOffsetIncluding ? LastByte : CurrentEndOffsetIncluding;
			Request->Parameters.Range.SetEndIncluding(LastByte);
			Request->Parameters.ConnectTimeout = FTimeValue().SetFromMilliseconds(HttpParams.ConnectTimeoutMillis);
			Request->Parameters.NoDataTimeout = FTimeValue().SetFromMilliseconds(HttpParams.NoDataTimeoutMillis);
			Request->Parameters.AcceptEncoding.Set(TEXT("identity"));
			Request->Parameters.RequestHeaders = HttpParams.RequestHeaders;
			Request->ProgressListener = ProgressListener;
			Request->ResponseCache = HttpParams.HTTPResponseCache;
			Request->ReceiveBuffer = TargetBuffer;

			HttpParams.HTTPManager->AddRequest(Request, false);
			ReadCompleted.WaitAndReset();
			HttpParams.HTTPManager->RemoveRequest(Request, false);
			LastError = Request->ConnectionInfo.StatusInfo.ErrorDetail.GetMessage();
			Request.Reset();

			// Check if this is the first attempt for which we may have requested additional data to be read
			// and the server returned an error because of that.
			if (bIsFirstRead && bHasErrored && ConnectionInfo.StatusInfo.HTTPStatus == 416)
			{
				bIsFirstRead = false;
				bForceFirstReadAsSmall = true;
				CurrentEndOffsetIncluding = -1;
				bHasErrored = false;
				ConnectionInfo = HTTP::FConnectionInfo();
				continue;
			}

			if (bHasErrored)
			{
				return EResult::ReadError;
			}
			else if (bWasAborted)
			{
				return EResult::Canceled;
			}

			if (TargetBuffer->Num() < InNumBytes)
			{
				LastError = FString::Printf(TEXT("Got only %lld bytes instead of %lld"), TargetBuffer->Num(), (long long int)InNumBytes);
				return EResult::ReadError;
			}
			if (bReadViaTempBuffer)
			{
				FMemory::Memcpy(InOutBuffer, TargetBuffer->GetLinearReadData(), InNumBytes);
			}
			break;
		}
		CurrentOffset = InFromOffset + InNumBytes;
		return InNumBytes;
	}
}

bool FMP4DataLoader::HasReachedEOF()
{
	bool bEOS = FileSize >= 0 && CurrentOffset >= FileSize;
	return bEOS;
}

} // namespace Electra
