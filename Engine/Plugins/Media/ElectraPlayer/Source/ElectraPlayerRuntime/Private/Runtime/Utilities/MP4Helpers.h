// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "HAL/FileManager.h"
#include "Serialization/Archive.h"
#include "HTTP/HTTPManager.h"
#include "Core/MediaEventSignal.h"
#include "MP4DataReader.h"

namespace Electra
{
	class FMP4DataLoader : public MP4Utilities::IMP4DataReaderBase, public TSharedFromThis<FMP4DataLoader, ESPMode::ThreadSafe>
	{
	public:
		static TSharedPtr<FMP4DataLoader, ESPMode::ThreadSafe> Create()
		{ return MakeShareable(new FMP4DataLoader); }
		virtual ~FMP4DataLoader();

		struct FHttpParams
		{
			TSharedPtr<IElectraHttpManager, ESPMode::ThreadSafe> HTTPManager;
			TSharedPtr<IHTTPResponseCache, ESPMode::ThreadSafe> HTTPResponseCache;
			TArray<HTTP::FHTTPHeader> RequestHeaders;
			int32 ConnectTimeoutMillis = 8000;
			int32 NoDataTimeoutMillis = 6000;
		};
		void Open(const FString& InURL, const FHttpParams& InHttpParams);
		void Close();

		//! Methods from MP4Utilities::IMP4DataReaderBase
		int64 ReadData(void* InOutBuffer, int64 InNumBytes, int64 InFromOffset, FCancellationCheckDelegate InCheckCancellationDelegate) override;
		int64 GetTotalFileSize() override
		{ return FileSize; }
		int64 GetCurrentFileOffset() override
		{ return CurrentOffset; }
		bool HasReachedEOF() override;
		FString GetLastError() override
		{ return LastError; }

		const HTTP::FConnectionInfo& GetConnectionInfo() const
		{ return ConnectionInfo; }

	private:
		FMP4DataLoader() = default;
		void HTTPCompletionHandle(const IElectraHttpManager::FRequest* InRequest);
		FHttpParams HttpParams;
		HTTP::FConnectionInfo ConnectionInfo;
		FString URL;
		FString LastError;
		FMediaEvent ReadCompleted;
		TSharedPtr<IElectraHttpManager::FProgressListener, ESPMode::ThreadSafe> ProgressListener;
		FArchive* Archive = nullptr;
		int64 FileSize = -1;
		int64 CurrentOffset = 0;
		int64 CurrentEndOffsetIncluding = -1;
		bool bIsFirstRead = true;
		volatile bool bHasErrored = false;
		volatile bool bWasAborted = false;
	};

} // namespace Electra
