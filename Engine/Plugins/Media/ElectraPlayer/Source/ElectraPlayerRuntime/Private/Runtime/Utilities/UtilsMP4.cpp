// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/UtilsMP4.h"
#include "Misc/Base64.h"
#include "MP4Utilities.h"
#include "HTTP/HTTPResponseCache.h"


namespace Electra
{

UtilsMP4::FMP4RootBoxLocator::~FMP4RootBoxLocator()
{
}

bool UtilsMP4::FMP4RootBoxLocator::LocateRootBoxes(TArray<FBoxInfo>& OutBoxInfos, const TSharedPtrTS<IElectraHttpManager>& InHTTPManager, const FString& InURL, const TArray<HTTP::FHTTPHeader>& InRequestHeaders, const TArray<uint32>& InFirstBoxes, const TArray<uint32>& InStopAfterBoxes, const TArray<uint32>& InReadDataOfBoxes, FCancellationCheckDelegate InCheckCancellationDelegate)
{
	FMediaEvent ReadCompleted;
	FString URL(InURL);
	volatile bool bAbort = false;
	FileSize = -1;
	bHasErrored = false;

	// Create a HTTP response cache to avoid unnecessary requests.
	TSharedPtrTS<IHTTPResponseCache> LocalCache = IHTTPResponseCache::Create(1024*1024, 32, nullptr);

	TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->ProgressDelegate = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateLambda([&](const IElectraHttpManager::FRequest* InRequest)->int32
	{
		bAbort = InCheckCancellationDelegate.Execute();
		return bAbort ? 1 : 0;
	});
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateLambda([&](const IElectraHttpManager::FRequest* InRequest)
	{
		bool bFailed = InRequest->ConnectionInfo.StatusInfo.ErrorDetail.IsError();
		ConnectionInfo = InRequest->ConnectionInfo;
		if (!bFailed)
		{
			// Set the size of the resource if we don't have it yet.
			if (FileSize < 0)
			{
				ElectraHTTPStream::FHttpRange crh;
				if (crh.ParseFromContentRangeResponse(InRequest->ConnectionInfo.ContentRangeHeader))
				{
					FileSize = crh.GetDocumentSize();
				}
			}

			if (ConnectionInfo.EffectiveURL.Len())
			{
				URL = ConnectionInfo.EffectiveURL;
			}
		}
		bHasErrored = bFailed;
		ReadCompleted.Signal();
	});


	auto CreateReadRequestAndBuffer = [&](TSharedPtrTS<FWaitableBuffer>& OutReceiveBuffer, int64 InFromOffset, int64 InNumBytes) -> TSharedPtrTS<IElectraHttpManager::FRequest>
	{
		TSharedPtrTS<IElectraHttpManager::FRequest> Req = MakeSharedTS<IElectraHttpManager::FRequest>();
		Req->Parameters.URL = URL;
		Req->Parameters.Range.SetStart(InFromOffset);
		int64 LastByte = InFromOffset + InNumBytes - 1;
		if (FileSize >= 0 && LastByte > FileSize-1)
		{
			LastByte = FileSize - 1;
		}
		Req->Parameters.Range.SetEndIncluding(LastByte);
		Req->Parameters.ConnectTimeout = FTimeValue().SetFromMilliseconds(1000 * 8);
		Req->Parameters.NoDataTimeout = FTimeValue().SetFromMilliseconds(1000 * 6);
		Req->Parameters.AcceptEncoding.Set(TEXT("identity"));
		Req->Parameters.RequestHeaders = InRequestHeaders;
		OutReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
		OutReceiveBuffer->Reserve(InNumBytes);
		Req->ReceiveBuffer = OutReceiveBuffer;
		Req->ProgressListener = ProgressListener;
		Req->ResponseCache = LocalCache;
		return Req;
	};

	const int32 MinRequiredReadSize = 32; // uint32(size) + uint32(type) + uint64(largesize) + uuid
	const int32 ChunkSize = 4096;
	int64 StartOffset = 0;
	int64 CurrentEndOffset = -1;
	bool bSuccess = false;
	bool bIsFirst = true;
	TSharedPtrTS<IElectraHttpManager::FRequest> Request;
	TSharedPtrTS<FWaitableBuffer> ReceiveBuffer;
	while(1)
	{
		int64 SizeToRead = CurrentEndOffset < StartOffset+MinRequiredReadSize ? ChunkSize : MinRequiredReadSize;
		Request = CreateReadRequestAndBuffer(ReceiveBuffer, StartOffset, SizeToRead);
		InHTTPManager->AddRequest(Request, false);
		ReadCompleted.WaitAndReset();
		if (bAbort || bHasErrored)
		{
			break;
		}
		TSharedPtrTS<FWaitableBuffer> DataBuffer(MoveTemp(ReceiveBuffer));
		InHTTPManager->RemoveRequest(Request, false);
		Request.Reset();

		if (DataBuffer.IsValid() && DataBuffer->Num() >= 8)
		{
			const int64 End = StartOffset + DataBuffer->Num();
			CurrentEndOffset = CurrentEndOffset < End ? End : CurrentEndOffset;

			FBoxInfo bi;
			const uint32* Data = reinterpret_cast<const uint32*>(DataBuffer->GetLinearReadData());
			bi.Size = (int64) MEDIA_FROM_BIG_ENDIAN(Data[0]);
			bi.Type = MEDIA_FROM_BIG_ENDIAN(Data[1]);
			bi.Offset = StartOffset;
			uint32 BoxInternalOffset = 8;
			if (bIsFirst)
			{
				// The way we read the file we now need to know its actual size.
				// An open ended chunked transfer will not work here.
				if (FileSize < 0)
				{
					ErrorMsg = TEXT("Invalid mp4 file: Unknown file size. Cannot parse the file.");
					break;
				}

				if (InFirstBoxes.Num() && !InFirstBoxes.Contains(bi.Type))
				{
					ErrorMsg = TEXT("Invalid mp4 file: First box is not of expected type");
					break;
				}
				bIsFirst = false;
			}

			// Check the box size value.
			if (bi.Size == 0)
			{
				// Zero size means "until the end of the file".
				bi.Size = FileSize - StartOffset;
			}
			else if (bi.Size == 1)
			{
				// A size of 1 indicates that the size is expressed as a 64 bit value following the box type.
				if (DataBuffer->Num() < 16)
				{
					ErrorMsg = TEXT("Invalid mp4 file: Box requiring 64 bit size value is truncated");
					break;
				}
				bi.Size = (int64) MEDIA_FROM_BIG_ENDIAN(reinterpret_cast<const uint64*>(Data)[1]);
				BoxInternalOffset += 8;
			}

			if (bi.Type == MP4Utilities::MakeBoxAtom('u','u','i','d'))
			{
				if (DataBuffer->Num() < BoxInternalOffset + 16)
				{
					ErrorMsg = TEXT("Invalid mp4 file: UUID box is truncated");
					break;
				}
				FMemory::Memcpy(bi.UUID, reinterpret_cast<const uint8*>(Data) + BoxInternalOffset, 16);
			}

			// Read this box?
			if (!bAbort && InReadDataOfBoxes.Contains(bi.Type))
			{
				Request = CreateReadRequestAndBuffer(bi.DataBuffer, bi.Offset, bi.Size);
				InHTTPManager->AddRequest(Request, false);
				ReadCompleted.WaitAndReset();
				if (bAbort || bHasErrored)
				{
					break;
				}
				InHTTPManager->RemoveRequest(Request, false);
				Request.Reset();
			}

			StartOffset += bi.Size;
			bool bStopNow = InStopAfterBoxes.Contains(bi.Type);
			OutBoxInfos.Emplace(MoveTemp(bi));

			// Done?
			if (bStopNow || StartOffset >= FileSize)
			{
				if (StartOffset > FileSize)
				{
					ErrorMsg = TEXT("Invalid mp4 file: File shorter than box sizes indicate");
					break;
				}
				else
				{
					bSuccess = true;
				}
				break;
			}
		}
		else
		{
			break;
		}
	}

	if (Request.IsValid())
	{
		InHTTPManager->RemoveRequest(Request, false);
		Request.Reset();
	}
	ProgressListener.Reset();
	ReceiveBuffer.Reset();

	return bSuccess;
}

TSharedPtrTS<FWaitableBuffer> UtilsMP4::FMP4ChunkLoader::LoadChunk(const int64 InOffset, const int64 InSize, const TSharedPtrTS<IElectraHttpManager>& InHTTPManager, const TSharedPtrTS<IHTTPResponseCache>& InHttpResponseCache, const FString& InURL, FCancellationCheckDelegate InCheckCancellationDelegate)
{
	FMediaEvent ReadCompleted;
	FString URL(InURL);
	volatile bool bAbort = false;
	FileSize = -1;
	bHasErrored = false;

	TSharedPtrTS<IElectraHttpManager::FProgressListener> ProgressListener = MakeSharedTS<IElectraHttpManager::FProgressListener>();
	ProgressListener->ProgressDelegate = IElectraHttpManager::FProgressListener::FProgressDelegate::CreateLambda([&](const IElectraHttpManager::FRequest* InRequest)->int32
	{
		bAbort = InCheckCancellationDelegate.Execute();
		return bAbort ? 1 : 0;
	});
	ProgressListener->CompletionDelegate = IElectraHttpManager::FProgressListener::FCompletionDelegate::CreateLambda([&](const IElectraHttpManager::FRequest* InRequest)
	{
		const bool bFailed = InRequest->ConnectionInfo.StatusInfo.ErrorDetail.IsError();
		ConnectionInfo = InRequest->ConnectionInfo;
		if (!bFailed)
		{
			// Set the size of the resource if we don't have it yet.
			if (FileSize < 0)
			{
				ElectraHTTPStream::FHttpRange crh;
				if (crh.ParseFromContentRangeResponse(InRequest->ConnectionInfo.ContentRangeHeader))
				{
					FileSize = crh.GetDocumentSize();
				}
			}

			if (ConnectionInfo.EffectiveURL.Len())
			{
				URL = ConnectionInfo.EffectiveURL;
			}
		}
		bHasErrored = bFailed;
		ReadCompleted.Signal();
	});

	auto CreateReadRequestAndBuffer = [&](TSharedPtrTS<FWaitableBuffer>& OutReceiveBuffer, int64 InFromOffset, int64 InNumBytes) -> TSharedPtrTS<IElectraHttpManager::FRequest>
	{
		TSharedPtrTS<IElectraHttpManager::FRequest> Req = MakeSharedTS<IElectraHttpManager::FRequest>();
		Req->Parameters.URL = URL;
		Req->Parameters.Range.SetStart(InFromOffset);
		int64 LastByte = InFromOffset + InNumBytes - 1;
		if (FileSize >= 0 && LastByte > FileSize-1)
		{
			LastByte = FileSize - 1;
		}
		Req->Parameters.Range.SetEndIncluding(LastByte);
		Req->Parameters.ConnectTimeout = FTimeValue().SetFromMilliseconds(1000 * 8);
		Req->Parameters.NoDataTimeout = FTimeValue().SetFromMilliseconds(1000 * 6);
		OutReceiveBuffer = MakeSharedTS<FWaitableBuffer>();
		OutReceiveBuffer->Reserve(InNumBytes);
		Req->ReceiveBuffer = OutReceiveBuffer;
		Req->ProgressListener = ProgressListener;
		Req->ResponseCache = InHttpResponseCache;
		return Req;
	};

	TSharedPtrTS<FWaitableBuffer> ReceiveBuffer;
	const TSharedPtrTS<IElectraHttpManager::FRequest> Request = CreateReadRequestAndBuffer(ReceiveBuffer, InOffset, InSize);
	InHTTPManager->AddRequest(Request, false);
	ReadCompleted.WaitAndReset();
	InHTTPManager->RemoveRequest(Request, false);

	return !(bAbort || bHasErrored) ? ReceiveBuffer : nullptr;
}

}
