// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include "HTTP/HTTPManager.h"

namespace Electra
{

class UtilsMP4
{
public:

	class FMP4RootBoxLocator
	{
	public:
		struct FBoxInfo
		{
			int64 Size = 0;
			int64 Offset = 0;
			uint32 Type = 0;
			uint8 UUID[16] {};
			TSharedPtrTS<FWaitableBuffer> DataBuffer;
		};
		FMP4RootBoxLocator() = default;
		~FMP4RootBoxLocator();
		DECLARE_DELEGATE_RetVal(bool, FCancellationCheckDelegate);
		bool LocateRootBoxes(TArray<FBoxInfo>& OutBoxInfos, const TSharedPtrTS<IElectraHttpManager>& InHTTPManager, const FString& InURL, const TArray<HTTP::FHTTPHeader>& InRequestHeaders, const TArray<uint32>& InFirstBoxes, const TArray<uint32>& InStopAfterBoxes, const TArray<uint32>& InReadDataOfBoxes, FCancellationCheckDelegate InCheckCancellationDelegate);

		int64 GetFileSize() const
		{ return FileSize; }
		bool DidDownloadFail() const
		{ return bHasErrored; }
		const HTTP::FConnectionInfo& GetConnectionInfo() const
		{ return ConnectionInfo; }

		const FString& GetErrorMessage() const
		{ return ErrorMsg; }
	private:
		HTTP::FConnectionInfo ConnectionInfo;
		FString ErrorMsg;
		int64 FileSize = -1;
		volatile bool bHasErrored = false;
	};

	class FMP4ChunkLoader
	{
	public:
		FMP4ChunkLoader() = default;
		~FMP4ChunkLoader() = default;

		DECLARE_DELEGATE_RetVal(bool, FCancellationCheckDelegate);
		TSharedPtrTS<FWaitableBuffer> LoadChunk(const int64 InOffset, const int64 InSize, const TSharedPtrTS<IElectraHttpManager>& InHTTPManager, const TSharedPtrTS<IHTTPResponseCache>& InHttpResponseCache, const FString& InURL, FCancellationCheckDelegate InCheckCancellationDelegate);

		int64 GetFileSize() const
		{ return FileSize; }
		bool DidDownloadFail() const
		{ return bHasErrored; }
		const HTTP::FConnectionInfo& GetConnectionInfo() const
		{ return ConnectionInfo; }

		const FString& GetErrorMessage() const
		{ return ErrorMsg; }
	private:
		HTTP::FConnectionInfo ConnectionInfo;
		FString ErrorMsg;
		int64 FileSize = -1;
		volatile bool bHasErrored = false;
	};
};

} // namespace Electra
