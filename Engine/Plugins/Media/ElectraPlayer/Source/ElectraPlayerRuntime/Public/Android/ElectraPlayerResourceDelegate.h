// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"
#include "Android/AndroidApplication.h"

class IElectraPlayerResourceDelegate
{
public:
	virtual ~IElectraPlayerResourceDelegate() = default;

	virtual jobject GetCodecSurface() = 0;
};
