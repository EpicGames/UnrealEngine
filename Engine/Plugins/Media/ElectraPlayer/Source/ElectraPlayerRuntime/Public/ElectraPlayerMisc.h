// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"

/** Log category for the ElectraPlayer module. */
ELECTRAPLAYERRUNTIME_API DECLARE_LOG_CATEGORY_EXTERN(LogElectraPlayer, Log, All);
ELECTRAPLAYERRUNTIME_API DECLARE_LOG_CATEGORY_EXTERN(LogElectraPlayerStats, Log, All);

/** CSV Profiler */
CSV_DECLARE_CATEGORY_MODULE_EXTERN(ELECTRAPLAYERRUNTIME_API, MediaStreaming);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(ELECTRAPLAYERRUNTIME_API, ElectraPlayer);

/** UE Profiler */
DECLARE_STATS_GROUP(TEXT("ElectraPlayer"), STATGROUP_ElectraPlayer, STATCAT_Advanced);
