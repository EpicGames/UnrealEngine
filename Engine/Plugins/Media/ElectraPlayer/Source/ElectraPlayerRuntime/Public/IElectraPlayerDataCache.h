// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IElectraPlayerDataCache : public TSharedFromThis<IElectraPlayerDataCache, ESPMode::ThreadSafe>
{
public:
	virtual ~IElectraPlayerDataCache() = default;

	struct FItemInfo
	{
		struct FRange
		{
			/** 
			 * Offset to the first byte in the resource.
			 */
			int64 Start = -1;
			
			/**
			 * Offset to the last byte, including.
			 * 
			 * A range of `Start` = 0, `EndIncluding` = 0 describes one byte at offset 0.
			 */
			int64 EndIncluding = -1;

			/** 
			 * The total size of the resource. This _may_ be set when adding to the cache
			 * but is not required as there are cases where the total resource size is unknown.
			 * This does not describe the number of bytes between `Start` and `EndIncluding`,
			 * but the total size of the resource of which the range is a part of.
			 */
			int64 TotalSize = -1;
		};

		enum class EStreamType
		{
			Video,
			Audio,
			Other
		};

		/**
		 * Unique resource identifier. Typically a URL.
		 */
		FString URI;

		/**
		 * Possible sub-range within the resource. Will be set when adding to the cache.
		 * When asking the cache for a hit the `TotalSize` member will not be set (-1) but is
		 * expected to be filled in when returning the cached element with the value that was
		 * provided when adding the entry to the cache.
		 */
		FRange Range;

		/**
		 * Type of elementary stream this request is for.
		 */
		EStreamType StreamType = EStreamType::Other;
		
		/**
		 * The quality index of this stream. Zero indicates the "worst" (or only) quality.
		 * If this is the highest quality that is available this value will be the same
		 * as `MaxQualityIndex`.
		 */
		int32 QualityIndex = 0;

		/**
		 * The maximum quality index of this stream. Zero indicates this is the only available
		 * quality level.
		 * This can be used to cache only the highest quality stream, or any other.
		 */
		int32 MaxQualityIndex = 0;
	};

	using FCacheDataPtr = TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe>;

	/**
	 * Asks to add an item to the cache.
	 * Whether or not the item gets added is under the purview of the cache implementation.
	 * As such, this method has no return value to indicate success.
	 * 
	 * This method should not block to wait until the cache has been updated.
	 * It should merely schedule an asynchronous update and defer writing the data to the storage
	 * to some asynchronous task.
	 */
	virtual void AddElementToCache(const FItemInfo& ItemInfoToCache, FCacheDataPtr DataToCache) = 0;

	enum class ECacheResult
	{
		// Element not available in the cache.
		Miss,
		// Element available in the cache.
		Hit
	};

	DECLARE_DELEGATE_ThreeParams(FCachedDataReadCompleted, ECacheResult /*Result*/, FItemInfo /*ItemInfoFromCache*/, FCacheDataPtr /*DataFromCache*/);

	/**
	 * Asks to get an item from the cache.
	 * The item to get is described by `InItemToGetFromCache`, with the exception of the `.Range.TotalSize` member
	 * which is not known ahead of time.
	 * If the cache fulfills the request the `OutCachedItemInfo` is expected to be set up, including the `.Range.TotalSize` member.
	 */
	virtual void GetElementFromCache(const FItemInfo& ItemToGetFromCache, FCachedDataReadCompleted CompletionDelegate) = 0;
};
