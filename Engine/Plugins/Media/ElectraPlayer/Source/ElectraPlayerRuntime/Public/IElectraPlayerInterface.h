// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Misc/Guid.h"
#include "IMediaPlayer.h"
#include "IMediaEventSink.h"
#include "IMediaOptions.h"
#include "IAnalyticsProviderET.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtrTemplates.h"
#include "UObject/WeakObjectPtr.h"


#define UE_API ELECTRAPLAYERRUNTIME_API

// ---------------------------------------------------------------------------------------------

DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FElectraPlayerSendAnalyticMetricsDelegate, const TSharedPtr<IAnalyticsProviderET>& /*InAnalyticsProvider*/, const FGuid& /*InPlayerGuid*/);
DECLARE_TS_MULTICAST_DELEGATE_OneParam(FElectraPlayerSendAnalyticMetricsPerMinuteDelegate, const TSharedPtr<IAnalyticsProviderET>& /*InAnalyticsProvider*/);
DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FElectraPlayerReportVideoStreamingErrorDelegate, const FGuid& /*InPlayerGuid*/, const FString& /*InLastError*/);
DECLARE_TS_MULTICAST_DELEGATE_FourParams(FElectraPlayerReportSubtitlesMetricsDelegate, const FGuid& /*InPlayerGuid*/, const FString& /*InURL*/, double /*InResponseTime*/, const FString& /*InLastError*/);


/**
 * This class is used to get safe access to an IMediaOptions interface.
 * When passing IMediaOptions through media framework as a POD pointer there is the risk
 * that it is actually from a derived class like UMediaSource which is subject to GC.
 * Thus it is not safe to keep the POD IMediaOptions around.
 * This class is intended to be derived and instantiated from and stored as a TSharedPtr
 * in a derived UMediaSource class.
 * Then, as the media player is opened and the POD IMediaOptions is passed along, a
 * GetMediaOption("GetSafeMediaOptions") query will be made on it to get this instance.
 * If future access to the IMediaOptions is needed it will be made through this class
 * by first locking, getting and using the IMediaOptions pointer returned here if it
 * not null and unlocking afterwards.
 */
class IElectraSafeMediaOptionInterface : public IMediaOptions::FDataContainer
{
public:
	virtual ~IElectraSafeMediaOptionInterface() = default;
	virtual void Lock() = 0;
	virtual void Unlock() = 0;
	virtual IMediaOptions* GetMediaOptionInterface() = 0;

	class FScopedLock
	{
	public:
		FScopedLock(const TSharedPtr<IElectraSafeMediaOptionInterface, ESPMode::ThreadSafe>& InSafeMediaOptionInterface)
			: SafeMediaOptionInterface(InSafeMediaOptionInterface)
		{
			if (SafeMediaOptionInterface.IsValid())
			{
				SafeMediaOptionInterface->Lock();
			}
		}
		~FScopedLock()
		{
			if (SafeMediaOptionInterface.IsValid())
			{
				SafeMediaOptionInterface->Unlock();
			}
		}

	private:
		TSharedPtr<IElectraSafeMediaOptionInterface, ESPMode::ThreadSafe> SafeMediaOptionInterface;
	};
};


/**
 * A ready to use implementation of the IElectraSafeMediaOptionInterface
 */
class FElectraSafeMediaOptionInterface : public IElectraSafeMediaOptionInterface
{
public:
	FElectraSafeMediaOptionInterface(IMediaOptions* InOwner)
		: Owner(InOwner)
		, OwnerObject(InOwner ? InOwner->ToUObject() : nullptr)
	{ }
	virtual ~FElectraSafeMediaOptionInterface()
	{
		ClearOwner();
	}
	void ClearOwner()
	{
		FScopeLock lock(&OwnerLock);
		Owner = nullptr;
		OwnerObject.Reset();
	}
	virtual void Lock() override
	{
		OwnerLock.Lock();
	}
	virtual void Unlock() override
	{
		OwnerLock.Unlock();
	}
	virtual IMediaOptions* GetMediaOptionInterface() override
	{
		return OwnerObject.IsStale(true, true) ? nullptr : Owner;
	}
private:
	FCriticalSection OwnerLock;
	IMediaOptions* Owner = nullptr;
	FWeakObjectPtr OwnerObject;
};



//! Data type for use with media options interface
class FElectraSeekablePositions : public IMediaOptions::FDataContainer
{
public:
	FElectraSeekablePositions(const TArray<FTimespan>& InData) : Data(InData) {}
	virtual ~FElectraSeekablePositions() = default;

	TArray<FTimespan> Data;
};


class FElectraPlayerRuntimeFactory
{
public:
	static UE_API TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& InEventSink,
																			FElectraPlayerSendAnalyticMetricsDelegate& InSendAnalyticMetricsDelegate,
																			FElectraPlayerSendAnalyticMetricsPerMinuteDelegate& InSendAnalyticMetricsPerMinuteDelegate,
																			FElectraPlayerReportVideoStreamingErrorDelegate& InReportVideoStreamingErrorDelegate,
																			FElectraPlayerReportSubtitlesMetricsDelegate& InReportSubtitlesFileMetricsDelegate);
};

// ---------------------------------------------------------------------------------------------

#undef UE_API
