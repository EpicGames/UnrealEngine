// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"

/**
 * Interface for the ElectraPlayerRuntime module.
 */
class IElectraPlayerRuntimeModule
	: public IModuleInterface
{
public:

	/**
	 * Is the ElectraPlayerRuntime module initialized?
	 * @return True if the module is initialized.
	 */
	virtual bool IsInitialized() const = 0;

	virtual ~IElectraPlayerRuntimeModule() = default;
};
