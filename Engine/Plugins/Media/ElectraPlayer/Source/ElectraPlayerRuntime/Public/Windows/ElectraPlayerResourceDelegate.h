// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/FunctionFwd.h"

struct ID3D12CommandQueue;

class IElectraPlayerResourceDelegate
{
public:
	virtual ~IElectraPlayerResourceDelegate() = default;

	virtual void ExecuteCodeWithCopyCommandQueueUsage(TFunction<void(ID3D12CommandQueue*)>&& CodeToRun) = 0;
};
