// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraProtron : ModuleRules
	{
		public ElectraProtron(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ElectraProtronFactory",
					"Core",
					"Engine",
					"MediaUtils",
					"RenderCore",
					"RHI",
					"ElectraBase",
                    "ElectraDecoders",
					"ElectraSamples",
                    "MP4Utilities",
                    "MP4Boxes"
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PrivateDependencyModuleNames.Add("D3D12RHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
			}
		}
	}
}
