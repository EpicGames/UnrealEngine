// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraProtronPrivate.h"

#include "Modules/ModuleManager.h"

#include "IElectraProtronModule.h"
#include "Player/ElectraProtronPlayer.h"

DEFINE_LOG_CATEGORY(LogElectraProtron);

class FElectraProtronModule
	: public IElectraProtronModule
{
public:
	//~ IElectraProtronModule interface
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		return MakeShared<FElectraProtronPlayer, ESPMode::ThreadSafe>(EventSink);
	}

	void StartupModule() override
	{
		// Load the modules we depend on. They may have been loaded already, but we do it explicitly here to ensure that
		// they will not be unloaded on shutdown before this module here.
		FModuleManager::Get().LoadModule(TEXT("ElectraBase"));
		FModuleManager::Get().LoadModule(TEXT("ElectraSamples"));
		FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));
	}
	void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FElectraProtronModule, ElectraProtron)
