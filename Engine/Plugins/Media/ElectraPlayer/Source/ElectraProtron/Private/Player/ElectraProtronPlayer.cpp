// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraProtronPlayer.h"
#include "ElectraProtronPlayerImpl.h"
#include "ElectraProtronPrivate.h"
#include "MediaSamples.h"
#include "MP4Boxes.h"
#include "ElectraTextureSample.h"
#include "IElectraAudioSample.h"



FElectraProtronPlayer::FElectraProtronPlayer(IMediaEventSink& InEventSink)
	: EventSink(InEventSink)
{
	CurrentState = EMediaState::Closed;
	CurrentStatus = EMediaStatus::None;
}

FElectraProtronPlayer::~FElectraProtronPlayer()
{
	Close();
}

void FElectraProtronPlayer::Close()
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> ToClose;
	Swap(CurrentPlayer, ToClose);
	CurrentState = EMediaState::Closed;
	CurrentStatus = EMediaStatus::None;
	CurrentURL.Empty();
	CurrentInternalState = EInternalState::Closed;
	if (ToClose.IsValid())
	{
		ToClose->Close(FImpl::FCompletionDelegate());
		ToClose.Reset();
		EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
	}
	CurrentTexturePool.Reset();
	CurrentAudioSamplePool.Reset();
}

IMediaCache& FElectraProtronPlayer::GetCache()
{
	return *this;
}

IMediaControls& FElectraProtronPlayer::GetControls()
{
	return *this;
}

FGuid FElectraProtronPlayer::GetPlayerPluginGUID() const
{
	// Same GUID as in the factory!
	static FGuid PlayerPluginGUID(0x2899727b, 0xfc934ccb, 0x94119db7, 0x185741d8);
	return PlayerPluginGUID;
}

IMediaSamples& FElectraProtronPlayer::GetSamples()
{
	return *this;
}

IMediaTracks& FElectraProtronPlayer::GetTracks()
{
	return *this;
}

FString FElectraProtronPlayer::GetUrl() const
{
	return CurrentURL;
}

IMediaView& FElectraProtronPlayer::GetView()
{
	return *this;
}

bool FElectraProtronPlayer::Open(const FString& InUrl, const IMediaOptions* InOptions)
{
	// We expect this player to be used with a FileMediaSource, so the URL needs to
	// start with "file://" and the remainder is a filename as-is, without escaped URL characters.
	if (!InUrl.StartsWith(TEXT("file://")))
	{
		LastError = FString::Printf(TEXT("File to open does not start with the file:// scheme"));
		UE_LOGF(LogElectraProtron, Error, "%ls", *LastError);
		return false;
	}

	if (CurrentPlayer.IsValid())
	{
		Close();
	}

	CurrentURL = InUrl;

	// Remove the "file://" prefix.
	FString Filename = CurrentURL.RightChop(7);

	// Create a new implementation player.
	CurrentPlayer = MakeShared<FImpl, ESPMode::ThreadSafe>();
	CurrentState = EMediaState::Preparing;
	CurrentStatus = EMediaStatus::Connecting;
	CurrentInternalState = EInternalState::Opening;
	CurrentSequenceIndex = 0;

	CurrentTexturePool = MakeShareable(new FElectraTextureSamplePool);
	CurrentAudioSamplePool = MakeShareable(new FElectraAudioSamplePool);

	EventSink.ReceiveMediaEvent(EMediaEvent::MediaConnecting);
	FImpl::FOpenParam op;
	op.Filename = Filename;
	op.TexturePool = CurrentTexturePool;
	op.AudioSamplePool = CurrentAudioSamplePool;
	op.InitialPlaybackRange = CurrentPlaybackRange;
	CurrentPlayer->Open(op, FImpl::FCompletionDelegate::CreateSPLambda(AsShared(), [this](FImpl::ImplPointer InImpl)
	{
		if (InImpl == CurrentPlayer)
		{
			// Change the state to opened, whether successful or not, and let
			// the media thread do the state transition.
			CurrentInternalState = EInternalState::Opened;
		}
	}));
	return true;
}

bool FElectraProtronPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options)
{
	return false;
}

bool FElectraProtronPlayer::Open(const FString& InUrl, const IMediaOptions* InOptions, const FMediaPlayerOptions* InPlayerOptions)
{
	// Not currently supported, use player-option-less Open() instead.
	return Open(InUrl, InOptions);
}

FVariant FElectraProtronPlayer::GetMediaInfo(FName InInfoName) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetMediaInfo(InInfoName) : FVariant();
}


bool FElectraProtronPlayer::GetPlayerFeatureFlag(EFeatureFlag InWhichFeature) const
{
	switch(InWhichFeature)
	{
		case EFeatureFlag::AllowShutdownOnClose:
		case EFeatureFlag::UsePlaybackTimingV2:
		case EFeatureFlag::PlayerUsesInternalFlushOnSeek:
		case EFeatureFlag::IsTrackSwitchSeamless:
		case EFeatureFlag::PlayerSelectsDefaultTracks:
		{
			return true;
		}
		default:
		{
			return IMediaPlayer::GetPlayerFeatureFlag(InWhichFeature);
		}
	}

}


void FElectraProtronPlayer::CheckForStateChanges()
{
	// Check for open completion on the media thread.
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	if (CurrentInternalState == EInternalState::Opened)
	{
		// Did the player fail to open?
		LastError = Player->GetLastError();
		if (LastError.IsEmpty())
		{
			CurrentInternalState = EInternalState::Ready;
			CurrentState = EMediaState::Stopped;
			CurrentStatus = EMediaStatus::None;
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
			EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
		}
		else
		{
			CurrentInternalState = EInternalState::Failed;
			CurrentState = EMediaState::Error;
			CurrentStatus = EMediaStatus::None;
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
			UE_LOGF(LogElectraProtron, Error, "%ls", *LastError);
		}
	}

	// Check for state changes when the player is ready.
	if (CurrentInternalState == EInternalState::Ready)
	{
		// Got an error?
		if (LastError.IsEmpty())
		{
			LastError = Player->GetLastError();
			if (!LastError.IsEmpty())
			{
				CurrentInternalState = EInternalState::Failed;
				CurrentState = EMediaState::Error;
				CurrentStatus = EMediaStatus::None;
				EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
				UE_LOGF(LogElectraProtron, Error, "%ls", *LastError);
				return;
			}
		}

		// Rate changes?
		float CurrentRate = Player->GetRate();
		if (CurrentRate == 0.0f && (CurrentState == EMediaState::Playing || CurrentState == EMediaState::Stopped))
		{
			const EMediaState PreviousState = CurrentState;
			CurrentState = EMediaState::Paused;
			if (PreviousState == EMediaState::Playing)
			{
				EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
			}
		}
		else if (CurrentRate != 0.0f && (CurrentState == EMediaState::Paused || CurrentState == EMediaState::Stopped))
		{
			CurrentState = EMediaState::Playing;
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
		}

		// Reached end of playback?
		if (CurrentState == EMediaState::Playing)
		{
			if (Player->HasReachedEnd())
			{
				Player->SetRate(0.0f);
				CurrentState = EMediaState::Paused;
				EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);
				EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
			}
		}
	}
}


void FElectraProtronPlayer::TickFetch(FTimespan InDeltaTime, FTimespan InTimecode)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	if (Player.IsValid())
	{
		Player->TickFetch(InDeltaTime, InTimecode);
	}
}

void FElectraProtronPlayer::TickInput(FTimespan InDeltaTime, FTimespan InTimecode)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	if (Player.IsValid())
	{
		Player->TickInput(InDeltaTime, InTimecode);
	}
	CheckForStateChanges();
}

FString FElectraProtronPlayer::GetInfo() const
{
	return FString();
}

FString FElectraProtronPlayer::GetStats() const
{
	return FString();
}

bool FElectraProtronPlayer::CanControl(EMediaControl InControl) const
{
	if (InControl == EMediaControl::BlockOnFetch)
	{
		return CurrentState == EMediaState::Playing || CurrentState == EMediaState::Paused;
	}
	else if (InControl == EMediaControl::Pause)
	{
		return CurrentState == EMediaState::Playing;
	}
	else if (InControl == EMediaControl::Resume)
	{
		return CurrentState == EMediaState::Paused || CurrentState == EMediaState::Stopped;
	}
	else if (InControl == EMediaControl::Seek || InControl == EMediaControl::Scrub)
	{
		return CurrentState == EMediaState::Playing || CurrentState == EMediaState::Paused || CurrentState == EMediaState::Stopped;
	}
	else if (InControl == EMediaControl::PlaybackRange)
	{
		return true;
	}
	return false;
}

FTimespan FElectraProtronPlayer::GetDuration() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetDuration() : FTimespan();
}

float FElectraProtronPlayer::GetRate() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetRate() : 0.0f;
}

EMediaState FElectraProtronPlayer::GetState() const
{
	return CurrentState;
}

EMediaStatus FElectraProtronPlayer::GetStatus() const
{
	return CurrentStatus;
}

TRangeSet<float> FElectraProtronPlayer::GetSupportedRates(EMediaRateThinning InThinning) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetSupportedRates(InThinning) : TRangeSet<float>();
}

FTimespan FElectraProtronPlayer::GetTime() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetTime() : FTimespan();
}

bool FElectraProtronPlayer::IsLooping() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->IsLooping() : false;
}

bool FElectraProtronPlayer::Seek(const FTimespan& InTime, const FMediaSeekParams& InAdditionalParams)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	if (!Player.IsValid())
	{
		return false;
	}
	check(InAdditionalParams.NewSequenceIndex.IsSet());
	CurrentSequenceIndex = InAdditionalParams.NewSequenceIndex.GetValue();
	TOptional<int32> NewLoopIndex = 0;
	Player->Seek(InTime, CurrentSequenceIndex, NewLoopIndex);
	// Send a seek complete event even if that is not really true and the seek is in progress or
	// has not even started. This is mostly to satisfy code that cannot handle async seeking or
	// seeks that override an ongoing seek (so the number of completed seeks will not match the
	// number of issued seeks).
	EventSink.ReceiveMediaEvent(EMediaEvent::SeekCompleted);
	return true;
}

bool FElectraProtronPlayer::SetLooping(bool bInLooping)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->SetLooping(bInLooping) : false;
}

bool FElectraProtronPlayer::SetRate(float InRate)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->SetRate(InRate) : false;
}

TRange<FTimespan> FElectraProtronPlayer::GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetPlaybackTimeRange(InRangeToGet) : (InRangeToGet == EMediaTimeRangeType::Current && CurrentPlaybackRange.IsSet()) ? CurrentPlaybackRange.GetValue() :TRange<FTimespan>(FTimespan(0), GetDuration());
}

bool FElectraProtronPlayer::SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	CurrentPlaybackRange = InTimeRange;
	return Player.IsValid() ? Player->SetPlaybackTimeRange(InTimeRange) : false;
}

int32 FElectraProtronPlayer::GetNumTracks(EMediaTrackType InTrackType) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetNumTracks(InTrackType) : 0;
}

int32 FElectraProtronPlayer::GetNumTrackFormats(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetNumTrackFormats(InTrackType, InTrackIndex) : 0;
}

int32 FElectraProtronPlayer::GetSelectedTrack(EMediaTrackType InTrackType) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetSelectedTrack(InTrackType) : -1;
}

FText FElectraProtronPlayer::GetTrackDisplayName(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetTrackDisplayName(InTrackType, InTrackIndex) : FText();
}

int32 FElectraProtronPlayer::GetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetTrackFormat(InTrackType, InTrackIndex) : 0;
}

FString FElectraProtronPlayer::GetTrackLanguage(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetTrackLanguage(InTrackType, InTrackIndex) : FString();
}

FString FElectraProtronPlayer::GetTrackName(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetTrackName(InTrackType, InTrackIndex) : FString();
}

bool FElectraProtronPlayer::GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetVideoTrackFormat(InTrackIndex, InFormatIndex, OutFormat) : false;
}

bool FElectraProtronPlayer::GetAudioTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetAudioTrackFormat(InTrackIndex, InFormatIndex, OutFormat) : false;
}

bool FElectraProtronPlayer::SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->SelectTrack(InTrackType, InTrackIndex) : false;
}

bool FElectraProtronPlayer::SetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex, int32 InFormatIndex)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->SetTrackFormat(InTrackType, InTrackIndex, InFormatIndex) : false;
}

TOptional<FTimespan> FElectraProtronPlayer::GetTrackDuration(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetTrackDuration(InTrackType, InTrackIndex) : TOptional<FTimespan>();
}

bool FElectraProtronPlayer::QueryCacheState(EMediaCacheState InState, TRangeSet<FTimespan>& OutTimeRanges) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->QueryCacheState(InState, OutTimeRanges) : false;
}

int32 FElectraProtronPlayer::GetSampleCount(EMediaCacheState InState) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->GetSampleCount(InState) : 0;
}



IMediaSamples::EFetchBestSampleResult FElectraProtronPlayer::FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& InTimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bInReverse, bool bInConsistentResult)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->FetchBestVideoSampleForTimeRange(InTimeRange, OutSample, bInReverse, bInConsistentResult) : IMediaSamples::EFetchBestSampleResult::NoSample;
}
bool FElectraProtronPlayer::FetchAudio(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->FetchAudio(InTimeRange, OutSample) : false;
}
bool FElectraProtronPlayer::FetchCaption(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->FetchCaption(InTimeRange, OutSample) : false;
}
bool FElectraProtronPlayer::FetchMetadata(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->FetchMetadata(InTimeRange, OutSample) : false;
}
bool FElectraProtronPlayer::FetchSubtitle(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->FetchSubtitle(InTimeRange, OutSample) : false;
}
void FElectraProtronPlayer::FlushSamples()
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	if (Player.IsValid())
	{
		Player->FlushSamples();
	}
}
void FElectraProtronPlayer::SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	if (Player.IsValid())
	{
		Player->SetMinExpectedNextSequenceIndex(InNextSequenceIndex);
	}
}
bool FElectraProtronPlayer::PeekVideoSampleTime(FMediaTimeStamp& OutTimeStamp)
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->PeekVideoSampleTime(OutTimeStamp) : false;
}
bool FElectraProtronPlayer::CanReceiveVideoSamples(uint32 Num) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->CanReceiveVideoSamples(Num) : false;
}
bool FElectraProtronPlayer::CanReceiveAudioSamples(uint32 Num) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->CanReceiveAudioSamples(Num) : false;
}
bool FElectraProtronPlayer::CanReceiveSubtitleSamples(uint32 Num) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->CanReceiveSubtitleSamples(Num) : false;
}
bool FElectraProtronPlayer::CanReceiveCaptionSamples(uint32 Num) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->CanReceiveCaptionSamples(Num) : false;
}
bool FElectraProtronPlayer::CanReceiveMetadataSamples(uint32 Num) const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->CanReceiveMetadataSamples(Num) : false;
}
int32 FElectraProtronPlayer::NumAudioSamples() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->NumAudioSamples() : 0;
}
int32 FElectraProtronPlayer::NumCaptionSamples() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->NumCaptionSamples() : 0;
}
int32 FElectraProtronPlayer::NumMetadataSamples() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->NumMetadataSamples() : 0;
}
int32 FElectraProtronPlayer::NumSubtitleSamples() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->NumSubtitleSamples() : 0;
}
int32 FElectraProtronPlayer::NumVideoSamples() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Player(GetCurrentPlayer());
	return Player.IsValid() ? Player->NumVideoSamples() : 0;
}
