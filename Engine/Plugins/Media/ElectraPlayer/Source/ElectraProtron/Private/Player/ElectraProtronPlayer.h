// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/Guid.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "IMediaTracks.h"
#include "IMediaView.h"
#include "IMediaSamples.h"
#include "IMediaEventSink.h"
#include "Misc/Timespan.h"
#include "Misc/Timecode.h"
#include "Templates/SharedPointer.h"
#include "ElectraTextureSample.h"


class FMediaSamples;
class FElectraAudioSamplePool;
class IMediaEventSink;

class FElectraProtronPlayer : public TSharedFromThis<FElectraProtronPlayer, ESPMode::ThreadSafe>
	, public IMediaPlayer
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaTracks
	, protected IMediaView
	, protected IMediaSamples
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 */
	FElectraProtronPlayer(IMediaEventSink& InEventSink);

	/** Virtual destructor. */
	virtual ~FElectraProtronPlayer();

public:

	//~ IMediaPlayer interface

	void Close() override;
	IMediaCache& GetCache() override;
	IMediaControls& GetControls() override;
	FString GetInfo() const override;
	FGuid GetPlayerPluginGUID() const override;
	IMediaSamples& GetSamples() override;
	FString GetStats() const override;
	IMediaTracks& GetTracks() override;
	FString GetUrl() const override;
	IMediaView& GetView() override;
	bool Open(const FString& InUrl, const IMediaOptions* InOptions) override;
	bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive, const FString& InOriginalUrl, const IMediaOptions* InOptions) override;
	bool Open(const FString& InUrl, const IMediaOptions* InOptions, const FMediaPlayerOptions* InPlayerOptions) override;
	FVariant GetMediaInfo(FName InInfoName) const override;
	bool GetPlayerFeatureFlag(EFeatureFlag InWhichFeature) const override;
	void TickFetch(FTimespan InDeltaTime, FTimespan InTimecode) override;
	void TickInput(FTimespan InDeltaTime, FTimespan InTimecode) override;

protected:
	//~ IMediaControls interface
	bool CanControl(EMediaControl InControl) const override;
	FTimespan GetDuration() const override;
	float GetRate() const override;
	EMediaState GetState() const override;
	EMediaStatus GetStatus() const override;
	TRangeSet<float> GetSupportedRates(EMediaRateThinning InThinning) const override;
	FTimespan GetTime() const override;
	bool IsLooping() const override;
	bool Seek(const FTimespan& Time) override
	{ check(!"You have to call the override with additional options!"); return false; }
	bool Seek(const FTimespan& InNewTime, const FMediaSeekParams& InAdditionalParams) override;
	bool SetLooping(bool bInLooping) override;
	bool SetRate(float InRate) override;
	TRange<FTimespan> GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet) const override;
	bool SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange) override;


	//~ IMediaTracks interface
	bool GetAudioTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	int32 GetNumTracks(EMediaTrackType InTrackType) const override;
	int32 GetNumTrackFormats(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	int32 GetSelectedTrack(EMediaTrackType InTrackType) const override;
	FText GetTrackDisplayName(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	int32 GetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	FString GetTrackLanguage(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	FString GetTrackName(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	bool GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	bool SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex) override;
	bool SetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex, int32 InFormatIndex) override;
	TOptional<FTimespan> GetTrackDuration(EMediaTrackType InTrackType, int32 InTrackIndex) const override;

	//~ IMediaCache interface
	bool QueryCacheState(EMediaCacheState InState, TRangeSet<FTimespan>& OutTimeRanges) const override;
	int32 GetSampleCount(EMediaCacheState InState) const override;

	//~ IMediaSamples interface
	EFetchBestSampleResult FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& InTimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bInReverse, bool bInConsistentResult) override;
	bool FetchAudio(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample) override;
	bool FetchCaption(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	bool FetchMetadata(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample) override;
	bool FetchSubtitle(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	void FlushSamples() override;
	void SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex) override;
	bool PeekVideoSampleTime(FMediaTimeStamp& OutTimeStamp) override;
	bool CanReceiveVideoSamples(uint32 InNum) const override;
	bool CanReceiveAudioSamples(uint32 InNum) const override;
	bool CanReceiveSubtitleSamples(uint32 InNum) const override;
	bool CanReceiveCaptionSamples(uint32 InNum) const override;
	bool CanReceiveMetadataSamples(uint32 InNum) const override;
	int32 NumAudioSamples() const override;
	int32 NumCaptionSamples() const override;
	int32 NumMetadataSamples() const override;
	int32 NumSubtitleSamples() const override;
	int32 NumVideoSamples() const override;

private:
	class FImpl;

	TSharedPtr<FImpl, ESPMode::ThreadSafe> GetCurrentPlayer() const
	{
		return CurrentPlayer;
	}

	enum class EInternalState
	{
		Closed,
		Opening,
		Opened,
		Ready,
		Failed
	};
	void CheckForStateChanges();


	TSharedPtr<FImpl, ESPMode::ThreadSafe> CurrentPlayer;
	TOptional<TRange<FTimespan>> CurrentPlaybackRange;
	int32 CurrentSequenceIndex = 0;
	EInternalState CurrentInternalState = EInternalState::Closed;

	IMediaEventSink& EventSink;

	FString LastError;
	FString CurrentURL;

	TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> CurrentTexturePool;
	TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> CurrentAudioSamplePool;

	/** Current player state. */
	EMediaState CurrentState;
	EMediaStatus CurrentStatus;
};
