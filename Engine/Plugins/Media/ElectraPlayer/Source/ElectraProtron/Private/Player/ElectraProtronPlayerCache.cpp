// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraProtronPlayerCache.h"

FProtronVideoCache::FProtronVideoCache()
{
}

void FProtronVideoCache::SetMaxFramesToCache(int32 InNumVideoFramesToCacheAhead, int32 InNumVideoFramesToCacheBehind)
{
	NumVideoFramesToCacheAhead = InNumVideoFramesToCacheAhead;
	NumVideoFramesToCacheBehind = InNumVideoFramesToCacheBehind;
	MaxFramesToCache = NumVideoFramesToCacheAhead + NumVideoFramesToCacheBehind;
	Empty();
}

void FProtronVideoCache::SetPlaybackRange(TRange<FTimespan> InRange)
{
	FScopeLock lock(&Lock);
	PlaybackRange = MoveTemp(InRange);
	PlaybackRangeEndInclusive = PlaybackRange.GetUpperBoundValue() - FTimespan(1);
	while(Entries.Num() && !Entries[0].RawRange.Overlaps(PlaybackRange))
	{
		Entries.RemoveAt(0);
	}
	TRange<FTimespan> TempInclusiveRange(PlaybackRange.GetLowerBoundValue(), PlaybackRange.GetUpperBoundValue() + FTimespan(1));
	while(Entries.Num() && !Entries.Last().RawRange.Overlaps(TempInclusiveRange))
	{
		Entries.Pop();
	}
}

void FProtronVideoCache::SeekIssuedTo(FTimespan InNewPosition)
{
	FScopeLock lock(&Lock);
	CurrentReadTime.Reset();

	// Check if we have an entry for this time.
	int32 Idx = INDEX_NONE;
	for(int32 i=0,iMax=Entries.Num(); i<iMax; ++i)
	{
		if (Entries[i].RawRange.Contains(InNewPosition))
		{
			Idx = i;
			break;
		}
	}
	if (Idx != INDEX_NONE)
	{
		PerformMaintenance(Idx, true, false);
	}
	else
	{
		Entries.Empty();
	}
}

void FProtronVideoCache::SetPlaybackRate(float InNewRate)
{
	FScopeLock lock(&Lock);
	PlaybackRate = InNewRate;
}

int32 FProtronVideoCache::Find(FTimespan InPTS)
{
	FScopeLock lock(&Lock);
	for(int32 i=0,iMax=Entries.Num(); i<iMax; ++i)
	{
		if (Entries[i].RawRange.GetLowerBoundValue() == InPTS)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

bool FProtronVideoCache::Contains(FTimespan InPTS)
{
	return Find(InPTS) != INDEX_NONE;
}

bool FProtronVideoCache::CanAccept(FTimespan InPTS)
{
	FScopeLock lock(&Lock);

	// If we are starting at a new time we can accept the sample.
	if (!CurrentReadTime.IsSet())
	{
		return true;
	}

	// If we already have that time then we can accept the sample again.
	if (Contains(InPTS))
	{
		return true;
	}

	// When we are full we can't accept anything.
	if (Entries.Num() >= MaxFramesToCache)
	{
		return false;
	}


	int32 CurrentReadTimeIndex = Find(CurrentReadTime.GetValue());
	check(CurrentReadTimeIndex != INDEX_NONE);

	// Playing forwards?
	TArray<int32> DummyIndices;
	int32 DummyWrapIndex;
	if (PlaybackRate >= 0.0f)
	{
		int32 HaveFuture = GetConsecutiveFutureSamples(DummyIndices, DummyWrapIndex, CurrentReadTimeIndex, true);
		return HaveFuture < NumVideoFramesToCacheAhead;
	}
	else
	{
		int32 HavePrevious = GetConsecutivePreviousSamples(DummyIndices, DummyWrapIndex, CurrentReadTimeIndex, true);
		return HavePrevious < NumVideoFramesToCacheAhead;
	}
}

void FProtronVideoCache::Empty()
{
	FScopeLock lock(&Lock);
	Entries.Empty();
	CurrentReadTime.Reset();
}

FProtronVideoCache::EGetResult FProtronVideoCache::GetFrame(TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutFrame, const TRange<FMediaTimeStamp>& InTimeRange, bool bIsLooping, bool bInReverse, bool bInUseFirstMatch)
{
	auto WrappedModulo = [](FTimespan InTime, FTimespan InDuration) -> FTimespan
	{
		return (InTime >= FTimespan::Zero()) ? (InTime % InDuration) : (InDuration + (InTime % InDuration));
	};


	/*
		If the time range to find a frame for wraps around for looping then we need to inspect multiple sub ranges.
		Here we do not unwrap more than one loop index since this makes no difference.
	*/
	TRange<FMediaTimeStamp> AdjustedTimeRange(InTimeRange);
	if (AdjustedTimeRange.GetLowerBoundValue().GetTime() < PlaybackRange.GetLowerBoundValue())
	{
		check(!"not handled");
	}
	// Note: This should probably have happened in the player facade GetCurrentPlaybackTimeRange() already, but it doesn't.
	//       All that happens there is checking if the range contains a wrap around, but not if the range as a whole needs to wrap!
	if (AdjustedTimeRange.GetLowerBoundValue().GetTime() > PlaybackRangeEndInclusive || AdjustedTimeRange.GetUpperBoundValue().GetTime() > PlaybackRangeEndInclusive)
	{
		check(!bInReverse);
		FTimespan s0 = WrappedModulo(AdjustedTimeRange.GetLowerBoundValue().GetTime(), PlaybackRange.GetUpperBoundValue());
		FTimespan s1 = WrappedModulo(AdjustedTimeRange.GetUpperBoundValue().GetTime(), PlaybackRange.GetUpperBoundValue());
		if (s0 < AdjustedTimeRange.GetLowerBoundValue().GetTime())
		{
			AdjustedTimeRange.SetLowerBoundValue(FMediaTimeStamp(s0, AdjustedTimeRange.GetLowerBoundValue().GetSequenceIndex(), AdjustedTimeRange.GetLowerBoundValue().GetLoopIndex() + 1));
		}
		if (s1 < AdjustedTimeRange.GetUpperBoundValue().GetTime())
		{
			AdjustedTimeRange.SetUpperBoundValue(FMediaTimeStamp(s1, AdjustedTimeRange.GetUpperBoundValue().GetSequenceIndex(), AdjustedTimeRange.GetUpperBoundValue().GetLoopIndex() + 1));
		}
	}

	TArray<TRange<FTimespan>> RangesToCheck;
	TArray<int32> LoopIndices;
	// Does the time range loop?
	if (AdjustedTimeRange.GetLowerBoundValue().GetLoopIndex() == AdjustedTimeRange.GetUpperBoundValue().GetLoopIndex())
	{
		// No. Add the range as it is to the list of ranges to check.
		RangesToCheck.Emplace(TRange<FTimespan>(AdjustedTimeRange.GetLowerBoundValue().GetTime(), AdjustedTimeRange.GetUpperBoundValue().GetTime()));
		LoopIndices.Emplace(AdjustedTimeRange.GetLowerBoundValue().GetLoopIndex());
	}
	// Looping forward?
	else if (AdjustedTimeRange.GetLowerBoundValue().GetLoopIndex() < AdjustedTimeRange.GetUpperBoundValue().GetLoopIndex())
	{
		RangesToCheck.Emplace(TRange<FTimespan>(AdjustedTimeRange.GetLowerBoundValue().GetTime(), PlaybackRange.GetUpperBoundValue()));
		RangesToCheck.Emplace(TRange<FTimespan>(FTimespan::Zero(), AdjustedTimeRange.GetUpperBoundValue().GetTime()));
		LoopIndices.Emplace(AdjustedTimeRange.GetLowerBoundValue().GetLoopIndex());
		LoopIndices.Emplace(AdjustedTimeRange.GetUpperBoundValue().GetLoopIndex());
	}
	// Looping backward.
	else
	{
		RangesToCheck.Emplace(TRange<FTimespan>(FTimespan::Zero(), AdjustedTimeRange.GetLowerBoundValue().GetTime()));
		RangesToCheck.Emplace(TRange<FTimespan>(AdjustedTimeRange.GetUpperBoundValue().GetTime(), PlaybackRange.GetUpperBoundValue()));
		LoopIndices.Emplace(AdjustedTimeRange.GetLowerBoundValue().GetLoopIndex());
		LoopIndices.Emplace(AdjustedTimeRange.GetUpperBoundValue().GetLoopIndex());
	}

	FScopeLock lock(&Lock);

	FTimespan OverlapDuration(FTimespan::Zero());
	int32 BestIndex = INDEX_NONE;
	TOptional<int32> LoopIndex;

	// Searching for forward play direction?
	if (!bInReverse)
	{
		// Do we want the first or last match?
		if (!bInUseFirstMatch)
		{
			// We want to find the *last* match with the largest overlap duration with the time range,
			// so we check the samples in reverse to find such a match fast.
			for(int32 nSubRangeIdx=RangesToCheck.Num()-1; nSubRangeIdx>=0; --nSubRangeIdx)
			{
				const TRange<FTimespan>& TimeRange(RangesToCheck[nSubRangeIdx]);
				for(int32 SmpIdx=Entries.Num()-1; SmpIdx>=0; --SmpIdx)
				{
					if (TimeRange.Overlaps(Entries[SmpIdx].RawRange))
					{
						FTimespan Overlap(TRange<FTimespan>::Intersection(Entries[SmpIdx].RawRange, TimeRange).Size<FTimespan>());
						// No candidate or a better overlap?
						if (BestIndex == INDEX_NONE || Overlap > OverlapDuration)
						{
							OverlapDuration = Overlap;
							BestIndex = SmpIdx;
							LoopIndex = LoopIndices[nSubRangeIdx];
						}
						else
						{
							break;
						}
					}
				}
			}
		}
		else
		{
			// We want to find the *first* match with the largest overlap duration.
			for(int32 nSubRangeIdx=0; nSubRangeIdx<RangesToCheck.Num(); ++nSubRangeIdx)
			{
				const TRange<FTimespan>& TimeRange(RangesToCheck[nSubRangeIdx]);
				for(int32 SmpIdx=0; SmpIdx<Entries.Num(); ++SmpIdx)
				{
					if (TimeRange.Overlaps(Entries[SmpIdx].RawRange))
					{
						FTimespan Overlap(TRange<FTimespan>::Intersection(Entries[SmpIdx].RawRange, TimeRange).Size<FTimespan>());
						// No candidate or a better overlap?
						if (BestIndex == INDEX_NONE || Overlap > OverlapDuration)
						{
							OverlapDuration = Overlap;
							BestIndex = SmpIdx;
							LoopIndex = LoopIndices[nSubRangeIdx];
						}
						else
						{
							break;
						}
					}
				}
			}
		}
	}
	// Searching for reverse play direction
	else
	{
		// Do we want the first or last match?
		if (!bInUseFirstMatch)
		{
			// We want to find the *last* match with the largest overlap duration with the time range
			// but in reverse direction, so we check the samples forwards to find such a match fast.
			for(int32 nSubRangeIdx=RangesToCheck.Num()-1; nSubRangeIdx>=0; --nSubRangeIdx)
			{
				const TRange<FTimespan>& TimeRange(RangesToCheck[nSubRangeIdx]);
				for(int32 SmpIdx=0; SmpIdx<Entries.Num(); ++SmpIdx)
				{
					if (TimeRange.Overlaps(Entries[SmpIdx].RawRange))
					{
						FTimespan Overlap(TRange<FTimespan>::Intersection(Entries[SmpIdx].RawRange, TimeRange).Size<FTimespan>());
						// No candidate or a better overlap?
						if (BestIndex == INDEX_NONE || Overlap > OverlapDuration)
						{
							OverlapDuration = Overlap;
							BestIndex = SmpIdx;
							LoopIndex = LoopIndices[nSubRangeIdx];
						}
						else
						{
							break;
						}
					}
				}
			}
		}
		else
		{
			// We want to find the *first* match with the largest overlap duration,
			// but in reverse direction.
			for(int32 nSubRangeIdx=0; nSubRangeIdx<RangesToCheck.Num(); ++nSubRangeIdx)
			{
				const TRange<FTimespan>& TimeRange(RangesToCheck[nSubRangeIdx]);
				for(int32 SmpIdx=Entries.Num()-1; SmpIdx>=0; --SmpIdx)
				{
					if (TimeRange.Overlaps(Entries[SmpIdx].RawRange))
					{
						FTimespan Overlap(TRange<FTimespan>::Intersection(Entries[SmpIdx].RawRange, TimeRange).Size<FTimespan>());
						// No candidate or a better overlap?
						if (BestIndex == INDEX_NONE || Overlap > OverlapDuration)
						{
							OverlapDuration = Overlap;
							BestIndex = SmpIdx;
							LoopIndex = LoopIndices[nSubRangeIdx];
						}
						else
						{
							break;
						}
					}
				}
			}
		}
	}


	// Found?
	if (BestIndex != INDEX_NONE)
	{
		OutFrame = Entries[BestIndex].Frame;
		// Set the sequence and loop index
		check(LoopIndex.IsSet());
		static_cast<FElectraTextureSample*>(OutFrame.Get())->SetTime(FMediaTimeStamp(OutFrame->GetTime().GetTime(), AdjustedTimeRange.GetLowerBoundValue().GetSequenceIndex(), LoopIndex.GetValue()));

		CurrentReadTime = Entries[BestIndex].RawRange.GetLowerBoundValue();

		PerformMaintenance(BestIndex, bIsLooping, bInReverse);
		return EGetResult::Hit;
	}
	// Not found.
	else
	{
		// If there are samples we need to dump all of them and return that fact.
		if (Entries.Num())
		{
			Empty();
			return EGetResult::PurgedEmpty;
		}
		return EGetResult::Miss;
	}
}


void FProtronVideoCache::PerformMaintenance(int32 InAtIndex, bool bIsLooping, bool bInReverse)
{
	TArray<int32> NextIndices, PrevIndices;
	int32 NextIndexWrap, PrevIndexWrap;
	GetConsecutiveFutureSamples(NextIndices, NextIndexWrap, InAtIndex, bIsLooping);
	GetConsecutivePreviousSamples(PrevIndices, PrevIndexWrap, InAtIndex, bIsLooping);

	// If the next indices contains the oldest then we have the entire video in the cache.
	// So either the video is really short or the cache massively large.
	// Either way, we do not need to perform any evicting of frames.
	if (NextIndices.Num() && PrevIndices.Num() && NextIndices.Contains(PrevIndices.Last()))
	{
		return;
	}

	// Perform cache maintenance.
	if (!bInReverse)
	{
		if (PrevIndices.Num() > NumVideoFramesToCacheBehind)
		{
			// Remove the indices of the samples we want to keep.
			PrevIndices.RemoveAt(0, NumVideoFramesToCacheBehind);
			// Check the indices we want to remove for being a non-wrapping range.
			if (PrevIndices[0] >= PrevIndices.Last())
			{
				Entries.RemoveAt(PrevIndices.Last(), PrevIndices[0] - PrevIndices.Last() + 1);
			}
			else
			{
				check(PrevIndexWrap != INDEX_NONE);
				PrevIndexWrap -= NumVideoFramesToCacheBehind;
				if (PrevIndexWrap < 0)
				{
					PrevIndexWrap = 0;
				}
				// First remove the range with the larger indices, which ensures the array doesn't shift down.
				Entries.RemoveAt(PrevIndices.Last(), PrevIndices[PrevIndexWrap] - PrevIndices.Last() + 1);
				// Then, if there is still a valid wrap index, remove the ones with the smaller indices which causes the
				// array to shift down.
				if (PrevIndexWrap > 0)
				{
					Entries.RemoveAt(PrevIndices[PrevIndexWrap - 1], PrevIndices[0] - PrevIndices[PrevIndexWrap - 1] + 1);
				}
			}
		}
	}
	else
	{
		if (NextIndices.Num() > NumVideoFramesToCacheBehind)
		{
			// Remove the indices of the samples we want to keep.
			NextIndices.RemoveAt(0, NumVideoFramesToCacheBehind);
			// Check the indices we want to remove for being a non-wrapping range.
			if (NextIndices[0] <= NextIndices.Last())
			{
				Entries.RemoveAt(NextIndices[0], NextIndices.Last() - NextIndices[0] + 1);
			}
			else
			{
				check(NextIndexWrap != INDEX_NONE);
				NextIndexWrap -= NumVideoFramesToCacheBehind;
				if (NextIndexWrap < 0)
				{
					NextIndexWrap = 0;
				}
				// First remove the range with the larger indices, like above. Here these are at the start of the list.
				if (NextIndexWrap > 0)
				{
					Entries.RemoveAt(NextIndices[0], NextIndices[NextIndexWrap - 1] - NextIndices[0] + 1);
				}
				Entries.RemoveAt(NextIndices[NextIndexWrap], NextIndices.Last() - NextIndices[NextIndexWrap] + 1);
			}
		}
	}
}


int32 FProtronVideoCache::GetConsecutiveFutureSamples(TArray<int32>& OutIndices, int32& OutWrapIndex, int32 InStartFrameIndex, bool bInLooping)
{
	TRange<FTimespan> SampleRange(Entries[InStartFrameIndex].RawRange);
	int32 curIdx = InStartFrameIndex;
	OutWrapIndex = INDEX_NONE;
	for(int32 i=1,iMax=Entries.Num(); i<iMax; ++i)
	{
		int32 nextIdx = (curIdx + 1) % iMax;
		TRange<FTimespan> NextSampleRange(Entries[nextIdx].RawRange);
		// Wrap around?
		if (nextIdx < curIdx)
		{
			// When not considering looping we are done.
			if (!bInLooping)
			{
				break;
			}
			// Are we wrapping around on the last sample of the range?
			// If this is not the last sample then we are not looping back to the beginning.
			if (!SampleRange.Contains(PlaybackRangeEndInclusive))
			{
				break;
			}

			// If the next sample range, which is the range of the lowest sample in the cache,
			// does not contain the start of the play range then this is not a contiguous loop.
			if (!NextSampleRange.Contains(PlaybackRange.GetLowerBoundValue()))
			{
				break;
			}
			else
			{
				OutWrapIndex = OutIndices.Num();
				OutIndices.Emplace(nextIdx);
			}
		}
		else
		{
			if (NextSampleRange.Adjoins(SampleRange))
			{
				OutIndices.Emplace(nextIdx);
			}
			else
			{
				break;
			}
		}
		curIdx = nextIdx;
		SampleRange = MoveTemp(NextSampleRange);
	}
	return OutIndices.Num();
}


int32 FProtronVideoCache::GetConsecutivePreviousSamples(TArray<int32>& OutIndices, int32& OutWrapIndex, int32 InStartFrameIndex, bool bInLooping)
{
	TRange<FTimespan> SampleRange(Entries[InStartFrameIndex].RawRange);
	int32 curIdx = InStartFrameIndex;
	OutWrapIndex = INDEX_NONE;
	for(int32 i=1,iMax=Entries.Num(); i<iMax; ++i)
	{
		int32 nextIdx = curIdx - 1;
		if (nextIdx < 0)
		{
			nextIdx += iMax;
		}
		TRange<FTimespan> PrevSampleRange(Entries[nextIdx].RawRange);
		// Wrap around?
		if (nextIdx > curIdx)
		{
			// When not considering looping we are done.
			if (!bInLooping)
			{
				break;
			}
			// Are we wrapping around on the first sample of the range?
			// If this is not the first sample then we are not looping back to the end.
			if (!SampleRange.Contains(PlaybackRange.GetLowerBoundValue()))
			{
				break;
			}

			// If the previous sample range, which is the range of the highest sample in the cache,
			// does not contain the end of the play range then this is not a contiguous loop.
			if (!PrevSampleRange.Contains(PlaybackRangeEndInclusive))
			{
				break;
			}
			else
			{
				OutWrapIndex = OutIndices.Num();
				OutIndices.Emplace(nextIdx);
			}
		}
		else
		{
			if (PrevSampleRange.Adjoins(SampleRange))
			{
				OutIndices.Emplace(nextIdx);
			}
			else
			{
				break;
			}
		}
		curIdx = nextIdx;
		SampleRange = MoveTemp(PrevSampleRange);
	}
	return OutIndices.Num();
}


void FProtronVideoCache::AddFrame(TSharedRef<IMediaTextureSample, ESPMode::ThreadSafe> InFrame, FTimespan InRawPTS, FTimespan InRawDuration)
{
	/*
		Note: We must not try to make room in the cache here.
			    If we do that then we basically allow the decoder to run freely from start to end
				as we would be throwing out frames that haven't been used yet to make room for the
				next frame to add.
	*/
	FScopeLock lock(&Lock);

	// If there is no known time of the sample value last returned in GetFrame(), which is the case at the
	// start or after a seek, then we set that time to the time of the sample being delivered now.
	// That will become the time around which we check which cache entries to keep and which to evict.
	if (!CurrentReadTime.IsSet())
	{
		CurrentReadTime = InRawPTS;
	}

	int32 Idx = Find(InRawPTS);
	if (Idx == INDEX_NONE)
	{
		FEntry ne;
		ne.RawRange = TRange<FTimespan>(InRawPTS, InRawPTS + InRawDuration);
		ne.Frame = MoveTemp(InFrame);
		Entries.Emplace(MoveTemp(ne));
		Entries.Sort([](const FEntry& a, const FEntry& b) { return a.RawRange.GetLowerBoundValue() < b.RawRange.GetLowerBoundValue(); });
	}
}


void FProtronVideoCache::QueryCacheState(TRangeSet<FTimespan>& OutTimeRanges)
{
	FScopeLock lock(&Lock);
	for(auto &e : Entries)
	{
		OutTimeRanges.Add(e.RawRange);
	}
}
