// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ElectraProtronPlayer.h"
#include "Containers/Queue.h"
#include "Misc/TVariant.h"
#include "Misc/Optional.h"
#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "MediaSamples.h"


class FProtronVideoCache
{
public:
	FProtronVideoCache();
	void SetMaxFramesToCache(int32 InNumVideoFramesToCacheAhead, int32 InNumVideoFramesToCacheBehind);
	void SetPlaybackRange(TRange<FTimespan> InRange);
	void SeekIssuedTo(FTimespan InNewPosition);
	void SetPlaybackRate(float InNewRate);

	bool Contains(FTimespan InPTS);
	int32 Find(FTimespan InPTS);
	bool CanAccept(FTimespan InPTS);
	void Empty();

	void QueryCacheState(TRangeSet<FTimespan>& OutTimeRanges);

	enum class EGetResult
	{
		Hit,
		Miss,
		PurgedEmpty
	};

	EGetResult GetFrame(TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutFrame, const TRange<FMediaTimeStamp>& InTimeRange, bool bIsLooping, bool bInReverse, bool bInUseFirstMatch);

	void PerformMaintenance(int32 InAtIndex, bool bIsLooping, bool bInReverse);

	void AddFrame(TSharedRef<IMediaTextureSample, ESPMode::ThreadSafe> InFrame, FTimespan InRawPTS, FTimespan InRawDuration);

private:
	FCriticalSection Lock;
	struct FEntry
	{
		TRange<FTimespan> RawRange;
		TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Frame;
	};

	int32 GetConsecutiveFutureSamples(TArray<int32>& OutIndices, int32& OutWrapIndex, int32 InStartFrameIndex, bool bInLooping);
	int32 GetConsecutivePreviousSamples(TArray<int32>& OutIndices, int32& OutWrapIndex, int32 InStartFrameIndex, bool bInLooping);



	TArray<FEntry> Entries;
	TRange<FTimespan> PlaybackRange;
	FTimespan PlaybackRangeEndInclusive;

	// The position around which to check for consecutive future and past samples when adding.
	TOptional<FTimespan> CurrentReadTime;

	float PlaybackRate = 0.0f;
	int32 MaxFramesToCache = 0;
	int32 NumVideoFramesToCacheAhead = 0;
	int32 NumVideoFramesToCacheBehind = 0;
};
