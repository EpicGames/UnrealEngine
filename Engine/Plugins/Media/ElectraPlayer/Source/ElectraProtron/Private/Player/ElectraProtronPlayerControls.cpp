// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraProtronPlayerImpl.h"
#include "ElectraProtronPrivate.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "CodecTypeFormat.h"


TRangeSet<float> FElectraProtronPlayer::FImpl::GetSupportedRates(EMediaRateThinning InThinning)
{
	// For now we do not handle thinned rates differently from unthinned rates.
	// If we have a selected audio track or a selected video track that does not have keyframes only
	// we do not support reverse playback.
	if (!bAreRatesValid)
	{
		bAreRatesValid = true;

		TRangeSet<float> TempRates;
		// Pause and 1x forward are always supported.
		TempRates.Add(TRange<float>(0.0f));
		TempRates.Add(TRange<float>(1.0f));

		// No audio?
		if (TrackSelection.SelectedTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)] == -1)
		{
			// Video selected?
			const auto TypeIdx = CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video);
			if (TrackSelection.SelectedTrackIndex[TypeIdx] != -1)
			{
				auto Track = Tracks[UsableTrackArrayIndicesByType[TypeIdx][TrackSelection.SelectedTrackIndex[TypeIdx]]];
				if (Track.IsValid() && Track->bIsKeyframeOnlyFormat)
				{
					TempRates.Add(TRange<float>::Inclusive(0.0f, 8.0f));
					TempRates.Add(TRange<float>::Inclusive(-8.0f, 0.0f));
				}
			}
		}

		ThinnedRates = TempRates;
		UnthinnedRates = TempRates;
	}
	return InThinning == EMediaRateThinning::Unthinned ? UnthinnedRates : ThinnedRates;
}

void FElectraProtronPlayer::FImpl::HandleActiveTrackChanges()
{
	if (TrackSelection.bChanged)
	{
		auto ChangeTrack = [&](int32 InCodecType, FDecoderThread& InDecoderThread, FLoaderThread& InLoaderThread)
		{
			if (TrackSelection.SelectedTrackIndex[InCodecType] != TrackSelection.ActiveTrackIndex[InCodecType])
			{
				if (TrackSelection.SelectedTrackIndex[InCodecType] >= 0)
				{
					// Select.
					auto Track = Tracks[UsableTrackArrayIndicesByType[InCodecType][TrackSelection.SelectedTrackIndex[InCodecType]]];
					check(Track.IsValid());
					if (Track.IsValid() && TrackSampleBuffers.Contains(Track->TrackID))
					{
						auto tsb = TrackSampleBuffers[Track->TrackID];
						if (tsb.IsValid())
						{
							if (InCodecType == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
							{
								CurrentSampleQueueInterface->SetVideoTrackEnabled(true);
							}
							else if (InCodecType == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio))
							{
								CurrentSampleQueueInterface->SetAudioTrackEnabled(true);
							}
							InDecoderThread.SetSampleBuffer(tsb, FGetSampleDlg::CreateRaw(&InLoaderThread, &FLoaderThread::GetSample));
							TrackSelection.ActiveTrackIndex[InCodecType] = TrackSelection.SelectedTrackIndex[InCodecType];
						}
					}
				}
				else
				{
					// Deselect.
					if (InCodecType == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
					{
						CurrentSampleQueueInterface->SetVideoTrackEnabled(false);
					}
					else if (InCodecType == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio))
					{
						CurrentSampleQueueInterface->SetAudioTrackEnabled(false);
					}
					InDecoderThread.DisconnectSampleBuffer();
					TrackSelection.ActiveTrackIndex[InCodecType] = TrackSelection.SelectedTrackIndex[InCodecType];
				}
			}
		};

		auto videoCI = CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video);
		auto audioCI = CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio);
		// See GetSupportedRates() above. When we are currently playing in reverse and switching tracks would result in
		// reverse playback being disabled, we need to either set the rate to forward play or pause.
		if (CurrentRate < 0.0f || SharedPlayParams->PlaybackDirection < 0.0f)
		{
			bool bReversePossible = false;
			if (TrackSelection.SelectedTrackIndex[audioCI] == -1 && TrackSelection.SelectedTrackIndex[videoCI] != -1)
			{
				auto Track = Tracks[UsableTrackArrayIndicesByType[videoCI][TrackSelection.SelectedTrackIndex[videoCI]]];
				bReversePossible = Track.IsValid() && Track->bIsKeyframeOnlyFormat;
			}
			if (!bReversePossible)
			{
				UE_LOGF(LogElectraProtron, Warning, "New track selection disallows reverse playback. Switching to pause.");
				// Pause.
				IntendedRate = 0.0f;
				HandleRateChanges();
			}
		}

		ChangeTrack(videoCI, VideoDecoderThread, VideoLoaderThread);
		ChangeTrack(audioCI, AudioDecoderThread, AudioLoaderThread);
		TrackSelection.bChanged = !(TrackSelection.SelectedTrackIndex[videoCI] == TrackSelection.ActiveTrackIndex[videoCI] &&
									TrackSelection.SelectedTrackIndex[audioCI] == TrackSelection.ActiveTrackIndex[audioCI]);
	}
}

void FElectraProtronPlayer::FImpl::HandleRateChanges()
{
	if (IntendedRate != CurrentRate)
	{
		// Update the general playback direction. When going into pause retain
		// the last direction for the loader to know into which direction
		// the operation went before pausing.
		// Other than at start the playback direction should not be zero.
		if (IntendedRate == 0.0f && CurrentRate != 0.0f)
		{
			SharedPlayParams->PlaybackDirection = CurrentRate;
		}
		else if (IntendedRate != 0.0f)
		{
			SharedPlayParams->PlaybackDirection = IntendedRate;
		}
		SharedPlayParams->DesiredPlayRate = IntendedRate;

		CurrentSampleQueueInterface->SetPlaybackRate(IntendedRate);
		VideoDecoderThread.SetRate(IntendedRate);
		AudioDecoderThread.SetRate(IntendedRate);

		CurrentRate = IntendedRate;
	}
}

void FElectraProtronPlayer::FImpl::HandleSeekRequest(const FSeekRequest& InSeekRequest)
{
	// Stop and flush the decoder threads
	TSharedPtr<FMediaEvent, ESPMode::ThreadSafe> VidFlushed = MakeShared<FMediaEvent, ESPMode::ThreadSafe>();
	TSharedPtr<FMediaEvent, ESPMode::ThreadSafe> AudFlushed = MakeShared<FMediaEvent, ESPMode::ThreadSafe>();

	VideoDecoderThread.PauseForSeek();
	AudioDecoderThread.PauseForSeek();

	VideoDecoderThread.Flush(VidFlushed);
	AudioDecoderThread.Flush(AudFlushed);
	VidFlushed->Wait();
	AudFlushed->Wait();


	CurrentPlayPosTime = InSeekRequest.NewTime;
	UpdateTrackLoader(CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video));
	UpdateTrackLoader(CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio));

	VideoDecoderThread.SetTime(InSeekRequest.NewTime, InSeekRequest.NewSequenceIndex, InSeekRequest.NewLoopIndex);
	VideoDecoderThread.ResumeAfterSeek();

	AudioDecoderThread.SetTime(InSeekRequest.NewTime, InSeekRequest.NewSequenceIndex, InSeekRequest.NewLoopIndex);
	AudioDecoderThread.ResumeAfterSeek();
}
