// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraProtronPlayerImpl.h"
#include "ElectraProtronPrivate.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "CodecTypeFormat.h"

#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "IElectraDecoderOutputAudio.h"
#include "ElectraDecodersUtils.h"

#include "MediaSamples.h"
#include "MediaDecoderOutput.h"
#include "ElectraTextureSample.h"
#include "IElectraAudioSample.h"



void FElectraProtronPlayer::FImpl::FDecoderThread::StartThread(const FOpenParam& InParam, const TSharedPtr<FSharedPlayParams, ESPMode::ThreadSafe>& InSharedPlayParams)
{
	Params = InParam;
	SharedPlayParams = InSharedPlayParams;
	// Video decoder does not need to hold the reference to the audio sample pool.
	if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
	{
		Params.AudioSamplePool.Reset();
	}
	// Audio decoder does not need to hold the reference to the texture pool.
	else if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio))
	{
		Params.TexturePool.Reset();
	}

	if (!Thread)
	{
		Thread = FRunnableThread::Create(this, TEXT("Electra Protron Decoder"), 0, TPri_Normal);
	}
}

void FElectraProtronPlayer::FImpl::FDecoderThread::StopThread()
{
	if (Thread)
	{
		bTerminate = true;
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}
}

uint32 FElectraProtronPlayer::FImpl::FDecoderThread::Run()
{
	double PreviousTime = FPlatformTime::Seconds();
	const double kDefaultMinLoopSleepTime = 0.005;
	MinLoopSleepTime = kDefaultMinLoopSleepTime;
	while(!bTerminate)
	{
		// When handling to the target seek time we want to go as quickly as possible,
		// so we only yield here but not wait.
		if (SeekTimeToHandleTo.IsSet())
		{
			FPlatformProcess::YieldThread();
		}
		else
		{
			double TimeNow = FPlatformTime::Seconds();
			double ElapsedTime = TimeNow - PreviousTime;
			if (ElapsedTime < MinLoopSleepTime)
			{
				FPlatformProcess::Sleep(MinLoopSleepTime - ElapsedTime);
			}
			PreviousTime = TimeNow;
			MinLoopSleepTime = kDefaultMinLoopSleepTime;
		}
		if (bFlushPending)
		{
			bFlushPending = false;
			PerformFlush();
			FlushedSignal->Signal();
		}
		while(HandleOutputFrame())
		{}

		// If an error occurred, do not do anything to make matters worse.
		// Just wait for the player to be closed.
		if (!LastErrorMessage.IsEmpty())
		{
			continue;
		}

		// Is a track buffer change pending?
		if (PendingBufferChange.bIsSet)
		{
			FScopeLock lock(&PendingBufferChange.Lock);
			PendingBufferChange.bIsSet = false;
			if (PendingBufferChange.NewTrackSampleBuffer.IsValid())
			{
				TSharedPtr<FMP4TrackSampleBuffer, ESPMode::ThreadSafe> NewTrackSampleBuffer(MoveTemp(PendingBufferChange.NewTrackSampleBuffer));
				FGetSampleDlg NewGetSampleDelegate(MoveTemp(PendingBufferChange.NewGetSampleDelegate));
				lock.Unlock();
				DestroyDecoder();
				TrackSampleBuffer = MoveTemp(NewTrackSampleBuffer);
				GetSampleDelegate = MoveTemp(NewGetSampleDelegate);
				UpdateTrackSampleDurationMap();
				FirstRangeSampleIt.Reset();
				LastRangeSampleIt.Reset();
				HandlePlaybackRangeChanges();
				TimeLock.Lock();
				CurrentTime = ClampTimeIntoPlaybackRange(CurrentTime);
				FTimespan Time = CurrentTime;
				TimeLock.Unlock();
				UpdateTrackIterator(Time);
				// When switching buffers we can skip decoding of samples prior to the current time.
				SeekTimeToDecodeTo = SeekTimeToHandleTo = Time;
				SeekTimeNumFramesDecoded = SeekTimeNumFramesSkipped = 0;
			}
			else
			{
				lock.Unlock();
				PerformFlush();
				TrackSampleBuffer.Reset();
				GetSampleDelegate.Unbind();
				TrackIterator.Reset();
				bReachedEnd = true;
				continue;
			}
		}

		// Check for rate changes.
		if (IntendedRate != CurrentRate)
		{
			const bool bDirectionChange = (IntendedRate < 0.0f && PlaybackDirection >= 0.0f) || (IntendedRate > 0.0f && PlaybackDirection < 0.0f);
			FTimespan NewTime;
			if (bDirectionChange)
			{
				TimeLock.Lock();
				NewTime = CurrentTime;
				TimeLock.Unlock();
				PerformFlush();
			}

			// Going into pause?
			if (IntendedRate == 0.0f)
			{
				if (CurrentRate != 0.0f)
				{
					PlaybackDirection = CurrentRate;
				}
				if (!IsPaused())
				{
					Pause();
				}
			}
			else
			{
				PlaybackDirection = IntendedRate;
				if (IsPaused())
				{
					Resume();
				}
			}
			CurrentRate = IntendedRate;

			if (bDirectionChange)
			{
				UpdateTrackIterator(NewTime);
			}
		}

		// Are we to do something?
		if (!TrackSampleBuffer.IsValid() || IsPausedForSeek() || (IsPaused() && !SeekTimeToHandleTo.IsSet() && !PendingSeek.bIsSet))
		{
			continue;
		}

		// Moving to a new time?
		if (PendingSeek.bIsSet)
		{
			PendingSeek.Lock.Lock();
			FTimespan NewTime = PendingSeek.NewTime;
			int32 NewSequenceIndex = PendingSeek.NewSeqIdx;
			TOptional<int32> NewLoopIndex = PendingSeek.NewLoopIdx;
			PendingSeek.bIsSet = false;
			PendingSeek.Lock.Unlock();

			TimeLock.Lock();
			CurrentTime = NewTime;
			SequenceIndex = NewSequenceIndex;
			LoopIndex = NewLoopIndex.Get(LoopIndex);
			TimeLock.Unlock();

			UpdateTrackIterator(NewTime);
			// Handle decoding to this time now.
			SeekTimeToHandleTo = NewTime;
			SeekTimeToDecodeTo = NewTime;
			SeekTimeNumFramesDecoded = 0;
			SeekTimeNumFramesSkipped = 0;
		}

		// Update the playback range if it changed.
		HandlePlaybackRangeChanges();
// FIXME: need to check if we are out of bounds and need to flush?

		DecodeOneFrame();
	}
	DestroyDecoder();
	return 0;
}


void FElectraProtronPlayer::FImpl::FDecoderThread::SetRate(float InNewRate)
{
	bool bChange = IntendedRate != InNewRate;
	IntendedRate = InNewRate;
	if (bChange)
	{
		WorkSignal.Signal();
	}
}

bool FElectraProtronPlayer::FImpl::FDecoderThread::SetLooping(bool bInLooping)
{
	if (bInLooping && !bShouldLoop)
	{
		// Check if play direction is reverse and non-keyframe only.
		if ((PlaybackDirection < 0.0f || CurrentRate < 0.0f) && TrackSampleBuffer.IsValid() && !TrackSampleBuffer->TrackAndCodecInfo->bIsKeyframeOnlyFormat)
		{
			UE_LOGF(LogElectraProtron, Warning, "Cannot enable looping while playing in reverse for tracks that use a non-keyframe-only codec!");
			return false;
		}
		// Enable
		bShouldLoop = true;
	}
	else if (!bInLooping && bShouldLoop)
	{
		// Disable
		bShouldLoop = false;
	}
	return true;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::SetPlaybackRange(TRange<FTimespan> InRange)
{
	FScopeLock lock(&PendingPlayRangeChange.Lock);
	PendingPlayRangeChange.NewRange = MoveTemp(InRange);
	PendingPlayRangeChange.bIsSet = true;
}


bool FElectraProtronPlayer::FImpl::FDecoderThread::HasReachedEnd()
{
	return bReachedEnd;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::Pause()
{
	bIsPaused = true;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::Resume()
{
	bIsPaused = false;
}

bool FElectraProtronPlayer::FImpl::FDecoderThread::IsPaused()
{
	return bIsPaused;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::PauseForSeek()
{
	bPausedForSeek = true;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::ResumeAfterSeek()
{
	bPausedForSeek = false;
}

bool FElectraProtronPlayer::FImpl::FDecoderThread::IsPausedForSeek()
{
	return bPausedForSeek;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::SetSampleBuffer(const TSharedPtr<FElectraProtronPlayer::FImpl::FMP4TrackSampleBuffer, ESPMode::ThreadSafe>& InTrackSampleBuffer, FElectraProtronPlayer::FImpl::FGetSampleDlg InGetSampleDelegate)
{
	FScopeLock lock(&PendingBufferChange.Lock);
	check(InGetSampleDelegate.IsBound());
	PendingBufferChange.NewTrackSampleBuffer = InTrackSampleBuffer;
	PendingBufferChange.NewGetSampleDelegate = MoveTemp(InGetSampleDelegate);
	PendingBufferChange.bIsSet = true;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::DisconnectSampleBuffer()
{
	FScopeLock lock(&PendingBufferChange.Lock);
	PendingBufferChange.NewTrackSampleBuffer = nullptr;
	PendingBufferChange.bIsSet = true;
}


void FElectraProtronPlayer::FImpl::FDecoderThread::SetTime(FTimespan InTime, int32 InSeqIdx, TOptional<int32> InLoopIdx)
{
	check(IsPausedForSeek());
	FScopeLock lock(&PendingSeek.Lock);
	PendingSeek.NewTime = InTime;
	PendingSeek.NewSeqIdx = InSeqIdx;
	PendingSeek.NewLoopIdx = InLoopIdx;
	PendingSeek.bIsSet = true;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::SetEstimatedPlaybackTime(FTimespan InTime)
{
	FScopeLock lock(&TimeLock);
	CurrentTime = InTime;
}

FTimespan FElectraProtronPlayer::FImpl::FDecoderThread::GetEstimatedPlaybackTime()
{
	FScopeLock lock(&TimeLock);
	return CurrentTime;
}


void FElectraProtronPlayer::FImpl::FDecoderThread::Flush(const TSharedPtr<FMediaEvent, ESPMode::ThreadSafe>& InFlushedSignal)
{
	FlushedSignal = InFlushedSignal;
	bFlushPending = true;
}

void FElectraProtronPlayer::FImpl::FDecoderThread::UpdateTrackSampleDurationMap()
{
	// Only do this for video tracks.
	if (DecoderTypeIndex != CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
	{
		return;
	}

	// Clear the video sample cache.
	Params.SampleQueueInterface->GetVideoCache().Empty();

	// Iterate the track and collect the effective PTS values into a sorted list,
	// then use their difference as the sample duration.
	FTimespan ClipDur(TrackSampleBuffer->TrackAndCodecInfo->MP4Track->GetFullMovieDuration().GetAsTimespan());
	struct FSampleTimes
	{
		FTimespan PTS;
		FTimespan Duration;
	};
	TArray<FSampleTimes> SampleTimeDurations;
	TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe> TkIt = TrackSampleBuffer->TrackAndCodecInfo->MP4Track->CreateIterator();
	SampleTimeDurations.Reserve(TkIt->GetNumSamples());
	auto It = TrackSampleBuffer->TrackAndCodecInfo->MP4Track->CreateIterator();
	do
	{
		SampleTimeDurations.Emplace(FSampleTimes({.PTS=It->GetEffectivePTS().GetAsTimespan()}));
	} while(It->Next());
	SampleTimeDurations.Sort([](const FSampleTimes& a, const FSampleTimes& b){return a.PTS < b.PTS;});
	for(int32 i=1,iMax=SampleTimeDurations.Num(); i<iMax; ++i)
	{
		SampleTimeDurations[i-1].Duration = SampleTimeDurations[i].PTS - SampleTimeDurations[i-1].PTS;
	}
	// The last sample's duration extends to the end of the clip.
	SampleTimeDurations.Last().Duration = ClipDur - SampleTimeDurations.Last().PTS;
	// Get the sample durations over into the lookup map.
	SampleTimeToDurationMap.Empty();
	for(auto& st : SampleTimeDurations)
	{
		SampleTimeToDurationMap.Emplace(st.PTS, st.Duration);
	}
}

void FElectraProtronPlayer::FImpl::FDecoderThread::UpdateTrackIterator(const FTimespan& InForTime)
{
	if (!TrackSampleBuffer.IsValid())
	{
		return;
	}

	TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe> TkIt = TrackSampleBuffer->TrackAndCodecInfo->MP4Track->CreateIteratorAtKeyframe(InForTime, FTimespan::FromMilliseconds(TrackSampleBuffer->TrackAndCodecInfo->bIsKeyframeOnlyFormat ? 0 : Config.NextKeyframeThresholdMillis));
	if (!TkIt.IsValid())
	{
		LastErrorMessage = TrackSampleBuffer->TrackAndCodecInfo->MP4Track->GetLastError();
		UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
		return;
	}
	// Searching for a keyframe in the track returns the frame on or before the time. When playing in reverse
	// (for keyframe-only codecs) we may need to pick the next frame.
	if (PlaybackDirection < 0.0f && TrackSampleBuffer->TrackAndCodecInfo->bIsKeyframeOnlyFormat)
	{
		auto NextIt = TkIt->Clone();
		NextIt->NextEffective();
		int64 t0 = FMath::Abs((TkIt->GetEffectivePTS().GetAsTimespan() - InForTime).GetTicks());
		int64 t1 = FMath::Abs((NextIt->GetEffectivePTS().GetAsTimespan() - InForTime).GetTicks());
		if (t1 < t0)
		{
			TkIt = MoveTemp(NextIt);
		}
	}

	// Use the new iterator.
	TrackIterator = MoveTemp(TkIt);
	bReachedEnd = false;
	// Recalculate the playrange iterators.
	FirstRangeSampleIt.Reset();
	LastRangeSampleIt.Reset();
	HandlePlaybackRangeChanges();
}


FTimespan FElectraProtronPlayer::FImpl::FDecoderThread::ClampTimeIntoPlaybackRange(const FTimespan& InTime)
{
	if (InTime < PlaybackRange.GetLowerBoundValue())
	{
		return PlaybackRange.GetLowerBoundValue();
	}
	else if (InTime > PlaybackRange.GetUpperBoundValue())
	{
		return PlaybackRange.GetUpperBoundValue();
	}
	return InTime;
}


void FElectraProtronPlayer::FImpl::FDecoderThread::HandlePlaybackRangeChanges()
{
	// Can't do anything without a sample buffer.
	if (!TrackSampleBuffer.IsValid())
	{
		return;
	}

	// Change in playback range?
	if (PendingPlayRangeChange.bIsSet)
	{
		FScopeLock lock(&TimeLock);
		PendingPlayRangeChange.bIsSet = false;
		if (PlaybackRange != PendingPlayRangeChange.NewRange)
		{
			PlaybackRange = MoveTemp(PendingPlayRangeChange.NewRange);
			FirstRangeSampleIt.Reset();
			LastRangeSampleIt.Reset();
		}
	}
	// If the range is not valid we set it to encompass the entire clip.
	if (PlaybackRange.IsEmpty() || PlaybackRange.IsDegenerate() || !PlaybackRange.HasLowerBound() || !PlaybackRange.HasUpperBound() || PlaybackRange.GetLowerBoundValue() > PlaybackRange.GetUpperBoundValue())
	{
		PlaybackRange = TRange<FTimespan>(FTimespan(0), Params.SampleQueueInterface->GetMovieDuration());
	}
	// Need to reset the first/last iterators?
	if (!FirstRangeSampleIt.IsValid())
	{
		// Locate the first and last sample numbers for the range
		FTrackIterator RangeIt = TrackSampleBuffer->TrackAndCodecInfo->MP4Track->CreateIteratorAtKeyframe(PlaybackRange.GetLowerBoundValue(), FTimespan(0));
		if (!RangeIt.IsValid())
		{
			LastErrorMessage = TrackSampleBuffer->TrackAndCodecInfo->MP4Track->GetLastError();
			UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
			return;
		}
		FirstRangeSampleIt = RangeIt->Clone();
		// Move forward until we reach the end or both the effective DTS *and* PTS are greater or equal than the end of the range.
		// We need to look at both DTS and PTS because the effective PTS can be smaller than the effective DTS due to composition time offsets.
		while(!RangeIt->IsLastEffective())
		{
			if (RangeIt->GetEffectiveDTS().GetAsTimespan() >= PlaybackRange.GetUpperBoundValue() && RangeIt->GetEffectivePTS().GetAsTimespan() >= PlaybackRange.GetUpperBoundValue())
			{
				// We want the last iterator to represent the last sample included in the playback range,
				// so we need to step one back here as we are currently outside the range.
				RangeIt->PrevEffective();
				break;
			}
			RangeIt->NextEffective();
		}
		LastRangeSampleIt = MoveTemp(RangeIt);
	}
}



void FElectraProtronPlayer::FImpl::FDecoderThread::PerformFlush()
{
	CurrentDecoderOutput.Reset();
	CurrentInputSample.Reset();
	InputForCurrentDecoderOutput.Reset();
	InDecoderInput.Empty();
	TrackIterator.Reset();
	SeekTimeToHandleTo.Reset();
	SeekTimeToDecodeTo.Reset();
	SeekTimeNumFramesDecoded = 0;
	SeekTimeNumFramesSkipped = 0;
	bIsDrainingAtEOS = false;
	bReachedEnd = false;
	bWaitForSyncSample = true;
	bWarnedMissingSyncSample = false;
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->ResetToCleanStart();
	}
	if (DecoderBitstreamProcessor.IsValid())
	{
		DecoderBitstreamProcessor->Clear();
	}
}


void FElectraProtronPlayer::FImpl::FDecoderThread::FlushForEndOrLooping()
{
	// Non keyframe-only formats, and all audio formats, require that we drain the decoder to get
	// all the pending output before we start decoding the next (loop) sequence.
	bool bNeedsDrain = true;
	if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video) &&
		TrackSampleBuffer->TrackAndCodecInfo->bIsKeyframeOnlyFormat)
	{
		bNeedsDrain = false;
	}

	if (bNeedsDrain && DecoderInstance.IsValid())
	{
		IElectraDecoder::EDecoderError DecErr = DecoderInstance->SendEndOfData();
		if (ensure(DecErr == IElectraDecoder::EDecoderError::None))
		{
			bIsDrainingAtEOS = true;
			bWaitForSyncSample = true;
			bWarnedMissingSyncSample = false;
		}
		else
		{
			LastErrorMessage = FString::Printf(TEXT("Failed to flush decoder at the end of the stream"));
		}
	}
}


void FElectraProtronPlayer::FImpl::FDecoderThread::DecodeOneFrame()
{
	// Do not decode new data when draining at end-of-stream.
	if (bIsDrainingAtEOS)
	{
		return;
	}

	// If we are still hanging on to output we could not deliver we will not
	// decode new data that will likewise not be able to deliver.
	if (CurrentDecoderOutput.IsValid())
	{
		return;
	}

	// If the end of the stream has been reached we are done.
	if (bReachedEnd)
	{
		return;
	}

	if (TrackSampleBuffer.IsValid())
	{
		if (!CreateDecoder())
		{
			return;
		}
		if (TrackIterator.IsValid())
		{
			bool bSkipDecoding = false;

			if (!CurrentInputSample.IsValid())
			{
				CurrentInputSample = MakeUnique<FInDecoder>();
			}
			if (!CurrentInputSample->Sample.IsValid())
			{
				// Get the frame the iterator is pointing at from the buffer.
				FMP4SamplePtr Sample = GetSampleDelegate.Execute(TrackSampleBuffer, TrackIterator, 0);
				if (!Sample.IsValid())
				{
					return;
				}

				CurrentInputSample->Sample = MoveTemp(Sample);
				// Do we have to make a copy of the sample data so we can modify it without affecting the original in the cache?
				if (DecoderBitstreamProcessor.IsValid() && DecoderBitstreamProcessor->WillModifyBitstreamInPlace())
				{
					CurrentInputSample->DataCopy = CurrentInputSample->Sample->Data;
					CurrentInputSample->DecAU.Data = CurrentInputSample->DataCopy.GetData();
					CurrentInputSample->DecAU.DataSize = CurrentInputSample->DataCopy.Num();
				}
				else
				{
					CurrentInputSample->DecAU.Data = CurrentInputSample->Sample->Data.GetData();
					CurrentInputSample->DecAU.DataSize = CurrentInputSample->Sample->Data.Num();
				}
				CurrentInputSample->DecAU.DTS = CurrentInputSample->Sample->DTS;
				CurrentInputSample->DecAU.PTS = CurrentInputSample->Sample->PTS;
				const FTimespan* DurFromMap = SampleTimeToDurationMap.Find(CurrentInputSample->Sample->EffectivePTS);
				CurrentInputSample->DecAU.Duration = DurFromMap ? *DurFromMap : CurrentInputSample->Sample->Duration;
				CurrentInputSample->DecAU.UserValue = NextUserValue;
				CurrentInputSample->DecAU.Flags = CurrentInputSample->Sample->bIsSyncOrRap ? EElectraDecoderFlags::IsSyncSample : EElectraDecoderFlags::None;
				if (CurrentInputSample->Sample->bIsSyncOrRap)
				{
					CurrentInputSample->CSDOptions = CurrentCodecSpecificData;
				}
				if (DecoderBitstreamProcessor.IsValid())
				{
					IElectraDecoderBitstreamProcessor::EProcessResult BSResult = DecoderBitstreamProcessor->ProcessInputForDecoding(CurrentInputSample->BSI, CurrentInputSample->DecAU, CurrentInputSample->CSDOptions);
					if (BSResult == IElectraDecoderBitstreamProcessor::EProcessResult::Error)
					{
						LastErrorMessage = DecoderBitstreamProcessor->GetLastError();
						UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
						return;
					}
				}

				// If handling to a specific time see if this sample is discardable and does not require decoding.
				if (SeekTimeToDecodeTo.IsSet())
				{
					FTimespan Start = CurrentInputSample->Sample->EffectivePTS;
					FTimespan End = Start + CurrentInputSample->DecAU.Duration;
					FTimespan X = SeekTimeToDecodeTo.GetValue();
					if (X >= Start && X < End)
					{
						SeekTimeToDecodeTo.Reset();
					}
					else if ((CurrentInputSample->DecAU.Flags & EElectraDecoderFlags::IsDiscardable) != EElectraDecoderFlags::None)
					{
						bSkipDecoding = true;
						CurrentInputSample.Reset();
						++SeekTimeNumFramesSkipped;
					}
				}
			}

			// Is this really a sync sample if we need one?
			if (!bSkipDecoding && bWaitForSyncSample && (CurrentInputSample->DecAU.Flags & EElectraDecoderFlags::IsSyncSample) == EElectraDecoderFlags::None)
			{
				if (!bWarnedMissingSyncSample)
				{
					bWarnedMissingSyncSample = true;
					UE_LOGF(LogElectraProtron, Warning, "Expected a sync sample at PTS %lld, but did not get one. The stream may be packaged incorrectly. Dropping frames until one arrives, which may take a while. Please wait!", (long long int)CurrentInputSample->Sample->EffectivePTS.GetTicks());
				}
				bSkipDecoding = true;
				CurrentInputSample.Reset();
			}

			if (!bSkipDecoding)
			{
				IElectraDecoder::EDecoderError DecErr = DecoderInstance->DecodeAccessUnit(CurrentInputSample->DecAU, CurrentInputSample->CSDOptions);
				if (DecErr == IElectraDecoder::EDecoderError::None)
				{
					if ((CurrentInputSample->DecAU.Flags & EElectraDecoderFlags::DoNotOutput) == EElectraDecoderFlags::None)
					{
						// The copied data is no longer necessary to keep around.
						CurrentInputSample->DataCopy.Empty();
						// Set the associated indices
						CurrentInputSample->SequenceIndex = SequenceIndex;
						CurrentInputSample->LoopIndex = LoopIndex;
						InDecoderInput.Emplace(NextUserValue, MoveTemp(CurrentInputSample));
						++NextUserValue;
					}
					bWaitForSyncSample = false;
					bWarnedMissingSyncSample = false;
				}
				else if (DecErr == IElectraDecoder::EDecoderError::NoBuffer)
				{
					// Try again later...
					return;
				}
				else if (DecErr == IElectraDecoder::EDecoderError::LostDecoder)
				{
					if (DecoderInstance->GetError().IsSet())
					{
						LastErrorMessage = DecoderInstance->GetError().GetMessage();
					}
					else
					{
						LastErrorMessage = FString::Printf(TEXT("Lost the decoder"));
					}
					UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
					return;
				}
				else
				{
					if (DecoderInstance->GetError().IsSet())
					{
						LastErrorMessage = DecoderInstance->GetError().GetMessage();
					}
					else
					{
						LastErrorMessage = FString::Printf(TEXT("Decoder error"));
					}
					UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
					return;
				}
			}

			// Move to the next or previous frame
			StepTrackIterator();
		}
	}
}

void FElectraProtronPlayer::FImpl::FDecoderThread::StepTrackIterator()
{
	if (!TrackSampleBuffer.IsValid() || !TrackIterator.IsValid() || !FirstRangeSampleIt.IsValid() || !LastRangeSampleIt.IsValid())
	{
		return;
	}

	// Going forward or backwards?
	if (PlaybackDirection >= 0.0f)
	{
		// Move to the next sample. If we are on the last sample of the playback range (inclusive) or there is no next sample
		// we either have to loop or are done.
		if (TrackIterator->GetSampleNumber() == LastRangeSampleIt->GetSampleNumber() || !TrackIterator->NextEffective())
		{
			FlushForEndOrLooping();
			if (bShouldLoop)
			{
				TrackIterator = FirstRangeSampleIt->Clone();
				++LoopIndex;
			}
			else
			{
				bReachedEnd = true;
			}
		}
	}
	else
	{
		// If we are on the first sample of the playback range (inclusive) or there is no previous sample to go back to
		// we either have to loop or are done.
		if (TrackIterator->GetSampleNumber() == FirstRangeSampleIt->GetSampleNumber() || !TrackIterator->PrevEffective())
		{
			FlushForEndOrLooping();
			if (bShouldLoop)
			{
				// Re-check this is keyframe-only. For non keyframe-only we cannot go back to the last frame since we need to
				// go to whatever the last keyframe is to decode up to the last frame from there.
				if (TrackSampleBuffer.IsValid() && TrackSampleBuffer->TrackAndCodecInfo->bIsKeyframeOnlyFormat)
				{
					TrackIterator = LastRangeSampleIt->Clone();
					--LoopIndex;
				}
				else
				{
					bReachedEnd = true;
				}
			}
			else
			{
				bReachedEnd = true;
			}
		}
	}
}


bool FElectraProtronPlayer::FImpl::FDecoderThread::HandleOutputFrame()
{
	if (!DecoderInstance.IsValid())
	{
		return false;
	}
	bool bGetNext = false;

	// If we still have unhandled output, do not get new output yet.
	if (!CurrentDecoderOutput.IsValid())
	{
		InputForCurrentDecoderOutput.Reset();
		// Does the decoder have new output?
		IElectraDecoder::EOutputStatus OutputStatus = DecoderInstance->HaveOutput();
		if (OutputStatus == IElectraDecoder::EOutputStatus::Available)
		{
			TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = DecoderInstance->GetOutput();
			if (!Out.IsValid())
			{
				// No output although advertised?
				return false;
			}
			if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video) && Out->GetType() == IElectraDecoderOutput::EType::Video)
			{
				CurrentDecoderOutput = MoveTemp(Out);
			}
			else if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio) && Out->GetType() == IElectraDecoderOutput::EType::Audio)
			{
				CurrentDecoderOutput = MoveTemp(Out);
			}
			// Unhandled output type?
			if (Out.IsValid())
			{
				Out.Reset();
				return false;
			}
			// Find the matching input sample
			if (!InDecoderInput.RemoveAndCopyValue(CurrentDecoderOutput->GetUserValue(), InputForCurrentDecoderOutput))
			{
				check(!"no input for this output?");
				CurrentDecoderOutput.Reset();
				return false;
			}
		}
		else if (OutputStatus == IElectraDecoder::EOutputStatus::EndOfData)
		{
			check(bIsDrainingAtEOS);
			bIsDrainingAtEOS = false;
			// Assuming we loop back to the playback range set the skip-until to the start of the range.
			// When we do not loop or seek somewhere else setting this up here does no harm either.
			SeekTimeToDecodeTo = SeekTimeToHandleTo = PlaybackRange.GetLowerBoundValue();
			SeekTimeNumFramesDecoded = SeekTimeNumFramesSkipped = 0;
		}
	}
	// Handle the output.
	if (CurrentDecoderOutput.IsValid() && InputForCurrentDecoderOutput.IsValid() && Params.SampleQueueInterface.IsValid())
	{
		TOptional<FTimespan> AdjustToTime;
		bool bSendToOutput = true;

		auto AdjustForSeek = [&]() -> void
		{
			// Is this the frame we were supposed to decode up to?
			if (SeekTimeToHandleTo.IsSet())
			{
				FTimespan Start = InputForCurrentDecoderOutput->Sample->EffectivePTS;
				FTimespan End = InputForCurrentDecoderOutput->Sample->EffectivePTS + InputForCurrentDecoderOutput->DecAU.Duration;
				FTimespan X = SeekTimeToHandleTo.GetValue();
				if (X >= Start && X < End)
				{
					AdjustToTime = X;
					SeekTimeToHandleTo.Reset();
					if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video) && (SeekTimeNumFramesDecoded + SeekTimeNumFramesSkipped) > 1)
					{
						UE_LOGF(LogElectraProtron, Log, "Processed %d frames to reach seek point, of which %d could be skipped", SeekTimeNumFramesDecoded + SeekTimeNumFramesSkipped, SeekTimeNumFramesSkipped);
					}
				}
				else
				{
					bSendToOutput = false;
					++SeekTimeNumFramesDecoded;
				}
			}
		};

		// Get the timestamps of the media local timeline needed to adjust the decoded samples.
		FTimespan FirstSampleEffectiveStartTime = PlaybackRange.GetLowerBoundValue();
		FTimespan LatestSampleEffectiveEndTime = PlaybackRange.GetUpperBoundValue();

		if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
		{
			AdjustForSeek();

			FTimespan RawPTS(InputForCurrentDecoderOutput->Sample->EffectivePTS);
			FTimespan RawDur(InputForCurrentDecoderOutput->DecAU.Duration);

			// When sending the output we need to wait until there is room to receive it.
			// If the output is not to be sent we can handle it quickly without waiting.
			if (bSendToOutput)
			{
				Params.SampleQueueInterface->VideoSampleWillBeReady();
			}
			if (!bSendToOutput || (Params.SampleQueueInterface->CanEnqueueVideoSample(RawPTS) && Params.SampleQueueInterface->AreRequiredSamplesReady()))
			{
				// First perform any processing that needs to be done regardless of whether the output will be sent or not.

				// Get information from the bitstream that the decoder does not provide.
				if (DecoderBitstreamProcessor.IsValid())
				{
					TMap<FString, FVariant> BSIProperties;
					DecoderBitstreamProcessor->SetPropertiesOnOutput(BSIProperties, InputForCurrentDecoderOutput->BSI);
					if (BSIProperties.Num())
					{
						// Colorimetry?
						TArray<uint8> CommonColorimetry(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::CommonColorimetry));
						if (CommonColorimetry.Num() == sizeof(ElectraDecodersUtil::MPEG::FCommonColorimetry))
						{
							const ElectraDecodersUtil::MPEG::FCommonColorimetry& Colorimetry(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FCommonColorimetry*>(CommonColorimetry.GetData()));
							if (!CurrentColorimetry.IsSet())
							{
								CurrentColorimetry = Electra::MPEG::FColorimetryHelper();
							}
							CurrentColorimetry.GetValue().Update(Colorimetry.colour_primaries, Colorimetry.transfer_characteristics, Colorimetry.matrix_coeffs, Colorimetry.video_full_range_flag, Colorimetry.video_format);
						}

						// HDR parameters?
						TArray<uint8> Mdcv(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::SeiMasteringDisplayColorVolume));
						if (Mdcv.Num() == sizeof(ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume))
						{
							if (!CurrentHDR.IsSet())
							{
								CurrentHDR = Electra::MPEG::FHDRHelper();
							}
							CurrentHDR.GetValue().UpdateWith(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume*>(Mdcv.GetData()));
						}
						TArray<uint8> Clli(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::SeiContentLightLeveInfo));
						if (Clli.Num() == sizeof(ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info))
						{
							if (!CurrentHDR.IsSet())
							{
								CurrentHDR = Electra::MPEG::FHDRHelper();
							}
							CurrentHDR.GetValue().UpdateWith(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info*>(Clli.GetData()));
						}
						TArray<uint8> Altc(ElectraDecodersUtil::GetVariantValueUInt8Array(BSIProperties, IElectraDecoderBitstreamProcessorInfo::SeiAlternateTransferCharacteristics));
						if (Altc.Num() == sizeof(ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics))
						{
							if (!CurrentHDR.IsSet())
							{
								CurrentHDR = Electra::MPEG::FHDRHelper();
							}
							CurrentHDR.GetValue().UpdateWith(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics*>(Altc.GetData()));
						}
					}
				}

				FTimespan PTS(RawPTS);
				FTimespan Dur(RawDur);
				if (AdjustToTime.IsSet())
				{
					FTimespan Diff(AdjustToTime.GetValue() - RawPTS);
					PTS += Diff;
					Dur -= Diff;
				}
				// Need to trim at the start?
				else if (PTS < FirstSampleEffectiveStartTime)
				{
					FTimespan Diff = FirstSampleEffectiveStartTime - PTS;
					PTS += Diff;
					Dur -= Diff;
				}
				// Need to trim at the end?
				if (PTS + Dur > LatestSampleEffectiveEndTime)
				{
					FTimespan Diff = PTS + Dur - LatestSampleEffectiveEndTime;
					Dur -= Diff;
				}
				// Still useful for display?
				if (Dur <= FTimespan::Zero())
				{
					bSendToOutput = false;
				}

				// Send the output along?
				if (bSendToOutput)
				{
					auto VideoDecoderOutput = StaticCastSharedPtr<IElectraDecoderVideoOutput>(CurrentDecoderOutput);

					FElectraTextureSamplePtr OutputTextureSample = VideoDecoderOutputPool->AcquireShared();
					// Note: If we need to be informed when the output sample is no longer needed we
					//       could bind a delegate to it via GetReleaseDelegate()
					TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> BufferProperties = MakeShared<Electra::FParamDict, ESPMode::ThreadSafe>();

					Electra::FTimeValue OutputPTS, OutputDur;
					OutputDur.SetFromTimespan(Dur);
					uint64 seqIdx = (uint32) InputForCurrentDecoderOutput->LoopIndex;
					seqIdx |= (uint64)((uint32)InputForCurrentDecoderOutput->SequenceIndex) << 32U;
					OutputPTS.SetFromTimespan(PTS, (int64) seqIdx);
					BufferProperties->Set(IDecoderOutputOptionNames::PTS, Electra::FVariantValue(OutputPTS));
					BufferProperties->Set(IDecoderOutputOptionNames::Duration, Electra::FVariantValue(OutputDur));
					if (InputForCurrentDecoderOutput->Sample->AssociatedTimecode.IsSet())
					{
						BufferProperties->Set(IDecoderOutputOptionNames::TMCDTimecode, Electra::FVariantValue(InputForCurrentDecoderOutput->Sample->AssociatedTimecode.GetValue()));
						BufferProperties->Set(IDecoderOutputOptionNames::TMCDFramerate, Electra::FVariantValue(InputForCurrentDecoderOutput->Sample->AssociatedTimecodeFramerate.Get(FFrameRate())));
					}

					// Set the colorimetry, if available, on the output properties.
					if (CurrentColorimetry.IsSet())
					{
						CurrentColorimetry.GetValue().UpdateParamDict(*BufferProperties);
						// Also HDR information (which requires colorimetry!) if available.
						if (CurrentHDR.IsSet())
						{
							CurrentHDR.GetValue().SetHDRType(VideoDecoderOutput->GetNumberOfBits(), CurrentColorimetry.GetValue());
							CurrentHDR.GetValue().UpdateParamDict(*BufferProperties);
						}
					}

					FString DecoderOutputErrorMsg;
					if (ensure(FElectraDecodersPlatformResources::SetupOutputTextureSample(DecoderOutputErrorMsg, OutputTextureSample, VideoDecoderOutput, BufferProperties, DecoderPlatformResource)))
					{
						Params.SampleQueueInterface->EnqueueVideoSample(OutputTextureSample.ToSharedRef(), RawPTS, RawDur);
					}
				}
				CurrentDecoderOutput.Reset();
				InputForCurrentDecoderOutput.Reset();
				bGetNext = true;
			}
		}
		else if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio))
		{
			AdjustForSeek();

			// When sending the output we need to wait until there is room to receive it.
			// If the output is not to be sent we can handle it quickly without waiting.
			if (bSendToOutput)
			{
				Params.SampleQueueInterface->AudioSampleWillBeReady();
			}

			if (!bSendToOutput || (Params.SampleQueueInterface->CanEnqueueAudioSample() && Params.SampleQueueInterface->AreRequiredSamplesReady()))
			{
				TSharedPtr<IElectraDecoderAudioOutput, ESPMode::ThreadSafe> AudioDecoderOutput = StaticCastSharedPtr<IElectraDecoderAudioOutput>(CurrentDecoderOutput);

				// Check that we get what we support. At the moment that is the only provided output format anyway, so this should pass.
				if (AudioDecoderOutput->GetType() == IElectraDecoderOutput::EType::Audio &&
					AudioDecoderOutput->IsInterleaved() &&
					AudioDecoderOutput->GetSampleFormat() == IElectraDecoderAudioOutput::ESampleFormat::Float)
				{
					const float* PCMBuffer = reinterpret_cast<const float*>(AudioDecoderOutput->GetData(0));
					const int32 SamplingRate = AudioDecoderOutput->GetSampleRate();
					const int32 NumberOfChannels = AudioDecoderOutput->GetNumChannels();
					const int32 NumBytesPerFrame = AudioDecoderOutput->GetBytesPerFrame();
					const int32 NumBytesPerSample = AudioDecoderOutput->GetBytesPerSample();
					int32 NumSamplesProduced = AudioDecoderOutput->GetNumFrames();

					if (AudioChannelMapper.IsInitialized() || (!AudioChannelMapper.IsInitialized() && AudioChannelMapper.Initialize(AudioDecoderOutput)))
					{
						int32 ByteOffsetToFirstSample = 0;
						FElectraAudioSampleRef AudioSample = Params.AudioSamplePool->AcquireShared();
						FMediaTimeStamp PTS(InputForCurrentDecoderOutput->Sample->EffectivePTS, InputForCurrentDecoderOutput->SequenceIndex, InputForCurrentDecoderOutput->LoopIndex);
						FTimespan Dur(InputForCurrentDecoderOutput->Sample->Duration);
						FTimespan TrimDurAtStart, TrimDurAtEnd;
						if (AdjustToTime.IsSet())
						{
							TrimDurAtStart = AdjustToTime.GetValue() - InputForCurrentDecoderOutput->Sample->EffectivePTS;
							PTS += TrimDurAtStart;
							Dur -= TrimDurAtStart;
						}
						// Need to trim at the start?
						else if (PTS.Time < FirstSampleEffectiveStartTime)
						{
							TrimDurAtStart = FirstSampleEffectiveStartTime - PTS.Time;
							PTS += TrimDurAtStart;
							Dur -= TrimDurAtStart;
						}
						// Need to trim at the end?
						if (PTS.Time + Dur > LatestSampleEffectiveEndTime)
						{
							TrimDurAtEnd = PTS.Time + Dur - LatestSampleEffectiveEndTime;
							Dur -= TrimDurAtEnd;
						}

						// Set the current time to be the end of this decoded frame, which is the PTS plus the
						// frame's duration when playing forward, or just the PTS when playing in reverse.
						// We do this for audio since it is difficult to get the actual playback position from the outside.
						TimeLock.Lock();
						CurrentTime = PTS.Time + (PlaybackDirection >= 0.0f ? Dur : FTimespan(0));
						TimeLock.Unlock();

						// Send the output along?
						if (bSendToOutput)
						{
							int32 SkipStartSampleNum = (int32) (TrimDurAtStart.GetTicks() * SamplingRate / ETimespan::TicksPerSecond);
							int32 SkipEndSampleNum = (int32) (TrimDurAtEnd.GetTicks() * SamplingRate / ETimespan::TicksPerSecond);

							if (SkipStartSampleNum + SkipEndSampleNum < NumSamplesProduced)
							{
								ByteOffsetToFirstSample = SkipStartSampleNum * NumBytesPerFrame;
								NumSamplesProduced -= SkipStartSampleNum;
								NumSamplesProduced -= SkipEndSampleNum;
							}
							else
							{
								NumSamplesProduced = 0;
							}

							// Anything remaining to pass along?
							if (NumSamplesProduced)
							{
								// Allocate the sample buffer.
								if (AudioSample->AllocateFor(EMediaAudioSampleFormat::Float, (uint32)NumberOfChannels, (uint32)NumSamplesProduced))
								{
									// Set the sample parameters and copy the data.
									AudioSample->SetParameters((uint32)SamplingRate, PTS, Dur);
									AudioChannelMapper.MapChannels(AudioSample->GetWritableBuffer(), (int32)AudioSample->GetAllocatedSize(),
																	ElectraDecodersUtil::AdvancePointer(PCMBuffer, ByteOffsetToFirstSample), NumSamplesProduced * NumBytesPerFrame, NumSamplesProduced);
									Params.SampleQueueInterface->EnqueueAudioSample(AudioSample);
								}
								else
								{
									// Out of memory. Leave gracefully. Bad things will happen soon somewhere, but at least not here.
								}
							}
						}
					}
					else
					{
						LastErrorMessage = FString::Printf(TEXT("Could not initialize the channel mapper"));
						UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
					}
				}
				else
				{
					LastErrorMessage = FString::Printf(TEXT("Unsupported audio output format"));
					UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
				}
				CurrentDecoderOutput.Reset();
				InputForCurrentDecoderOutput.Reset();
			}
		}
	}
	return bGetNext;
}



void FElectraProtronPlayer::FImpl::FDecoderThread::DestroyDecoder()
{
	PerformFlush();
	CurrentCodecSpecificData.Empty();
	CurrentColorimetry.Reset();
	CurrentHDR.Reset();
	AudioChannelMapper.Reset();

	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Close();
		DecoderInstance.Reset();
	}
	if (DecoderBitstreamProcessor.IsValid())
	{
		DecoderBitstreamProcessor->Clear();
		DecoderBitstreamProcessor.Reset();
	}
	if (DecoderPlatformResource)
	{
		FElectraDecodersPlatformResources::ReleasePlatformVideoResource(this, DecoderPlatformResource);
		DecoderPlatformResource = nullptr;
	}
}


bool FElectraProtronPlayer::FImpl::FDecoderThread::CreateDecoder()
{
	if (DecoderInstance.IsValid())
	{
		return true;
	}
	check(TrackSampleBuffer.IsValid());
	if (!TrackSampleBuffer.IsValid())
	{
		LastErrorMessage = FString::Printf(TEXT("Internal error, no track sample buffer is set when creating a decoder."));
		return false;
	}

	// Get the decoder factory module.
	IElectraCodecFactoryModule* FactoryModule = static_cast<IElectraCodecFactoryModule*>(FModuleManager::Get().GetModule(TEXT("ElectraCodecFactory")));
	check(FactoryModule);

	CurrentCodecSpecificData.Empty();
	CurrentColorimetry.Reset();
	CurrentHDR.Reset();
	TMap<FString, FVariant> DecoderCfgParams;
	const Electra::FCodecTypeFormat& ci(TrackSampleBuffer->TrackAndCodecInfo->CodecInfo);
	if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
	{
		check(ci.Type == Electra::FCodecTypeFormat::EType::Video);
		const Electra::FCodecTypeFormat::FVideo& vi(ci.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
		uint32 kNumBufs = Params.SampleQueueInterface->GetMaxVideoFramesToCache() > 0 ? (uint32)Params.SampleQueueInterface->GetMaxVideoFramesToCache() : 8U;
		// To support playback ranges with GOP codecs we need additional buffers when looping from the range end to its start
		// since it is possible that we need to decode-discard frames prior to the range. Although these won't be used for display they
		// will still be produced and thus need additional room.
		if (!TrackSampleBuffer->TrackAndCodecInfo->bIsKeyframeOnlyFormat)
		{
			kNumBufs += 5;
		}
		DecoderCfgParams.Add(TEXT("max_output_buffers"), FVariant(kNumBufs));
		DecoderCfgParams.Add(TEXT("max_width"), FVariant((uint32) vi.Width));
		DecoderCfgParams.Add(TEXT("max_height"), FVariant((uint32) vi.Height));
	}
	// Remember the CSD/DCR so it can be passed into the decode call more easily.
	if (ci.DCR.Num())
	{
		CurrentCodecSpecificData.Add(TEXT("dcr"), FVariant(ci.DCR));
	}
	if (ci.CSD.Num())
	{
		CurrentCodecSpecificData.Add(TEXT("csd"), FVariant(ci.CSD));
	}
	// Add the extra boxes found in the sample description.
	for(auto& boxIt : ci.ExtraBoxes)
	{
		FString BoxName = FString::Printf(TEXT("%s"), *MP4Utilities::Printable4CC(boxIt.Key));
		CurrentCodecSpecificData.Add(BoxName, FVariant(boxIt.Value));
	}

	// Try to find a decoder factory for the format.
	// This should succeed since we already checked the supported formats earlier.
	TMap<FString, FVariant> FormatInfo;
	TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> DecoderFactory = FactoryModule->GetBestDecoderFactoryForFormat(FormatInfo, ci, DecoderCfgParams);
	check(DecoderFactory.IsValid());
	if (!DecoderFactory.IsValid())
	{
		LastErrorMessage = FString::Printf(TEXT("No decoder factory found for format \"%s\"."), *ci.RFC6381);
		return false;
	}


	// Create platform specific resources to be used with the new decoder.
	check(DecoderPlatformResource == nullptr);
	if (DecoderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
	{
		DecoderPlatformResource = FElectraDecodersPlatformResources::CreatePlatformVideoResource(this, DecoderCfgParams);
	}

	// Create a decoder instance.
	DecoderInstance = DecoderFactory->CreateDecoder(ci, DecoderCfgParams);
	if (!DecoderInstance.IsValid())
	{
		FElectraDecodersPlatformResources::ReleasePlatformVideoResource(this, DecoderPlatformResource);
		DecoderPlatformResource = nullptr;
		LastErrorMessage = FString::Printf(TEXT("Failed to create decoder for format \"%s\"."), *ci.RFC6381);
		return false;
	}
	if (DecoderInstance->GetError().IsSet())
	{
		FElectraDecodersPlatformResources::ReleasePlatformVideoResource(this, DecoderPlatformResource);
		DecoderPlatformResource = nullptr;
		LastErrorMessage = DecoderInstance->GetError().GetMessage();
		return false;
	}
	// Get the bitstream processor for this decoder, if it requires one.
	DecoderBitstreamProcessor = DecoderInstance->CreateBitstreamProcessor();
	return true;
}
