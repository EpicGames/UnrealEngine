// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraProtronPlayerImpl.h"
#include "ElectraProtronPrivate.h"
#include "Utils/MPEG/ElectraUtilsMP4.h"
#include "CodecTypeFormat.h"
#include "Core/MediaThreads.h"

/*********************************************************************************************************************/
#include "HAL/IConsoleManager.h"
static TAutoConsoleVariable<bool> CVarProtronForceShortestDuration(
	TEXT("ElectraProtron.ForceShortestDuration"),
	true,
	TEXT("Set movie duration to the shortest track in the file.\n")
	TEXT(" true: use the shortest duration\n false: keep individual track durations"),
	ECVF_Default);
/*********************************************************************************************************************/

namespace ElectraProtronOptionNames
{
	const FName StartTimecodeValue(TEXT("StartTimecodeValue"));
	const FName StartTimecodeFrameRate(TEXT("StartTimecodeFrameRate"));
	const FName KeyframeInterval(TEXT("KeyframeInterval"));
	const FName VideoDecoderProvider(TEXT("VideoDecoderProvider"));
	const FName AudioDecoderProvider(TEXT("AudioDecoderProvider"));
}


FElectraProtronPlayer::FImpl::FImpl()
{
	// Create the playback parameter structure that has members changing at any moment in time.
	// This information is shared with the frame loader.
	SharedPlayParams = MakeShared<FSharedPlayParams, ESPMode::ThreadSafe>();

	// Create the track-by-type array upfront in case queries to tracks are made before opening a source.
	UsableTrackArrayIndicesByType.SetNum(UE_ARRAY_COUNT(kCodecTrackIndexMap));

	// Create the sample queue interface.
	const int32 kVideoFramesToCacheAhead = 8;
	const int32 kVideoFramesToCacheBehind = 8;
	CurrentSampleQueueInterface = MakeShared<FSampleQueueInterface, ESPMode::ThreadSafe>(kVideoFramesToCacheAhead, kVideoFramesToCacheBehind);
}

FElectraProtronPlayer::FImpl::~FImpl()
{
}

void FElectraProtronPlayer::FImpl::StartThread()
{
	if (!Thread)
	{
		Thread = FRunnableThread::Create(this, TEXT("Electra Protron"), 0, TPri_Normal);
	}
}


void FElectraProtronPlayer::FImpl::Open(const FOpenParam& InParam, FElectraProtronPlayer::FImpl::FCompletionDelegate InCompletionDelegate)
{
	FWorkerThreadMessage Msg;
	FWorkerThreadMessage::FParamOpen open { InParam };
	open.Param.SampleQueueInterface = CurrentSampleQueueInterface;
	Msg.Param.Emplace<FWorkerThreadMessage::FParamOpen>(MoveTemp(open));
	Msg.Self = AsShared();
	Msg.Type = FWorkerThreadMessage::EType::Open;
	Msg.CompletionDelegate = MoveTemp(InCompletionDelegate);
	SendWorkerThreadMessage(MoveTemp(Msg));
	StartThread();
}

void FElectraProtronPlayer::FImpl::Close(FElectraProtronPlayer::FImpl::FCompletionDelegate InCompletionDelegate)
{
	bAbort = true;
	if (Thread)
	{
		FWorkerThreadMessage Msg;
		Msg.Param.Emplace<FWorkerThreadMessage::FParamTerminate>(FWorkerThreadMessage::FParamTerminate());
		Msg.Self = AsShared();
		Msg.Type = FWorkerThreadMessage::EType::Terminate;
		Msg.CompletionDelegate = MoveTemp(InCompletionDelegate);
		SendWorkerThreadMessage(MoveTemp(Msg));
	}
	else
	{
		InCompletionDelegate.ExecuteIfBound(AsShared());
	}
}

FString FElectraProtronPlayer::FImpl::GetLastError()
{
	return LastErrorMessage;
}

bool FElectraProtronPlayer::FImpl::HasReachedEnd()
{
	const bool bIsVideoActive = TrackSelection.ActiveTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)] != -1;
	const bool bIsAudioActive = TrackSelection.ActiveTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)] != -1;

	bool bAllReachedEnd = true;
	if (bIsVideoActive && !VideoDecoderThread.HasReachedEnd())
	{
		bAllReachedEnd = false;
	}
	if (bIsAudioActive && !AudioDecoderThread.HasReachedEnd())
	{
		bAllReachedEnd = false;
	}
	return bAllReachedEnd;
}


void FElectraProtronPlayer::FImpl::SendWorkerThreadMessage(FElectraProtronPlayer::FImpl::FWorkerThreadMessage&& InMessage)
{
	WorkMessages.Enqueue(MoveTemp(InMessage));
	WorkMessageSignal.Signal();
}

void FElectraProtronPlayer::FImpl::Exit()
{
	// We are still within our own thread here, so we cannot wait for completion.
	// Use an async task to this if possible.
	if (GIsRunning)
	{
		FMediaRunnable::EnqueueAsyncTask([this]()
		{
			Thread->WaitForCompletion();
			delete Thread;
			SelfDuringTerminate.Reset();
		});
	}
	else
	{
		// Leave the thread dangling, we can't clean it up here.
		SelfDuringTerminate.Reset();
	}
}

uint32 FElectraProtronPlayer::FImpl::Run()
{
	bool bDone = false;
	while(!bDone)
	{
		WorkMessageSignal.WaitTimeoutAndReset(1000 * 20);
		FWorkerThreadMessage Msg;
		while(WorkMessages.Dequeue(Msg))
		{
			switch(Msg.Type)
			{
				case FWorkerThreadMessage::EType::Open:
				{
					const FWorkerThreadMessage::FParamOpen& open(Msg.Param.Get<FWorkerThreadMessage::FParamOpen>());
					InternalOpen(open.Param.Filename);
					// Start loader threads when opening was successful.
					if (LastErrorMessage.IsEmpty())
					{
						// Set the duration of the movie on the sample queue for looping/wrapping purposes.
						CurrentSampleQueueInterface->SetMovieDuration(Duration);
						// If there an initial playback range set then apply it, otherwise set the entire movie.
						SetPlaybackTimeRange(open.Param.InitialPlaybackRange.Get(TRange<FTimespan>::Empty()));
						// By default we start at the beginning of the playback range.
						CurrentPlayPosTime = CurrentPlaybackRange.GetLowerBoundValue();

						VideoLoaderThread.StartThread(open.Param.Filename, SharedPlayParams);
						AudioLoaderThread.StartThread(open.Param.Filename, SharedPlayParams);

						VideoDecoderThread.StartThread(open.Param, SharedPlayParams);
						AudioDecoderThread.StartThread(open.Param, SharedPlayParams);

						// Ensure the track selection is reset to initial values
						TrackSelection = FTrackSelection();

						// Select the first video and audio track by default (if they exist).
						SelectTrack(EMediaTrackType::Video, 0);
						SelectTrack(EMediaTrackType::Audio, 0);
					}
					break;
				}
				case FWorkerThreadMessage::EType::Terminate:
				{
					bDone = true;
					// Hold on to ourselves while we exit the loop.
					// Otherwise, if there are no other owners we may get destroyed too soon on our way out.
					SelfDuringTerminate = AsShared();

					// Stop decoder threads
					AudioDecoderThread.StopThread();
					VideoDecoderThread.StopThread();

					// Stop loader threads
					AudioLoaderThread.StopThread();
					VideoLoaderThread.StopThread();
					break;
				}
				default:
				{
					unimplemented();
					break;
				}
			}
			Msg.CompletionDelegate.ExecuteIfBound(AsShared());
		}

		// Is there a new seek request pending?
		SeekRequestLock.Lock();
		TOptional<FSeekRequest> NewSeekRequest(PendingSeekRequest);
		PendingSeekRequest.Reset();
		SeekRequestLock.Unlock();
		if (NewSeekRequest.IsSet())
		{
			HandleSeekRequest(NewSeekRequest.GetValue());
		}
	}
	return 0;
}


/**
 * Opens the given file and verifies that it can be used.
 */
void FElectraProtronPlayer::FImpl::InternalOpen(const FString& InFilename)
{
	// Open the file.
	TSharedPtr<MP4Utilities::FMP4FileDataReader, ESPMode::ThreadSafe> Reader = MP4Utilities::FMP4FileDataReader::Create();
	if (!Reader->Open(InFilename))
	{
		LastErrorMessage = Reader->GetLastError();
		return;
	}

	// Read the mp4 box structure.
	MP4Utilities::FMP4BoxLocator BoxLocator;
	const TArray<uint32> FirstBoxes { MP4Utilities::MakeBoxAtom('f','t','y','p'), MP4Utilities::MakeBoxAtom('s','t','y','p'), MP4Utilities::MakeBoxAtom('s','i','d','x'), MP4Utilities::MakeBoxAtom('f','r','e','e'), MP4Utilities::MakeBoxAtom('s','k','i','p') };
	const TArray<uint32> ReadBoxes;			// Empty means to read in all boxes except `mdat`
	const TArray<uint32> StopAfterBoxes;	// Empty means to read all boxes and not stop after a specific one.
	TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> RootBoxes;
	if (!BoxLocator.LocateAndReadRootBoxes(RootBoxes, Reader, FirstBoxes, StopAfterBoxes, false, ReadBoxes, MP4Utilities::IMP4DataReaderBase::FCancellationCheckDelegate::CreateLambda([&](){return bAbort;})))
	{
		LastErrorMessage = BoxLocator.GetLastError();
		return;
	}

	// In order to be usable the mp4 needs to have a `moov` box.
	if (!RootBoxes.FindByPredicate([](const TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>& In){ return In->Type == MP4Utilities::MakeBoxAtom('m','o','o','v'); }))
	{
		LastErrorMessage = FString::Printf(TEXT("No `moov` box found in this file. It does not appear to be an mp4/mov file."));
		return;
	}

	// Parse all the root boxes. There are typically no more than 4, maybe 6 unless it is a fragmented mp4.
	ParsedRootBoxes.SetNum(RootBoxes.Num());
	for(int32 i=0; i<RootBoxes.Num(); ++i)
	{
		MP4Boxes::FMP4BoxTreeParser tp;
		if (!tp.ParseBoxTree(RootBoxes[i]))
		{
			LastErrorMessage = FString::Printf(TEXT("Failed to parse the file's box structure."));
			return;
		}
		ParsedRootBoxes[i] = tp.GetBoxTree();
	}
	auto FindRootBox = [&](uint32 InType) -> TSharedPtr<MP4Boxes::FMP4BoxBase>
	{
		auto rb = ParsedRootBoxes.FindByPredicate([Type=InType](const TSharedPtr<MP4Boxes::FMP4BoxBase>& In) { return In->GetType() == Type;});
		return rb ? *rb : nullptr;
	};

	// We need the `moov` box now to determine what is inside this file.
	// Yes, we looked at the presence of it above already as a quick reject before parsing the boxes.
	// Now we need it for real and we know that it needs to be there, so here we go
	TSharedPtr<MP4Boxes::FMP4BoxMOOV, ESPMode::ThreadSafe> MoovBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxMOOV>(FindRootBox(MP4Utilities::MakeBoxAtom('m','o','o','v')));
	// Get the `mvhd` box for the duration of the movie and the timescale other values are measured in.
	auto MvhdBox = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxMVHD>(MP4Utilities::MakeBoxAtom('m','v','h','d'), 0);
	if (!MvhdBox.IsValid())
	{
		LastErrorMessage = FString::Printf(TEXT("No `mvhd` box found. This is not a usable mp4/mov file."));
		return;
	}

	// Check if this is a fragmented file. We do not look for `sidx` or `mfra` boxes since these are purely optional.
	TSharedPtr<MP4Boxes::FMP4BoxMOOF, ESPMode::ThreadSafe> FirstMoofBox = StaticCastSharedPtr<MP4Boxes::FMP4BoxMOOF>(FindRootBox(MP4Utilities::MakeBoxAtom('m','o','o','f')));
	auto MvexBox = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxMVEX>(MP4Utilities::MakeBoxAtom('m','v','e','x'), 0);
	if (FirstMoofBox && !MvexBox)
	{
		LastErrorMessage = FString::Printf(TEXT("No `mvex` box present in a file containing `moof` fragment boxes."));
		return;
	}
	const bool bIsFragmented = MvexBox || FirstMoofBox;

	MovieDuration = MvhdBox->GetDuration();
	if (!MovieDuration.IsValid())
	{
		LastErrorMessage = FString::Printf(TEXT("Duration in `mvhd` box is set to indefinite or is not valid. This is not a usable mp4/mov file."));
		return;
	}
	// If there is a movie extends header box we take the movie duration from there.
	if (bIsFragmented && MvexBox)
	{
		auto MehdBox = MvexBox->FindBoxRecursive<MP4Boxes::FMP4BoxMEHD>(MP4Utilities::MakeBoxAtom('m','e','h','d'), 0);
		/*
			From ISO/IEC 14496-12:2022 section 8.8.2.1
				If movie fragments are present but there is no MediaExtendsHeaderBox
				and the movie duration is 0, the movie duration should be interpreted as indefinite duration.
		*/
		if (MehdBox)
		{
			MovieDuration.SetNumerator(MehdBox->GetFragmentDuration());
		}
		else if (MovieDuration.GetNumerator() == 0)
		{
			LastErrorMessage = FString::Printf(TEXT("Duration in `mvhd` box is set to zero in a fragmented file that is missing a `mehd` box. This is not a usable mp4/mov file."));
			return;
		}
	}

	// Get all the tracks.
	TArray<TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>> AllTracks;
	MoovBox->GetAllBoxInstances(AllTracks, MP4Utilities::MakeBoxAtom('t','r','a','k'));
	// Prepare the internal track structure and check which tracks we can use and which ones we cannot.
	Tracks.SetNum(AllTracks.Num());
	int32 NumTimecodeTracks = 0;
	int64 LongestTrackDuration = -1;
	int64 LongestSupportedTrackDuration = -1;
	int64 ShortestSupportedTrackDuration = TNumericLimits<int64>::Max();
	for(int32 i=0; i<AllTracks.Num(); ++i)
	{
		const TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>& ThisTrakBox(AllTracks[i]);

		Tracks[i] = MakeShared<FTrackInfo>();
		Tracks[i]->TrackBox = ThisTrakBox;

		auto Tkhd = ThisTrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxTKHD>(MP4Utilities::MakeBoxAtom('t','k','h','d'), 1);
		auto Mdhd = ThisTrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxMDHD>(MP4Utilities::MakeBoxAtom('m','d','h','d'), 3);
		if (!Tkhd)
		{
			LastErrorMessage = FString::Printf(TEXT("No `tkhd` box found on track %d. This file cannot be used."), i);
			Tracks.Empty();
			return;
		}
		if (!Mdhd)
		{
			LastErrorMessage = FString::Printf(TEXT("No `mdhd` box found on track %d. This file cannot be used."), i);
			Tracks.Empty();
			return;
		}
		uint32 TrackID = Tracks[i]->TrackID = Tkhd->GetTrackID();
		const bool bIsTrackEnabled = Tkhd->IsEnabled();

		// This track's duration must not be indefinite.
		int64 TrackDuration = Tkhd->GetDuration();
		if (TrackDuration == TNumericLimits<int64>::Max())
		{
			UE_LOGF(LogElectraProtron, Warning, "Track #%u has indefinite duration, ignoring this track.", TrackID);
			continue;
		}

		// If the file is fragmented we collect all the track fragment boxes for this track.
		if (bIsFragmented)
		{
			Tracks[i]->FragmentInfo = MakeShared<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe>();
			if (!Tracks[i]->FragmentInfo->Prepare(ThisTrakBox, MvexBox, ParsedRootBoxes))
			{
				UE_LOGF(LogElectraProtron, Warning, "Fragmented track #%u could not be prepared, ignoring this track.", TrackID);
				continue;
			}

			// The summed duration is in media timescale, so we need to transform it into movie timescale.
			Electra::FTimeFraction MediaDur(Tracks[i]->FragmentInfo->GetTotalDuration(), Mdhd->GetTimescale());
			TrackDuration = MediaDur.GetAsTimebase(MovieDuration.GetDenominator());
			// It is possible that the movie duration is only describing the content contained in the moov box.
			// If that is smaller than the track duration, update the movie duration with the track duration.
			if (MovieDuration.GetNumerator() < TrackDuration)
			{
				MovieDuration.SetNumerator(TrackDuration);
			}
		}

		// Take note of the track with the longest duration (any track, even unsupported ones)
		LongestTrackDuration = LongestTrackDuration < TrackDuration ? TrackDuration : LongestTrackDuration;

		Electra::FCodecTypeFormat& CodecInfo(Tracks[i]->CodecInfo);
		Electra::FDRMTypeFormat& DrmFormat(Tracks[i]->DrmFormat);
		Electra::FDecoderInformation& DecoderInformation(Tracks[i]->DecoderInformation);
		FString Msg;
		if (!ElectraDecodersUtil::MP4::GetTrackFormatInfo(Msg, CodecInfo, DrmFormat, DecoderInformation, ThisTrakBox, TrackID, true))
		{
			UE_LOGF(LogElectraProtron, Warning, "%ls", *Msg);
			continue;
		}
		if (DrmFormat.IsEncrypted())
		{
			UE_LOGF(LogElectraProtron, Verbose, "Encrypted tracks are not handled right now, ignoring track #%u.", TrackID);
			continue;
		}

		/*
			From the standard:
				"Tracks that are marked as not enabled (track_enabled set to 0) shall be ignored and treated as if
				not present."
			However, timecode tracks are commonly marked as disabled by some muxers (e.g. FFmpeg/Lavf).
			We still need them for start timecode extraction.
		*/
		if (!bIsTrackEnabled && CodecInfo.Type != Electra::FCodecTypeFormat::EType::Timecode)
		{
			UE_LOGF(LogElectraProtron, Warning, "Track #%u is flagged as disabled, ignoring this track.", TrackID);
			continue;
		}

		// Timecode tracks are inherently usable.
		if (CodecInfo.Type == Electra::FCodecTypeFormat::EType::Timecode)
		{
			Tracks[i]->bIsUsable = true;
			Tracks[i]->bIsKeyframeOnlyFormat = true;
			++NumTimecodeTracks;
		}
		// Check with the decoder factory if this format can be decoded.
		else if (CodecInfo.Type == Electra::FCodecTypeFormat::EType::Video || CodecInfo.Type == Electra::FCodecTypeFormat::EType::Audio)
		{
			if (DecoderInformation.bIsDecodable)
			{
				Tracks[i]->bIsUsable = true;
				Tracks[i]->bIsKeyframeOnlyFormat = CodecInfo.KeyframeMode == Electra::FCodecTypeFormat::EKeyframeMode::OnlyKeyframes;
				// Take note of the supported track with the longest duration.
				LongestSupportedTrackDuration = LongestSupportedTrackDuration < TrackDuration ? TrackDuration : LongestSupportedTrackDuration;
				// Likewise for the shortest.
				ShortestSupportedTrackDuration = ShortestSupportedTrackDuration > TrackDuration ? TrackDuration : ShortestSupportedTrackDuration;
			}
			else
			{
				UE_LOGF(LogElectraProtron, Warning, "No decoder to handle sample type \"%ls\" of track #%u, ignoring this track.", *CodecInfo.RFC6381, TrackID);
			}
		}
		else if (CodecInfo.Type == Electra::FCodecTypeFormat::EType::Subtitle)
		{
			UE_LOGF(LogElectraProtron, Verbose, "Subtitles in track #%u are not handled right now, ignoring this track.", TrackID);
		}
	}

	Duration = MovieDuration.GetAsTimespan();
	// Check that the duration given in the `mvhd` box is not larger than the longest track is.
	int64 MvhdDur = MovieDuration.GetNumerator();

	if (MvhdDur > LongestTrackDuration)
	{
		UE_LOGF(LogElectraProtron, Warning, "Movie duration in `mvhd` box (%lld) is larger than the longest track (%lld) in the file, adjusting movie duration down.", (long long int)MvhdDur, (long long int)LongestTrackDuration);
		MovieDuration.SetFromND(LongestTrackDuration, MovieDuration.GetDenominator());
		Duration = MovieDuration.GetAsTimespan();
		MvhdDur = LongestTrackDuration;
	}
	MP4Utilities::FFractionalTime EntireMovieDuration(MovieDuration);

	if (CVarProtronForceShortestDuration.GetValueOnAnyThread())
	{
		// Check that the movie duration is not larger than the shortest supported track.
		if (ShortestSupportedTrackDuration < TNumericLimits<int64>::Max() && MvhdDur > ShortestSupportedTrackDuration)
		{
			UE_LOGF(LogElectraProtron, Warning, "Movie duration in `mvhd` box (%lld) is larger than the shortest supported track (%lld) in the file, adjusting movie duration down.", (long long int)MvhdDur, (long long int)ShortestSupportedTrackDuration);
			MovieDuration.SetFromND(ShortestSupportedTrackDuration, MovieDuration.GetDenominator());
			Duration = MovieDuration.GetAsTimespan();
		}
	}

	// If there are timecode tracks, find which tracks references them.
	// If there are none, then - if there is only a single timecode track - we apply it to all other tracks.
	// Other references are of no interest at the moment.
	if (NumTimecodeTracks)
	{
		bool bAnyTrackReferencesTimecodeExplicitly = false;
		for(int32 TrkNum=0; TrkNum<Tracks.Num(); ++TrkNum)
		{
			// We need to check every track, even the ones we cannot use. Otherwise we
			// may think the timecode is not referenced and use it for everything!
			// We do not check if a timecode track references another one as this would be silly.
			if (Tracks[TrkNum]->CodecInfo.Type != Electra::FCodecTypeFormat::EType::Timecode)
			{
				auto Tref = Tracks[TrkNum]->TrackBox->FindBoxRecursive<MP4Boxes::FMP4BoxTREF>(MP4Utilities::MakeBoxAtom('t','r','e','f'), 1);
				if (Tref.IsValid())
				{
					// Get timecode references, if any.
					auto References = Tref->GetEntriesOfType(MP4Utilities::MakeBoxAtom('t','m','c','d'));
					if (References.Num())
					{
						if (References.Num() > 1)
						{
							UE_LOGF(LogElectraProtron, Warning, "Track #%u contains more than one `tmcd` reference box. Using first reference only.", Tracks[TrkNum]->TrackID);
						}
						if (References[0].TrackIDs.Num())
						{
							if (References[0].TrackIDs.Num() > 1)
							{
								UE_LOGF(LogElectraProtron, Warning, "Track #%u references more than one timecode track. Using first reference only.", Tracks[TrkNum]->TrackID);
							}
							// Whether the reference is actually valid or not, a track makes an explicit reference
							// and so we must not assign unreferenced timecode tracks later.
							bAnyTrackReferencesTimecodeExplicitly = true;
							// Either way we need to tag every referenced track.
							for(auto& RefTkId : References[0].TrackIDs)
							{
								TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>* ReferencedTrack = Tracks.FindByPredicate([RefTkId](const TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>& e){ return e->TrackID == RefTkId; });
								if (ReferencedTrack)
								{
									if (!Tracks[TrkNum]->ReferencedTimecodeTrack.IsValid())
									{
										Tracks[TrkNum]->ReferencedTimecodeTrack = *ReferencedTrack;
									}
									(*ReferencedTrack)->IsReferencedByTracks.Add(Tracks[TrkNum]);
								}
								else
								{
									UE_LOGF(LogElectraProtron, Warning, "Track #%u references a non-existing timecode track #%u. Ignoring.", Tracks[TrkNum]->TrackID, RefTkId);
									Tracks[TrkNum]->ReferencedTimecodeTrack.Reset();
								}
							}
						}
					}
				}
			}
		}

		// Now check for tracks that are not explicitly referencing a timecode track, but only when NO track
		// makes an explicit reference. If some do and other don't we do not interfere.
		if (!bAnyTrackReferencesTimecodeExplicitly)
		{
			if (NumTimecodeTracks == 1)
			{
				// Which track is the timecode?
				TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>* TimecodeTrack = Tracks.FindByPredicate([](const TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>& In){ return In->CodecInfo.Type == Electra::FCodecTypeFormat::EType::Timecode; });
				check(TimecodeTrack); // there has to be one, otherwise we would not even get here, but for safeties sake...
				if (TimecodeTrack)
				{
					// We set this up for video tracks only.
					for(int32 i=0; i<Tracks.Num(); ++i)
					{
						if (Tracks[i]->CodecInfo.Type != Electra::FCodecTypeFormat::EType::Video)
						{
							continue;
						}
						Tracks[i]->ReferencedTimecodeTrack = *TimecodeTrack;
						(*TimecodeTrack)->IsReferencedByTracks.Add(Tracks[i]);
					}
				}
			}
			else
			{
				UE_LOGF(LogElectraProtron, Warning, "There are %d timecode tracks that are not referenced by any other track. Ignoring all of them.", NumTimecodeTracks);
			}
		}
	}
	else
	{
		// See if there is an `udta` box in the `moov` that contains `(c)TIM` and `(C)TSC` boxes.
		auto UdtaBox = MoovBox->FindBoxRecursive<MP4Boxes::FMP4BoxUDTA>(MP4Utilities::MakeBoxAtom('u','d','t','a'), 0);
		if (UdtaBox.IsValid())
		{
			auto CTIMBox = UdtaBox->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xa9U,'T','I','M'), 0);
			auto CTSCBox = UdtaBox->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xa9U,'T','S','C'), 0);
			auto CTSZBox = UdtaBox->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xa9U,'T','S','Z'), 0);
			if (CTIMBox.IsValid() && CTSCBox.IsValid() && CTSZBox.IsValid())
			{
				auto GetValue = [](const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& InBox) -> FString
				{
					MP4Utilities::FMP4AtomReader timReader(InBox->GetBoxData());
					FString String;
					uint16 StringLength, UnknownValue;
					if (timReader.Read(StringLength)&&  timReader.Read(UnknownValue))
					{
						if (timReader.ReadString(String, StringLength))
						{
							return String;
						}
					}
					return FString();
				};
				FTrackInfo::FFirstSampleTimecode tc;
				FFrameRate fr;
				tc.Timecode = GetValue(CTIMBox);
				tc.Framerate = FString::Printf(TEXT("%s/%s"), *GetValue(CTSCBox), *GetValue(CTSZBox));
				TOptional<FTimecode> ptc = FTimecode::ParseTimecode(tc.Timecode);
				if (ptc.IsSet() && TryParseString(fr, *tc.Framerate))
				{
					FFrameNumber fn = ptc.GetValue().ToFrameNumber(fr);
					tc.TimecodeValue = (uint32) fn.Value;
				}
				for(int32 i=0; i<Tracks.Num(); ++i)
				{
					if (Tracks[i]->bIsUsable)
					{
						Tracks[i]->FirstSampleTimecode = tc;
					}
				}
			}
		}
	}

	// One last pass to count how many usable tracks per type we have.
	for(int32 TkTypIdx=0; TkTypIdx<UE_ARRAY_COUNT(kCodecTrackIndexMap); ++TkTypIdx)
	{
		for(int32 TkIdx=0; TkIdx<Tracks.Num(); ++TkIdx)
		{
			if (Tracks[TkIdx]->bIsUsable && TkTypIdx==(int32) Tracks[TkIdx]->CodecInfo.Type)
			{
				UsableTrackArrayIndicesByType[TkTypIdx].Emplace(TkIdx);
			}
		}
	}

	// Neither video nor audio?
	if (UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num() == 0 &&
		UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num() == 0)
	{
		LastErrorMessage = FString::Printf(TEXT("This file contains no playable video or audio tracks."));
		return;
	}

	// Prepare the tracks
	for(int32 i=0; i<Tracks.Num(); ++i)
	{
		if (!Tracks[i]->bIsUsable)
		{
			continue;
		}
		Tracks[i]->MP4Track = MP4Boxes::FMP4Track::Create(Tracks[i]->TrackBox, Tracks[i]->FragmentInfo);
		if (!Tracks[i]->MP4Track->Prepare(EntireMovieDuration, MovieDuration))
		{
			LastErrorMessage = Tracks[i]->MP4Track->GetLastError();
			check(!LastErrorMessage.IsEmpty());
			return;
		}
		Tracks[i]->Duration = Tracks[i]->MP4Track->GetMappedTrackDuration().GetAsTimespan();

		// If this is a timecode track we may want to read in the first timecode.
		if (Config.bReadFirstTimecode && Tracks[i]->CodecInfo.Type == Electra::FCodecTypeFormat::EType::Timecode &&
			Tracks[i]->CodecInfo.FourCC == MP4Utilities::MakeBoxAtom('t','m','c','d'))
		{
			auto It = Tracks[i]->MP4Track->CreateIterator();
			if (!It.IsValid())
			{
				LastErrorMessage = Tracks[i]->MP4Track->GetLastError();
				check(!LastErrorMessage.IsEmpty());
				return;
			}
			// Read the first sample.
			int64 SampleSize = It->GetSampleSize();
			int64 SampleFileOffset = It->GetSampleFileOffset();
			if (SampleSize != 4)
			{
				// Not a tmcd sample.
				LastErrorMessage = FString::Printf(TEXT("Invalid timecode sample size."));
				return;
			}
			TArray<uint32> TimecodeBuffer;
			TimecodeBuffer.SetNumUninitialized(Align(SampleSize, sizeof(uint32)));
			int64 NumRead = Reader->ReadData(TimecodeBuffer.GetData(), SampleSize, SampleFileOffset, MP4Utilities::IMP4DataReaderBase::FCancellationCheckDelegate::CreateLambda([&](){return bAbort;}));
			if (NumRead != SampleSize)
			{
				LastErrorMessage = FString::Printf(TEXT("Failed to read first timecode sample."));
				return;
			}
			// Get the timecode description from the codec info.
			Electra::FCodecTypeFormat::FTMCDTimecode TimecodeInfo(Tracks[i]->CodecInfo.Properties.Get<Electra::FCodecTypeFormat::FTMCDTimecode>());
			// Set the first timecode sample on the track.
			FTrackInfo::FFirstSampleTimecode firstTC;
			firstTC.TimecodeValue = MP4Utilities::GetFromBigEndian(TimecodeBuffer[0]);
			firstTC.Framerate = TimecodeInfo.GetFrameRate().ToPrettyText().ToString();
			firstTC.Timecode = TimecodeInfo.ConvertToTimecode(firstTC.TimecodeValue).ToString();
			Tracks[i]->FirstSampleTimecode.Emplace(MoveTemp(firstTC));

			// For convenience also set this on the tracks that reference this timecode track
			for(auto ReferencingTrack : Tracks[i]->IsReferencedByTracks)
			{
				if (auto PinnedRefTrk = ReferencingTrack.Pin())
				{
					PinnedRefTrk->FirstSampleTimecode = Tracks[i]->FirstSampleTimecode;
				}
			}
		}
	}
}

FVariant FElectraProtronPlayer::FImpl::GetMediaInfo(FName InInfoName)
{
	if (InInfoName == ElectraProtronOptionNames::StartTimecodeValue || InInfoName == ElectraProtronOptionNames::StartTimecodeFrameRate ||
		InInfoName == ElectraProtronOptionNames::KeyframeInterval || InInfoName == ElectraProtronOptionNames::VideoDecoderProvider)
	{
		auto ci = CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video);
		int32 SelectedVideoTrackIndex = TrackSelection.SelectedTrackIndex[ci];
		if (SelectedVideoTrackIndex >= 0 && SelectedVideoTrackIndex < UsableTrackArrayIndicesByType[ci].Num())
		{
			const FTrackInfo& ti(*Tracks[UsableTrackArrayIndicesByType[ci][SelectedVideoTrackIndex]]);

			if (InInfoName == ElectraProtronOptionNames::StartTimecodeValue && ti.FirstSampleTimecode.IsSet())
			{
				return FVariant(ti.FirstSampleTimecode.GetValue().Timecode);
			}
			else if (InInfoName == ElectraProtronOptionNames::StartTimecodeFrameRate && ti.FirstSampleTimecode.IsSet())
			{
				return FVariant(ti.FirstSampleTimecode.GetValue().Framerate);
			}
			else if (InInfoName == ElectraProtronOptionNames::KeyframeInterval)
			{
				return FVariant(ti.bIsKeyframeOnlyFormat ? (int32)1 : (int32)0);
			}
			else if (InInfoName == ElectraProtronOptionNames::VideoDecoderProvider && ti.DecoderInformation.ProviderInfo.Name.Len())
			{
				return FVariant(ti.DecoderInformation.ProviderInfo.Name);
			}
		}
	}
	else if (InInfoName == ElectraProtronOptionNames::AudioDecoderProvider)
	{
		auto ci = CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio);
		int32 SelectedAudioTrackIndex = TrackSelection.SelectedTrackIndex[ci];
		if (SelectedAudioTrackIndex >= 0 && SelectedAudioTrackIndex < UsableTrackArrayIndicesByType[ci].Num())
		{
			const FTrackInfo& ti(*Tracks[UsableTrackArrayIndicesByType[ci][SelectedAudioTrackIndex]]);
			if (InInfoName == ElectraProtronOptionNames::AudioDecoderProvider && ti.DecoderInformation.ProviderInfo.Name.Len())
			{
				return FVariant(ti.DecoderInformation.ProviderInfo.Name);
			}
		}
	}
	return FVariant();
}



int32 FElectraProtronPlayer::FImpl::GetNumTracks(EMediaTrackType InTrackType)
{
	switch(InTrackType)
	{
		case EMediaTrackType::Video:
		{
			return UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num();
		}
		case EMediaTrackType::Audio:
		{
			return UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num();
		}
	}
	return 0;
}
int32 FElectraProtronPlayer::FImpl::GetNumTrackFormats(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackIndex >= 0)
	{
		// Every track this player supports, if the track exists, only has a single format.
		switch(InTrackType)
		{
			case EMediaTrackType::Video:
			{
				return InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num() ? 1 : 0;
			}
			case EMediaTrackType::Audio:
			{
				return InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num() ? 1 : 0;
			}
		}
	}
	return 0;
}

int32 FElectraProtronPlayer::FImpl::GetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackIndex >= 0)
	{
		// Every track this player supports, if the track exists, only has a single format.
		switch(InTrackType)
		{
			case EMediaTrackType::Video:
			{
				return InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num() ? 0 : -1;
			}
			case EMediaTrackType::Audio:
			{
				return InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num() ? 0 : -1;
			}
		}
	}
	return -1;
}

FText FElectraProtronPlayer::FImpl::GetTrackDisplayName(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackIndex >= 0)
	{
		switch(InTrackType)
		{
			case EMediaTrackType::Video:
			{
				if (InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num())
				{
					const FTrackInfo& ti(*Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)][InTrackIndex]]);
					FString Name(ti.MP4Track->GetCommonMetadata().Name);
					if (Name.IsEmpty())
					{
						Name = FString::Printf(TEXT("Video track #%u"), ti.TrackID);
					}
					return FText::FromString(Name);
				}
				break;
			}
			case EMediaTrackType::Audio:
			{
				if (InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num())
				{
					const FTrackInfo& ti(*Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)][InTrackIndex]]);
					FString Name(ti.MP4Track->GetCommonMetadata().Name);
					if (Name.IsEmpty())
					{
						Name = FString::Printf(TEXT("Audio track #%u"), ti.TrackID);
					}
					return FText::FromString(Name);
				}
				break;
			}
		}
	}
	return FText();
}

FString FElectraProtronPlayer::FImpl::GetTrackLanguage(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackIndex >= 0)
	{
		switch(InTrackType)
		{
			case EMediaTrackType::Video:
			{
				if (InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num())
				{
					return Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)][InTrackIndex]]->CodecInfo.LanguageTag.Get();
				}
				break;
			}
			case EMediaTrackType::Audio:
			{
				if (InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num())
				{
					return Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)][InTrackIndex]]->CodecInfo.LanguageTag.Get();
				}
				break;
			}
		}
	}
	return FString();
}

FString FElectraProtronPlayer::FImpl::GetTrackName(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackIndex >= 0)
	{
		switch(InTrackType)
		{
			case EMediaTrackType::Video:
			{
				if (InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num())
				{
					return FString::Printf(TEXT("%u"), Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)][InTrackIndex]]->TrackID);
				}
				break;
			}
			case EMediaTrackType::Audio:
			{
				if (InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num())
				{
					return FString::Printf(TEXT("%u"), Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)][InTrackIndex]]->TrackID);
				}
				break;
			}
		}
	}
	return FString();
}

bool FElectraProtronPlayer::FImpl::GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat)
{
	if (InTrackIndex >= 0 && InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num() && InFormatIndex == 0)
	{
		const FTrackInfo& ti(*Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)][InTrackIndex]]);
		const Electra::FCodecTypeFormat::FVideo& vi(ti.CodecInfo.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
		OutFormat.Dim.X = (int32) vi.Width;
		OutFormat.Dim.Y = (int32) vi.Height;
		if (vi.FrameRate.IsValid())
		{
			OutFormat.FrameRate = (float) vi.FrameRate.AsDecimal();
			OutFormat.FrameRates = TRange<float>{ OutFormat.FrameRate };
		}
		else
		{
			OutFormat.FrameRate = 0.0f;
			OutFormat.FrameRates = TRange<float>{ OutFormat.FrameRate };//TRange<float>::Empty();
		}
		OutFormat.TypeName = ti.CodecInfo.HumanReadableFormatInfo;
		return true;
	}
	return false;
}

bool FElectraProtronPlayer::FImpl::GetAudioTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaAudioTrackFormat& OutFormat)
{
	if (InTrackIndex >= 0 && InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num() && InFormatIndex == 0)
	{
		const FTrackInfo& ti(*Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)][InTrackIndex]]);
		const Electra::FCodecTypeFormat::FAudio& ai(ti.CodecInfo.Properties.Get<Electra::FCodecTypeFormat::FAudio>());
		OutFormat.BitsPerSample = 16;
		OutFormat.NumChannels = ai.NumChannels;
		OutFormat.SampleRate = ai.SampleRate;
		OutFormat.TypeName = ti.CodecInfo.HumanReadableFormatInfo;
		return true;
	}
	return false;
}

int32 FElectraProtronPlayer::FImpl::GetSelectedTrack(EMediaTrackType InTrackType)
{
	switch(InTrackType)
	{
		case EMediaTrackType::Video:
		{
			return TrackSelection.SelectedTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)];
		}
		case EMediaTrackType::Audio:
		{
			return TrackSelection.SelectedTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)];
		}
	}
	return -1;
}


bool FElectraProtronPlayer::FImpl::SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackIndex < -1)
	{
		InTrackIndex = -1;
	}
	switch(InTrackType)
	{
		case EMediaTrackType::Video:
		{
			const auto TypeIndex = CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video);
			if (InTrackIndex < UsableTrackArrayIndicesByType[TypeIndex].Num())
			{
				if (TrackSelection.SelectedTrackIndex[TypeIndex] != InTrackIndex)
				{
					TrackSelection.SelectedTrackIndex[TypeIndex] = InTrackIndex;
					TrackSelection.bChanged = true;
					bAreRatesValid = false;
					UpdateTrackLoader(TypeIndex);
					HandleActiveTrackChanges();
				}
				return true;
			}
			break;
		}
		case EMediaTrackType::Audio:
		{
			const auto TypeIndex = CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio);
			if (InTrackIndex < UsableTrackArrayIndicesByType[TypeIndex].Num())
			{
				if (TrackSelection.SelectedTrackIndex[TypeIndex] != InTrackIndex)
				{
					TrackSelection.SelectedTrackIndex[TypeIndex] = InTrackIndex;
					TrackSelection.bChanged = true;
					bAreRatesValid = false;
					UpdateTrackLoader(TypeIndex);
					HandleActiveTrackChanges();
				}
				return true;
			}
			break;
		}
		default:
		{
			break;
		}
	}
	return false;
}

bool FElectraProtronPlayer::FImpl::SetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex, int32 InFormatIndex)
{
	return false;
}

TOptional<FTimespan> FElectraProtronPlayer::FImpl::GetTrackDuration(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackIndex >= 0)
	{
		switch(InTrackType)
		{
			case EMediaTrackType::Video:
			{
				if (InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)].Num())
				{
					return Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)][InTrackIndex]]->Duration;
				}
				break;
			}
			case EMediaTrackType::Audio:
			{
				if (InTrackIndex < UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)].Num())
				{
					return Tracks[UsableTrackArrayIndicesByType[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)][InTrackIndex]]->Duration;
				}
				break;
			}
			default:
			{
				break;
			}
		}
	}
	return TOptional<FTimespan>();
}


bool FElectraProtronPlayer::FImpl::QueryCacheState(EMediaCacheState InState, TRangeSet<FTimespan>& OutTimeRanges)
{
	if (InState == EMediaCacheState::Loading)
	{
		if (TrackSelection.SelectedTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)] >= 0)
		{
			OutTimeRanges = VideoLoaderThread.GetTimeRangesToLoad();
			return true;
		}
		else if (TrackSelection.SelectedTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)] >= 0)
		{
			OutTimeRanges = AudioLoaderThread.GetTimeRangesToLoad();
			return true;
		}
	}
	else if (InState == EMediaCacheState::Loaded)
	{
		if (TrackSelection.SelectedTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)] >= 0)
		{
			TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> sqi(GetCurrentSampleQueueInterface());
			if (sqi.IsValid())
			{
				sqi->GetVideoCache().QueryCacheState(OutTimeRanges);
				return true;
			}
		}
	}
	return false;
}

int32 FElectraProtronPlayer::FImpl::GetSampleCount(EMediaCacheState InState)
{
	return 0;
}

float FElectraProtronPlayer::FImpl::GetRate()
{
	return CurrentRate;
}

bool FElectraProtronPlayer::FImpl::SetRate(float InRate)
{
	HandleActiveTrackChanges();

	IntendedRate = InRate;
	HandleRateChanges();

	return true;
}

FTimespan FElectraProtronPlayer::FImpl::GetTime()
{
	return CurrentPlayPosTime;
}

bool FElectraProtronPlayer::FImpl::SetLooping(bool bInLooping)
{
	bool bOk = true;
	if (bOk && !VideoDecoderThread.SetLooping(bInLooping))
	{
		bOk = false;
	}
	if (bOk && !AudioDecoderThread.SetLooping(bInLooping))
	{
		bOk = false;
	}
	SharedPlayParams->bShouldLoop = bInLooping && bOk;
	return bOk;
}

bool FElectraProtronPlayer::FImpl::IsLooping()
{
	return SharedPlayParams->bShouldLoop;
}

void FElectraProtronPlayer::FImpl::Seek(const FTimespan& InTime, int32 InNewSequenceIndex, const TOptional<int32>& InNewLoopIndex)
{
	FSeekRequest sr;
	sr.NewTime = InTime;
	sr.NewSequenceIndex = InNewSequenceIndex;
	sr.NewLoopIndex = InNewLoopIndex;
	if (CurrentSampleQueueInterface.IsValid())
	{
		CurrentSampleQueueInterface->SeekIssuedTo(InTime, TOptional<int32>(InNewSequenceIndex));
	}
	SeekRequestLock.Lock();
	PendingSeekRequest = MoveTemp(sr);
	SeekRequestLock.Unlock();
	WorkMessageSignal.Signal();
}

TRange<FTimespan> FElectraProtronPlayer::FImpl::GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet)
{
	return InRangeToGet == EMediaTimeRangeType::Absolute ? TRange<FTimespan>(FTimespan(0), GetDuration()) : CurrentPlaybackRange;
}

bool FElectraProtronPlayer::FImpl::SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange)
{
	// For proper validation we need to have the content duration.
	if (Duration <= FTimespan::Zero() || InTimeRange.IsDegenerate() || !InTimeRange.HasLowerBound() || !InTimeRange.HasUpperBound() || InTimeRange.GetLowerBoundValue() > InTimeRange.GetUpperBoundValue())
	{
		return false;
	}
	// If we get an empty range we instead set the range to be the entire movie.
	if (InTimeRange.IsEmpty())
	{
		CurrentPlaybackRange = TRange<FTimespan>(FTimespan(0), GetDuration());
	}
	else
	{
		TRange<FTimespan> r(InTimeRange);
		if (r.GetLowerBoundValue() < FTimespan::Zero())
		{
			UE_LOGF(LogElectraProtron, Warning, "Clamping start of playback range to zero as it was set negative.");
			r.SetLowerBoundValue(FTimespan::Zero());
		}
		if (r.GetUpperBoundValue() > Duration)
		{
			UE_LOGF(LogElectraProtron, Warning, "Clamping end of playback range to movie duration as it was set larger.");
			r.SetUpperBoundValue(Duration);
		}
		CurrentPlaybackRange = MoveTemp(r);
	}
	CurrentSampleQueueInterface->SetPlaybackRange(CurrentPlaybackRange);
	VideoLoaderThread.SetPlaybackRange(CurrentPlaybackRange);
	AudioLoaderThread.SetPlaybackRange(CurrentPlaybackRange);
	VideoDecoderThread.SetPlaybackRange(CurrentPlaybackRange);
	AudioDecoderThread.SetPlaybackRange(CurrentPlaybackRange);
	return true;
}

void FElectraProtronPlayer::FImpl::TickFetch(FTimespan InDeltaTime, FTimespan InTimecode)
{
}

void FElectraProtronPlayer::FImpl::TickInput(FTimespan InDeltaTime, FTimespan InTimecode)
{
	if (LastErrorMessage.IsEmpty())
	{
		if (!VideoDecoderThread.GetLastError().IsEmpty())
		{
			LastErrorMessage = VideoDecoderThread.GetLastError();
		}
		else if (!AudioDecoderThread.GetLastError().IsEmpty())
		{
			LastErrorMessage = AudioDecoderThread.GetLastError();
		}
		else if (!VideoLoaderThread.GetLastError().IsEmpty())
		{
			LastErrorMessage = VideoLoaderThread.GetLastError();
		}
		else if (!AudioLoaderThread.GetLastError().IsEmpty())
		{
			LastErrorMessage = AudioLoaderThread.GetLastError();
		}
	}

	auto sqi = GetCurrentSampleQueueInterface();
	if (sqi.IsValid())
	{
		FMediaTimeStamp ts = sqi->GetLastHandedOutTimestamp();
		const bool bIsVideoActive = TrackSelection.ActiveTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)] != -1;
		const bool bIsAudioActive = TrackSelection.ActiveTrackIndex[CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)] != -1;
		if (ts.IsValid())
		{
			FTimespan NewPos = ts.GetTime();
			if (bIsVideoActive)
			{
				VideoDecoderThread.SetEstimatedPlaybackTime(NewPos);
				CurrentPlayPosTime = NewPos;
			}
			if (!bIsAudioActive)
			{
				AudioDecoderThread.SetEstimatedPlaybackTime(NewPos);
			}
		}
		else if (bIsAudioActive)
		{
			FTimespan NewPos = AudioDecoderThread.GetEstimatedPlaybackTime();
			VideoDecoderThread.SetEstimatedPlaybackTime(NewPos);
			CurrentPlayPosTime = NewPos;
		}
	}
}


void FElectraProtronPlayer::FImpl::UpdateTrackLoader(int32 InCodecTypeIndex)
{
	if (TrackSelection.SelectedTrackIndex[InCodecTypeIndex] >= 0)
	{
		auto Track = Tracks[UsableTrackArrayIndicesByType[InCodecTypeIndex][TrackSelection.SelectedTrackIndex[InCodecTypeIndex]]];
		check(Track.IsValid());
		if (Track.IsValid())
		{
			// Do we have a track sample buffer for this track?
			if (!TrackSampleBuffers.Contains(Track->TrackID))
			{
				// No, create it now.
				TSharedPtr<FMP4TrackSampleBuffer, ESPMode::ThreadSafe> tsb = MakeShared<FMP4TrackSampleBuffer, ESPMode::ThreadSafe>();
				tsb->TrackAndCodecInfo = Track;
				tsb->TrackID = Track->TrackID;
				TrackSampleBuffers.Emplace(Track->TrackID, MoveTemp(tsb));
			}

			if (InCodecTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
			{
				VideoLoaderThread.RequestLoad(TrackSampleBuffers[Track->TrackID], CurrentPlayPosTime);
			}
			else if (InCodecTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio))
			{
				AudioLoaderThread.RequestLoad(TrackSampleBuffers[Track->TrackID], CurrentPlayPosTime);
			}
		}
	}
}



IMediaSamples::EFetchBestSampleResult FElectraProtronPlayer::FImpl::FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& InTimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bInReverse, bool bInConsistentResult)
{
	TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> sqi(GetCurrentSampleQueueInterface());
	if (sqi.IsValid())
	{
		FProtronVideoCache::EGetResult gr = sqi->GetVideoCache().GetFrame(OutSample, InTimeRange, IsLooping(), bInReverse, bInConsistentResult);
		if (gr == FProtronVideoCache::EGetResult::Hit)
		{
			sqi->UpdateNextExpectedTimestamp(OutSample, bInReverse, IsLooping());
			sqi->UpdateLastHandedOutTimestamp(OutSample);
			return IMediaSamples::EFetchBestSampleResult::Ok;
		}
		else if (gr == FProtronVideoCache::EGetResult::PurgedEmpty)
		{
			sqi->ResetCurrentTimestamps();
			return IMediaSamples::EFetchBestSampleResult::PurgedToEmpty;
		}
	}
	return IMediaSamples::EFetchBestSampleResult::NoSample;
}
bool FElectraProtronPlayer::FImpl::FetchAudio(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> sqi(GetCurrentSampleQueueInterface());
	return sqi.IsValid() ? sqi->GetCurrentSampleQueue()->FetchAudio(InTimeRange, OutSample) : false;
}
bool FElectraProtronPlayer::FImpl::FetchCaption(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	return false;
}
bool FElectraProtronPlayer::FImpl::FetchMetadata(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample)
{
	return false;
}
bool FElectraProtronPlayer::FImpl::FetchSubtitle(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	return false;
}
void FElectraProtronPlayer::FImpl::FlushSamples()
{
}
void FElectraProtronPlayer::FImpl::SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex)
{
	TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> sqi(GetCurrentSampleQueueInterface());
	if (sqi.IsValid())
	{
		sqi->GetCurrentSampleQueue()->SetMinExpectedNextSequenceIndex(InNextSequenceIndex);
	}
}
bool FElectraProtronPlayer::FImpl::PeekVideoSampleTime(FMediaTimeStamp& OutTimeStamp)
{
	TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> sqi(GetCurrentSampleQueueInterface());
	return sqi.IsValid() ? sqi->PeekVideoSampleTime(OutTimeStamp) : false;
}
bool FElectraProtronPlayer::FImpl::CanReceiveVideoSamples(uint32 InNum) const
{
	return true;
}
bool FElectraProtronPlayer::FImpl::CanReceiveAudioSamples(uint32 InNum) const
{
	TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> sqi(GetCurrentSampleQueueInterface());
	return sqi.IsValid() ? sqi->CanEnqueueAudioSample() : true;
}
bool FElectraProtronPlayer::FImpl::CanReceiveSubtitleSamples(uint32 InNum) const
{
	return true;
}
bool FElectraProtronPlayer::FImpl::CanReceiveCaptionSamples(uint32 InNum) const
{
	return true;
}
bool FElectraProtronPlayer::FImpl::CanReceiveMetadataSamples(uint32 InNum) const
{
	return true;
}
int32 FElectraProtronPlayer::FImpl::NumAudioSamples() const
{
	return 0;
}
int32 FElectraProtronPlayer::FImpl::NumCaptionSamples() const
{
	return 0;
}
int32 FElectraProtronPlayer::FImpl::NumMetadataSamples() const
{
	return 0;
}
int32 FElectraProtronPlayer::FImpl::NumSubtitleSamples() const
{
	return 0;
}
int32 FElectraProtronPlayer::FImpl::NumVideoSamples() const
{
	return 0;
}