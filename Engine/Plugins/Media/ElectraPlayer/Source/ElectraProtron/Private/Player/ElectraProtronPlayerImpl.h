// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ElectraProtronPlayer.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Core/MediaEventSignal.h"
#include "Containers/Queue.h"
#include "Misc/TVariant.h"
#include "Misc/Optional.h"
#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "MP4Utilities.h"
#include "MP4DataReader.h"
#include "MP4Boxes.h"
#include "MP4Track.h"
#include "CodecTypeFormat.h"
#include "ElectraDecodersPlatformResources.h"
#include "MediaSamples.h"
#include "ElectraTextureSample.h"
#include "Utils/AudioChannelMapper.h"
#include "Utils/VideoDecoderHelpers.h"
#include "ElectraProtronPlayerCache.h"
#include "Utilities/BCP47-Helpers.h"



/**
 * Private player implementation managed through a thread-safe pointer and a worker thread
 * to not block the game thread.
 */
class FElectraProtronPlayer::FImpl : public TSharedFromThis<FElectraProtronPlayer::FImpl, ESPMode::ThreadSafe>, public FRunnable
{
public:
	class FSampleQueueInterface : public TSharedFromThis<FSampleQueueInterface, ESPMode::ThreadSafe>
	{
	public:
		FSampleQueueInterface(int32 InNumVideoFramesToCacheAhead, int32 InNumVideoFramesToCacheBehind)
			: SampleQueue(MakeShared<FMediaSamples, ESPMode::ThreadSafe>())
			, NumVideoFramesToCache(InNumVideoFramesToCacheAhead + InNumVideoFramesToCacheBehind)
		{
			// We need to have some future frames to mimick the behavior of the FMediaSample struct.
			check(InNumVideoFramesToCacheAhead >= 4);
			// And we also need to retain some old samples, which is the whole point of having a cache.
			check(InNumVideoFramesToCacheBehind >= 4);
			VideoCache.SetMaxFramesToCache(InNumVideoFramesToCacheAhead, InNumVideoFramesToCacheBehind);
		}

		int32 GetMaxVideoFramesToCache()
		{
			return NumVideoFramesToCache;
		}

		void SetMovieDuration(FTimespan InDuration)
		{
			Duration = InDuration;
			VideoCache.SetPlaybackRange(TRange<FTimespan>(FTimespan(0), InDuration));
		}

		FTimespan GetMovieDuration()
		{
			return Duration;
		}

		void SetPlaybackRange(TRange<FTimespan> InRange)
		{
			PlaybackRange = MoveTemp(InRange);
		}

		TRange<FTimespan> GetPlaybackRange()
		{
			return PlaybackRange;
		}

		void SetPlaybackRate(float InNewRate)
		{
			PlaybackRate = InNewRate;
			VideoCache.SetPlaybackRate(InNewRate);
		}

		void SeekIssuedTo(FTimespan InToTime, TOptional<int32> InNextSequenceIndex)
		{
			MinSeqIdx = InNextSequenceIndex;
			SampleQueue->SetMinExpectedNextSequenceIndex(InNextSequenceIndex);
			VideoCache.SeekIssuedTo(InToTime);
			FScopeLock lock(&TimestampLock);
			NextExpectedTimestamp.Invalidate();
			LastHandedOutTimestamp.Invalidate();
			bVideoSampleWillBeReady = false;
			bAudioSampleWillBeReady = false;
		}

		void SetVideoTrackEnabled(bool bInEnabled)
		{
			bIsVideoTrackEnabled = bInEnabled;
		}
		void SetAudioTrackEnabled(bool bInEnabled)
		{
			bIsAudioTrackEnabled = bInEnabled;
		}
		bool IsVideoTrackEnabled() const
		{
			return bIsVideoTrackEnabled;
		}
		bool IsAudioTrackEnabled() const
		{
			return bIsAudioTrackEnabled;
		}
		void VideoSampleWillBeReady()
		{
			bVideoSampleWillBeReady = true;
		}
		void AudioSampleWillBeReady()
		{
			bAudioSampleWillBeReady = true;
		}
		bool AreRequiredSamplesReady() const
		{
			return (!bIsVideoTrackEnabled || (bIsVideoTrackEnabled && bVideoSampleWillBeReady)) &&
				   (!bIsAudioTrackEnabled || (bIsAudioTrackEnabled && bAudioSampleWillBeReady));
		}

		bool CanEnqueueVideoSample(FTimespan InPTS)
		{
			return VideoCache.CanAccept(InPTS);
		}

		bool CanEnqueueAudioSample()
		{
			return SampleQueue->CanReceiveAudioSamples(1);
		}

		void EnqueueVideoSample(const TSharedRef<IMediaTextureSample, ESPMode::ThreadSafe>& InSample, FTimespan InRawPTS, FTimespan InRawDuration)
		{
			if (InSample->GetTime().GetSequenceIndex() < MinSeqIdx.Get(0))
			{
				return;
			}
			VideoCache.AddFrame(InSample, InRawPTS, InRawDuration);

			FScopeLock lock(&TimestampLock);
			if (!NextExpectedTimestamp.IsValid())
			{
				NextExpectedTimestamp = InSample->GetTime();
			}
		}

		void EnqueueAudioSample(const TSharedRef<IMediaAudioSample, ESPMode::ThreadSafe>& InSample)
		{
			SampleQueue->AddAudio(InSample);
		}

		TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> GetCurrentSampleQueue()
		{
			return SampleQueue;
		}

		bool PeekVideoSampleTime(FMediaTimeStamp& OutTimeStamp)
		{
			FScopeLock lock(&TimestampLock);
			if (NextExpectedTimestamp.IsValid())
			{
				OutTimeStamp = NextExpectedTimestamp;
				return true;
			}
			return false;
		}

		void UpdateLastHandedOutTimestamp(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& InSample)
		{
			FScopeLock lock(&TimestampLock);
			LastHandedOutTimestamp = InSample->GetTime();
		}

		FMediaTimeStamp GetLastHandedOutTimestamp()
		{
			FScopeLock lock(&TimestampLock);
			return LastHandedOutTimestamp;
		}

		void UpdateNextExpectedTimestamp(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& InSample, bool bInReverse, bool bInIsLooping)
		{
			FScopeLock lock(&TimestampLock);
			if (!bInReverse)
			{
				NextExpectedTimestamp = InSample->GetTime() + InSample->GetDuration();
				if (NextExpectedTimestamp.GetTime() >= PlaybackRange.GetUpperBoundValue())
				{
					if (bInIsLooping)
					{
						NextExpectedTimestamp -= PlaybackRange.GetUpperBoundValue() - PlaybackRange.GetLowerBoundValue();
						NextExpectedTimestamp.AdjustLoopIndex(1);
					}
					else
					{
						// Set to the time of the last sample. This must be less than the end of the
						// playback range to work.
						NextExpectedTimestamp.SetTime(InSample->GetTime().GetTime());
					}
				}
			}
			else
			{
				NextExpectedTimestamp = InSample->GetTime() - InSample->GetDuration();
				if (NextExpectedTimestamp.GetTime() < PlaybackRange.GetLowerBoundValue())
				{
					if (bInIsLooping)
					{
						NextExpectedTimestamp += PlaybackRange.GetUpperBoundValue() - PlaybackRange.GetLowerBoundValue();
						NextExpectedTimestamp.AdjustLoopIndex(-1);
					}
					else
					{
						// Set to the lower bound of the playback range
						NextExpectedTimestamp.SetTime(PlaybackRange.GetLowerBoundValue());
					}
				}
			}
		}

		void ResetCurrentTimestamps()
		{
			FScopeLock lock(&TimestampLock);
			LastHandedOutTimestamp.Invalidate();
			NextExpectedTimestamp.Invalidate();
		}

		FProtronVideoCache& GetVideoCache()
		{
			return VideoCache;
		}

	private:
		FProtronVideoCache VideoCache;
		TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> SampleQueue;
		FCriticalSection TimestampLock;
		FMediaTimeStamp NextExpectedTimestamp;
		FMediaTimeStamp LastHandedOutTimestamp;
		TOptional<int32> MinSeqIdx;
		FTimespan Duration;
		TRange<FTimespan> PlaybackRange;
		float PlaybackRate = 0.0f;
		int32 NumVideoFramesToCache = 0;
		bool bIsVideoTrackEnabled = false;
		bool bIsAudioTrackEnabled = false;
		bool bVideoSampleWillBeReady = false;
		bool bAudioSampleWillBeReady = false;
	};


	// Define pointer type to get around the ',' in the name that confuses the delegate declaration macro.
	using ImplPointer = TSharedPtr<FImpl, ESPMode::ThreadSafe>;
	DECLARE_DELEGATE_OneParam(FCompletionDelegate, ImplPointer);

	FImpl();
	~FImpl();
	struct FOpenParam
	{
		FString Filename;
		TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> SampleQueueInterface;
		TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> TexturePool;
		TSharedPtr<FElectraAudioSamplePool, ESPMode::ThreadSafe> AudioSamplePool;
		TOptional<TRange<FTimespan>> InitialPlaybackRange;
	};
	void Open(const FOpenParam& InParam, FCompletionDelegate InCompletionDelegate);
	void Close(FCompletionDelegate InCompletionDelegate);
	FString GetLastError();
	bool HasReachedEnd();

	// Implementation methods from FElectraProtronPlayer
	FTimespan GetDuration()
	{ return Duration; }
	FVariant GetMediaInfo(FName InInfoName);
	bool GetAudioTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaAudioTrackFormat& OutFormat);
	int32 GetNumTracks(EMediaTrackType InTrackType);
	int32 GetNumTrackFormats(EMediaTrackType InTrackType, int32 InTrackIndex);
	int32 GetSelectedTrack(EMediaTrackType InTrackType);
	FText GetTrackDisplayName(EMediaTrackType InTrackType, int32 InTrackIndex);
	int32 GetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex);
	FString GetTrackLanguage(EMediaTrackType InTrackType, int32 InTrackIndex);
	FString GetTrackName(EMediaTrackType InTrackType, int32 InTrackIndex);
	bool GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat);
	bool SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex);
	bool SetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex, int32 InFormatIndex);
	TOptional<FTimespan> GetTrackDuration(EMediaTrackType InTrackType, int32 InTrackIndex);
	bool QueryCacheState(EMediaCacheState InState, TRangeSet<FTimespan>& OutTimeRanges);
	int32 GetSampleCount(EMediaCacheState InState);

	TRangeSet<float> GetSupportedRates(EMediaRateThinning InThinning);
	float GetRate();
	bool SetRate(float InRate);
	FTimespan GetTime();
	bool SetLooping(bool bInLooping);
	bool IsLooping();
	void Seek(const FTimespan& InTime, int32 InNewSequenceIndex, const TOptional<int32>& InNewLoopIndex);
	TRange<FTimespan> GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet);
	bool SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange);

	void TickFetch(FTimespan InDeltaTime, FTimespan InTimecode);
	void TickInput(FTimespan InDeltaTime, FTimespan InTimecode);

	// Inherited from FRunnable
	uint32 Run() override;
	void Exit() override;

	// Forwarded IMediaSamples interface methods from the enclosing FElectraProtronPlayer
	EFetchBestSampleResult FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bInReverse, bool bInConsistentResult);
	bool FetchAudio(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample);
	bool FetchCaption(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample);
	bool FetchMetadata(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample);
	bool FetchSubtitle(TRange<FMediaTimeStamp> InTimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample);
	void FlushSamples();
	void SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex);
	bool PeekVideoSampleTime(FMediaTimeStamp& OutTimeStamp);
	bool CanReceiveVideoSamples(uint32 InNum) const;
	bool CanReceiveAudioSamples(uint32 InNum) const;
	bool CanReceiveSubtitleSamples(uint32 InNum) const;
	bool CanReceiveCaptionSamples(uint32 InNum) const;
	bool CanReceiveMetadataSamples(uint32 InNum) const;
	int32 NumAudioSamples() const;
	int32 NumCaptionSamples() const;
	int32 NumMetadataSamples() const;
	int32 NumSubtitleSamples() const;
	int32 NumVideoSamples() const;

private:
	static constexpr int32 CodecTypeIndex(const Electra::FCodecTypeFormat::EType InType)
	{ return (int32) InType; }

	struct FConfig
	{
		FTimespan DurationCacheAhead { ETimespan::TicksPerSecond * 2 };
		FTimespan DurationCacheBehind { ETimespan::TicksPerSecond * 1 };
		int32 NextKeyframeThresholdMillis { 2 };
		bool bReadFirstTimecode = true;
		bool bReadSampleTimecode = true;
	};

	struct FSharedPlayParams
	{
		float DesiredPlayRate = 0.0f;
		float PlaybackDirection = 0.0f;
		bool bShouldLoop = false;
	};

	struct FSeekRequest
	{
		FTimespan NewTime;
		int32 NewSequenceIndex = 0;
		TOptional<int32> NewLoopIndex;
	};

	struct FWorkerThreadMessage
	{
		enum class EType
		{
			Nop,
			Open,
			Terminate
		};
		struct FParamOpen
		{
			FOpenParam Param;
		};
		struct FParamTerminate
		{
		};

		EType Type = EType::Nop;
		ImplPointer Self;
		FCompletionDelegate CompletionDelegate;
		TVariant<FParamOpen, FParamTerminate> Param;
	};

	struct FTrackInfo
	{
		TArray<TWeakPtr<FTrackInfo, ESPMode::ThreadSafe>> IsReferencedByTracks;
		Electra::FCodecTypeFormat CodecInfo;
		Electra::FDRMTypeFormat DrmFormat;
		Electra::FDecoderInformation DecoderInformation;

		FTimespan Duration;
		TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe> TrackBox;
		TSharedPtr<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe> FragmentInfo;
		TSharedPtr<MP4Boxes::FMP4Track, ESPMode::ThreadSafe> MP4Track;
		TWeakPtr<FTrackInfo, ESPMode::ThreadSafe> ReferencedTimecodeTrack;
		uint32 TrackID = 0;
		bool bIsUsable = false;
		bool bIsKeyframeOnlyFormat = false;
		struct FFirstSampleTimecode
		{
			FString Timecode;
			FString Framerate;
			uint32 TimecodeValue = 0;
		};
		TOptional<FFirstSampleTimecode> FirstSampleTimecode;
	};

	struct FTrackSelection
	{
		int32 SelectedTrackIndex[4] {-1, -1, -1, -1};
		int32 ActiveTrackIndex[4] {-1, -1, -1, -1};
		bool bChanged = false;
	};

	struct FMP4Sample
	{
		TArray<uint8> Data;
		FTimespan DTS;
		FTimespan PTS;
		FTimespan EffectiveDTS;
		FTimespan EffectivePTS;
		FTimespan Duration;
		int64 SizeInBytes;
		int64 OffsetInFile;
		uint32 TrackID;
		uint32 SampleNumber;
		bool bIsSyncOrRap;
		TOptional<FTimecode> AssociatedTimecode;
		TOptional<FFrameRate> AssociatedTimecodeFramerate;
	};
	using FMP4SamplePtr = TSharedPtr<FMP4Sample, ESPMode::ThreadSafe>;

	using FTrackIterator = TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe>;

	struct FMP4TrackSampleBuffer
	{
		FCriticalSection Lock;
		TRangeSet<uint32> SampleRanges;
		TSortedMap<uint32, FMP4SamplePtr> SampleMap;
		TSharedPtr<FTrackInfo, ESPMode::ThreadSafe> TrackAndCodecInfo;
		uint32 TrackID;

		// Used by the sample loader
		TRange<FTimespan> CurrentPlaybackRange;
		FTrackIterator FirstRangeSampleIt;
		FTrackIterator LastRangeSampleIt;
	};
	using FMP4TrackSampleBufferPtr = TSharedPtr<FMP4TrackSampleBuffer, ESPMode::ThreadSafe>;


	void StartThread();
	void SendWorkerThreadMessage(FWorkerThreadMessage&& InMessage);

	void InternalOpen(const FString& InFilename);

	void UpdateTrackLoader(int32 InCodecTypeIndex);

	void HandleActiveTrackChanges();
	void HandleRateChanges();
	void HandleSeekRequest(const FSeekRequest& InSeekRequest);

	const int32 kCodecTrackIndexMap[(int32) Electra::FCodecTypeFormat::EType::MAX] =
	{
		CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video),
		CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio),
		CodecTypeIndex(Electra::FCodecTypeFormat::EType::Subtitle),
		CodecTypeIndex(Electra::FCodecTypeFormat::EType::Timecode)
	};

	FConfig Config;

	FRunnableThread* Thread = nullptr;
	ImplPointer SelfDuringTerminate;
	FMediaEvent WorkMessageSignal;
	TQueue<FWorkerThreadMessage, EQueueMode::Mpsc> WorkMessages;
	FString LastErrorMessage;
	bool bAbort = false;
	TArray<TSharedPtr<MP4Boxes::FMP4BoxBase>> ParsedRootBoxes;
	TArray<TSharedPtr<FTrackInfo, ESPMode::ThreadSafe>> Tracks;
	TArray<TArray<int32>> UsableTrackArrayIndicesByType;

	MP4Utilities::FFractionalTime MovieDuration;
	FTimespan Duration;						// already converted `MovieDuration` for faster access
	FTrackSelection TrackSelection;
	TRangeSet<float> UnthinnedRates;
	TRangeSet<float> ThinnedRates;
	bool bAreRatesValid = false;
	FTimespan CurrentPlayPosTime;
	TRange<FTimespan> CurrentPlaybackRange;
	float CurrentRate = 0.0f;
	float IntendedRate = 0.0f;

	FCriticalSection SeekRequestLock;
	TOptional<FSeekRequest> PendingSeekRequest;

	TMap<uint32, FMP4TrackSampleBufferPtr> TrackSampleBuffers;

	TSharedPtr<FSharedPlayParams, ESPMode::ThreadSafe> SharedPlayParams;

	TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> CurrentSampleQueueInterface;
	TSharedPtr<FSampleQueueInterface, ESPMode::ThreadSafe> GetCurrentSampleQueueInterface() const
	{
		return CurrentSampleQueueInterface;
	}


	// ===== Loader and decoder threads =====

	DECLARE_DELEGATE_RetVal_ThreeParams(FMP4SamplePtr, FGetSampleDlg, FMP4TrackSampleBufferPtr /*FromBuffer*/, const FTrackIterator& /*AtIterator*/, int32 /*WaitMicros*/);

	class FLoaderThread : public FRunnable
	{
	public:
		FLoaderThread(const FConfig& InConfig, int32 InCodecTypeIndex)
			: Config(InConfig), LoaderTypeIndex(InCodecTypeIndex)
		{ }

		FString GetLastError()
		{ return LastErrorMessage; }

		// Inherited from FRunnable
		uint32 Run() override;

		// Thread start/stop
		void StartThread(const FString& InFilename, const TSharedPtr<FSharedPlayParams, ESPMode::ThreadSafe>& InSharedPlayParams);
		void StopThread();

		// Called by the player
		void SetPlaybackRange(TRange<FTimespan> InRange);
		void RequestLoad(FMP4TrackSampleBufferPtr InTrackSampleBuffer, FTimespan InTime);
		TRangeSet<FTimespan> GetTimeRangesToLoad();

		// Called by the decoder
		FMP4SamplePtr GetSample(FMP4TrackSampleBufferPtr InFromBuffer, const FTrackIterator& InAtIterator, int32 InWaitMicroseconds);
	private:
		struct FOpenRequest
		{
			FString Filename;
			TSharedPtr<FSharedPlayParams, ESPMode::ThreadSafe> SharedPlayParams;
		};
		struct FLoadRequest
		{
			void Empty()
			{
				TrackSampleBuffer.Reset();
				StartAtIterator.Reset();
				UpdateAtIterator.Reset();
			}
			FMP4TrackSampleBufferPtr TrackSampleBuffer;
			FTrackIterator StartAtIterator;
			FTrackIterator UpdateAtIterator;
		};

		const FConfig& Config;
		int32 LoaderTypeIndex;

		FRunnableThread* Thread = nullptr;
		FMediaEvent WorkSignal;
		volatile bool bTerminateThread = false;

		TSharedPtr<FSharedPlayParams, ESPMode::ThreadSafe> SharedPlayParams;
		TSharedPtr<MP4Utilities::FMP4FileDataReader, ESPMode::ThreadSafe> Reader;
		FString LastErrorMessage;

		FCriticalSection LoadRequestLock;
		FOpenRequest OpenRequest;
		FLoadRequest PendingLoadRequest;
		FLoadRequest ActiveLoadRequest;
		TRange<FTimespan> PlaybackRange;
		FCriticalSection TimeRangeLock;
		TRangeSet<FTimespan> TimeRangesToLoad;
		volatile int32 LoadRequestDirty = -1;

		enum class ELoadResult
		{
			Ok,
			Error,
			Canceled
		};

		ELoadResult Load(FMP4TrackSampleBufferPtr InTrackSampleBuffer, FTrackIterator InAtIterator);
		FMP4SamplePtr RetrieveSample(const FMP4TrackSampleBufferPtr& InTrackSampleBuffer, const FTrackIterator& InSampleIt, const FTrackIterator& InOptionalTimecodeIt, const Electra::FCodecTypeFormat::FTMCDTimecode& InTimecodeInfo);

		struct FSampleRange
		{
			TRangeSet<FTimespan> TimeRanges;
			TRangeSet<uint32> SampleRanges;
			int32 NumSamplesAfter = 0;
			int32 NumSamplesBefore = 0;
			int32 NumRemainingToLoadAfter = 0;
			int32 NumRemainingToLoadBefore = 0;
		};
		void CalcRangeToLoad(FSampleRange& OutRange, const FMP4TrackSampleBufferPtr& InTrackSampleBuffer, const FTrackIterator& InSampleIt);

		void GetUnreferencedFrames(TArray<uint32>& OutFramesToRemove, const FMP4TrackSampleBufferPtr& InTrackSampleBuffer, const FSampleRange& InActiveSampleRange);
	};

	class FDecoderThread : public FRunnable
	{
	public:
		FDecoderThread(const FConfig& InConfig, int32 InCodecTypeIndex)
			: Config(InConfig), DecoderTypeIndex(InCodecTypeIndex)
		{
			VideoDecoderOutputPool = MakeShareable(new FElectraTextureSamplePool);
		}

		// Inherited from FRunnable
		uint32 Run() override;

		void StartThread(const FOpenParam& InParam, const TSharedPtr<FSharedPlayParams, ESPMode::ThreadSafe>& InSharedPlayParams);
		void StopThread();

		FString GetLastError()
		{ return LastErrorMessage; }

		// Sets playback rate.
		void SetRate(float InNewRate);
		// Whether or not to loop.
		bool SetLooping(bool bInLooping);

		// Sets the active playback range.
		void SetPlaybackRange(TRange<FTimespan> InRange);

		// Did decoding reach the last sample (or first sample when going in reverse)?
		bool HasReachedEnd();

		// Pauses decoding. Used when switching buffers or setting a new position.
		void Pause();
		// Resumes decoding. Decoder starts in paused state and needs to be resumed after setting buffer and time.
		void Resume();
		// Set the buffer to get the samples to decode from.
		void SetSampleBuffer(const FMP4TrackSampleBufferPtr& InTrackSampleBuffer, FGetSampleDlg InGetSampleDelegate);
		void DisconnectSampleBuffer();

		bool IsPaused();
		void PauseForSeek();
		void ResumeAfterSeek();
		bool IsPausedForSeek();
		void SetTime(FTimespan InTime, int32 InSeqIdx, TOptional<int32> InLoopIdx);
		void SetEstimatedPlaybackTime(FTimespan InTime);
		FTimespan GetEstimatedPlaybackTime();

		void Flush(const TSharedPtr<FMediaEvent, ESPMode::ThreadSafe>& InFlushedSignal);
	private:

		void UpdateTrackSampleDurationMap();
		void HandlePlaybackRangeChanges();
		FTimespan ClampTimeIntoPlaybackRange(const FTimespan& InTime);
		void UpdateTrackIterator(const FTimespan& InForTime);
		void StepTrackIterator();
		bool CreateDecoder();
		void DestroyDecoder();
		void DecodeOneFrame();
		bool HandleOutputFrame();
		void FlushForEndOrLooping();
		void PerformFlush();

		struct FInDecoder
		{
			TMap<FString, FVariant> CSDOptions;
			IElectraDecoder::FInputAccessUnit DecAU;
			TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> BSI;
			FMP4SamplePtr Sample;
			TArray<uint8> DataCopy;
			int32 SequenceIndex = 0;
			int32 LoopIndex = 0;
		};

		struct FPendingBufferChange
		{
			FCriticalSection Lock;
			TSharedPtr<FMP4TrackSampleBuffer, ESPMode::ThreadSafe> NewTrackSampleBuffer;
			FGetSampleDlg NewGetSampleDelegate;
			bool bIsSet = false;
		};

		struct FPendingSeek
		{
			FCriticalSection Lock;
			FTimespan NewTime;
			int32 NewSeqIdx = 0;
			TOptional<int32> NewLoopIdx;
			bool bIsSet = false;
		};

		struct FPendingPlayRange
		{
			FCriticalSection Lock;
			TRange<FTimespan> NewRange;
			bool bIsSet = false;
		};

		const FConfig& Config;
		int32 DecoderTypeIndex;
		FRunnableThread* Thread = nullptr;
		FMediaEvent WorkSignal;
		bool bTerminate = false;
		FString LastErrorMessage;
		double MinLoopSleepTime = 0.0;

		FOpenParam Params;
		TSharedPtr<FSharedPlayParams, ESPMode::ThreadSafe> SharedPlayParams;
		TSharedPtr<FElectraProtronPlayer::FImpl::FMP4TrackSampleBuffer, ESPMode::ThreadSafe> TrackSampleBuffer;
		TSortedMap<FTimespan, FTimespan> SampleTimeToDurationMap;
		FGetSampleDlg GetSampleDelegate;
		FTrackIterator TrackIterator;
		FTrackIterator FirstRangeSampleIt;
		FTrackIterator LastRangeSampleIt;


		FCriticalSection TimeLock;
		FTimespan CurrentTime;
		TRange<FTimespan> PlaybackRange;
		float CurrentRate = 0.0f;
		float IntendedRate = 0.0f;
		float PlaybackDirection = 0.0f;
		bool bShouldLoop = false;
		bool bReachedEnd = false;
		volatile bool bIsPaused = true;
		volatile bool bPausedForSeek = false;

		TSharedPtr<FMediaEvent, ESPMode::ThreadSafe> FlushedSignal;
		volatile bool bFlushPending = false;
		volatile bool bIsDrainingAtEOS = false;

		FPendingBufferChange PendingBufferChange;

		FPendingPlayRange PendingPlayRangeChange;

		FPendingSeek PendingSeek;
		TOptional<FTimespan> SeekTimeToHandleTo;
		TOptional<FTimespan> SeekTimeToDecodeTo;
		int32 SeekTimeNumFramesDecoded = 0;
		int32 SeekTimeNumFramesSkipped = 0;

		TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> DecoderInstance;
		TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> DecoderBitstreamProcessor;

		TMap<FString, FVariant> CurrentCodecSpecificData;
		FElectraDecodersPlatformResources::IDecoderPlatformResource* DecoderPlatformResource = nullptr;
		TUniquePtr<FInDecoder> CurrentInputSample;
		TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> CurrentDecoderOutput;
		TUniquePtr<FInDecoder> InputForCurrentDecoderOutput;
		TMap<uint64, TUniquePtr<FInDecoder>> InDecoderInput;
		TOptional<Electra::MPEG::FColorimetryHelper> CurrentColorimetry;
		TOptional<Electra::MPEG::FHDRHelper> CurrentHDR;
		Electra::FAudioChannelMapper AudioChannelMapper;

		uint64 NextUserValue = 0;
		int32 SequenceIndex = 0;
		int32 LoopIndex = 0;
		bool bWaitForSyncSample = true;
		bool bWarnedMissingSyncSample = false;

		TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> VideoDecoderOutputPool;
	};

	FLoaderThread VideoLoaderThread {Config, CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)};
	FLoaderThread AudioLoaderThread {Config, CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)};

	FDecoderThread VideoDecoderThread {Config, CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video)};
	FDecoderThread AudioDecoderThread {Config, CodecTypeIndex(Electra::FCodecTypeFormat::EType::Audio)};
};
