// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraProtronPlayerImpl.h"
#include "ElectraProtronPrivate.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "CodecTypeFormat.h"



void FElectraProtronPlayer::FImpl::FLoaderThread::StartThread(const FString& InFilename, const TSharedPtr<FSharedPlayParams, ESPMode::ThreadSafe>& InSharedPlayParams)
{
	if (!Thread)
	{
		bTerminateThread = false;
		OpenRequest.Filename = InFilename;
		OpenRequest.SharedPlayParams = InSharedPlayParams;
		WorkSignal.Signal();
		Thread = FRunnableThread::Create(this, TEXT("Electra Protron Loader"), 0, TPri_Normal);
	}
}

void FElectraProtronPlayer::FImpl::FLoaderThread::StopThread()
{
	if (Thread)
	{
		bTerminateThread = true;
		Thread->WaitForCompletion();
	}
}

uint32 FElectraProtronPlayer::FImpl::FLoaderThread::Run()
{
// TODO: clamp look ahead/behind values to 0 if negative and an internal limit if too large.
	while(!bTerminateThread)
	{
		WorkSignal.WaitTimeoutAndReset(1000 * 20);

		bool bNeedToLoad = false;

		FScopeLock lock(&LoadRequestLock);
		// Open a file?
		if (!OpenRequest.Filename.IsEmpty())
		{
			SharedPlayParams = MoveTemp(OpenRequest.SharedPlayParams);
			check(SharedPlayParams.IsValid());
			Reader = MP4Utilities::FMP4FileDataReader::Create();
			FString Filename = MoveTemp(OpenRequest.Filename);
			if (!Reader->Open(Filename))
			{
				// Failure is not really an option seeing as how we already opened the file
				// successfully in Open().
				check(!"how could this fail?");
				LastErrorMessage = Reader->GetLastError();
				Reader.Reset();
			}
		}
		// New load request trigger by user interaction?
		else if (PendingLoadRequest.StartAtIterator.IsValid())
		{
			// Only when there hasn't been an error yet.
			if (LastErrorMessage.IsEmpty())
			{
				ActiveLoadRequest = MoveTemp(PendingLoadRequest);
				bNeedToLoad = true;
			}
		}
		// New update request?
		else if (ActiveLoadRequest.UpdateAtIterator.IsValid())
		{
			check(!ActiveLoadRequest.StartAtIterator.IsValid());
			ActiveLoadRequest.StartAtIterator = MoveTemp(ActiveLoadRequest.UpdateAtIterator);
			bNeedToLoad = true;
		}

		if (bNeedToLoad)
		{
			FMP4TrackSampleBufferPtr TrackSampleBuffer = ActiveLoadRequest.TrackSampleBuffer;
			FTrackIterator StartAtIterator = MoveTemp(ActiveLoadRequest.StartAtIterator);
			LoadRequestDirty = -1;
			check(StartAtIterator.IsValid());
			lock.Unlock();
			ELoadResult Result = Load(TrackSampleBuffer, StartAtIterator);
			if (Result == ELoadResult::Error)
			{
				if (LastErrorMessage.IsEmpty())
				{
					LastErrorMessage = FString::Printf(TEXT("Error loading media samples"));
				}
			}
		}
	}
	PendingLoadRequest.Empty();
	ActiveLoadRequest.Empty();
	return 0;
}


void FElectraProtronPlayer::FImpl::FLoaderThread::CalcRangeToLoad(FSampleRange& OutRange, const FMP4TrackSampleBufferPtr& InTrackSampleBuffer, const FTrackIterator& InSampleIt)
{
	OutRange.NumSamplesAfter = 0;
	OutRange.NumSamplesBefore = 0;
	OutRange.NumRemainingToLoadAfter = 0;
	OutRange.NumRemainingToLoadBefore = 0;

	// Given a track iterator, figure out the samples we need to load prior and following the iterator's current position.
	check(InSampleIt->IsValid());
	check(InTrackSampleBuffer->FirstRangeSampleIt.IsValid() && InTrackSampleBuffer->LastRangeSampleIt.IsValid());
	if (!InSampleIt->IsValid() || !InTrackSampleBuffer->FirstRangeSampleIt.IsValid() || !InTrackSampleBuffer->LastRangeSampleIt.IsValid())
	{
		return;
	}

	const uint32 TrackTimescale = InSampleIt->GetTimescale();
	check(TrackTimescale);

	int64 FwdDurNeeded = Electra::FTimeFraction(Config.DurationCacheAhead).GetAsTimebase(TrackTimescale);
	int64 RevDurNeeded = Electra::FTimeFraction(Config.DurationCacheBehind).GetAsTimebase(TrackTimescale);
	// See if the combined cache ahead and behind duration encompasses the entire playback range.
	if (FwdDurNeeded + RevDurNeeded >= InTrackSampleBuffer->LastRangeSampleIt->GetEffectiveDTS().GetNumerator() - InTrackSampleBuffer->FirstRangeSampleIt->GetEffectiveDTS().GetNumerator())
	{
		const uint32 NumTrackSamples = InTrackSampleBuffer->LastRangeSampleIt->GetSampleNumber() + 1 - InTrackSampleBuffer->FirstRangeSampleIt->GetSampleNumber();
		OutRange.NumSamplesAfter = SharedPlayParams->PlaybackDirection >= 0.0f ? NumTrackSamples * 3 / 4 : NumTrackSamples / 4;
		OutRange.NumSamplesBefore = NumTrackSamples - OutRange.NumSamplesAfter;
		OutRange.TimeRanges.Add(InTrackSampleBuffer->CurrentPlaybackRange);
		OutRange.SampleRanges.Add(TRange<uint32>(InTrackSampleBuffer->FirstRangeSampleIt->GetSampleNumber(), InTrackSampleBuffer->LastRangeSampleIt->GetSampleNumber() + 1));
		return;
	}

	if (SharedPlayParams->PlaybackDirection < 0.0f)
	{
		Swap(FwdDurNeeded, RevDurNeeded);
	}

	uint32 StartSampleNum = InSampleIt->GetSampleNumber();
	int64 DurHandled = 0;
	FTrackIterator FwdTkIt(InSampleIt->Clone());
	int64 StartDTS = FwdTkIt->GetEffectiveDTS().GetNumerator();
	while(DurHandled < FwdDurNeeded)
	{
		const int64 dur = FwdTkIt->GetDuration().GetNumerator();
		DurHandled += dur;
		++OutRange.NumSamplesAfter;
		if (!FwdTkIt->NextEffective() || FwdTkIt->GetSampleNumber() >= InTrackSampleBuffer->LastRangeSampleIt->GetSampleNumber())
		{
			OutRange.TimeRanges.Add(TRange<FTimespan>(Electra::FTimeFraction(StartDTS, TrackTimescale).GetAsTimespan(), Electra::FTimeFraction(FwdTkIt->GetEffectiveDTS().GetNumerator() + dur, TrackTimescale).GetAsTimespan()));
			OutRange.SampleRanges.Add(TRange<uint32>(StartSampleNum, FwdTkIt->GetSampleNumber()+1));
			FwdTkIt = InTrackSampleBuffer->FirstRangeSampleIt->Clone();
			StartDTS = FwdTkIt->GetEffectiveDTS().GetNumerator();
			StartSampleNum = FwdTkIt->GetSampleNumber();
		}
	}
	OutRange.TimeRanges.Add(TRange<FTimespan>(Electra::FTimeFraction(StartDTS, TrackTimescale).GetAsTimespan(), Electra::FTimeFraction(FwdTkIt->GetEffectiveDTS().GetNumerator(), TrackTimescale).GetAsTimespan()));
	OutRange.SampleRanges.Add(TRange<uint32>(StartSampleNum, FwdTkIt->GetSampleNumber()));

	// Reverse scanning
	const bool bNeedKeyframe = !InTrackSampleBuffer->TrackAndCodecInfo->bIsKeyframeOnlyFormat;
	DurHandled = 0;
	FTrackIterator RevTkIt(InSampleIt->Clone());
	int64 EndDTS = RevTkIt->GetEffectiveDTS().GetNumerator();
	uint32 EndSampleNum = RevTkIt->GetSampleNumber();
	bool bHaveEnough = DurHandled >= RevDurNeeded;
	while(!bHaveEnough)
	{
		if (RevTkIt->GetSampleNumber() <= InTrackSampleBuffer->FirstRangeSampleIt->GetSampleNumber() || !RevTkIt->PrevEffective())
		{
			TRange<uint32> SmpRng(RevTkIt->GetSampleNumber(), EndSampleNum);
			if (!SmpRng.IsEmpty())
			{
				int64 DTS = RevTkIt->GetEffectiveDTS().GetNumerator();
				FTimespan ts = Electra::FTimeFraction(DTS, TrackTimescale).GetAsTimespan();
				FTimespan te = Electra::FTimeFraction(EndDTS, TrackTimescale).GetAsTimespan();
				OutRange.TimeRanges.Add(TRange<FTimespan>(ts, te));
				OutRange.SampleRanges.Add(SmpRng);
			}
			RevTkIt = InTrackSampleBuffer->LastRangeSampleIt->Clone();
			EndDTS = RevTkIt->GetEffectiveDTS().GetNumerator() + RevTkIt->GetDuration().GetNumerator();
			EndSampleNum = RevTkIt->GetSampleNumber() + 1;
		}
		const int64 dur = RevTkIt->GetDuration().GetNumerator();
		DurHandled += dur;
		++OutRange.NumSamplesBefore;
		bHaveEnough = DurHandled >= RevDurNeeded;
		if (bHaveEnough && bNeedKeyframe)
		{
			bHaveEnough = RevTkIt->IsSyncOrRAPSample();
		}
	}
	TRange<uint32> SmpRng(RevTkIt->GetSampleNumber(), EndSampleNum);
	if (!SmpRng.IsEmpty())
	{
		int64 DTS = RevTkIt->GetEffectiveDTS().GetNumerator();
		FTimespan ts = Electra::FTimeFraction(DTS, TrackTimescale).GetAsTimespan();
		FTimespan te = Electra::FTimeFraction(EndDTS, TrackTimescale).GetAsTimespan();
		OutRange.TimeRanges.Add(TRange<FTimespan>(ts, te));
		OutRange.SampleRanges.Add(TRange<uint32>(RevTkIt->GetSampleNumber(), EndSampleNum));
	}
}


void FElectraProtronPlayer::FImpl::FLoaderThread::GetUnreferencedFrames(TArray<uint32>& OutFramesToRemove, const FMP4TrackSampleBufferPtr& InTrackSampleBuffer, const FSampleRange& InActiveSampleRange)
{
	TArray<uint32> FramesInMap;
	InTrackSampleBuffer->Lock.Lock();
	FramesInMap.Reserve(InTrackSampleBuffer->SampleMap.Num());
	InTrackSampleBuffer->SampleMap.GenerateKeyArray(FramesInMap);
	InTrackSampleBuffer->Lock.Unlock();
	for(int32 i=0, iMax=FramesInMap.Num(); i<iMax; ++i)
	{
		if (!InActiveSampleRange.SampleRanges.Contains(FramesInMap[i]))
		{
			OutFramesToRemove.Add(FramesInMap[i]);
		}
	}
}


FElectraProtronPlayer::FImpl::FMP4SamplePtr FElectraProtronPlayer::FImpl::FLoaderThread::GetSample(FMP4TrackSampleBufferPtr InFromBuffer, const FTrackIterator& InAtIterator, int32 InWaitMicroseconds)
{
	InFromBuffer->Lock.Lock();
	uint32 SampleNum = InAtIterator->GetSampleNumber();
	FMP4SamplePtr Sample = InFromBuffer->SampleMap.FindRef(SampleNum);
	InFromBuffer->Lock.Unlock();

	// If the buffer is still the one that is active we need to trigger a fetch of new samples
	// regardless of whether we got (and thus consumed) the sample asked for.
	// We should have the sample ready. If not then triggering the load request is especially important.
	LoadRequestLock.Lock();
	if (ActiveLoadRequest.TrackSampleBuffer == InFromBuffer)
	{
		ActiveLoadRequest.UpdateAtIterator = InAtIterator->Clone();
		LoadRequestDirty = (int32) SampleNum;
		WorkSignal.Signal();
	}
	LoadRequestLock.Unlock();

	return Sample;
}

TRangeSet<FTimespan> FElectraProtronPlayer::FImpl::FLoaderThread::GetTimeRangesToLoad()
{
	FScopeLock lock(&TimeRangeLock);
	return TimeRangesToLoad;
}

void FElectraProtronPlayer::FImpl::FLoaderThread::SetPlaybackRange(TRange<FTimespan> InRange)
{
	TimeRangeLock.Lock();
	PlaybackRange = MoveTemp(InRange);
	TimeRangeLock.Unlock();
}

void FElectraProtronPlayer::FImpl::FLoaderThread::RequestLoad(FMP4TrackSampleBufferPtr InTrackSampleBuffer, FTimespan InTime)
{
	if (!Reader.IsValid() || !InTrackSampleBuffer.IsValid())
	{
		return;
	}

	// Did the playback range change?
	TimeRangeLock.Lock();
	TRange<FTimespan> RangeNow(PlaybackRange);
	TimeRangeLock.Unlock();
	if (RangeNow != InTrackSampleBuffer->CurrentPlaybackRange)
	{
		InTrackSampleBuffer->CurrentPlaybackRange = RangeNow;
		// Locate the first and last sample numbers for the range
		FTrackIterator RangeIt = InTrackSampleBuffer->TrackAndCodecInfo->MP4Track->CreateIteratorAtKeyframe(RangeNow.GetLowerBoundValue(), FTimespan(0));
		if (!RangeIt.IsValid())
		{
			LastErrorMessage = InTrackSampleBuffer->TrackAndCodecInfo->MP4Track->GetLastError();
			UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
			return;
		}
		InTrackSampleBuffer->FirstRangeSampleIt = RangeIt->Clone();

		// Move forward until we reach the end or both the effective DTS *and* PTS are greater or equal than the end of the range.
		// We need to look at both DTS and PTS because the effective PTS can be smaller than the effective DTS due to composition time offsets.
		while(!RangeIt->IsLastEffective())
		{
			if (RangeIt->GetEffectiveDTS().GetAsTimespan() >= RangeNow.GetUpperBoundValue() && RangeIt->GetEffectivePTS().GetAsTimespan() >= RangeNow.GetUpperBoundValue())
			{
				// We want the last iterator to represent the last sample included in the playback range,
				// so we need to step one back here as we are currently outside the range.
				RangeIt->PrevEffective();
				break;
			}
			RangeIt->NextEffective();
		}
		InTrackSampleBuffer->LastRangeSampleIt = MoveTemp(RangeIt);
	}

	FTrackIterator TkIt = InTrackSampleBuffer->TrackAndCodecInfo->MP4Track->CreateIteratorAtKeyframe(InTime, FTimespan::FromMilliseconds(InTrackSampleBuffer->TrackAndCodecInfo->bIsKeyframeOnlyFormat ? 0 : Config.NextKeyframeThresholdMillis));
	if (!TkIt.IsValid())
	{
		LastErrorMessage = InTrackSampleBuffer->TrackAndCodecInfo->MP4Track->GetLastError();
		UE_LOGF(LogElectraProtron, Error, "%ls", *LastErrorMessage);
		return;
	}

	LoadRequestLock.Lock();
	PendingLoadRequest.Empty();
	PendingLoadRequest.TrackSampleBuffer = MoveTemp(InTrackSampleBuffer);
	PendingLoadRequest.StartAtIterator = MoveTemp(TkIt);
	LoadRequestDirty = -2;
	WorkSignal.Signal();
	LoadRequestLock.Unlock();
}

FElectraProtronPlayer::FImpl::FLoaderThread::ELoadResult FElectraProtronPlayer::FImpl::FLoaderThread::Load(FMP4TrackSampleBufferPtr InTrackSampleBuffer, FTrackIterator InAtIterator)
{
	// Determine the range of samples to load.
	check(InAtIterator.IsValid());
	check(InTrackSampleBuffer.IsValid());
	FSampleRange RangeToLoad;
	CalcRangeToLoad(RangeToLoad, InTrackSampleBuffer, InAtIterator);
	// Need to load something...
	check(RangeToLoad.NumSamplesAfter || RangeToLoad.NumSamplesBefore);
	if (!(RangeToLoad.NumSamplesAfter || RangeToLoad.NumSamplesBefore))
	{
		return ELoadResult::Ok;
	}

	TimeRangeLock.Lock();
	TimeRangesToLoad = RangeToLoad.TimeRanges;
	TimeRangeLock.Unlock();

	// Get the frames that are not referenced now we have to evict from the data map.
	TArray<uint32> FramesToRemove;
	GetUnreferencedFrames(FramesToRemove, InTrackSampleBuffer, RangeToLoad);

	// Are timecodes from another track referenced?
	FTrackIterator FwdTkItTC, RevTkItTC;
	Electra::FCodecTypeFormat::FTMCDTimecode TimecodeInfo;
	if (Config.bReadSampleTimecode && LoaderTypeIndex == CodecTypeIndex(Electra::FCodecTypeFormat::EType::Video))
	{
		if (auto TimecodeTrack = InTrackSampleBuffer->TrackAndCodecInfo->ReferencedTimecodeTrack.Pin())
		{
			// The timecode track needs to have as many samples as this track, otherwise there would be
			// a mismatch somewhere and the timecode couldn't be used.
			if (TimecodeTrack->MP4Track->GetNumberOfSamples() == InTrackSampleBuffer->TrackAndCodecInfo->MP4Track->GetNumberOfSamples())
			{
				// Get the timecode description from the codec info.
				TimecodeInfo = TimecodeTrack->CodecInfo.Properties.Get<Electra::FCodecTypeFormat::FTMCDTimecode>();
				FwdTkItTC = TimecodeTrack->MP4Track->CreateIterator(InAtIterator->GetSampleNumber());
				if (FwdTkItTC.IsValid())
				{
					RevTkItTC = FwdTkItTC->Clone();
				}
			}
		}
	}

	// Calculate the ratio of samples to fetch ahead vs. fetch behind.
	// We want to fetch more samples in the direction we're going than from
	// where we came.
	// For this we do not need to look at the playback direction, we just take
	// the number of samples to load in either direction since that is determined
	// by the current play direction. Whichever is the one with more samples is the
	// direction we are going in.
	double fwd = RangeToLoad.NumSamplesAfter;
	double rev = RangeToLoad.NumSamplesBefore;
	int32 RatioF = (fwd > rev && rev > 0.0) ? FMath::CeilToInt(fwd / rev) : 1;
	int32 RatioR = (rev > fwd && fwd > 0.0) ? FMath::CeilToInt(rev / fwd) : 1;

	FTrackIterator FwdTkIt(InAtIterator->Clone());
	FTrackIterator RevTkIt(InAtIterator->Clone());

	RangeToLoad.NumRemainingToLoadAfter = RangeToLoad.NumSamplesAfter;
	RangeToLoad.NumRemainingToLoadBefore = RangeToLoad.NumSamplesBefore;
	const int32 kMinFramesToLoad = 2;
	while(RangeToLoad.NumRemainingToLoadAfter || RangeToLoad.NumRemainingToLoadBefore)
	{
		// Remove one old frame now.
		if (FramesToRemove.Num())
		{
			InTrackSampleBuffer->Lock.Lock();
			InTrackSampleBuffer->SampleMap.Remove(FramesToRemove.Last());
			InTrackSampleBuffer->Lock.Unlock();
			FramesToRemove.Pop(EAllowShrinking::No);
		}

		for(int32 i=RatioF; i>0 && RangeToLoad.NumRemainingToLoadAfter; --i)
		{
			FMP4SamplePtr Sample = RetrieveSample(InTrackSampleBuffer, FwdTkIt, FwdTkItTC, TimecodeInfo);
			// Abort loading immediately or when we have loaded at least the minimum number of required frames?
			if (LoadRequestDirty < -1 || (LoadRequestDirty >= 0 && (-RangeToLoad.NumRemainingToLoadAfter + RangeToLoad.NumSamplesAfter > kMinFramesToLoad)))
			{
				//UE_LOGF(LogTemp, Log, "Load canceled with %d/%d fwd and %d/%d rev remaining", RangeToLoad.NumRemainingToLoadAfter, RangeToLoad.NumSamplesAfter, RangeToLoad.NumRemainingToLoadBefore, RangeToLoad.NumSamplesBefore);
				return ELoadResult::Canceled;
			}
			if (!Sample.IsValid())
			{
				return ELoadResult::Error;
			}

			if (FwdTkItTC.IsValid())
			{
				FwdTkItTC->Next();
			}
			if (FwdTkIt->GetSampleNumber() >= InTrackSampleBuffer->LastRangeSampleIt->GetSampleNumber() || !FwdTkIt->NextEffective())
			{
				FwdTkIt = InTrackSampleBuffer->FirstRangeSampleIt->Clone();
				FwdTkItTC.Reset();
				if (auto TimecodeTrack = InTrackSampleBuffer->TrackAndCodecInfo->ReferencedTimecodeTrack.Pin())
				{
					FwdTkItTC = TimecodeTrack->MP4Track->CreateIterator(FwdTkIt->GetSampleNumber());
				}
			}
			--RangeToLoad.NumRemainingToLoadAfter;
		}

		for(int32 i=RatioR; i>0 && RangeToLoad.NumRemainingToLoadBefore; --i)
		{
			if (RevTkItTC.IsValid())
			{
				RevTkItTC->Prev();
			}
			if (RevTkIt->GetSampleNumber() <= InTrackSampleBuffer->FirstRangeSampleIt->GetSampleNumber() || !RevTkIt->PrevEffective())
			{
				RevTkIt = InTrackSampleBuffer->LastRangeSampleIt->Clone();
				RevTkItTC.Reset();
				if (auto TimecodeTrack = InTrackSampleBuffer->TrackAndCodecInfo->ReferencedTimecodeTrack.Pin())
				{
					RevTkItTC = TimecodeTrack->MP4Track->CreateIterator(RevTkIt->GetSampleNumber());
				}
			}

			FMP4SamplePtr Sample = RetrieveSample(InTrackSampleBuffer, RevTkIt, RevTkItTC, TimecodeInfo);
			// Abort loading immediately or when we have loaded at least the minimum number of required frames?
			if (LoadRequestDirty < -1 || (LoadRequestDirty >= 0 && (-RangeToLoad.NumRemainingToLoadBefore + RangeToLoad.NumSamplesBefore > kMinFramesToLoad)))
			{
				//UE_LOGF(LogTemp, Log, "Load canceled with %d/%d fwd and %d/%d rev remaining", RangeToLoad.NumRemainingToLoadAfter, RangeToLoad.NumSamplesAfter, RangeToLoad.NumRemainingToLoadBefore, RangeToLoad.NumSamplesBefore);
				return ELoadResult::Canceled;
			}
			if (!Sample.IsValid())
			{
				return ELoadResult::Error;
			}
			--RangeToLoad.NumRemainingToLoadBefore;
		}
	}
	// Remove any remaining old frames now.
	InTrackSampleBuffer->Lock.Lock();
	for(int32 i=0,iMax=FramesToRemove.Num(); i<iMax; ++i)
	{
		InTrackSampleBuffer->SampleMap.Remove(FramesToRemove[i]);
	}
	InTrackSampleBuffer->Lock.Unlock();

	return ELoadResult::Ok;
}

TSharedPtr<FElectraProtronPlayer::FImpl::FMP4Sample, ESPMode::ThreadSafe> FElectraProtronPlayer::FImpl::FLoaderThread::RetrieveSample(const FMP4TrackSampleBufferPtr& InTrackSampleBuffer, const FTrackIterator& InSampleIt, const FTrackIterator& InOptionalTimecodeIt, const Electra::FCodecTypeFormat::FTMCDTimecode& InTimecodeInfo)
{
	InTrackSampleBuffer->Lock.Lock();
	FMP4SamplePtr* CurrentSample = InTrackSampleBuffer->SampleMap.Find(InSampleIt->GetSampleNumber());
	InTrackSampleBuffer->Lock.Unlock();
	if (CurrentSample)
	{
		check((*CurrentSample)->DTS == InSampleIt->GetDTS().GetAsTimespan());
		check((*CurrentSample)->PTS == InSampleIt->GetPTS().GetAsTimespan());
		check((*CurrentSample)->EffectiveDTS == InSampleIt->GetEffectiveDTS().GetAsTimespan());
		check((*CurrentSample)->EffectivePTS == InSampleIt->GetEffectivePTS().GetAsTimespan());
		check((*CurrentSample)->Duration == InSampleIt->GetDurationAsTimespan());
		check((*CurrentSample)->SizeInBytes == InSampleIt->GetSampleSize());
		check((*CurrentSample)->OffsetInFile == InSampleIt->GetSampleFileOffset());
		check((*CurrentSample)->TrackID == InSampleIt->GetTrackID());
		check((*CurrentSample)->SampleNumber == InSampleIt->GetSampleNumber());
		check((*CurrentSample)->bIsSyncOrRap == InSampleIt->IsSyncOrRAPSample());
		return *CurrentSample;
	}

	auto CheckAbort = MP4Utilities::IMP4DataReaderBase::FCancellationCheckDelegate::CreateLambda([&]()
	{
		// We do NOT abort loading of a frame!
		return false;
	});

	FMP4SamplePtr Sample = MakeShared<FMP4Sample, ESPMode::ThreadSafe>();
	Sample->DTS = InSampleIt->GetDTS().GetAsTimespan();
	Sample->PTS = InSampleIt->GetPTS().GetAsTimespan();
	Sample->EffectiveDTS = InSampleIt->GetEffectiveDTS().GetAsTimespan();
	Sample->EffectivePTS = InSampleIt->GetEffectivePTS().GetAsTimespan();
	Sample->Duration = InSampleIt->GetDurationAsTimespan();
	Sample->SizeInBytes = InSampleIt->GetSampleSize();
	Sample->OffsetInFile = InSampleIt->GetSampleFileOffset();
	Sample->TrackID = InSampleIt->GetTrackID();
	Sample->SampleNumber = InSampleIt->GetSampleNumber();
	Sample->bIsSyncOrRap = InSampleIt->IsSyncOrRAPSample();
	Sample->Data.SetNumUninitialized(Sample->SizeInBytes);
	int64 NumRead = Reader->ReadData(Sample->Data.GetData(), Sample->SizeInBytes, Sample->OffsetInFile, CheckAbort);
	if (NumRead != Sample->SizeInBytes)
	{
		return nullptr;
	}

	// Optionally read the timecode sample from the associated track
	if (InOptionalTimecodeIt.IsValid())
	{
		int64 tcSampleSize = InOptionalTimecodeIt->GetSampleSize();
		if (tcSampleSize == 4)
		{
			TArray<uint32> TimecodeBuffer;
			TimecodeBuffer.SetNumUninitialized(Align(tcSampleSize, 4));
			NumRead = Reader->ReadData(TimecodeBuffer.GetData(), tcSampleSize, InOptionalTimecodeIt->GetSampleFileOffset(), CheckAbort);
			if (NumRead == tcSampleSize)
			{
				Sample->AssociatedTimecode = InTimecodeInfo.ConvertToTimecode(MP4Utilities::GetFromBigEndian(TimecodeBuffer[0]));
				Sample->AssociatedTimecodeFramerate = InTimecodeInfo.GetFrameRate();
			}
		}
	}

	uint32 SampleNumber = Sample->SampleNumber;
	FScopeLock lock(&InTrackSampleBuffer->Lock);
	InTrackSampleBuffer->SampleRanges.Add(TRange<uint32>(SampleNumber, SampleNumber+1));
	InTrackSampleBuffer->SampleMap.Emplace(SampleNumber, Sample);
	return Sample;
}
