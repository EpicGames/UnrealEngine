// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ElectraProtronFactory : ModuleRules
    {
        public ElectraProtronFactory(ReadOnlyTargetRules Target) : base(Target)
        {
		    bLegalToDistributeObjectCode = true;

			DynamicallyLoadedModuleNames.AddRange(
                new string[] {
                    "Media",
                    "ElectraProtron"
                });

            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "MediaAssets",
                });

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
					"ElectraProtron"
				});

            PrivateDependencyModuleNames.Add("ElectraBase");

            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                });

            if (Target.Type == TargetType.Editor)
            {
                DynamicallyLoadedModuleNames.Add("Settings");
                PrivateIncludePathModuleNames.Add("Settings");
            }

            if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
            {
                PublicDefinitions.Add("_CRT_SECURE_NO_WARNINGS");
            }
        }
    }
}
