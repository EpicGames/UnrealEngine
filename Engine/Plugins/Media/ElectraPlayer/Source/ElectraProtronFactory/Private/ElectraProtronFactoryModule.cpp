// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraProtronFactoryPrivate.h"
#include "ElectraProtronFactorySettings.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IMediaModule.h"
#include "IMediaOptions.h"
#include "IMediaPlayerFactory.h"
#include "Modules/ModuleManager.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "Internationalization/Internationalization.h"
#include "UObject/NameTypes.h"

#include "Utilities/URLParser.h"

#include "IElectraProtronModule.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#endif

DEFINE_LOG_CATEGORY(LogElectraProtronFactory);

#define LOCTEXT_NAMESPACE "ElectraProtronFactoryModule"

class ElectraProtronFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:
	ElectraProtronFactoryModule() { }

public:
	// IMediaPlayerFactory interface

	bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		// Split the URL apart.
		Electra::FURL_RFC3986 UrlParser;
		if (!UrlParser.Parse(Url))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("MalformedURI", "The URI '{0}' could not be parsed"), FText::FromString(Url)));
			}
		}
		// Check scheme
		FString Scheme = UrlParser.GetScheme();
		if (Scheme.IsEmpty())
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}
			return 0;
		}
		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}
			return 0;
		}

		// Check for known extensions
		TArray<FString> PathComponents;
		UrlParser.GetPathComponents(PathComponents);
		FString LowerCaseExtension = PathComponents.Num() ? FPaths::GetExtension(PathComponents.Last().ToLower()) : FString();
		// If the extension is known, we are confident that we can play this.
		// At this point there is no information provided on the codecs used in the media,
		// so we cannot check for this.
		if (SupportedFileExtensions.Contains(LowerCaseExtension))
		{
			// See if Protron is preferred over Electra for the current executable.
			const UElectraProtronFactorySettings* Settings = GetDefault<UElectraProtronFactorySettings>();
			const bool bIsInGame = !GIsEditor || IsRunningGame();
			if (Settings && ((bIsInGame && Settings->bPreferProtronInGame) || (!bIsInGame && Settings->bPreferProtronInEditor)))
			{
				// Electra's confidence score is 100 (and most other players are 80).
				return 101;
			}
			
			// Low score to select other players instead.
			return 1;
		}
		return 0;
	}

	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto PlayerModule = FModuleManager::LoadModulePtr<IElectraProtronModule>("ElectraProtron");
		return (PlayerModule != nullptr) ? PlayerModule->CreatePlayer(EventSink) : nullptr;
	}

	FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Electra Protron mp4 playback");
	}

	FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("ElectraProtron"));
		return PlayerName;
	}

	FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0x2899727b, 0xfc934ccb, 0x94119db7, 0x185741d8);
		return PlayerPluginGUID;
	}

	const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::AudioSamples) ||
				(Feature == EMediaFeature::AudioTracks) ||
				(Feature == EMediaFeature::VideoSamples) ||
				(Feature == EMediaFeature::VideoTracks));
	}

public:

	// IModuleInterface interface

	void StartupModule() override
	{
		// supported platforms
		MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		check(MediaModule);

		AddSupportedPlatform(FGuid(0xd1d5f296, 0xff834a87, 0xb20faaa9, 0xd6b8e9a6));
		AddSupportedPlatform(FGuid(0x003be296, 0x17004f0c, 0x8e1f7860, 0x81efbb1f));

		// supported schemes
		SupportedUriSchemes.Add(TEXT("file"));

		// supported file extensions
		SupportedFileExtensions.Add(TEXT("mp4"));
		SupportedFileExtensions.Add(TEXT("m4v"));
		SupportedFileExtensions.Add(TEXT("m4a"));
		SupportedFileExtensions.Add(TEXT("mov"));

		// register player factory
		MediaModule->RegisterPlayerFactory(*this);

#if WITH_EDITOR
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "ElectraProtronFactory",
				LOCTEXT("ElectraProtronFactorySettingsName", "Electra Protron Factory"),
				LOCTEXT("ElectraProtronFactorySettingsDescription", "Configure the Electra Protron Factory."),
				GetMutableDefault<UElectraProtronFactorySettings>()
			);
		}
#endif //WITH_EDITOR
	}

	void ShutdownModule() override
	{
#if WITH_EDITOR
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "ElectraProtronFactory");
		}
#endif //WITH_EDITOR

		// Get media module once more to be sure it is still there
		MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		// unregister player factory
		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}
	}

private:
	void AddSupportedPlatform(const FGuid& PlatformGuid)
	{
		FName PlatformName = MediaModule->GetPlatformName(PlatformGuid);
		if (!PlatformName.IsNone())
		{
			SupportedPlatforms.Add(PlatformName.ToString());
		}
	}

	/** Media module */
	IMediaModule* MediaModule;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(ElectraProtronFactoryModule, ElectraProtronFactory);
