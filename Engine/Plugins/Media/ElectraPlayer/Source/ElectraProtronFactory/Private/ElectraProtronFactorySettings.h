// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ElectraProtronFactorySettings.generated.h"

/**
 * Settings for the Electra Protron Factory module.
 */
UCLASS(config=Engine)
class UElectraProtronFactorySettings : public UObject
{
	GENERATED_BODY()
	
public:
	/** When the media source leaves the player selection to automatic, Protron will be used instead of Electra in editor. */
	UPROPERTY(config, EditAnywhere, Category=General)
	bool bPreferProtronInEditor = false;

	/** When the media source leaves the player selection to automatic, Protron will be used instead of Electra in game. */
	UPROPERTY(config, EditAnywhere, Category=General)
	bool bPreferProtronInGame = false;
};